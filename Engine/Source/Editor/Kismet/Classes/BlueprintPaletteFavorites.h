// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintNodeSignature.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphSchema.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "BlueprintPaletteFavorites.generated.h"

#define UE_API KISMET_API

class UBlueprintNodeSpawner;
struct FBlueprintActionInfo;
struct FEdGraphSchemaAction;

/*******************************************************************************
* FFavoritedPaletteItem
*******************************************************************************/

USTRUCT()
struct FFavoritedBlueprintPaletteItem
{
	GENERATED_USTRUCT_BODY()

public:
	UE_API FFavoritedBlueprintPaletteItem();

	/**
	 * Sometime favorites can be coming from user edited .ini files, so this 
	 * converts that readable text into a favorite (since the strings are user 
	 * generated, there could be some error, so sure to check its validity).
	 * 
	 * @param  SerializedAction		A string representing a serialized favorite.
	 */
	UE_API FFavoritedBlueprintPaletteItem(FString const& SerializedAction);

	/**
	 * Constructs a favorite from the specified palette action (some palette actions
	 * cannot be favorited, so make sure to check its validity).
	 * 
	 * @param  PaletteAction	The action you wish to favorite.
	 */
	UE_API FFavoritedBlueprintPaletteItem(const FEdGraphSchemaAction& InPaletteAction);

	UE_DEPRECATED(5.5, "Provide a dereferenced FEdGraphSchemaAction, after validating it in the caller")
	UE_API FFavoritedBlueprintPaletteItem(TSharedPtr<FEdGraphSchemaAction> PaletteAction);

	/**
	 * 
	 * 
	 * @param  BlueprintAction	
	 * @return 
	 */
	UE_API FFavoritedBlueprintPaletteItem(UBlueprintNodeSpawner const* BlueprintAction);

	/**
	 * Sometimes we're not able to construct favorites from specified actions, 
	 * so this provides us a way to check this item's validity.
	 * 
	 * @return True if this favorite is valid (refers to a specific node), false if not.
	 */
	UE_API bool IsValid() const;

	/**
	 * Checks to see if this favorite matches the specified one.
	 * 
	 * @param  Rhs	The other favorite you want to compare with.
	 * @return True if this favorite matches the other one, false if not.
	 */
	UE_API bool operator==(FFavoritedBlueprintPaletteItem const& Rhs) const;

	/**
	 * Checks to see if this favorite represents the supplied ed-graph action 
	 * (so we can match them together, and construct a favorites list).
	 * 
	 * @param  PaletteAction	The action you want to compare with.
	 * @return True if this favorite represents the specified action, false if not.
	 */
	UE_API bool operator==(TSharedPtr<FEdGraphSchemaAction> PaletteAction) const;

	/**
	 * We want to be able to specify some of these in .ini files, so let we have
	 * to have a readable string representation for them.
	 * 
	 * @return A string representation of this item.
	 */
	UE_API FString const& ToString() const;

private:
	FBlueprintNodeSignature ActionSignature;
};

/*******************************************************************************
* FBlueprintPaletteFavorites
*******************************************************************************/

UCLASS(MinimalAPI, config=EditorPerProjectUserSettings)
class UBlueprintPaletteFavorites : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UObject Interface
	UE_API virtual void PostInitProperties() override;
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface

	/**
	 * Not all palette actions can be turned into a favorite. This method is
	 * meant to catch those cases.
	 * 
	 * @param  PaletteAction	The action you wish to check.
	 * @return True if the action can be turned into a favorite, false if not.
	 */
	UE_API bool CanBeFavorited(TSharedPtr<FEdGraphSchemaAction> PaletteAction) const;

	/**
	 * This method can be used to see if a specified action is already favorited 
	 * by the user.
	 * 
	 * @param  PaletteAction	The action you wish to check.
	 * @return True if this action is already favorited, false if it not.
	 */
	UE_API bool IsFavorited(TSharedPtr<FEdGraphSchemaAction> PaletteAction) const;
	UE_API bool IsFavorited(const FEdGraphSchemaAction& InCurrentAction) const;
	
	/**
	 * 
	 * 
	 * @param  BlueprintAction	
	 * @return 
	 */
	UE_API bool IsFavorited(FBlueprintActionInfo& BlueprintAction) const;

	/**
	 * 
	 * 
	 * @param  BlueprintAction	
	 * @return 
	 */
	UE_API bool IsFavorited(UBlueprintNodeSpawner const* BlueprintAction) const;

	/**
	 * Adds the specified action to the current favorites list (fails if the action
	 * action can't be favorited, or if the favorite already exists). Will also 
	 * convert the user's profile to a custom one (if it isn't already).
	 * 
	 * @param  PaletteAction	The action you wish to add.
	 */
	UE_API void AddFavorite(TSharedPtr<FEdGraphSchemaAction> PaletteAction);

	/**
	 * Adds the specified actions to the current favorites list. Will also 
	 * convert the user's profile to a custom one (if it isn't already).
	 * 
	 * @param  PaletteActions	An array of action you wish to add.
	 */
	UE_API void AddFavorites(TArray< TSharedPtr<FEdGraphSchemaAction> > PaletteActions);

	/**
	 * Removes the specified action to the current favorites list (if it's 
	 * there). Will also convert the user's profile to a custom one (if it isn't 
	 * already).
	 * 
	 * @param  PaletteAction	The action you wish to remove.
	 */
	UE_API void RemoveFavorite(TSharedPtr<FEdGraphSchemaAction> PaletteAction);

	/**
	 * Remove the specified actions from the current favorites list. Will also 
	 * convert the user's profile to a custom one (if it isn't already).
	 * 
	 * @param  PaletteActions	An array of action you wish to remove.
	 */
	UE_API void RemoveFavorites(TArray< TSharedPtr<FEdGraphSchemaAction> > PaletteActions);

	/**
	 * Throws out all current favorites and loads in ones for the specified 
	 * profile (explicitly laid out in the editor .ini file).
	 * 
	 * @param  ProfileName	The name of the profile you wish to load.
	 */
	UE_API void LoadProfile(FString const& ProfileName);

	/**
	 * Provides any easy way to see if the user is currently using their own
	 * manual profile (one they set up through the tool).
	 * 
	 * @return True if the user is using their own profile, false if it is a predefined one.
	 */
	UE_API bool IsUsingCustomProfile() const;

	/**
	 * Gets the user's currently set profile. If the user hasn't manually set 
	 * one themselves, then it'll return the default profile identifier.
	 * 
	 * @return A string name, representing the currently set profile (defined in the editor .ini file)
	 */
	UE_API FString const& GetCurrentProfile() const;

	/**
	 * Removes every single favorite and sets the user's profile to a custom one
	 * (if it isn't already).
	 */
	UE_API void ClearAllFavorites();

	/** 
	 * A event for users to hook into (specifically the UI), so they can be 
	 * notified when a change to the favorites has been made.
	 */
	DECLARE_EVENT(UBlueprintPaletteFavorites, FBlueprintFavoritesUpdatedEvent);
	FBlueprintFavoritesUpdatedEvent OnFavoritesUpdated;

private:
	/**
	 * Gets the default profile id specified by the user's .ini file (so we can 
	 * easily change it without modifying code).
	 * 
	 * @return A string name, representing the default profile (defined in the editor .ini file)
	 */
	UE_API FString const& GetDefaultProfileId() const;

	/**
	 * Throws out all current favorites and loads in ones for specified by 
	 * CurrentProfile.
	 */
	UE_API void LoadSetProfile();

	/**
	 * Fills the CurrentFavorites array with items that have been loaded into 
	 * the CustomFavorites array.
	 */
	UE_API void LoadCustomFavorites();

	/**
	 * Modifies the CurrentProfile member, but flags the property as edited so 
	 * that it invokes a save of the config.
	 * 
	 * @param  NewProfileName	The new profile that you want favorites saved with.
	 */
	UE_API void SetProfile(FString const& NewProfileName);

public:
	/** 
	 * A list of strings that are used to identify specific palette actions. 
	 * This is what gets saved out when the user has customized their own set, 
	 * and is not updated until PreSave.
	 */
	UPROPERTY(config)
	TArray<FString> CustomFavorites;

	/** 
	 * A list of favorites that is constructed in PostLoad() (either from a 
	 * profile or the user's set of CustomFavorites). This list is up to date 
	 * and maintained at runtime.
	 */
	UPROPERTY(Transient)
	TArray<FFavoritedBlueprintPaletteItem> CurrentFavorites;

	/** 
	 * Users could load pre-existing profiles (intended to share favorites, and 
	 * hook into tutorials). If empty, the default profile will be loaded; if 
	 * the user has customized a pre-existing profile, then this will be "CustomProfile".
	 */
	UPROPERTY(config)
	FString CurrentProfile;
};

#undef UE_API
