// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Cooker/CookDependency.h"

#include "BlueprintDependencies.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogCookBlueprint, Display, All);

UENUM()
enum class EBPDependencyType
{
	Asset,
	Struct,
	Class,
	GameplayTag
};

USTRUCT()
struct FBlueprintDependency
{
	GENERATED_BODY()
	
	UPROPERTY()
	EBPDependencyType DependencyType = EBPDependencyType::Asset;

	UPROPERTY()
	FName PackageName;

	UPROPERTY()
	FName NativeObjectName;

	UPROPERTY()
	FString Hash;
};

USTRUCT()
struct FBlueprintDependencies
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FBlueprintDependency> BlueprintDependencies;
};

namespace UE::Private::BlueprintDependencies
{
KISMET_API UE::Cook::FCookDependency RecordCookDependencies(const UBlueprint* BP);
}
