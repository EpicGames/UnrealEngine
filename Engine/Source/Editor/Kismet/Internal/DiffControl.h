// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SCSDiff.h"
#include "DetailsDiff.h"
#include "DiffResults.h"
#include "IAssetTypeActions.h"
#include "IDetailsView.h"
#include "ReviewComments.h"
#include "Widgets/SWidget.h"
#include "DiffUtils.h"

struct FReviewCommentsDiffControl;
class SMultiLineEditableTextBox;
struct FDiffResultItem;
class SBlueprintDiff;
class UEdGraph;
struct FEdGraphEditAction;
class SCheckBox;

class FAsyncDetailViewDiff;

/** Interface responsible for generating FBlueprintDifferenceTreeEntry's for visual diff tools */
class IDiffControl
{
public:
	virtual ~IDiffControl() = default;

	virtual void Tick() {};
	
	/** Adds widgets to the tree of differences to show */
	virtual void GenerateTreeEntries(TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutTreeEntries, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutRealDifferences) = 0;
	
	static KISMET_API FText RightRevision;
	static KISMET_API TSharedRef<SWidget> GenerateSimpleDiffWidget(FText DiffText);
	static KISMET_API TSharedRef<SWidget> GenerateObjectDiffWidget(FSingleObjectDiffEntry DiffEntry, FText ObjectName);

	// to support comment posting, set this to the tree view that contains the comments.
	KISMET_API void EnableComments(TWeakPtr<STreeView<TSharedPtr<FBlueprintDifferenceTreeEntry>>> TreeView, const UObject* OldObject, const UObject* NewObject);
	KISMET_API void EnableComments(TWeakPtr<STreeView<TSharedPtr<FBlueprintDifferenceTreeEntry>>> TreeView, const TArray<const UObject*>& Objects);
	virtual void EnableComments(TWeakPtr<STreeView<TSharedPtr<FBlueprintDifferenceTreeEntry>>> TreeView) {}
	
protected:
	// to be called at the end of GenerateTreeEntries if you want this diff control to support review comments
	KISMET_API void GenerateCategoryCommentTreeEntries(TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& , TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutRealDifferences, FString CategoryKey);
	
	TSharedPtr<FReviewCommentsDiffControl> ReviewCommentsDiffControl;
};

/** Shows all differences for the blueprint structure itself that aren't picked up elsewhere */
class FMyBlueprintDiffControl : public TSharedFromThis<FMyBlueprintDiffControl>, public IDiffControl
{
public:
	FMyBlueprintDiffControl(const UBlueprint* InOldBlueprint, const UBlueprint* InNewBlueprint, FOnDiffEntryFocused InSelectionCallback);

	virtual void GenerateTreeEntries(TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutTreeEntries, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutRealDifferences) override;

	// to support comment posting, set this to the tree view that contains the comments.
	virtual void EnableComments(TWeakPtr<STreeView<TSharedPtr<FBlueprintDifferenceTreeEntry>>> TreeView) override;

private:
	FOnDiffEntryFocused SelectionCallback;
	const UBlueprint* OldBlueprint;
	const UBlueprint* NewBlueprint;
};

/** 
 * Each difference in the tree will either be a tree node that is added in one Blueprint 
 * or a tree node and an FName of a property that has been added or edited in one Blueprint
 */
class FSCSDiffControl : public TSharedFromThis<FSCSDiffControl>, public IDiffControl
{
public:
	FSCSDiffControl(const UBlueprint* InOldBlueprint, const UBlueprint* InNewBlueprint, FOnDiffEntryFocused InSelectionCallback);

	virtual void GenerateTreeEntries(TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutTreeEntries, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutRealDifferences) override;

	TSharedRef<SWidget> OldTreeWidget() { return OldSCS.TreeWidget(); }
	TSharedRef<SWidget> NewTreeWidget() { return NewSCS.TreeWidget(); }

	// to support comment posting, set this to the tree view that contains the comments.
	virtual void EnableComments(TWeakPtr<STreeView<TSharedPtr<FBlueprintDifferenceTreeEntry>>> TreeView) override;

private:
	FOnDiffEntryFocused SelectionCallback;
	FSCSDiffRoot DifferingProperties;

	FSCSDiff OldSCS;
	FSCSDiff NewSCS;
};

/** Generic wrapper around a details view template parameter determines whether TreeEntries are populated */
class FDetailsDiffControl : public TSharedFromThis<FDetailsDiffControl>, public IDiffControl
{
public:
	KISMET_API FDetailsDiffControl(const UObject* InOldObject, const UObject* InNewObject, FOnDiffEntryFocused InSelectionCallback, bool bPopulateOutTreeEntries);
	KISMET_API ~FDetailsDiffControl();

	KISMET_API virtual void Tick() override;
	
	KISMET_API virtual void GenerateTreeEntries(TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutTreeEntries, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutRealDifferences) override;
	KISMET_API void GenerateTreeEntriesWithoutComments(TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutTreeEntries, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutRealDifferences);

	KISMET_API TSharedRef<IDetailsView> InsertObject(const UObject* Object, bool bScrollbarOnLeft = false, int32 Index = INDEX_NONE);
	KISMET_API TSharedRef<IDetailsView> GetDetailsWidget(const UObject* Object) const;
	KISMET_API TSharedPtr<IDetailsView> TryGetDetailsWidget(const UObject* Object) const;
	KISMET_API TSharedPtr<FAsyncDetailViewDiff> GetDifferencesWithLeft(const UObject* Object) const;
	KISMET_API TSharedPtr<FAsyncDetailViewDiff> GetDifferencesWithRight(const UObject* Object) const;
	KISMET_API int32 IndexOfObject(const UObject* Object) const;

	// to support comment posting, set this to the tree view that contains the comments.
	KISMET_API virtual void EnableComments(TWeakPtr<STreeView<TSharedPtr<FBlueprintDifferenceTreeEntry>>> TreeView) override;

	DiffUtils::FOnGenerateCustomDiffEntries GenerateCustomEntriesCallback;
	DiffUtils::FOnGenerateCustomDiffEntryWidget GenerateCustomEntryWidgetCallback;
	DiffUtils::FOnOrganizeDiffEntries OrganizeEntriesCallback;
protected:
	KISMET_API virtual void OnSelectDiffEntry(FPropertySoftPath PropertyName);
	KISMET_API TAttribute<TArray<FVector2f>> GetLinkedScrollRateAttribute(const TSharedRef<IDetailsView>& OldDetailsView, const TSharedRef<IDetailsView>& NewDetailsView);
	

	// helper function that analyzes two details views and determines the rate they should scroll relative to one another to be in sync
	KISMET_API TArray<FVector2f> GetLinkedScrollRate(TSharedRef<IDetailsView> LeftDetailsView, TSharedRef<IDetailsView> RightDetailsView) const;
private:
	KISMET_API void HandlePropertiesChanged();

	FDetailsDiffControl& operator=(const FDetailsDiffControl&) = delete;
	FDetailsDiffControl( const FDetailsDiffControl& ) = delete;
	FDetailsDiffControl& operator=(FDetailsDiffControl&&) = delete;
	FDetailsDiffControl( FDetailsDiffControl&& ) = delete;
protected:

	FOnDiffEntryFocused SelectionCallback;
	
	TMap<const UObject*,FDetailsDiff> DetailsDiffs;
	TArray<const UObject*> ObjectDisplayOrder;
	struct FPropertyTreeDiffPairs
	{
		TSharedPtr<FAsyncDetailViewDiff> Left;
		TSharedPtr<FAsyncDetailViewDiff> Right;
	};
	TMap<const UObject*, FPropertyTreeDiffPairs> PropertyTreeDifferences;

	TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> > Children;
	const bool bPopulateOutTreeEntries;
	
	TSet<FPropertyPath> PropertyAllowList;
};

/** Override for CDO special case */
class FCDODiffControl : public FDetailsDiffControl
{
public:
	FCDODiffControl(const UObject* InOldObject, const UObject* InNewObject, FOnDiffEntryFocused InSelectionCallback);

	virtual void GenerateTreeEntries(TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutTreeEntries, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutRealDifferences) override;
};

/** Override for class class settings */
class FClassSettingsDiffControl : public FDetailsDiffControl
{
public:
	FClassSettingsDiffControl(const UObject* InOldObject, const UObject* InNewObject, FOnDiffEntryFocused InSelectionCallback);

	virtual void GenerateTreeEntries(TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutTreeEntries, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutRealDifferences) override;
};

/** Diff control to handle finding type-specific differences */
struct FBlueprintTypeDiffControl : public TSharedFromThis<FBlueprintTypeDiffControl>, public IDiffControl
{
	struct FSubObjectDiff
	{
		FDiffSingleResult SourceResult;
		FDetailsDiff OldDetails;
		FDetailsDiff NewDetails;
		TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>> Diffs;

		FSubObjectDiff(const FDiffSingleResult& InSourceResult, const UObject* OldObject, const UObject* NewObject)
			: SourceResult(InSourceResult)
			, OldDetails(OldObject)
			, NewDetails(NewObject)
		{}
	};

	struct FDiffCustomObject
	{
		FDiffSingleResult SourceResult;
		TSharedPtr<IDiffCustomObject> DiffWidget;
		TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>> Diffs;
		FDiffCustomObject(const FDiffSingleResult& InSourceResult, TSharedPtr<IDiffCustomObject> DiffWidget)
			: SourceResult(InSourceResult)
			, DiffWidget(DiffWidget)
		{}
	};

	FBlueprintTypeDiffControl(const UBlueprint* InBlueprintOld, const UBlueprint* InBlueprintNew, FOnDiffEntryFocused InSelectionCallback);

	/** Generate difference tree widgets */
	virtual void GenerateTreeEntries(TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutTreeEntries, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutRealDifferences) override;
	
	// to support comment posting, set this to the tree view that contains the comments.
	virtual void EnableComments(TWeakPtr<STreeView<TSharedPtr<FBlueprintDifferenceTreeEntry>>> TreeView) override;
	
	/** The old blueprint (left) */
	const UBlueprint* BlueprintOld;

	/** The new blueprint(right) */
	const UBlueprint* BlueprintNew;

	/** Boxes that will display the details diffs */
	TSharedPtr<SBox> OldDetailsBox;
	TSharedPtr<SBox> NewDetailsBox;

private:
	/** Generate Widget for top category */
	TSharedRef<SWidget> GenerateCategoryWidget(bool bHasRealDiffs);

	/** Build up the Diff Source Array*/
	void BuildDiffSourceArray();

	/** Handle selecting a diff */
	void OnSelectSubobjectDiff(FPropertySoftPath Identifier, TSharedPtr<FSubObjectDiff> SubObjectDiff);

	/** Handle selecting a custom object diff */
	void OnSelectDiffCustomObject(TSharedPtr<IDiffCustomProperty> CustomProperty, TSharedPtr<FDiffCustomObject> DiffCustomObject);

	/** List of objects with differences */
	TArray<TSharedPtr<FSubObjectDiff>> SubObjectDiffs;
	
	/** List of custom object differences */
	TArray<TSharedPtr<FDiffCustomObject>> DiffCustomObjects;

	/** Source for list view */
	TArray<TSharedPtr<FDiffResultItem>> DiffListSource;

	/** Selection callback */
	FOnDiffEntryFocused SelectionCallback;

	/** Did diff generation succeed? */
	bool bDiffSucceeded;
};


/** Category list item for a graph*/
struct FGraphToDiff	: public TSharedFromThis<FGraphToDiff>, IDiffControl
{
	FGraphToDiff(SBlueprintDiff* DiffWidget, UEdGraph* GraphOld, UEdGraph* GraphNew, const FRevisionInfo& RevisionOld, const FRevisionInfo& RevisionNew);
	virtual ~FGraphToDiff() override;

	/** Add widgets to the differences tree */
	virtual void GenerateTreeEntries(TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutTreeEntries, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutRealDifferences) override;

	/** Get old(left) graph*/
	UEdGraph* GetGraphOld() const;

	/** Get new(right) graph*/
	UEdGraph* GetGraphNew() const;

	// to support comment posting, set this to the tree view that contains the comments.
	virtual void EnableComments(TWeakPtr<STreeView<TSharedPtr<FBlueprintDifferenceTreeEntry>>> TreeView) override;
	
	/** Source for list view */
	TArray<TSharedPtr<FDiffResultItem>> DiffListSource;
	TSharedPtr<TArray<FDiffSingleResult>> FoundDiffs;

	/** Index of the first item in RealDifferences that was generated by this graph */
	int32 RealDifferencesStartIndex = INDEX_NONE;

private:
	/** Get tooltip for category */
	FText GetToolTip();

	/** Generate Widget for category list */
	TSharedRef<SWidget> GenerateCategoryWidget();

	/** Called when the Newer Graph is modified*/
	void OnGraphChanged(const FEdGraphEditAction& Action);

	/** Build up the Diff Source Array*/
	void BuildDiffSourceArray();

	/** Diff widget */
	class SBlueprintDiff* DiffWidget;

	/** The old graph(left)*/
	UEdGraph* GraphOld;

	/** The new graph(right)*/
	UEdGraph* GraphNew;

	/** Description of Old and new graph*/
	FRevisionInfo	RevisionOld, RevisionNew;

	/** Handle to the registered OnGraphChanged delegate. */
	FDelegateHandle OnGraphChangedDelegateHandle;
};

class FCommentTreeEntry : public FBlueprintDifferenceTreeEntry, public TSharedFromThis<FCommentTreeEntry>
{
public:
	FCommentTreeEntry(TWeakPtr<FReviewCommentsDiffControl> InCommentsControl, const FReviewComment& InComment, const TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& InChildren);
	virtual ~FCommentTreeEntry() override;
	static TSharedRef<FCommentTreeEntry> Make(TWeakPtr<FReviewCommentsDiffControl> CommentsControl, const FReviewComment& Comment, const TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& Children = {});

	int32 GetCommentIDChecked() const;
	void AwaitCommentPost();
private:
	// FBlueprintDifferenceTreeEntry callbacks
	TSharedRef<SWidget> CreateWidget();

	// slate callbacks
	bool IsCommentTextBoxReadonly() const;
	FReply OnClickReply();
	FReply OnClickEdit();
	FReply OnLikeToggle();
	FText GetLikeTooltip() const;
	const FSlateBrush* GetLikeIcon() const;
	FSlateColor GetUsernameColor() const;
	FSlateColor GetLikeIconColor() const;
	EVisibility GetEditReplyButtonGroupVisibility() const;
	EVisibility GetEditButtonVisibility() const;
	EVisibility GetSubmitCancelButtonGroupVisibility() const;
	bool IsSubmitButtonEnabled() const;
	FReply OnEditSubmitClicked();
	FReply OnEditCancelClicked();

	// comment api callbacks
	void OnCommentPosted(const FReviewComment& Comment);

	FString GetCommentString() const;
	void SetCommentString(const FString& NewComment);

	// during a pending edit, returns whether the comment is different from the saved version
	bool HasCommentStringChanged() const;
	
	bool IsEditMode() const;
	void SetEditMode(bool bIsEditMode);
	
	FReviewComment Comment;

	TSharedPtr<SWidget> Content;
	TSharedPtr<SHorizontalBox> EditReplyButtonGroup;
	TSharedPtr<SHorizontalBox> SubmitCancelButtonGroup;
	TSharedPtr<SMultiLineEditableTextBox> CommentTextBox;
	
	TWeakPtr<FReviewCommentsDiffControl> CommentsControl;
	FDelegateHandle OnCommentPostedHandle;

	// prefer the use of IsEditMode().
	bool bExpectedEditMode = false;
};

class FCommentDraftTreeEntry : public FBlueprintDifferenceTreeEntry, public TSharedFromThis<FCommentDraftTreeEntry>
{
public:
	FCommentDraftTreeEntry(TWeakPtr<FReviewCommentsDiffControl> InCommentsControl, TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>* InSiblings, int32 InReplyID = -1);
	static TSharedRef<FCommentDraftTreeEntry> MakeCommentDraft(TWeakPtr<FReviewCommentsDiffControl> CommentsControl, TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>* Siblings);
	static TSharedRef<FCommentDraftTreeEntry> MakeReplyDraft(TWeakPtr<FReviewCommentsDiffControl> CommentsControl, TSharedPtr<FCommentTreeEntry> InParent);

	void ReassignReplyParent(TSharedPtr<FCommentTreeEntry> InParent);

	TSharedPtr<SMultiLineEditableTextBox> GetCommentTextBox() const {return CommentTextBox;}

	bool IsReply() const;
private:
	// FBlueprintDifferenceTreeEntry callbacks
	TSharedRef<SWidget> CreateWidget();
	
	// slate callbacks
	bool IsPostButtonEnabled() const;
	FReply OnCommentPostClicked();
	
	void ReassignSiblings(TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>* InSiblings, int32 InReplyID);
	
	TSharedPtr<SWidget> Content;
	TSharedPtr<SMultiLineEditableTextBox> CommentTextBox;
	TSharedPtr<SCheckBox> FlagAsTaskCheckBox;
	
	TWeakPtr<FReviewCommentsDiffControl> CommentsControl;
	
	TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>* Siblings;
	
	// if this is a reply, store the comment id of it's parent
	int32 ReplyID = -1;
};

/** Category list item for a graph*/
struct FReviewCommentsDiffControl : public TSharedFromThis<FReviewCommentsDiffControl>, IDiffControl
{
	FReviewCommentsDiffControl(const FString& InCommentFilePath, TWeakPtr<STreeView<TSharedPtr<FBlueprintDifferenceTreeEntry>>> TreeView);

	virtual void GenerateTreeEntries(TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutTreeEntries, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutRealDifferences) override;

	void SetCategory(const FString& CategoryKey);

	void PostComment(FReviewComment& Comment);
	void DraftReply(TSharedPtr<FCommentTreeEntry> ParentComment);
	void RebuildListView() const;

	const FString& GetCommentFilePath() {return CommentFilePath;}
	const FString& GetCommentCategory() {return CommentCategory;}
protected:
	
	void GenerateCommentThreadRecursive(const FReviewComment& Comment,
		const TMap<int32, TArray<const FReviewComment*>>& CommentReplyMap,
		TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& OutTreeEntries);
	
	FString CommentFilePath;
	FString CommentCategory;
	TWeakPtr<STreeView<TSharedPtr<FBlueprintDifferenceTreeEntry>>> CommentsTreeView;

	// we only want one reply draft widget in existence so we can use a static to track it
	static TWeakPtr<FCommentDraftTreeEntry> ReplyDraftEntry;
};
