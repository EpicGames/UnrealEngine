// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "Containers/UnrealString.h"
#include "Misc/EnumClassFlags.h"

namespace EHostType { enum Type : int; }
class UClass;
class UFunction;
class UPackage;

namespace UE::ProjectUtilities
{
	/** 
	 * FBuildTargetSet is a value type that represents a set of build targets, e.g. Server|Client|Editor or Client|Server. 
	 * The underlying target types are meant to be abstracted away, instead the set can be constructed from a piece of 
	 * reflection data (e.g. a UFunction to call or UClass to use)
	 */
	struct FBuildTargetSet
	{
		/**
		 * Returns a FBuildTargetSet representing the set of targets that the caller supports, that the callee does not.
		 * 
		 * @param Caller	A class that intends to call the Callee
		 * @param Callee	The function being called
		 * @return			FBuildTargetSet representing the unsupported targets that a caller supports that the 
		 *					callee does not - useful for detecting a caller that may exist on some target (e.g. client) 
		 *					using a function that will not exist on said target.
		 */
		static KISMET_API FBuildTargetSet GetCallerTargetsUnsupportedByCallee(const UClass* Caller, const UFunction* Callee);

		/** @return			A String representation of this, useful for messaging to a user */
		KISMET_API FString LexToString() const;

		/** Consistent compiler generated default comparison operations */
		friend auto operator<=>(const FBuildTargetSet&, const FBuildTargetSet&) = default;
	private:
		// This internal state may become more restrictive as we add validations - e.g.
		// for platforms or specialty programs that go beyond the server/client/editor 
		// paradigm:
		enum class EBuildTargetFlags : uint32
		{
			None = 0,
			Server = 1 << 0,
			Client = 1 << 1,
			Editor = 1 << 2,
		};
		FRIEND_ENUM_CLASS_FLAGS(EBuildTargetFlags)
		static KISMET_API FString LexToStringImpl(EBuildTargetFlags Flags);
		static EBuildTargetFlags GetSupportedTargetsForNativeClass(const UClass* NativeClass);
		static EBuildTargetFlags GetCallerTargetsUnsupportedByCalleeImpl(EBuildTargetFlags CallerTargets, EBuildTargetFlags CalleeTargets);

		EBuildTargetFlags BuildTargetFlags = EBuildTargetFlags::None;
	};
	ENUM_CLASS_FLAGS(FBuildTargetSet::EBuildTargetFlags);

	/**
	 * Given a native UPackage attempts to find the module descriptor host type, returning EHostType::Max 
	 * if it cannot be determined.
	 */
	KISMET_API const EHostType::Type FindModuleDescriptorHostType(const UPackage* ForNativePackage);
} 
