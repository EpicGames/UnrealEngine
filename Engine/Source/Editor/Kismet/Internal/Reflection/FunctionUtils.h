// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FunctionUtilsPrivate.h"

namespace UE::Reflection
{

/** 
 * Template function for testing the signature of a UFuction, the template
 * argument should be a function signature, similar to TFunctionRef or 
 * std::function. Returns true if the UFunction is notionally compatible
 * with the function signature. For example, if you wish to identify functions
 * have a return param of int32 that takes in two int32s you would write:
 * 
 *	using UE::Reflection;
 *	const bool bMatches = DoesStaticFunctionSignatureMatch<int(int, int)>(TestFunction);
 *
 * Reference types should be tested with their templated variant, e.g. 
 * TObjectPtr<UObject> instead of UObject*, and TSubclassOf<UObject> instead 
 * of UClass*.
 * 
 * Current shortcomings:
 *	Any function with OutParams is not currently testable with this
 * function, they will always return false.
 *	Some intrinsic structs (like FVector) are not currently testable
 *	Delegate parameters are implemented but untested
 */
template<typename T>
bool DoesStaticFunctionSignatureMatch(const UFunction* TestFunction)
{
	using namespace UE::Private;
	return TDoesStaticFunctionSignatureMatchImpl<T>::DoesMatch(TestFunction);
}

}
