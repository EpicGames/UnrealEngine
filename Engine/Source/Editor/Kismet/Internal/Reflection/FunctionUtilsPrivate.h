// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/UnrealType.h"
#include "UObject/PropertyOptional.h"

namespace UE::Private
{
	template<typename T>
	struct TDoesStaticFunctionSignatureMatchImpl;

	// These simple helpers aren't enough to truly detect all UEnums and UStructs,
	// we would also need specializations for non-UHT generated UStructs,to make
	// this edge case more obvious I have chosen these names:
	template <typename T, typename = void>
	struct TIsUHTUEnum : std::false_type { };

	template <typename T>
	struct TIsUHTUEnum <T, std::void_t<decltype(&T::StaticEnum)>> : std::true_type {};

	template <typename T, typename = void>
	struct TIsUHTUStruct : std::false_type { };
	
	template <typename T>
	struct TIsUHTUStruct <T, std::void_t<decltype(&T::StaticStruct)>> : std::true_type {};

	/*
		FProperty <-> CPPType mappings, encoded in TCPPTypeToPropertyType:
		FInt8Property	<-> int8
		FInt16Property	<-> int16
		FIntProperty	<-> int32
		FInt64Property	<-> int64
		FByteProperty	<-> uint8
		FUInt16Property	<-> uint16
		FUInt32Property	<-> uint32
		FUInt64Property	<-> uint64
		FFloatProperty	<-> float
		FDoubleProperty	<-> double
		FBoolProperty	<-> bool
		FStrProperty	<-> FString
		FNameProperty	<-> FName
		FTextProperty	<-> FText
		FObjectProperty	<-> TObjectPtr
		FClassProperty	<-> TSubclassOf
		FSoftObjectProperty	<-> TSoftObjectPtr
		FSoftClassProperty	<-> TSoftClassPtr
		FWeakObjectProperty	<-> TWeakObjectPtr
		FLazyObjectProperty	<-> TLazyObjectPtr
		FSetProperty	<-> TSet
		FArrayProperty	<-> TArray
		FMapProperty	<-> TMap
		FOptionalProperty	<-> TOptional
		FInterfaceProperty	<-> TScriptInterface
		FMulticastInlineDelegateProperty	<-> TMulticastDelegate
		FMulticastSparseDelegateProperty	<-> TSparseDynamicDelegate
		FDelegateProperty	<-> TScriptDelegate
		FEnumProperty	<-> UEnum
		FStructProperty	<-> UStruct
	*/
	template<typename T>
	struct TCPPTypeToPropertyType
	{
		using PropertyType = std::conditional< 
			TIsUHTUStruct<T>::value, FStructProperty,
				std::conditional<TIsUHTUEnum<T>::value, FEnumProperty, std::false_type>>;
	};

	template<> struct TCPPTypeToPropertyType<int8> { using PropertyType = FInt8Property; };
	template<> struct TCPPTypeToPropertyType<int16> { using PropertyType = FInt16Property; };
	template<> struct TCPPTypeToPropertyType<int32> { using PropertyType = FIntProperty; };
	template<> struct TCPPTypeToPropertyType<int64> { using PropertyType = FInt64Property; };
	template<> struct TCPPTypeToPropertyType<uint8> { using PropertyType = FByteProperty; };
	template<> struct TCPPTypeToPropertyType<uint16> { using PropertyType = FUInt16Property; };
	template<> struct TCPPTypeToPropertyType<uint32> { using PropertyType = FUInt32Property; };
	template<> struct TCPPTypeToPropertyType<uint64> { using PropertyType = FUInt64Property; };
	template<> struct TCPPTypeToPropertyType<float> { using PropertyType = FFloatProperty; };
	template<> struct TCPPTypeToPropertyType<double> { using PropertyType = FDoubleProperty; };
	template<> struct TCPPTypeToPropertyType<bool> { using PropertyType = FBoolProperty; };
	template<> struct TCPPTypeToPropertyType<FString> { using PropertyType = FStrProperty; };
	template<> struct TCPPTypeToPropertyType<FName> { using PropertyType = FNameProperty; };
	template<> struct TCPPTypeToPropertyType<FText> { using PropertyType = FTextProperty; };
	template<typename T> struct TCPPTypeToPropertyType<TObjectPtr<T>> { using PropertyType = FObjectProperty; };
	template<typename T> struct TCPPTypeToPropertyType<TSubclassOf<T>> { using PropertyType = FClassProperty; };
	template<typename T> struct TCPPTypeToPropertyType<TSoftObjectPtr<T>> { using PropertyType = FSoftObjectProperty; };
	template<typename T> struct TCPPTypeToPropertyType<TSoftClassPtr<T>> { using PropertyType = FSoftClassProperty; };
	template<typename T> struct TCPPTypeToPropertyType<TWeakObjectPtr<T>> { using PropertyType = FWeakObjectProperty; };
	template<typename T> struct TCPPTypeToPropertyType<TLazyObjectPtr<T>> { using PropertyType = FLazyObjectProperty; };
	template<typename T> struct TCPPTypeToPropertyType<TArray<T>> { using PropertyType = FArrayProperty; };
	template<typename K, typename V> struct TCPPTypeToPropertyType<TMap<K, V>> { using PropertyType = FMapProperty; };
	template<typename T> struct TCPPTypeToPropertyType<TOptional<T>> { using PropertyType = FOptionalProperty; };
	template<typename T> struct TCPPTypeToPropertyType<TScriptInterface<T>> { using PropertyType = FInterfaceProperty; };
	template<typename T> struct TCPPTypeToPropertyType<TMulticastDelegate<T>> { using PropertyType = FMulticastInlineDelegateProperty; };
	template<typename A, typename B, typename C> struct TCPPTypeToPropertyType<TSparseDynamicDelegate<A,B,C>> { using PropertyType = FMulticastSparseDelegateProperty; };
	template<typename T> struct TCPPTypeToPropertyType<TScriptDelegate<T>> { using PropertyType = FDelegateProperty; };

	// This implementation handles UStructs, UEnums, and all non-parameterized types, specializations below handle parameterized types
	template<typename T>
	struct TDoesParamMatchImpl
	{
		static bool DoesMatchProperty(const FProperty* Property)
		{
			using PropertyType = typename TCPPTypeToPropertyType<T>::PropertyType;
			if constexpr (TIsUHTUStruct<T>::value)
			{
				const PropertyType* SP = CastField<PropertyType>(Property);
				return SP && SP->Struct == T::StaticStruct();
			}
			else if constexpr (TIsUHTUEnum<T>::value)
			{
				const PropertyType* EP = CastField<PropertyType>(Property);
				return EP && EP->Enum == T::StaticEnum();
			}
			else
			{
				return CastField<PropertyType>(Property) != nullptr;
			}
		}
	};

	// This handles all reference types (TObjectPtr, TSubclassOf, TSoftObjectPtr, TSoftClassPtr, TWeakObjectPtr, TLazyObjectPtr:
	template<typename T>
	bool DoesReferencePropertyMatch(const T* Property, UClass* Type)
	{
		check(Type);
		return Property && Type->IsChildOf(Property->PropertyClass);
	}
	template<>
	inline bool DoesReferencePropertyMatch<FClassProperty>(const FClassProperty* Property, UClass* Type)
	{
		check(Type);
		return Property && Type->IsChildOf(Property->MetaClass);
	}
	template<>
	inline bool DoesReferencePropertyMatch<FSoftClassProperty>(const FSoftClassProperty* Property, UClass* Type)
	{
		check(Type);
		return Property && Type->IsChildOf(Property->MetaClass);
	}

	template<template <typename> typename T, typename ReferenceType>
	struct TDoesParamMatchImpl<T<ReferenceType>>
	{
		static bool DoesMatchProperty(const FProperty* P)
		{
			return DoesReferencePropertyMatch(CastField<typename TCPPTypeToPropertyType<T<ReferenceType>>::PropertyType>(P), ReferenceType::StaticClass());
		}
	};
	// Container Types - we could make these more dense, but I found this easy to write:
	template<typename ContainedType>
	struct TDoesParamMatchImpl<TSet<ContainedType>> 
	{
		static bool DoesMatchProperty(const FProperty* P)
		{
			using PropertyType = typename TCPPTypeToPropertyType<TSet<ContainedType>>::PropertyType;
			const PropertyType* SetProperty = CastField<PropertyType>(P);
			return SetProperty && TDoesParamMatchImpl<ContainedType>::DoesMatchProperty(SetProperty->ElementProp);
		}
	};
	template<typename ContainedType>
	struct TDoesParamMatchImpl<TArray<ContainedType>> 
	{
		static bool DoesMatchProperty(const FProperty* P)
		{
			using PropertyType = typename TCPPTypeToPropertyType<TArray<ContainedType>>::PropertyType;
			const PropertyType* ArrayProperty = CastField<PropertyType>(P);
			return ArrayProperty && TDoesParamMatchImpl<ContainedType>::DoesMatchProperty(ArrayProperty->Inner);
		}
	};
	template<typename ContainedType>
	struct TDoesParamMatchImpl<TOptional<ContainedType>>
	{
		static bool DoesMatchProperty(const FProperty* P)
		{
			using PropertyType = typename TCPPTypeToPropertyType<TOptional<ContainedType>>::PropertyType;
			const PropertyType* OptionalProperty = CastField<PropertyType>(P);
			return OptionalProperty && TDoesParamMatchImpl<ContainedType>::DoesMatchProperty(OptionalProperty->GetValueProperty());
		}
	};
	template<typename KeyType, typename ValueType>
	struct TDoesParamMatchImpl<TMap<KeyType, ValueType>>
	{
		static bool DoesMatchProperty(const FProperty* P)
		{
			using PropertyType = typename TCPPTypeToPropertyType<TMap<KeyType, ValueType>>::PropertyType;
			const PropertyType* MapProperty = CastField<PropertyType>(P);
			return MapProperty && TDoesParamMatchImpl<KeyType>::DoesMatchProperty(MapProperty->KeyProp)
				&& TDoesParamMatchImpl<ValueType>::DoesMatchProperty(MapProperty->ValueProp);
		}
	};
	// delegate types:
	template<typename SignatureType>
	struct TDoesParamMatchImpl<TMulticastDelegate<SignatureType>>
	{
		static bool DoesMatchProperty(const FProperty* P)
		{
			using PropertyType = typename TCPPTypeToPropertyType<TMulticastDelegate<SignatureType>>::PropertyType;
			const PropertyType* MulticastDelegateProperty = CastField<PropertyType>(P);
			return MulticastDelegateProperty && TDoesStaticFunctionSignatureMatchImpl<SignatureType>::DoesMatch(MulticastDelegateProperty->SignatureFunction);
		}
	};
	template<typename SignatureType, typename OwningClass, typename InfoGetter>
	struct TDoesParamMatchImpl<TSparseDynamicDelegate<SignatureType, OwningClass, InfoGetter>>
	{
		static bool DoesMatchProperty(const FProperty* P)
		{
			using PropertyType = typename TCPPTypeToPropertyType<TSparseDynamicDelegate<SignatureType, OwningClass, InfoGetter>>::PropertyType;
			const PropertyType* MulticastDelegateProperty = CastField<PropertyType>(P);
			return MulticastDelegateProperty && TDoesStaticFunctionSignatureMatchImpl<SignatureType>::DoesMatch(MulticastDelegateProperty->SignatureFunction);
		}
	};
	template<typename SignatureType>
	struct TDoesParamMatchImpl<TScriptDelegate<SignatureType>>
	{
		static bool DoesMatchProperty(const FProperty* P)
		{
			using PropertyType = typename TCPPTypeToPropertyType<TScriptDelegate<SignatureType>>::PropertyType;
			const PropertyType* DelegateProperty = CastField<PropertyType>(P);
			return DelegateProperty && TDoesStaticFunctionSignatureMatchImpl<SignatureType>::DoesMatch(DelegateProperty->SignatureFunction);
		}
	};

	template<typename Arg, typename... Args>
	bool DoesParamMatch(const FProperty* Param)
	{
		if (!Param ||
			!Param->HasAnyPropertyFlags(CPF_Parm) ||
			Param->HasAnyPropertyFlags(CPF_OutParm))
		{
			return false;
		}

		if constexpr (std::is_reference<Arg>::value)
		{
			if (!Param->HasAnyPropertyFlags(CPF_ReferenceParm))
			{
				return false;
			}
		}
		else
		{
			if (Param->HasAnyPropertyFlags(CPF_ReferenceParm))
			{
				return false;
			}
		}

		const bool bMatches = TDoesParamMatchImpl<Arg>::DoesMatchProperty(Param);
		if constexpr (sizeof...(Args))
		{
			return bMatches && DoesParamMatch<Args...>(static_cast<const FProperty*>(Param->Next));
		}
		else
		{
			// for the last arg, make sure we have no more next or it's a script local:
			return bMatches && (Param->Next == nullptr || !static_cast<const FProperty*>(Param->Next)->HasAnyPropertyFlags(CPF_Parm));
		}
	}

	// Two specializations, one for signatures and one for signatures with return types:
	template<typename... Args>
	struct TDoesStaticFunctionSignatureMatchImpl<void(Args...)>
	{
		static bool DoesMatch(const UFunction* Fn)
		{
			if (Fn->ReturnValueOffset != MAX_uint16 ||
				!Fn->HasAnyFunctionFlags(FUNC_Static))
			{
				return false;
			}

			return DoesParamMatch<Args...>(Fn->PropertyLink);
		}
	};

	template<typename ReturnType, typename... Args>
	struct TDoesStaticFunctionSignatureMatchImpl<ReturnType(Args...)>
	{
		static bool DoesMatch(const UFunction* Fn)
		{
			if (Fn->ReturnValueOffset == MAX_uint16 ||
				!Fn->HasAnyFunctionFlags(FUNC_Static))
			{
				return false;
			}

			return DoesParamMatch<Args...>(Fn->PropertyLink) &&
				DoesParamMatch<ReturnType>(Fn->GetReturnProperty());
		}
	};
}
