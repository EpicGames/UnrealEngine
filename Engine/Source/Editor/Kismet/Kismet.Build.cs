// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class Kismet : ModuleRules
{
	public Kismet(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(
			new string[] { 
				"AssetRegistry", 
				"AssetTools",
                "BlueprintRuntime",
                "ClassViewer",
				"Analytics",
                "LevelEditor",
				"GameProjectGeneration",
				"SourceCodeAccess",
			}
			);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AppFramework",
				"Core",
				"CoreUObject",
				"FieldNotification",
				"ApplicationCore",
				"Slate",
				"SlateCore",
				"EditorStyle",
				"EditorWidgets",
				"Engine",
				"Json",
				"Merge",
				"MessageLog",
				"EditorFramework",
				"UnrealEd",
				"GraphEditor",
				"KismetWidgets",
				"KismetCompiler",
				"BlueprintGraph",
				"BlueprintEditorLibrary",
				"AnimGraph",
				"PropertyEditor",
				"SourceControl",
				"SharedSettingsWidgets",
				"InputCore",
				"EngineSettings",
				"Projects",
				"JsonUtilities",
				"DesktopPlatform",
				"HotReload",
				"JsonObjectGraph",
				"UMGEditor",
				"UMG", // for SBlueprintDiff
				"WorkspaceMenuStructure",
				"DeveloperSettings",
				"ToolMenus",
				"Sockets",
				"SubobjectEditor",
				"SubobjectDataInterface",
				"ToolWidgets",
				"TraceLog",
				"GameplayTags",
				"InteractiveToolsFramework",
				"TypedElementRuntime"
			}
			);

		// we only 'need' live coding so that we can clear cached 
		// native visibility information when a live coding event 
		// occurs:
		if (Target.bWithLiveCoding)
		{
			PrivateDependencyModuleNames.Add("LiveCoding");
		}

		DynamicallyLoadedModuleNames.AddRange(
            new string[] {
                "BlueprintRuntime",
                "ClassViewer",
				"Documentation",
				"GameProjectGeneration",
			}
            );

		// Circular references that need to be cleaned up
		CircularlyReferencedDependentModules.AddRange(
			[
				"BlueprintGraph",
				"UMGEditor",
				"Merge",
				"KismetCompiler",
				"SubobjectEditor",
			]
		); 
	}
}
