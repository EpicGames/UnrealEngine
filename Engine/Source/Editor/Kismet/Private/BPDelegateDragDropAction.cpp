// Copyright Epic Games, Inc. All Rights Reserved.

#include "BPDelegateDragDropAction.h"

#include "BlueprintEditor.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node_AddDelegate.h"
#include "K2Node_CallDelegate.h"
#include "K2Node_ClearDelegate.h"
#include "K2Node_CustomEvent.h"
#include "K2Node_RemoveDelegate.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Layout/WidgetPath.h"
#include "ScopedTransaction.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Textures/SlateIcon.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealNames.h"

class SWidget;

#define LOCTEXT_NAMESPACE "DelegateDragDropAction"

void FKismetDelegateDragDropAction::MakeEvent(FNodeConstructionParams Params)
{
	check(Params.Graph && Params.Property);
	const FMulticastDelegateProperty* MulticastDelegateProperty = CastField<const FMulticastDelegateProperty>(Params.Property);
	const UFunction* SignatureFunction = MulticastDelegateProperty ? MulticastDelegateProperty->SignatureFunction.Get() : NULL;
	if(SignatureFunction)
	{
		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "K2_AddNode", "Add Node") );
		Params.Graph->Modify();

		const FString FunctionName = FString::Printf(TEXT("%s_Event"), *Params.Property->GetName());
		UK2Node_CustomEvent* NewNode = UK2Node_CustomEvent::CreateFromFunction(Params.GraphPosition, Params.Graph, FunctionName, SignatureFunction);

		if( NewNode )
		{
			FBlueprintEditorUtils::AnalyticsTrackNewNode( NewNode );
		}

		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(Params.Graph);
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
		Params.AnalyticCallback.ExecuteIfBound();
	}
}

void FKismetDelegateDragDropAction::AssignEvent(FNodeConstructionParams Params)
{
	check(Params.Graph && Params.Property);
	const FMulticastDelegateProperty* MulticastDelegateProperty = CastField<const FMulticastDelegateProperty>(Params.Property);
	const UFunction* SignatureFunction = MulticastDelegateProperty ? MulticastDelegateProperty->SignatureFunction.Get() : NULL;
	if (SignatureFunction)
	{
		UK2Node_AddDelegate* TemplateNode = NewObject<UK2Node_AddDelegate>();
		TemplateNode->SetFromProperty(Params.Property, Params.bSelfContext, Params.Property->GetOwnerClass());
		FEdGraphSchemaAction_K2AssignDelegate::AssignDelegate(TemplateNode, Params.Graph, NULL, Params.GraphPosition, true);
		Params.AnalyticCallback.ExecuteIfBound();
	}
}

FReply FKismetDelegateDragDropAction::DroppedOnPanel(const TSharedRef< SWidget >& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph)
{
	if(IsValid())
	{
		FNodeConstructionParams NewNodeParams;
		NewNodeParams.Property = GetVariableProperty();
		const UClass* VariableSourceClass = NewNodeParams.Property->GetOwnerChecked<UClass>();
		const UBlueprint* DropOnBlueprint = FBlueprintEditorUtils::FindBlueprintForGraph(&Graph);
		NewNodeParams.Graph = &Graph;
		NewNodeParams.GraphPosition = GraphPosition;
		NewNodeParams.bSelfContext = VariableSourceClass == NULL ||
			(DropOnBlueprint->SkeletonGeneratedClass && DropOnBlueprint->SkeletonGeneratedClass->IsChildOf(VariableSourceClass));
		NewNodeParams.AnalyticCallback = AnalyticCallback;

		FMenuBuilder MenuBuilder(true, NULL);
		const FText VariableNameText = FText::FromName( VariableName );
		MenuBuilder.BeginSection("BPDelegateDroppedOn", VariableNameText );
		{

			const bool bBlueprintCallable = NewNodeParams.Property->HasAllPropertyFlags(CPF_BlueprintCallable);
			if(bBlueprintCallable)
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("CallDelegate", "Call"),
					FText::Format( LOCTEXT("CallDelegateToolTip", "Call {0}"), VariableNameText ),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateStatic(&FKismetDelegateDragDropAction::MakeMCDelegateNode<UK2Node_CallDelegate>,NewNodeParams)
					));
			}

			if(NewNodeParams.Property->HasAllPropertyFlags(CPF_BlueprintAssignable))
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("AddDelegate", "Bind"),
					FText::Format( LOCTEXT("AddDelegateToolTip", "Bind event to {0}"), VariableNameText ),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateStatic(&FKismetDelegateDragDropAction::MakeMCDelegateNode<UK2Node_AddDelegate>,NewNodeParams)
					));

				MenuBuilder.AddMenuEntry(
					LOCTEXT("AddRemove", "Unbind"),
					FText::Format( LOCTEXT("RemoveDelegateToolTip", "Unbind event from {0}"), VariableNameText ),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateStatic(&FKismetDelegateDragDropAction::MakeMCDelegateNode<UK2Node_RemoveDelegate>, NewNodeParams)
					));

				MenuBuilder.AddMenuEntry(
					LOCTEXT("AddClear", "Unbind all"),
					FText::Format( LOCTEXT("ClearDelegateToolTip", "Unbind all events from {0}"), VariableNameText ),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateStatic(&FKismetDelegateDragDropAction::MakeMCDelegateNode<UK2Node_ClearDelegate>, NewNodeParams)
					));

				const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
				check(Schema);
				const EGraphType GraphType = Schema->GetGraphType(&Graph);
				const bool bSupportsEventGraphs = (DropOnBlueprint && FBlueprintEditorUtils::DoesSupportEventGraphs(DropOnBlueprint));
				const bool bAllowEvents = ((GraphType == GT_Ubergraph) && bSupportsEventGraphs);
				if(bAllowEvents)
				{
					MenuBuilder.AddMenuEntry(
						LOCTEXT("AddEvent", "Event"),
						FText::Format( LOCTEXT("EventDelegateToolTip", "Create event with the {0} signature"), VariableNameText ),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateStatic(&FKismetDelegateDragDropAction::MakeEvent, NewNodeParams)
						));

					MenuBuilder.AddMenuEntry(
						LOCTEXT("AssignEvent", "Assign"),
						LOCTEXT("AssignDelegateToolTip", "Create and bind event"),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateStatic(&FKismetDelegateDragDropAction::AssignEvent, NewNodeParams)
						));
				}
			}
		}
		MenuBuilder.EndSection();
		FSlateApplication::Get().PushMenu(Panel, FWidgetPath(), MenuBuilder.MakeWidget(), ScreenPosition, FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu));
	}
	return FReply::Handled();
}

bool FKismetDelegateDragDropAction::IsValid() const
{
	return VariableSource.IsValid() &&
		(VariableName != NAME_None) &&
		(NULL != FindFProperty<FMulticastDelegateProperty>(VariableSource.Get(), VariableName));
}

bool FKismetDelegateDragDropAction::IsSupportedBySchema(const class UEdGraphSchema* Schema) const
{
	auto SchemaK2 = Cast<UEdGraphSchema_K2>(Schema);
	return SchemaK2 && SchemaK2->DoesSupportEventDispatcher();
}

#undef LOCTEXT_NAMESPACE
