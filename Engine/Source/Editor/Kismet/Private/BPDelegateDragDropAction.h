// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BPVariableDragDropAction.h"
#include "BlueprintEditor.h"
#include "Containers/UnrealString.h"
#include "EdGraphSchema_K2_Actions.h"
#include "GraphEditorDragDropAction.h"
#include "Input/DragAndDrop.h"
#include "Input/Reply.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "MyBlueprintItemDragDropAction.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API KISMET_API

class SWidget;
class UEdGraph;
struct FEdGraphSchemaAction;

/** DragDropAction class for dropping a Variable onto a graph */
class FKismetDelegateDragDropAction : public FKismetVariableDragDropAction
{
public:
	DRAG_DROP_OPERATOR_TYPE(FKismetDelegateDragDropAction, FKismetVariableDragDropAction)

	// FGraphEditorDragDropAction interface
	virtual void HoverTargetChanged() override { FMyBlueprintItemDragDropAction::HoverTargetChanged(); }

	UE_API virtual FReply DroppedOnPanel(const TSharedRef< SWidget >& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph) override;

	UE_API virtual bool IsSupportedBySchema(const class UEdGraphSchema* Schema) const override;
	// End of FGraphEditorDragDropAction

	UE_API bool IsValid() const;
	
	static TSharedRef<FKismetDelegateDragDropAction> New(TSharedPtr<FEdGraphSchemaAction> InSourceAction, FName InVariableName, UStruct* InSource, FNodeCreationAnalytic AnalyticCallback)
	{
		TSharedRef<FKismetDelegateDragDropAction> Operation = MakeShareable(new FKismetDelegateDragDropAction);
		Operation->SourceAction = InSourceAction;
		Operation->VariableName = InVariableName;
		Operation->VariableSource = InSource;
		Operation->AnalyticCallback = AnalyticCallback;
		Operation->Construct();
		return Operation;
	}

	/** Structure for required node construction parameters */
	struct FNodeConstructionParams
	{
		FDeprecateSlateVector2D GraphPosition;
		UEdGraph* Graph;
		bool bSelfContext;
		const FProperty* Property;
		FNodeCreationAnalytic AnalyticCallback;
	};

	template<class TNode> static void MakeMCDelegateNode(FNodeConstructionParams Params)
	{
		check(Params.Graph && Params.Property);
		TNode* Node = NewObject<TNode>();
		FEdGraphSchemaAction_K2NewNode::SpawnNode<TNode>(
			Params.Graph,
			Params.GraphPosition,
			EK2NewNodeFlags::SelectNewNode,
			[&Params](TNode* NewInstance)
			{
				NewInstance->SetFromProperty(Params.Property, Params.bSelfContext, Params.Property->GetOwnerClass());
			}
		);
		Params.AnalyticCallback.ExecuteIfBound();
	}

	/** Create new custom event node from construction parameters */
	static UE_API void MakeEvent(FNodeConstructionParams Params);

	/** Assign new delegate node from construction parameters */
	static UE_API void AssignEvent(FNodeConstructionParams Params);

protected:
	FKismetDelegateDragDropAction() {}
};

#undef UE_API
