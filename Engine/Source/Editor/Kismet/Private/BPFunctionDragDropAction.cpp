// Copyright Epic Games, Inc. All Rights Reserved.

#include "BPFunctionDragDropAction.h"

#include "BlueprintEditor.h"
#include "BlueprintFunctionNodeSpawner.h"
#include "BlueprintNodeBinder.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphSchema_K2_Actions.h"
#include "Engine/Blueprint.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "K2Node_Event.h"
#include "K2Node_MacroInstance.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Script.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"

class SWidget;
class UEdGraphPin;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "FunctionDragDropAction"

/*******************************************************************************
* Static File Helpers
*******************************************************************************/

/**
 * A default for function node drag-drop operations to use if one wasn't 
 * supplied. Checks to see if a function node is allowed to be placed where it 
 * is currently dragged.
 * 
 * @param  DropActionIn		The action that will be executed when the user drops the dragged item.
 * @param  HoveredGraphIn	A pointer to the graph that the user currently has the item dragged over.
 * @param  ImpededReasonOut	If this returns false, this will be filled out with a reason to present the user with.
 * @param  FunctionIn		The function that the associated node would be calling.
 * @return True is the dragged palette item can be dropped where it currently is, false if not.
 */
static bool CanFunctionBeDropped(TSharedPtr<FEdGraphSchemaAction> /*DropActionIn*/, UEdGraph* HoveredGraphIn, FText& ImpededReasonOut, UFunction const* FunctionIn)
{
	bool bCanBePlaced = true;
	if (HoveredGraphIn == NULL)
	{
		bCanBePlaced = false;
		ImpededReasonOut = LOCTEXT("DropOnlyInGraph", "Nodes can only be placed inside the blueprint graph");
	}
	else if (!HoveredGraphIn->GetSchema()->IsA(UEdGraphSchema_K2::StaticClass()))
	{
		bCanBePlaced = false;
		ImpededReasonOut = LOCTEXT("CannotCreateInThisSchema", "Cannot call functions in this type of graph");
	}
	else if (FunctionIn == NULL)
	{
		bCanBePlaced = false;
		ImpededReasonOut = LOCTEXT("InvalidFuncAction", "Invalid function for placement");
	}
	else if ((HoveredGraphIn->GetSchema()->GetGraphType(HoveredGraphIn) == EGraphType::GT_Function) && FunctionIn->HasMetaData(FBlueprintMetadata::MD_Latent))
	{
		bCanBePlaced = false;
		ImpededReasonOut = LOCTEXT("CannotCreateLatentInGraph", "Cannot call latent functions in function graphs");
	}

	return bCanBePlaced;
}

/**
 * Checks to see if a macro node is allowed to be placed where it is currently 
 * dragged.
 * 
 * @param  DropActionIn		The action that will be executed when the user drops the dragged item.
 * @param  HoveredGraphIn	A pointer to the graph that the user currently has the item dragged over.
 * @param  ImpededReasonOut	If this returns false, this will be filled out with a reason to present the user with.
 * @param  MacroGraph		The macro graph that the node would be executing.
 * @param  bInIsLatentMacro	True if the macro has latent functions in it
 * @return True is the dragged palette item can be dropped where it currently is, false if not.
 */
static bool CanMacroBeDropped(TSharedPtr<FEdGraphSchemaAction> /*DropActionIn*/, UEdGraph* HoveredGraphIn, FText& ImpededReasonOut, UEdGraph* MacroGraph, bool bIsLatentMacro)
{
	bool bCanBePlaced = true;
	if (HoveredGraphIn == NULL)
	{
		bCanBePlaced = false;
		ImpededReasonOut = LOCTEXT("DropOnlyInGraph", "Nodes can only be placed inside the blueprint graph");
	}
	else if (!HoveredGraphIn->GetSchema()->IsA(UEdGraphSchema_K2::StaticClass()))
	{
		bCanBePlaced = false;
		ImpededReasonOut = LOCTEXT("CannotCreateInThisSchema_Macro", "Cannot call macros in this type of graph");
	}
	else if (MacroGraph == HoveredGraphIn)
	{
		bCanBePlaced = false;
		ImpededReasonOut = LOCTEXT("CannotRecurseMacro", "Cannot place a macro instance in its own graph");
	}
	else if( bIsLatentMacro && HoveredGraphIn->GetSchema()->GetGraphType(HoveredGraphIn) == GT_Function)
	{
		bCanBePlaced = false;
		ImpededReasonOut = LOCTEXT("CannotPlaceLatentMacros", "Cannot place a macro instance with latent functions in function graphs!");
	}

	return bCanBePlaced;
}

/*******************************************************************************
* FKismetDragDropAction
*******************************************************************************/

//------------------------------------------------------------------------------
void FKismetDragDropAction::HoverTargetChanged()
{
	UEdGraph* TheHoveredGraph = GetHoveredGraph();

	FText CannotDropReason = FText::GetEmpty();
	if (ActionWillShowExistingNode())
	{
		FSlateBrush const* ShowsExistingIcon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.ShowNode"));
		FText DragingText = FText::Format(LOCTEXT("ShowExistingNode", "Show '{0}'"), SourceAction->GetMenuDescription());
		SetSimpleFeedbackMessage(ShowsExistingIcon, FLinearColor::White, DragingText);
	}
	// it should be obvious that we can't drop on anything but a graph, so no need to point that out
	else if ((TheHoveredGraph == nullptr) || !CanBeDroppedDelegate.IsBound() || CanBeDroppedDelegate.Execute(SourceAction, TheHoveredGraph, CannotDropReason))
	{
		FMyBlueprintItemDragDropAction::HoverTargetChanged();
	}
	else 
	{
		SetFeedbackMessageError(CannotDropReason);
	}
}

//------------------------------------------------------------------------------
FReply FKismetDragDropAction::DroppedOnPanel( const TSharedRef< SWidget >& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph)
{	
	FReply Reply = FReply::Unhandled();

	FText CannotDropReason = FText::GetEmpty();
	if (!CanBeDroppedDelegate.IsBound() || CanBeDroppedDelegate.Execute(SourceAction, GetHoveredGraph(), CannotDropReason))
	{
		Reply = FMyBlueprintItemDragDropAction::DroppedOnPanel(Panel, ScreenPosition, GraphPosition, Graph);
	}

	if (Reply.IsEventHandled())
	{
		AnalyticCallback.ExecuteIfBound();
	}

	return Reply;
}

//------------------------------------------------------------------------------
bool FKismetDragDropAction::ActionWillShowExistingNode() const
{
	bool bWillFocusOnExistingNode = false;

	UEdGraph* TheHoveredGraph = GetHoveredGraph();
	if (SourceAction.IsValid() && (TheHoveredGraph != nullptr))
	{
		bWillFocusOnExistingNode = (SourceAction->GetTypeId() == FEdGraphSchemaAction_K2TargetNode::StaticGetTypeId()) ||
			(SourceAction->GetTypeId() == FEdGraphSchemaAction_K2InputAction::StaticGetTypeId());

		if (!bWillFocusOnExistingNode)
		{
			if (SourceAction->GetTypeId() == FEdGraphSchemaAction_K2AddEvent::StaticGetTypeId())
			{
				FEdGraphSchemaAction_K2AddEvent* AddEventAction = (FEdGraphSchemaAction_K2AddEvent*)SourceAction.Get();
				bWillFocusOnExistingNode = AddEventAction->EventHasAlreadyBeenPlaced(FBlueprintEditorUtils::FindBlueprintForGraph(TheHoveredGraph));
			}
			else if (SourceAction->GetTypeId() == FEdGraphSchemaAction_K2Event::StaticGetTypeId())
			{
				FEdGraphSchemaAction_K2Event* FuncAction = (FEdGraphSchemaAction_K2Event*)SourceAction.Get();
				// Drag and dropping custom events will place a Call Function and will not focus the existing event
				if (UK2Node_Event* Event = CastChecked<UK2Node_Event>(FuncAction->NodeTemplate))
				{
					UFunction* Function = FFunctionFromNodeHelper::FunctionFromNode(Event);
					if (Function && (Function->FunctionFlags & (FUNC_BlueprintCallable | FUNC_BlueprintPure)))
					{
						bWillFocusOnExistingNode = false;
					}
					else
					{
						bWillFocusOnExistingNode = true;
					}
				}
			}
		}
	}

	return bWillFocusOnExistingNode;
}

/*******************************************************************************
* FKismetFunctionDragDropAction
*******************************************************************************/

//------------------------------------------------------------------------------
TSharedRef<FKismetFunctionDragDropAction> FKismetFunctionDragDropAction::New(
	TSharedPtr<FEdGraphSchemaAction>	InActionNode,
	FName								InFunctionName, 
	UClass*								InOwningClass, 
	FMemberReference const&				InCallOnMember, 
	FNodeCreationAnalytic				AnalyticCallback, 
	FCanBeDroppedDelegate				CanBeDroppedDelegate /* = FCanBeDroppedDelegate() */)
{
	TSharedRef<FKismetFunctionDragDropAction> Operation = MakeShareable(new FKismetFunctionDragDropAction);
	Operation->FunctionName     = InFunctionName;
	Operation->OwningClass      = InOwningClass;
	Operation->CallOnMember     = InCallOnMember;
	Operation->AnalyticCallback = AnalyticCallback;
	Operation->CanBeDroppedDelegate = CanBeDroppedDelegate;
	Operation->SourceAction = InActionNode;

	if (!CanBeDroppedDelegate.IsBound())
	{
		Operation->CanBeDroppedDelegate = FCanBeDroppedDelegate::CreateStatic(&CanFunctionBeDropped, Operation->GetFunctionProperty());
	}

	Operation->Construct();
	return Operation;
}

//------------------------------------------------------------------------------
FKismetFunctionDragDropAction::FKismetFunctionDragDropAction()
	: OwningClass(NULL)
{

}

//------------------------------------------------------------------------------
FReply FKismetFunctionDragDropAction::DroppedOnPanel(TSharedRef<SWidget> const& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph)
{
	return DroppedOnPin(ScreenPosition, GraphPosition);
}

//------------------------------------------------------------------------------
FReply FKismetFunctionDragDropAction::DroppedOnPin(const FVector2f& ScreenPosition, const FVector2f& GraphPosition)
{
	FReply Reply = FReply::Unhandled();

	UEdGraph* Graph = GetHoveredGraph();

	// In certain cases, mouse movement messages can jump the event queue and process OnDragLeave before us.
	// This ends up setting our graph pointer to null, so we need to guard against that.
	if (Graph != nullptr)
	{
		// The ActionNode set during construction points to the Graph, this is suitable for displaying the mouse decorator but needs to be more complete based on the current graph
		UBlueprintFunctionNodeSpawner* FunctionNodeSpawner = GetDropAction(*Graph);

		if (FunctionNodeSpawner)
		{
			FText CannotDropReason = FText::GetEmpty();
			if (!CanBeDroppedDelegate.IsBound() || CanBeDroppedDelegate.Execute(nullptr, Graph, CannotDropReason))
			{
				UFunction const* Function = GetFunctionProperty();
				if ((Function != nullptr) && UEdGraphSchema_K2::CanUserKismetCallFunction(Function))
				{
					AnalyticCallback.ExecuteIfBound();

					const FScopedTransaction Transaction(LOCTEXT("KismetFunction_DroppedOnPanel", "Function Dropped on Graph"));

					IBlueprintNodeBinder::FBindingSet Bindings;
					UEdGraphNode* ResultNode = FunctionNodeSpawner->Invoke(Graph, Bindings, FDeprecateSlateVector2D(GraphPosition));

					// Autowire the node if we were dragging on top of a pin
					if (ResultNode != nullptr)
					{
						if (UEdGraphPin* FromPin = GetHoveredPin())
						{
							ResultNode->AutowireNewNode(FromPin);
						}
					}

					Reply = FReply::Handled();
				}
			}
		}
		else if (SourceAction->GetTypeId() == FEdGraphSchemaAction_K2Event::StaticGetTypeId())
		{
			if (FEdGraphSchemaAction_K2Event* FuncAction = (FEdGraphSchemaAction_K2Event*)SourceAction.Get())
			{
				FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(FuncAction->NodeTemplate);
			}
		}
	}

	return Reply;
}

//------------------------------------------------------------------------------
UFunction const* FKismetFunctionDragDropAction::GetFunctionProperty() const
{
	check(OwningClass != nullptr);
	check(FunctionName != NAME_None);

	UFunction* Function = FindUField<UFunction>(OwningClass, FunctionName);
	return Function;
}

//------------------------------------------------------------------------------
UBlueprintFunctionNodeSpawner* FKismetFunctionDragDropAction::GetDropAction(UEdGraph& Graph) const
{
	if (UEdGraph const* const TheHoveredGraph = &Graph)
	{
		if (UBlueprint* DropOnBlueprint = FBlueprintEditorUtils::FindBlueprintForGraph(TheHoveredGraph))
		{
			// make temp list builder
			FGraphActionListBuilderBase TempListBuilder;
			TempListBuilder.OwnerOfTemporaries = NewObject<UEdGraph>(DropOnBlueprint);
			TempListBuilder.OwnerOfTemporaries->SetFlags(RF_Transient);

			UEdGraphSchema_K2 const* K2Schema = GetDefault<UEdGraphSchema_K2>();
			check(K2Schema != nullptr);

			if (UFunction const* Function = GetFunctionProperty())
			{
				if (Function->FunctionFlags & (FUNC_BlueprintCallable | FUNC_BlueprintPure))
				{
					return UBlueprintFunctionNodeSpawner::Create(Function);
				}
			}
		}
	}
	return nullptr;
}

/*******************************************************************************
* FKismetMacroDragDropAction
*******************************************************************************/

//------------------------------------------------------------------------------
TSharedRef<FKismetMacroDragDropAction> FKismetMacroDragDropAction::New(
	TSharedPtr<FEdGraphSchemaAction> InActionNode,
	FName                 InMacroName, 
	UBlueprint*           InBlueprint, 
	UEdGraph*             InMacro, 
	FNodeCreationAnalytic AnalyticCallback)
{
	TSharedRef<FKismetMacroDragDropAction> Operation = MakeShareable(new FKismetMacroDragDropAction);
	Operation->SourceAction = InActionNode;
	Operation->MacroName = InMacroName;
	Operation->Macro = InMacro;
	Operation->Blueprint = InBlueprint;
	Operation->AnalyticCallback = AnalyticCallback;

	// Check to see if the macro has any latent functions in it, some graph types do not allow for latent functions
	bool bIsLatentMacro = FBlueprintEditorUtils::CheckIfGraphHasLatentFunctions(Operation->Macro);

	Operation->CanBeDroppedDelegate = FCanBeDroppedDelegate::CreateStatic(&CanMacroBeDropped, InMacro, bIsLatentMacro);

	Operation->Construct();
	return Operation;
}

//------------------------------------------------------------------------------
FKismetMacroDragDropAction::FKismetMacroDragDropAction()
	: Macro(nullptr)
	, Blueprint(nullptr)
{

}

//------------------------------------------------------------------------------
FReply FKismetMacroDragDropAction::DroppedOnPanel(TSharedRef<SWidget> const& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph)
{	
	check(Macro);
	check(CanBeDroppedDelegate.IsBound());

	FReply Reply = FReply::Unhandled();
	FText CannotDropReason = FText::GetEmpty();
	UEdGraph* MacroForCapture = Macro;
	FNodeCreationAnalytic& AnalyticCallbackForCapture = AnalyticCallback;

	if (CanBeDroppedDelegate.Execute(TSharedPtr<FEdGraphSchemaAction>(), &Graph, CannotDropReason))
	{
		FEdGraphSchemaAction_K2NewNode::SpawnNode<UK2Node_MacroInstance>(
			&Graph,
			FDeprecateSlateVector2D(GraphPosition),
			EK2NewNodeFlags::SelectNewNode,
			[MacroForCapture, &AnalyticCallbackForCapture](UK2Node_MacroInstance* NewInstance)
			{
				NewInstance->SetMacroGraph(MacroForCapture);
				AnalyticCallbackForCapture.ExecuteIfBound();
			}
		);
		Reply = FReply::Handled();
	}
	return Reply;
}

#undef LOCTEXT_NAMESPACE
