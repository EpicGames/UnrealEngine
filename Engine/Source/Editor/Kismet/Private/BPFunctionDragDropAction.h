// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintEditor.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Engine/MemberReference.h"
#include "Input/DragAndDrop.h"
#include "Input/Reply.h"
#include "Math/Vector2D.h"
#include "MyBlueprintItemDragDropAction.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

#define UE_API KISMET_API

class FText;
class UBlueprint;
class UClass;
class UEdGraph;
class UFunction;
struct FEdGraphSchemaAction;

/*******************************************************************************
* FKismetDragDropAction
*******************************************************************************/

class FKismetDragDropAction : public FMyBlueprintItemDragDropAction
{
public:
	DRAG_DROP_OPERATOR_TYPE(FKismetDragDropAction, FMyBlueprintItemDragDropAction)
		
	// FGraphEditorDragDropAction interface
	UE_API virtual void HoverTargetChanged() override;
	UE_API virtual FReply DroppedOnPanel( const TSharedRef< class SWidget >& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph) override;
	// End of FGraphSchemaActionDragDropAction

	DECLARE_DELEGATE_RetVal_ThreeParams(bool, FCanBeDroppedDelegate, TSharedPtr<FEdGraphSchemaAction> /*DropAction*/, UEdGraph* /*HoveredGraphIn*/, FText& /*ImpededReasonOut*/);

	static TSharedRef<FKismetDragDropAction> New(TSharedPtr<FEdGraphSchemaAction> InActionNode, FNodeCreationAnalytic AnalyticCallback, FCanBeDroppedDelegate CanBeDroppedDelegate)
	{
		TSharedRef<FKismetDragDropAction> Operation = MakeShareable(new FKismetDragDropAction);
		Operation->SourceAction = InActionNode;
		Operation->AnalyticCallback = AnalyticCallback;
		Operation->CanBeDroppedDelegate = CanBeDroppedDelegate;
		Operation->Construct();
		return Operation;
	}

protected:
	UE_API bool ActionWillShowExistingNode() const;

	/** */
	FCanBeDroppedDelegate CanBeDroppedDelegate;
};

/*******************************************************************************
* FKismetFunctionDragDropAction
*******************************************************************************/

class FKismetFunctionDragDropAction : public FKismetDragDropAction
{
public:
	DRAG_DROP_OPERATOR_TYPE(FKismetFunctionDragDropAction, FKismetDragDropAction)

	UE_API FKismetFunctionDragDropAction();

	// FGraphEditorDragDropAction interface
	UE_API virtual FReply DroppedOnPanel( const TSharedRef< class SWidget >& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph) override;
	UE_API virtual FReply DroppedOnPin(const FVector2f& ScreenPosition, const FVector2f& GraphPosition) override;
	// End of FGraphEditorDragDropAction

	static UE_API TSharedRef<FKismetFunctionDragDropAction> New(TSharedPtr<FEdGraphSchemaAction> InActionNode, FName InFunctionName, UClass* InOwningClass, const FMemberReference& InCallOnMember, FNodeCreationAnalytic AnalyticCallback, FCanBeDroppedDelegate CanBeDroppedDelegate = FCanBeDroppedDelegate());

protected:
	/** Name of function being dragged */
	FName FunctionName;
	/** Class that function belongs to */
	UClass* OwningClass;
	/** Call on member reference */
	FMemberReference CallOnMember;

	/** Looks up the functions field on OwningClass using FunctionName */
	UE_API UFunction const* GetFunctionProperty() const;

	/** Constructs an action to execute, placing a function call node for the associated function */
	UE_API class UBlueprintFunctionNodeSpawner* GetDropAction(UEdGraph& Graph) const;
};

/*******************************************************************************
* FKismetMacroDragDropAction
*******************************************************************************/

class FKismetMacroDragDropAction : public FKismetDragDropAction
{
public:
	DRAG_DROP_OPERATOR_TYPE(FKismetMacroDragDropAction, FKismetDragDropAction)

	UE_API FKismetMacroDragDropAction();

	// FGraphEditorDragDropAction interface
	UE_API virtual FReply DroppedOnPanel( const TSharedRef< class SWidget >& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph) override;
	// End of FGraphEditorDragDropAction

	static UE_API TSharedRef<FKismetMacroDragDropAction> New(TSharedPtr<FEdGraphSchemaAction> InActionNode, FName InMacroName, UBlueprint* InBlueprint, UEdGraph* InMacro, FNodeCreationAnalytic AnalyticCallback);

protected:
	// FMyBlueprintItemDragDropAction interface
	virtual UBlueprint* GetSourceBlueprint() const override
	{
		return Blueprint;
	}
	// End of FMyBlueprintItemDragDropAction interface

protected:
	/** Name of macro being dragged */
	FName MacroName;
	/** Graph for the macro being dragged */
	UEdGraph* Macro;
	/** Blueprint we are operating on */
	UBlueprint* Blueprint;
};

#undef UE_API
