// Copyright Epic Games, Inc. All Rights Reserved.

#include "BPGraphClipboardData.h"

#include "BlueprintEditor.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "Engine/MemberReference.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "K2Node_CallFunction.h"
#include "K2Node_EditablePinBase.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_Tunnel.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Misc/AssertionMacros.h"
#include "SFixupSelfContextDlg.h"
#include "Templates/Casts.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Script.h"
#include "UObject/UnrealNames.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BPGraphClipboardData)

class UObject;

FBPGraphClipboardData::FBPGraphClipboardData()
	: GraphType(GT_MAX)
{
	// this results in an invalid GraphData, which will not be able to Paste
}

FBPGraphClipboardData::FBPGraphClipboardData(const UEdGraph* InFuncGraph)
{
	SetFromGraph(InFuncGraph);
}

bool FBPGraphClipboardData::IsValid() const
{
	// the only way to set these is by populating this struct with a graph or using *mostly* valid serialized data
	return GraphName != NAME_None && !NodesString.IsEmpty() && GraphType != GT_MAX;
}

bool FBPGraphClipboardData::IsFunction() const
{
	return GraphType == GT_Function;
}

bool FBPGraphClipboardData::IsMacro() const
{
	return GraphType == GT_Macro;
}

void FBPGraphClipboardData::SetFromGraph(const UEdGraph* InFuncGraph)
{
	if (InFuncGraph)
	{
		OriginalBlueprint = Cast<UBlueprint>(InFuncGraph->GetOuter());

		GraphName = InFuncGraph->GetFName();

		if (const UEdGraphSchema* Schema = InFuncGraph->GetSchema())
		{
			GraphType = Schema->GetGraphType(InFuncGraph);
		}

		// TODO: Make this look nicer with an overload of ExportNodesToText that takes a TArray?
		// construct a TSet of the nodes in the graph for ExportNodesToText
		TSet<UObject*> Nodes;
		Nodes.Reserve(InFuncGraph->Nodes.Num());
		for (UEdGraphNode* Node : InFuncGraph->Nodes)
		{
			Nodes.Add(Node);
		}
		FEdGraphUtilities::ExportNodesToText(Nodes, NodesString);
	}
}

UEdGraph* FBPGraphClipboardData::CreateAndPopulateGraph(UBlueprint* InBlueprint, UBlueprint* FromBP, FBlueprintEditor* InBlueprintEditor, const FText& InCategoryOverride)
{
	if (InBlueprint && IsValid())
	{
		FKismetNameValidator Validator(InBlueprint);
		if (Validator.IsValid(GraphName) != EValidatorResult::Ok)
		{
			GraphName = FBlueprintEditorUtils::FindUniqueKismetName(InBlueprint, GraphName.GetPlainNameString());
		}
		UEdGraph* Graph = FBlueprintEditorUtils::CreateNewGraph(InBlueprint, GraphName, UEdGraph::StaticClass(), InBlueprintEditor->GetDefaultSchema());

		if (Graph)
		{
			if (!PopulateGraph(Graph, FromBP, InBlueprintEditor))
			{
				FBlueprintEditorUtils::RemoveGraph(InBlueprint, Graph);
				return nullptr;
			}

			if (GraphType == GT_Function)
			{
				InBlueprint->FunctionGraphs.Add(Graph);

				TArray<UK2Node_FunctionEntry*> Entry;
				Graph->GetNodesOfClass<UK2Node_FunctionEntry>(Entry);
				if (ensure(Entry.Num() == 1))
				{
					// Discard category
					Entry[0]->MetaData.Category = InCategoryOverride.IsEmpty() ? UEdGraphSchema_K2::VR_DefaultCategory : InCategoryOverride;

					// Add necessary function flags
					int32 AdditionalFunctionFlags = (FUNC_BlueprintEvent | FUNC_BlueprintCallable);
					if ((Entry[0]->GetExtraFlags() & FUNC_AccessSpecifiers) == FUNC_None)
					{
						AdditionalFunctionFlags |= FUNC_Public;
					}
					Entry[0]->AddExtraFlags(AdditionalFunctionFlags);

					Entry[0]->FunctionReference.SetExternalMember(Graph->GetFName(), nullptr);
				}
			}
			else if (ensure(GraphType == GT_Macro))
			{
				InBlueprint->MacroGraphs.Add(Graph);

				TArray<UK2Node_Tunnel*> Tunnels;
				Graph->GetNodesOfClass<UK2Node_Tunnel>(Tunnels);

				// discard category
				for (UK2Node_Tunnel* Tunnel : Tunnels)
				{
					if (Tunnel->DrawNodeAsEntry())
					{
						Tunnel->MetaData.Category = InCategoryOverride.IsEmpty() ? UEdGraphSchema_K2::VR_DefaultCategory : InCategoryOverride;
						break;
					}
				}

				// Mark the macro as public if it will be called from external objects
				if (InBlueprint->BlueprintType == BPTYPE_MacroLibrary)
				{
					Graph->SetFlags(RF_Public);
				}
			}

			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(InBlueprint);

			return Graph;
		}
	}

	return nullptr;
}

bool FBPGraphClipboardData::PopulateGraph(UEdGraph* InFuncGraph, UBlueprint* FromBP, FBlueprintEditor* InBlueprintEditor)
{
	if (FEdGraphUtilities::CanImportNodesFromText(InFuncGraph, NodesString))
	{
		TSet<UEdGraphNode*> PastedNodes;
		FEdGraphUtilities::ImportNodesFromText(InFuncGraph, NodesString, PastedNodes);

		// Only do this step if we can create functions on the blueprint (i.e. not macro graphs, etc)
		if (InBlueprintEditor->NewDocument_IsVisibleForType(FBlueprintEditor::CGT_NewFunctionGraph))
		{
			// Spawn Deferred Fixup Modal window if necessary
			TArray<UK2Node_CallFunction*> FixupNodes;
			for (UEdGraphNode* PastedNode : PastedNodes)
			{
				if (UK2Node_CallFunction* Node = Cast<UK2Node_CallFunction>(PastedNode))
				{
					if (Node->FunctionReference.IsSelfContext() && !Node->GetTargetFunction())
					{
						FixupNodes.Add(Node);
					}
				}
			}
			if (FixupNodes.Num() > 0)
			{
				if (!SFixupSelfContextDialog::CreateModal(FixupNodes, FromBP, InBlueprintEditor, FixupNodes.Num() != PastedNodes.Num()))
				{
					for (UEdGraphNode* Node : PastedNodes)
					{
						InFuncGraph->RemoveNode(Node);
					}

					return false;
				}
			}
		}
	}

	return true;
}
