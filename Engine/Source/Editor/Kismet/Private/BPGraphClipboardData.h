// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraphSchema.h"
#include "Engine/Blueprint.h"
#include "Internationalization/Text.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "BPGraphClipboardData.generated.h"

class FBlueprintEditor;
class UEdGraph;

/** A helper struct for copying a Blueprint Function to the clipboard */
USTRUCT()
struct FBPGraphClipboardData
{
	GENERATED_BODY()

	/** Default constructor */
	FBPGraphClipboardData();

	/** Constructs a FBPGraphClipboardData from a graph */
	FBPGraphClipboardData(const UEdGraph* InFuncGraph);
	
	/** Checks if the data is valid for configuring a graph */
	bool IsValid() const;

	/** Returns whether the graph represents a function */
	bool IsFunction() const;

	/** Returns whether the graph represents a macro */
	bool IsMacro() const;

	/** Populates the struct based on a graph */
	void SetFromGraph(const UEdGraph* InFuncGraph);

	UBlueprint* GetOriginalBlueprint() const { return OriginalBlueprint.IsValid() ? OriginalBlueprint.Get() : nullptr; }

	/**
	 * Creates and configures a new graph with data from this struct
	 *
	 * @param InBlueprint        The Blueprint to add the new graph to
	 * @param FromBP		The blueprint that this function is originally from
	 * @param InBlueprintEditor  Editor that is being pasted into
	 * @param InCategoryOverride Category to place the new graph into
	 * @return The new Graph, properly configured and populated if data is valid, or nullptr if data is invalid.
	 */
	UEdGraph* CreateAndPopulateGraph(UBlueprint* InBlueprint, UBlueprint* FromBP, FBlueprintEditor* InBlueprintEditor, const FText& InCategoryOverride = FText::GetEmpty());

private:
	/** Configures a graph with the nodes and settings, returns false if operation was aborted */
	bool PopulateGraph(UEdGraph* InFuncGraph, UBlueprint* FromBP, FBlueprintEditor* InBlueprintEditor);

private:
	/** Name of the Graph */
	UPROPERTY()
	FName GraphName;

	/** The type of graph */
	UPROPERTY()
	TEnumAsByte<EGraphType> GraphType;

	/** The original Blueprint that this function was copied from. Used to determine if nodes need to be created as functions or custom events */
	UPROPERTY()
	TWeakObjectPtr<UBlueprint> OriginalBlueprint;

	/** A string containing the exported text for the nodes in this function */
	UPROPERTY()
	FString NodesString;
};