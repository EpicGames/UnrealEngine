// Copyright Epic Games, Inc. All Rights Reserved.

#include "BPVariableDragDropAction.h"

#include "BlueprintEditor.h"
#include "Containers/EnumAsByte.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphSchema_K2_Actions.h"
#include "Engine/Blueprint.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GenericPlatform/GenericApplication.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node_Variable.h"
#include "K2Node_VariableGet.h"
#include "K2Node_VariableSet.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Layout/WidgetPath.h"
#include "Misc/AssertionMacros.h"
#include "ScopedTransaction.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

class SWidget;
struct FSlateBrush;
struct FSlateColor;

#define LOCTEXT_NAMESPACE "VariableDragDropAction"

FKismetVariableDragDropAction::FKismetVariableDragDropAction()
	: VariableName(NAME_None)
{
}

UBlueprint* FKismetVariableDragDropAction::GetSourceBlueprint() const
{
	check(VariableSource.IsValid());

	UClass* VariableSourceClass = nullptr;
	if (VariableSource.Get()->IsA(UClass::StaticClass()))
	{
		VariableSourceClass = CastChecked<UClass>(VariableSource.Get());
	}
	else
	{
		check(VariableSource.Get()->GetOuter());
		VariableSourceClass = CastChecked<UClass>(VariableSource.Get()->GetOuter());
	}
	return UBlueprint::GetBlueprintFromClass(VariableSourceClass);
}

void FKismetVariableDragDropAction::GetLinksThatWillBreak(	UEdGraphNode* Node, FProperty* NewVariableProperty, 
						   TArray<class UEdGraphPin*>& OutBroken)
{
	if(UK2Node_Variable* VarNodeUnderCursor = Cast<UK2Node_Variable>(Node))
	{
		if(const UEdGraphSchema_K2* Schema = Cast<const UEdGraphSchema_K2>(VarNodeUnderCursor->GetSchema()) )
		{
			FEdGraphPinType NewPinType;
			Schema->ConvertPropertyToPinType(NewVariableProperty,NewPinType);
			if(UEdGraphPin* Pin = VarNodeUnderCursor->FindPin(VarNodeUnderCursor->GetVarName()))
			{
				for(TArray<class UEdGraphPin*>::TIterator i(Pin->LinkedTo);i;i++)
				{
					UEdGraphPin* Link = *i;
					if(false == Schema->ArePinTypesCompatible(NewPinType, Link->PinType))
					{
						OutBroken.Add(Link);
					}
				}
			}
		}
	}
}

namespace
{
	/**
	* Helper function to determine if a node has any split pins on it.
	*
	* @return	True if there is a split pin on the node
	*/
	static bool NodeHasSplitPins(UEdGraphNode* InNode)
	{
		if (InNode)
		{
			for (UEdGraphPin* Pin : InNode->Pins)
			{
				// If a pin has no parent node but has SubPins then it is a split pin
				if (Pin && !Pin->ParentPin && Pin->SubPins.Num() > 0)
				{
					return true;
				}
			}
		}
		return false;
	}
}

void FKismetVariableDragDropAction::HoverTargetChanged()
{
	FProperty* VariableProperty = GetVariableProperty();
	if (VariableProperty == nullptr)
	{
		return;
	}

	FString VariableString = VariableName.ToString();

	// Icon/text to draw on tooltip
	FText Message = LOCTEXT("InvalidDropTarget", "Invalid drop target!");

	UEdGraphPin* PinUnderCursor = GetHoveredPin();

	bool bCanMakeSetter = true;
	bool bBadSchema = false;
	bool bBadGraph = false;
	UEdGraph* TheHoveredGraph = GetHoveredGraph();
	if (TheHoveredGraph)
	{
		if (!TheHoveredGraph->GetSchema()->CanVariableBeDropped(TheHoveredGraph, VariableProperty))
		{
			bBadSchema = true;
		}
		else if (!CanVariableBeDropped(VariableProperty, *TheHoveredGraph))
		{
			bBadGraph = true;
		}

		UStruct* Outer = VariableProperty->GetOwnerChecked<UStruct>();

		FNodeConstructionParams NewNodeParams;
		NewNodeParams.VariableName = VariableName;
		const UBlueprint* DropOnBlueprint = FBlueprintEditorUtils::FindBlueprintForGraph(TheHoveredGraph);
		NewNodeParams.Graph = TheHoveredGraph;
		NewNodeParams.VariableSource = Outer;
		
		bCanMakeSetter = CanExecuteMakeSetter(NewNodeParams, VariableProperty);
	}

	UEdGraphNode* VarNodeUnderCursor = Cast<UK2Node_Variable>(GetHoveredNode());

	if (bBadSchema)
	{
		SetFeedbackMessageError(LOCTEXT("CannotCreateInThisSchema", "Cannot access variables in this type of graph"));
	}
	else if (bBadGraph)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("VariableName"), FText::FromString(VariableString));
		Args.Add(TEXT("Scope"), FText::FromString(TheHoveredGraph->GetName()));

		if (IsFromBlueprint(FBlueprintEditorUtils::FindBlueprintForGraph(TheHoveredGraph)) && VariableProperty->GetOwner<UFunction>())
		{
			SetFeedbackMessageError(FText::Format( LOCTEXT("IncorrectGraphForLocalVariable_Error", "Cannot place local variable '{VariableName}' in external scope '{Scope}'"), Args));
		}
		else
		{
			SetFeedbackMessageError(FText::Format( LOCTEXT("IncorrectGraphForVariable_Error", "Cannot place variable '{VariableName}' in external scope '{Scope}'"), Args));
		}
	}
	else if (PinUnderCursor)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PinUnderCursor"), FText::FromName(PinUnderCursor->PinName));
		Args.Add(TEXT("VariableName"), FText::FromString(VariableString));

		if (CanVariableBeDropped(VariableProperty, *PinUnderCursor->GetOwningNode()->GetGraph()))
		{
			if (PinUnderCursor->bOrphanedPin)
			{
				SetFeedbackMessageError(FText::Format(LOCTEXT("OrphanedPin_Error", "Cannot make connection to orphaned pin {PinUnderCursor}"), Args));
			}
			else if (const UEdGraphSchema_K2* Schema = Cast<const UEdGraphSchema_K2>(PinUnderCursor->GetSchema()))
			{
				const bool bIsExecPin = Schema->IsExecPin(*PinUnderCursor);

				const bool bIsRead = (PinUnderCursor->Direction == EGPD_Input) && !bIsExecPin;
				const UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(PinUnderCursor->GetOwningNode());
				const bool bWritableProperty = (FBlueprintEditorUtils::IsPropertyWritableInBlueprint(Blueprint, VariableProperty) == FBlueprintEditorUtils::EPropertyWritableState::Writable);
				const bool bCanWriteIfNeeded = bIsRead || bWritableProperty;

				FEdGraphPinType VariablePinType;
				Schema->ConvertPropertyToPinType(VariableProperty, VariablePinType);
				const bool bTypeMatch = Schema->ArePinTypesCompatible(VariablePinType, PinUnderCursor->PinType) || bIsExecPin;
				const bool bCanAutoConvert = Schema->FindSpecializedConversionNode(VariablePinType, *PinUnderCursor, false).IsSet();
				bool bCanAutocast = false;
				if (PinUnderCursor->Direction == EGPD_Output)
				{
					bCanAutocast = Schema->SearchForAutocastFunction(PinUnderCursor->PinType, VariablePinType).IsSet();
				}
				else
				{
					bCanAutocast = Schema->SearchForAutocastFunction(VariablePinType, PinUnderCursor->PinType).IsSet();
				}
				
				Args.Add(TEXT("PinUnderCursor"), FText::FromName(PinUnderCursor->PinName));

				if ((bTypeMatch  || bCanAutocast || bCanAutoConvert) && bCanWriteIfNeeded)
				{
					SetFeedbackMessageOK(bIsRead ?
						FText::Format(LOCTEXT("MakeThisEqualThat_PinEqualVariableName", "Make {PinUnderCursor} = {VariableName}"), Args) :
						FText::Format(LOCTEXT("MakeThisEqualThat_VariableNameEqualPin", "Make {VariableName} = {PinUnderCursor}"), Args));
				}
				else
				{
					SetFeedbackMessageError(bCanWriteIfNeeded ?
						FText::Format(LOCTEXT("NotCompatible_Error", "The type of '{VariableName}' is not compatible with {PinUnderCursor}"), Args) :
						FText::Format(LOCTEXT("ReadOnlyVar_Error", "Cannot write to read-only variable '{VariableName}'"), Args));
				}
			}
		}
		else
		{
			Args.Add(TEXT("Scope"), FText::FromString(PinUnderCursor->GetOwningNode()->GetGraph()->GetName()));

			SetFeedbackMessageError(FText::Format( LOCTEXT("IncorrectGraphForPin_Error", "Cannot place local variable '{VariableName}' in external scope '{Scope}'"), Args));
		}
	}
	else if (VarNodeUnderCursor)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("VariableName"), FText::FromString(VariableString));

		if (CanVariableBeDropped(VariableProperty, *VarNodeUnderCursor->GetGraph()))
		{
			const bool bIsRead = VarNodeUnderCursor->IsA(UK2Node_VariableGet::StaticClass());
			const UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(VarNodeUnderCursor);
			const bool bWritableProperty = (FBlueprintEditorUtils::IsPropertyWritableInBlueprint(Blueprint, VariableProperty) == FBlueprintEditorUtils::EPropertyWritableState::Writable);
			const bool bCanWriteIfNeeded = bIsRead || bWritableProperty;
			// If this node has split pins then the reconstruction cannot handle it, so don't allow it
			const bool bHasSplitPins = NodeHasSplitPins(VarNodeUnderCursor);

			if (bHasSplitPins)
			{
				SetFeedbackMessageError(FText::Format(LOCTEXT("SplitPinVar_Error", "Cannot change '{VariableName}' because it has split pins"), Args));
			}
			else if (bCanWriteIfNeeded)
			{
				Args.Add(TEXT("ReadOrWrite"), bIsRead ? LOCTEXT("Read", "read") : LOCTEXT("Write", "write"));
				SetFeedbackMessageOK(WillBreakLinks(VarNodeUnderCursor, VariableProperty) ?
					FText::Format(LOCTEXT("ChangeNodeToWarnBreakLinks", "Change node to {ReadOrWrite} '{VariableName}', WARNING this will break links!"), Args) :
					FText::Format(LOCTEXT("ChangeNodeTo", "Change node to {ReadOrWrite} '{VariableName}'"), Args));
			}
			else
			{
				SetFeedbackMessageError(FText::Format( LOCTEXT("ReadOnlyVar_Error", "Cannot write to read-only variable '{VariableName}'"), Args));
			}
		}
		else
		{
			Args.Add(TEXT("Scope"), FText::FromString(VarNodeUnderCursor->GetGraph()->GetName()));

			SetFeedbackMessageError(FText::Format( LOCTEXT("IncorrectGraphForNodeReplace_Error", "Cannot replace node with local variable '{VariableName}' in external scope '{Scope}'"), Args));
		}
	}
	else if (bAltDrag && !bCanMakeSetter)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("VariableName"), FText::FromString(VariableString));

		SetFeedbackMessageError(FText::Format(LOCTEXT("CannotPlaceSetter", "Variable '{VariableName}' is readonly, you cannot set this variable."), Args));
	}
	else
	{
		FMyBlueprintItemDragDropAction::HoverTargetChanged();
	}
}

void FKismetVariableDragDropAction::GetDefaultStatusSymbol(const FSlateBrush*& PrimaryBrushOut, FSlateColor& IconColorOut, FSlateBrush const*& SecondaryBrushOut, FSlateColor& SecondaryColorOut) const
{
	PrimaryBrushOut = FBlueprintEditor::GetVarIconAndColor(VariableSource.Get(), VariableName, IconColorOut, SecondaryBrushOut, SecondaryColorOut);
}

FReply FKismetVariableDragDropAction::DroppedOnPin(const FVector2f& ScreenPosition, const FVector2f& GraphPosition)
{
	if (UEdGraphPin* TargetPin = GetHoveredPin())
	{
		if (!TargetPin->bOrphanedPin)
		{
			FProperty* VariableProperty = GetVariableProperty();
			if (!TargetPin->GetSchema()->CanVariableBeDropped(TargetPin->GetOwningNode()->GetGraph(), VariableProperty))
			{
				return FReply::Unhandled();
			}

			bool DropReply = ((UEdGraphSchema*)TargetPin->GetSchema())->RequestVariableDropOnPin(TargetPin->GetOwningNode()->GetGraph(), VariableProperty, TargetPin, FDeprecateSlateVector2D(GraphPosition), FDeprecateSlateVector2D(ScreenPosition));
			if (DropReply)
			{
				return FReply::Handled();
			}

			if (const UEdGraphSchema_K2* Schema = Cast<const UEdGraphSchema_K2>(TargetPin->GetSchema()))
			{
				const bool bIsExecPin = Schema->IsExecPin(*TargetPin);

				if (CanVariableBeDropped(VariableProperty, *TargetPin->GetOwningNode()->GetGraph()))
				{
					const bool bIsRead = (TargetPin->Direction == EGPD_Input) && !bIsExecPin;
					const UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(TargetPin->GetOwningNode());
					const bool bWritableProperty = (FBlueprintEditorUtils::IsPropertyWritableInBlueprint(Blueprint, VariableProperty) == FBlueprintEditorUtils::EPropertyWritableState::Writable);
					const bool bCanWriteIfNeeded = bIsRead || bWritableProperty;

					FEdGraphPinType VariablePinType;
					Schema->ConvertPropertyToPinType(VariableProperty, VariablePinType);
					const bool bTypeMatch = Schema->ArePinTypesCompatible(VariablePinType, TargetPin->PinType) || bIsExecPin;
					const bool bCanAutoConvert = Schema->FindSpecializedConversionNode(VariablePinType, *TargetPin, false).IsSet();
					bool bCanAutocast = false;
					if (TargetPin->Direction == EGPD_Output)
					{
						bCanAutocast = Schema->SearchForAutocastFunction(TargetPin->PinType, VariablePinType).IsSet();
					}
					else
					{
						bCanAutocast = Schema->SearchForAutocastFunction(VariablePinType, TargetPin->PinType).IsSet();
					}
					
					if ((bTypeMatch || bCanAutocast || bCanAutoConvert) && bCanWriteIfNeeded)
					{
						FEdGraphSchemaAction_K2NewNode Action;

						UK2Node_Variable* VarNode = bIsRead ? (UK2Node_Variable*)NewObject<UK2Node_VariableGet>() : (UK2Node_Variable*)NewObject<UK2Node_VariableSet>();
						Action.NodeTemplate = VarNode;

						UBlueprint* DropOnBlueprint = FBlueprintEditorUtils::FindBlueprintForGraph(TargetPin->GetOwningNode()->GetGraph());
						UEdGraphSchema_K2::ConfigureVarNode(VarNode, VariableName, VariableSource.Get(), DropOnBlueprint);

						Action.PerformAction(TargetPin->GetOwningNode()->GetGraph(), TargetPin, FDeprecateSlateVector2D(GraphPosition));
					}
				}
			}
		}
	}

	return FReply::Handled();
}

FReply FKismetVariableDragDropAction::DroppedOnNode(const FVector2f& ScreenPosition, const FVector2f& GraphPosition)
{
	if (UEdGraphNode* TargetNode = GetHoveredNode())
	{
		FProperty* VariableProperty = GetVariableProperty();
		if (!TargetNode->GetSchema()->CanVariableBeDropped(TargetNode->GetGraph(), VariableProperty))
		{
			return FReply::Unhandled();
		}

		bool DropReply = ((UEdGraphSchema*)TargetNode->GetSchema())->RequestVariableDropOnNode(TargetNode->GetGraph(), VariableProperty, TargetNode, FDeprecateSlateVector2D(GraphPosition), FDeprecateSlateVector2D(ScreenPosition));
		if (DropReply)
		{
			return FReply::Handled();
		}
	}

	UK2Node_Variable* TargetNode = Cast<UK2Node_Variable>(GetHoveredNode());

	if (TargetNode && (VariableName != TargetNode->GetVarName()))
	{
		FProperty* VariableProperty = GetVariableProperty();

		if (CanVariableBeDropped(VariableProperty, *TargetNode->GetGraph()) && !NodeHasSplitPins(TargetNode))
		{
			const FScopedTransaction Transaction(LOCTEXT("ReplacePinVariable", "Replace Pin Variable"));

			const FName OldVarName = TargetNode->GetVarName();
			const UEdGraphSchema_K2* Schema = Cast<const UEdGraphSchema_K2>(TargetNode->GetSchema());

			TArray<class UEdGraphPin*> BadLinks;
			GetLinksThatWillBreak(TargetNode,VariableProperty,BadLinks);

			// Change the variable name and context
			UBlueprint* DropOnBlueprint = FBlueprintEditorUtils::FindBlueprintForGraph(TargetNode->GetGraph());
			UEdGraphPin* Pin = TargetNode->FindPin(OldVarName);
			DropOnBlueprint->Modify();
			TargetNode->Modify();

			if (Pin != nullptr)
			{
				Pin->Modify();
			}

			UEdGraphSchema_K2::ConfigureVarNode(TargetNode, VariableName, VariableSource.Get(), DropOnBlueprint);

			if ((Pin == nullptr) || (Pin->LinkedTo.Num() == BadLinks.Num()) || (Schema == nullptr))
			{
				TargetNode->GetSchema()->ReconstructNode(*TargetNode);
			}
			else 
			{
				FEdGraphPinType NewPinType;
				Schema->ConvertPropertyToPinType(VariableProperty,NewPinType);

				Pin->PinName = VariableName;
				Pin->PinType = NewPinType;

				// break bad links
				for (TArray<class UEdGraphPin*>::TIterator OtherPinIt(BadLinks);OtherPinIt;++OtherPinIt)
				{
					Pin->BreakLinkTo(*OtherPinIt);
				}
			}

			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(DropOnBlueprint);

			return FReply::Handled();
		}
	}
	return FReply::Unhandled();
}

void FKismetVariableDragDropAction::MakeGetter(FNodeConstructionParams InParams)
{
	check(InParams.Graph);

	const UEdGraphSchema_K2* K2_Schema = Cast<const UEdGraphSchema_K2>(InParams.Graph->GetSchema());
	if (K2_Schema)
	{
		K2_Schema->SpawnVariableGetNode(InParams.GraphPosition, InParams.Graph, InParams.VariableName, InParams.VariableSource.Get());
	}
}

void FKismetVariableDragDropAction::MakeSetter(FNodeConstructionParams InParams)
{
	check(InParams.Graph);

	const UEdGraphSchema_K2* K2_Schema = Cast<const UEdGraphSchema_K2>(InParams.Graph->GetSchema());
	if (K2_Schema)
	{
		K2_Schema->SpawnVariableSetNode(InParams.GraphPosition, InParams.Graph, InParams.VariableName, InParams.VariableSource.Get());
	}
}

bool FKismetVariableDragDropAction::CanExecuteMakeSetter(FNodeConstructionParams InParams, FProperty* InVariableProperty)
{
	check(InVariableProperty);
	check(InParams.VariableSource.Get());
	check(InParams.Graph);

	if(UClass* VariableSourceClass = Cast<UClass>(InParams.VariableSource.Get()))
	{
		const UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(InParams.Graph);
		const bool bWritableProperty = (FBlueprintEditorUtils::IsPropertyWritableInBlueprint(Blueprint, InVariableProperty) == FBlueprintEditorUtils::EPropertyWritableState::Writable);

		bool bSupportsImpureNodes = false;
		const UEdGraphSchema_K2* K2_Schema = Cast<const UEdGraphSchema_K2>(InParams.Graph->GetSchema());
		if (K2_Schema)
		{
			bSupportsImpureNodes = K2_Schema->DoesGraphSupportImpureFunctions(InParams.Graph);
		}
		
		return (bSupportsImpureNodes && bWritableProperty && !VariableSourceClass->HasAnyClassFlags(CLASS_Const));
	}

	return true;
}

FReply FKismetVariableDragDropAction::DroppedOnPanel( const TSharedRef< SWidget >& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph)
{	
	FProperty* VariableProperty = GetVariableProperty();
	bool  DropReply = ((UEdGraphSchema*)Graph.GetSchema())->RequestVariableDropOnPanel(GetHoveredGraph(), VariableProperty, FDeprecateSlateVector2D(GraphPosition), FDeprecateSlateVector2D(ScreenPosition));
	if (DropReply)
	{
		return FReply::Handled();
	}

	if (Graph.GetSchema()->IsA<UEdGraphSchema_K2>())
	{
		if (VariableProperty && CanVariableBeDropped(VariableProperty, Graph))
		{
			UStruct* Outer = VariableProperty->GetOwnerChecked<UStruct>();
			
			FNodeConstructionParams NewNodeParams;
			NewNodeParams.VariableName = VariableName;
			NewNodeParams.Graph = &Graph;
			NewNodeParams.GraphPosition = GraphPosition;
			NewNodeParams.VariableSource= Outer;

			// call analytics
			AnalyticCallback.ExecuteIfBound();

			// Take into account current state of modifier keys in case the user changed their mind
			FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
			const bool bModifiedKeysActive = ModifierKeys.IsControlDown() || ModifierKeys.IsAltDown();
			const bool bAutoCreateGetter = bModifiedKeysActive ? ModifierKeys.IsControlDown() : bControlDrag;
			const bool bAutoCreateSetter = bModifiedKeysActive ? ModifierKeys.IsAltDown() : bAltDrag;
			// Handle Getter/Setters
			if (bAutoCreateGetter || bAutoCreateSetter)
			{
				if (bAutoCreateGetter || !CanExecuteMakeSetter(NewNodeParams, VariableProperty))
				{
					MakeGetter(NewNodeParams);
					NewNodeParams.GraphPosition.Y += 50.f;
				}
				if (bAutoCreateSetter && CanExecuteMakeSetter( NewNodeParams, VariableProperty))
				{
					MakeSetter(NewNodeParams);
				}
			}
			// Show selection menu
			else
			{
				FMenuBuilder MenuBuilder(true, NULL);
				const FText VariableNameText = FText::FromName( VariableName );

				MenuBuilder.BeginSection("BPVariableDroppedOn", VariableNameText );

				MenuBuilder.AddMenuEntry(
					FText::Format( LOCTEXT("CreateGetVariable", "Get {0}"), VariableNameText ),
					FText::Format( LOCTEXT("CreateVariableGetterToolTip", "Create Getter for variable '{0}'\n(Ctrl-drag to automatically create a getter)"), VariableNameText ),
					FSlateIcon(),
					FUIAction(
					FExecuteAction::CreateStatic(&FKismetVariableDragDropAction::MakeGetter, NewNodeParams), FCanExecuteAction())
					);

				MenuBuilder.AddMenuEntry(
					FText::Format( LOCTEXT("CreateSetVariable", "Set {0}"), VariableNameText ),
					FText::Format( LOCTEXT("CreateVariableSetterToolTip", "Create Setter for variable '{0}'\n(Alt-drag to automatically create a setter)"), VariableNameText ),
					FSlateIcon(),
					FUIAction(
					FExecuteAction::CreateStatic(&FKismetVariableDragDropAction::MakeSetter, NewNodeParams),
					FCanExecuteAction::CreateStatic(&FKismetVariableDragDropAction::CanExecuteMakeSetter, NewNodeParams, VariableProperty ))
					);

				TSharedRef< SWidget > PanelWidget = Panel;
				// Show dialog to choose getter vs setter
				FSlateApplication::Get().PushMenu(
					PanelWidget,
					FWidgetPath(),
					MenuBuilder.MakeWidget(),
					ScreenPosition,
					FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu)
					);

				MenuBuilder.EndSection();
			}
		}
	}

	return FReply::Handled();
}

bool FKismetVariableDragDropAction::CanVariableBeDropped(const FProperty* InVariableProperty, const UEdGraph& InGraph) const
{
	bool bCanVariableBeDropped = false;
	if (InVariableProperty)
	{
		// Only allow variables to be placed within the same blueprint (otherwise the self context on the dropped node will be invalid)
		bCanVariableBeDropped = IsFromBlueprint(FBlueprintEditorUtils::FindBlueprintForGraph(&InGraph));

		// Local variables have some special conditions for being allowed to be placed
		if (bCanVariableBeDropped && InVariableProperty->GetOwner<UFunction>())
		{
			// Check if the top level graph has the same name as the function, if they do not then the variable cannot be placed in the graph
			if (FBlueprintEditorUtils::GetTopLevelGraph(&InGraph)->GetFName() != InVariableProperty->GetOwner<UFunction>()->GetFName())
			{
				bCanVariableBeDropped = false;
			}
		}
	}
	return bCanVariableBeDropped;
}

UStruct* FKismetVariableDragDropAction::GetLocalVariableScope() const
{
	if( VariableSource->IsA(UFunction::StaticClass()) )
	{
		return VariableSource.Get();
	}
	return NULL;
}

#undef LOCTEXT_NAMESPACE
