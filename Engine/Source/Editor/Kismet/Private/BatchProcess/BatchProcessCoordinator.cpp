// Copyright Epic Games, Inc. All Rights Reserved.

#include "BatchProcess/BatchProcessCoordinator.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformTime.h"
#include "Misc/App.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/Paths.h"
#include "BatchProcess/BatchProcessLog.h"
#include "BatchProcess/BatchProcessSocketHelpers.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "SocketSubsystem.h"
#include "Sockets.h"

DEFINE_LOG_CATEGORY(LogBatchProcess);

namespace UE::Private
{
	FUtf8String ReadStringFromFile(const TCHAR* Filename);
	const TCHAR* const BatchCoordinatorName   = TEXT("BatchProcessCoordinator");
	const TCHAR* const CoordinatorIPStr       = TEXT("127.0.0.1");
	const uint32 CoordinatorIP                = 0x7f000001;
	const int32  MaxConnections               = 512;
	const float  TimeoutLengthSeconds         = 1.f;
	const float  JobStuckTimeoutSeconds       = 120.f; // kill a Running worker after 2 min silence
	const int32  MaxJobRetries                = 3;     // re-enqueue limit before marking a job failed
}

FUtf8String UE::Private::ReadStringFromFile(const TCHAR* Filename)
{
	TUniquePtr<FArchive> Reader(IFileManager::Get().CreateFileReader(Filename));
	if (!Reader.IsValid())
	{
		return FUtf8String();
	}
	FUtf8String ResultString;
	int64 TotalSize = Reader->TotalSize();
	if(TotalSize > 0 && TotalSize < TNumericLimits<int32>::Max())
	{
		ResultString.GetCharArray().Reserve(TotalSize+1);
		ResultString.GetCharArray().SetNum(TotalSize);
		Reader->Serialize(ResultString.GetCharArray().GetData(), TotalSize);
		ResultString.GetCharArray().Add(UTF8CHAR('\0'));
	}
	return ResultString;
}

FBatchProcessCoordinator::FBatchProcessCoordinator(const FString* ClientAddress, int32 InNumWorkers)
	: NumWorkers(InNumWorkers)
	, bIsCompleted(false)
	, bIsSealed(false)
{
	// create socket
	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get();
	check(SocketSubsystem);
	ListenAddr = SocketSubsystem->GetLocalBindAddr(*GLog);
	ListenAddr->SetIp(UE::Private::CoordinatorIP);
	ListenAddr->SetPort(UE::Private::BatchProcessSocketHelpers::DefaultPortNumber);
	CoordinatorSocket = SocketSubsystem->CreateSocket(NAME_Stream, UE::Private::BatchCoordinatorName, ListenAddr->GetProtocolType());
	CoordinatorSocket->SetReuseAddr();

	const bool bBindSuccessful = CoordinatorSocket->Bind(*ListenAddr);
	check(bBindSuccessful);

	const bool bListenSuccessful = CoordinatorSocket->Listen(UE::Private::MaxConnections);
	check(bListenSuccessful);

	// ClientAddress provided: we're in "dummy coordinator" mode — someone is launching a
	// coordinator just to exercise a single client (e.g. to debug a worker process).
	if(ClientAddress)
	{
		// connect back to the client and tell it our address so it can connect:
		TSharedPtr<class FInternetAddr> ClientAddr = SocketSubsystem->GetLocalBindAddr(*GLog);
		bool bIpValid = false;
		ClientAddr->SetIp(**ClientAddress, bIpValid);
		ClientAddr->SetPort(61580); // debug client listen port — see BatchProcessWorker
		check(bIpValid);

		FSocket* ToClient = SocketSubsystem->CreateSocket(NAME_Stream, UE::Private::BatchCoordinatorName, ClientAddr->GetProtocolType());
		bool bConnected = ToClient->Connect(*ClientAddr);
		check(bConnected);
		check(UE::Private::BatchProcessSocketHelpers::SendString(*ToClient, UE::Private::CoordinatorIPStr));

		ActiveWorkers.Add(
			{
				EWorkerState::Starting,
				FProcHandle(),
				FPlatformTime::Seconds(),
				TEXT("DummyWorker")
			}
		);

		SocketSubsystem->DestroySocket(ToClient);

		check(GetNumWorkers() == 1); // dummy mode only supports a single worker
	}
}

FBatchProcessCoordinator::~FBatchProcessCoordinator()
{
	Abort();
}

void FBatchProcessCoordinator::SubmitFile(const TCHAR* Filename)
{
	FUtf8String Command = UE::Private::ReadStringFromFile(Filename);
	if(Command == FUtf8String())
	{
		HandleError(FString::Printf(TEXT("Failed to read job file: %s"), Filename));
		return;
	}
	Submit(Command);
}

void FBatchProcessCoordinator::Submit(const FUtf8String& JsonCommand)
{
	TSharedPtr<FJsonObject> JsonObject;
	TSharedRef<TJsonReader<UTF8CHAR>> JsonReader = TJsonReaderFactory<UTF8CHAR>::Create(JsonCommand);
	if (!FJsonSerializer::Deserialize(JsonReader, JsonObject) || !JsonObject.IsValid())
	{
		HandleError(TEXT("Failed to parse job JSON"));
		return;
	}

	const bool bFoundFunction = JsonObject->TryGetStringField(TEXT("Function"), FunctionAsString);
	if(!bFoundFunction)
	{
		HandleError(TEXT("Job JSON missing required field: \"function\""));
		return;
	}

	const TArray<TSharedPtr<FJsonValue>>* ArgumentsField;
	if(!JsonObject->TryGetArrayField(TEXT("Arguments"), ArgumentsField))
	{
		HandleError(TEXT("Job JSON missing required field: \"arguments\""));
		return;
	}

	for (const TSharedPtr<FJsonValue>& Arg : *ArgumentsField)
	{
		AppendJob(Arg);
	}
}

void FBatchProcessCoordinator::AppendJob(TSharedPtr<FJsonValue> Argument)
{
	FBatchJobResult& Result = JobResults.AddDefaulted_GetRef();

	// Serialize the argument to a compact string for use in diagnostic output
	// when the job produces no log lines of its own.
	const TSharedPtr<FJsonObject>* ObjPtr = nullptr;
	if (Argument.IsValid() && Argument->TryGetObject(ObjPtr))
	{
		TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> Writer =
			TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&Result.ArgumentSummary);
		FJsonSerializer::Serialize((*ObjPtr).ToSharedRef(), Writer);
	}

	JobArguments.Add(MoveTemp(Argument));
	JobRetryCount.Add(0);
}

void FBatchProcessCoordinator::Seal()
{
	bIsSealed = true;
}

void FBatchProcessCoordinator::Abort()
{
	HandleCleanup();
}

bool FBatchProcessCoordinator::Tick()
{
	AcceptConnections();
	ReceiveHandshakes();
	LaunchNewWorkers();
	UpdateActiveWorkers();
	CheckCompletion();

	return !bIsCompleted;
}

bool FBatchProcessCoordinator::HasAnyErrors() const
{
	if(!ErrorRecord.IsEmpty())
	{
		return true;
	}
	for(const FWorkerStateData& Worker : ActiveWorkers)
	{
		if(IsErrorState(Worker.WorkerState))
		{
			return true;
		}
	}
	return false;
}

FString FBatchProcessCoordinator::GetErrorString() const
{
	return ErrorRecord;
}

int32 FBatchProcessCoordinator::GetNumJobs() const
{
	return JobResults.Num();
}

const FBatchJobResult& FBatchProcessCoordinator::GetJobResult(int32 Idx) const
{
	return JobResults[Idx];
}

const FString& FBatchProcessCoordinator::GetFunctionAsString() const
{
	return FunctionAsString;
}

void FBatchProcessCoordinator::HandleCleanup()
{
	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get();

	if(CoordinatorSocket)
	{
		SocketSubsystem->DestroySocket(CoordinatorSocket);
		CoordinatorSocket = nullptr;
	}

	for (FSocket* Socket : PendingWorkerSockets)
	{
		SocketSubsystem->DestroySocket(Socket);
	}
	PendingWorkerSockets.Reset();

	for (FWorkerStateData& Worker : ActiveWorkers)
	{
		if (Worker.Socket)
		{
			SocketSubsystem->DestroySocket(Worker.Socket);
			Worker.Socket = nullptr;
		}
	}
}

void FBatchProcessCoordinator::HandleError(const FString& Error)
{
	if(!ErrorRecord.IsEmpty())
	{
		ErrorRecord += TEXT("\n");
	}
	ErrorRecord += Error;
}

int32 FBatchProcessCoordinator::GetNumWorkers() const
{
	return NumWorkers;
}

int32 FBatchProcessCoordinator::GetNumLaunchingWorkers() const
{
	int32 Count = 0;
	for (const FWorkerStateData& Worker : ActiveWorkers)
	{
		if (Worker.WorkerState == EWorkerState::Starting)
		{
			++Count;
		}
	}
	return Count;
}

int32 FBatchProcessCoordinator::GetNumActiveWorkers() const
{
	int32 Count = 0;
	for (const FWorkerStateData& Worker : ActiveWorkers)
	{
		if (Worker.WorkerState == EWorkerState::Starting
		 || Worker.WorkerState == EWorkerState::WaitingForWork
		 || Worker.WorkerState == EWorkerState::Running)
		{
			++Count;
		}
	}
	return Count;
}

void FBatchProcessCoordinator::RegisterPendingWorker()
{
	// An invalid FProcHandle marks the slot as a "dummy" — FindCrashedWorkers skips it.
	ActiveWorkers.Add(
		{
			EWorkerState::Starting,
			FProcHandle(),
			FPlatformTime::Seconds(),
			TEXT("PrelaunchedWorker")
		}
	);
}

void FBatchProcessCoordinator::AcceptConnections()
{
	bool bHasPendingConnection = false;
	if (!CoordinatorSocket->HasPendingConnection(bHasPendingConnection) || !bHasPendingConnection)
	{
		return;
	}

	FSocket* ClientSocket = CoordinatorSocket->Accept(UE::Private::BatchCoordinatorName);
	if(!ClientSocket)
	{
		return;
	}

	PendingWorkerSockets.Add(ClientSocket);
}

void FBatchProcessCoordinator::ReceiveHandshakes()
{
	// Workers send their index as a 4-byte handshake. Don't block the update loop:
	// leave a socket in PendingWorkerSockets until its 4 bytes have arrived.
	for(int32 Idx = PendingWorkerSockets.Num() - 1; Idx >= 0; --Idx)
	{
		FSocket* Socket = PendingWorkerSockets[Idx];

		uint32 PendingBytes = 0;
		if (!Socket->HasPendingData(PendingBytes))
		{
			// No data in the receive buffer. A remotely-closed connection becomes
			// readable (select fires) but yields 0 bytes — detect and discard such
			// sockets so dead pending entries don't accumulate.
			if (Socket->Wait(ESocketWaitConditions::WaitForRead, FTimespan::Zero()))
			{
				HandleError(TEXT("Pending worker closed connection before sending handshake"));
				ISocketSubsystem::Get()->DestroySocket(Socket);
				PendingWorkerSockets.RemoveAt(Idx);
			}
			continue;
		}
		if (PendingBytes < sizeof(uint32))
		{
			continue;
		}

		uint32 WorkerId = 0;
		if (!UE::Private::BatchProcessSocketHelpers::RecvUint32(*Socket, WorkerId))
		{
			HandleError(TEXT("Worker connected but failed to send handshake ID"));
			ISocketSubsystem::Get()->DestroySocket(Socket);
			PendingWorkerSockets.RemoveAt(Idx);
			continue;
		}

		if (!ActiveWorkers.IsValidIndex((int32)WorkerId))
		{
			HandleError(FString::Printf(TEXT("Worker sent out-of-range ID %u (have %d slots)"),
				WorkerId, ActiveWorkers.Num()));
			ISocketSubsystem::Get()->DestroySocket(Socket);
			PendingWorkerSockets.RemoveAt(Idx);
			continue;
		}

		FWorkerStateData& Worker = ActiveWorkers[WorkerId];
		if (Worker.Socket != nullptr)
		{
			HandleError(FString::Printf(TEXT("Worker ID %u connected twice"), WorkerId));
			ISocketSubsystem::Get()->DestroySocket(Socket);
			PendingWorkerSockets.RemoveAt(Idx);
			continue;
		}

		Worker.Socket = Socket;
		const double BootElapsed = FPlatformTime::Seconds() - Worker.LastStateChangeTime;
		UE_LOGF(LogBatchProcess, Display, "Worker %d ready  (boot: %.1fs)", WorkerId, BootElapsed);
		Worker.WorkerState = EWorkerState::WaitingForWork;
		PendingWorkerSockets.RemoveAt(Idx);
	}
}

void FBatchProcessCoordinator::LaunchNewWorkers()
{
	if(CompletionCounter == JobResults.Num())
	{
		return;
	}

	// Never launch more workers than there are remaining jobs.
	// Count only live workers (Starting/WaitingForWork/Running) — ActiveWorkers may also
	// contain terminal entries from previous restarts (FailedToShutdown etc.) that no longer
	// consume a worker slot.
	const int32 RemainingJobs = JobResults.Num() - CompletionCounter;
	if (RemainingJobs <= 0)
	{
		return;
	}
	int32 NumLiveWorkers = 0;
	for (const FWorkerStateData& W : ActiveWorkers)
	{
		if (W.WorkerState == EWorkerState::Starting
		 || W.WorkerState == EWorkerState::WaitingForWork
		 || W.WorkerState == EWorkerState::Running)
		{
			++NumLiveWorkers;
		}
	}
	const int32 NumToLaunch = FMath::Min(GetNumWorkers(), RemainingJobs) - NumLiveWorkers;
	if(NumToLaunch <= 0)
	{
		return;
	}

	const FString LaunchExecutableName = FPlatformProcess::ExecutablePath();
	FString ProjectPath = FPaths::GetProjectFilePath();

	for(int32 I = 0; I < NumToLaunch; ++I)
	{
		// The worker identifies itself during handshake by its index in ActiveWorkers,
		// so capture the index before appending the new entry.
		const int32 NewWorkerIndex = ActiveWorkers.Num();
		FString ScratchLaunchString = FString::Printf(
			TEXT("%s -run=BatchWorkerCommandlet %s:%d %d -coordinatorpid=%u -unattended -nopause -nosplash -NoExceptionHandler -nullrhi -nosound -notexturestreaming"),
			*ProjectPath, UE::Private::CoordinatorIPStr, UE::Private::BatchProcessSocketHelpers::DefaultPortNumber, NewWorkerIndex,
			FPlatformProcess::GetCurrentProcessId());
		ActiveWorkers.Add(
			{
				EWorkerState::Starting,
				FPlatformProcess::CreateProc(
					*LaunchExecutableName,
					*ScratchLaunchString,
					false,
					true,
					true,
					nullptr,
					0,
					nullptr,
					nullptr),
				FPlatformTime::Seconds(),
				MoveTemp(ScratchLaunchString)
			}
		);
	}
}

void FBatchProcessCoordinator::UpdateActiveWorkers()
{
	FindCrashedWorkers();
	FindStuckWorkers();
	ProcessResultsFromWorkers(); // transitions finished workers to WaitingForWork
	GiveWorkToWorkers();         // immediately picks up any newly-idle workers
}

void FBatchProcessCoordinator::CheckCompletion()
{
	// Do not complete until Seal() has been called — the job list may still grow.
	if (!bIsSealed)
	{
		return;
	}

	if(CompletionCounter == JobResults.Num())
	{
		// All jobs are done.  Close every still-connected worker socket so that
		// PerformWork() on the worker side gets an EOF and returns cleanly.
		for(FWorkerStateData& Worker : ActiveWorkers)
		{
			if(Worker.Socket &&
				(Worker.WorkerState == EWorkerState::Running ||
				 Worker.WorkerState == EWorkerState::WaitingForWork))
			{
				Worker.Socket->Close();
				Worker.ChangeState(EWorkerState::ShuttingDown);
			}
		}
		bIsCompleted = true;
		return;
	}

	const auto IsTerminalState = [](EWorkerState WorkerState)
	{
		return WorkerState == EWorkerState::FailedToStart ||
			WorkerState == EWorkerState::FailedToShutdown ||
			WorkerState == EWorkerState::ShuttingDown;
	};

	bool bAllWorkersAreDone = true;
	for(FWorkerStateData& Worker : ActiveWorkers)
	{
		if(!IsTerminalState(Worker.WorkerState))
		{
			bAllWorkersAreDone = false;
		}
	}

	if(bAllWorkersAreDone && ActiveWorkers.Num() > 0)
	{
		FailAllRemainingJobs();
		bIsCompleted = true;
	}
	else
	{
		check(!bIsCompleted);
	}
}

void FBatchProcessCoordinator::FindCrashedWorkers()
{
	for(FWorkerStateData& Worker : ActiveWorkers)
	{
		if( FPlatformProcess::IsProcRunning(Worker.Handle) ||
			!Worker.Handle.IsValid()) // dummy worker
		{
			continue;
		}

		if(Worker.WorkerState == EWorkerState::Starting)
		{
			HandleError(TEXT("Batch worker failed to start"));
			Worker.ChangeState(EWorkerState::FailedToStart);
		}
		else
		{
			if(Worker.Socket)
			{
				Worker.Socket->Close();
			}

			HandleError(FString::Printf(TEXT("Worker crashed: %s"), *Worker.LaunchString));

			// Re-enqueue the in-flight job so another worker can pick it up.
			if (Worker.CurrentJobIndex >= 0)
			{
				const int32 JobIdx = Worker.CurrentJobIndex;
				Worker.CurrentJobIndex = -1;
				if (JobRetryCount[JobIdx]++ < UE::Private::MaxJobRetries)
				{
					RetryQueue.Add(JobIdx);
				}
				else
				{
					JobResults[JobIdx].State           = EBatchJobState::Completed;
					JobResults[JobIdx].bEncounteredError = true;
					JobResults[JobIdx].Output          = TEXT("Worker crashed repeatedly on this job");
					++CompletionCounter;
				}
			}

			Worker.ChangeState(EWorkerState::FailedToShutdown);
		}
	}
}

void FBatchProcessCoordinator::FindStuckWorkers()
{
	const double CurrentTime = FPlatformTime::Seconds();
	for(FWorkerStateData& Worker : ActiveWorkers)
	{
		if(Worker.WorkerState == EWorkerState::ShuttingDown)
		{
			if(Worker.LastStateChangeTime + UE::Private::TimeoutLengthSeconds < CurrentTime)
			{
				HandleError(TEXT("Worker failed to shut down in time"));
				Worker.ChangeState(EWorkerState::FailedToShutdown);
			}
		}
		else if (Worker.WorkerState == EWorkerState::Running
			&& Worker.LastResultTime + UE::Private::JobStuckTimeoutSeconds < CurrentTime)
		{
			HandleError(FString::Printf(TEXT("Worker silent for %.0fs, terminating: %s"),
				UE::Private::JobStuckTimeoutSeconds, *Worker.LaunchString));

			// Re-enqueue the in-flight job so another worker can pick it up.
			if (Worker.CurrentJobIndex >= 0)
			{
				const int32 JobIdx = Worker.CurrentJobIndex;
				Worker.CurrentJobIndex = -1;
				if (JobRetryCount[JobIdx]++ < UE::Private::MaxJobRetries)
				{
					RetryQueue.Add(JobIdx);
				}
				else
				{
					JobResults[JobIdx].State             = EBatchJobState::Completed;
					JobResults[JobIdx].bEncounteredError = true;
					JobResults[JobIdx].Output            = TEXT("Worker timed out repeatedly on this job");
					++CompletionCounter;
				}
			}

			FPlatformProcess::TerminateProc(Worker.Handle, true);
			if (Worker.Socket)
			{
				Worker.Socket->Close();
			}
			Worker.ChangeState(EWorkerState::FailedToShutdown);
		}
	}
}

void FBatchProcessCoordinator::GiveWorkToWorkers()
{
	for (FWorkerStateData& Worker : ActiveWorkers)
	{
		if (Worker.WorkerState != EWorkerState::WaitingForWork)
		{
			continue;
		}

		// Pick the next job: drain the retry queue first (crashed/timed-out jobs),
		// then advance the sequential dispatch pointer.
		int32 JobIdx = -1;
		if (RetryQueue.Num() > 0)
		{
			JobIdx = RetryQueue.Last();
			RetryQueue.RemoveAt(RetryQueue.Num() - 1, 1, EAllowShrinking::No);
		}
		else if (NextJobToDispatch < JobArguments.Num())
		{
			JobIdx = NextJobToDispatch++;
		}
		else
		{
			break; // no more jobs to dispatch right now
		}

		// Build a single-element batch packet.
		TArray<TSharedPtr<FJsonValue>> Arguments;
		Arguments.Add(JobArguments[JobIdx]);

		TSharedRef<FJsonObject> Object = ::MakeShared<FJsonObject>();
		Object->SetArrayField(TEXT("Arguments"), Arguments);
		Object->SetStringField(TEXT("Function"),   FunctionAsString);
		Object->SetNumberField(TEXT("StartIndex"), JobIdx);

		if (!UE::Private::BatchProcessSocketHelpers::SendJsonValue(*Worker.Socket, Object))
		{
			HandleError(FString::Printf(TEXT("Failed to send work to worker: %s"), *Worker.LaunchString));
			RetryQueue.Add(JobIdx); // put the job back so another worker can pick it up
			Worker.ChangeState(EWorkerState::FailedToShutdown);
			continue;
		}

		Worker.CurrentJobIndex = JobIdx;
		Worker.LastResultTime  = FPlatformTime::Seconds();
		Worker.ChangeState(EWorkerState::Running);
		JobResults[JobIdx].State = EBatchJobState::InProgress;
	}
}

void FBatchProcessCoordinator::ProcessResultsFromWorkers()
{
	// Phase 1 — drain bytes.
	// Read all currently-available socket bytes into each worker's per-worker buffer.
	// We do this across all workers before parsing so that no single worker's large
	// result payload can starve the others: every worker's send window is kept open
	// regardless of how quickly individual messages are parsed.
	for (FWorkerStateData& Worker : ActiveWorkers)
	{
		if (Worker.WorkerState != EWorkerState::Running)
		{
			continue;
		}

		uint32 Available = 0;
		while (Worker.Socket->HasPendingData(Available) && Available > 0)
		{
			const int32 Offset = Worker.RecvBuffer.AddUninitialized((int32)Available);
			int32 BytesRead = 0;
			if (!Worker.Socket->Recv(Worker.RecvBuffer.GetData() + Offset, (int32)Available, BytesRead)
				|| BytesRead == 0)
			{
				Worker.RecvBuffer.SetNum(Offset, EAllowShrinking::No); // undo the uninitialised reservation
				HandleError(FString::Printf(TEXT("Lost connection to worker mid-batch: %s"), *Worker.LaunchString));
				Worker.ChangeState(EWorkerState::FailedToShutdown);
				break;
			}
			Worker.RecvBuffer.SetNum(Offset + BytesRead, EAllowShrinking::No);
		}
	}

	// Phase 2 — parse messages.
	// Walk each worker's buffer and process every complete length-prefixed JSON frame.
	// Incomplete frames (body not yet fully received) are left in the buffer for the
	// next tick — the coordinator never blocks waiting for more socket data.
	for (FWorkerStateData& Worker : ActiveWorkers)
	{
		if (Worker.WorkerState != EWorkerState::Running)
		{
			continue;
		}

		while (true)
		{
			const int32 Remaining = Worker.RecvBuffer.Num() - Worker.RecvBufferReadOffset;

			// Need at least the 4-byte big-endian length header.
			if (Remaining < (int32)sizeof(uint32))
			{
				break;
			}

			const uint8* const Buf = Worker.RecvBuffer.GetData() + Worker.RecvBufferReadOffset;
			const uint32 MsgLen = (uint32(Buf[0]) << 24)
			                    | (uint32(Buf[1]) << 16)
			                    | (uint32(Buf[2]) <<  8)
			                    |  uint32(Buf[3]);

			// Guard against a corrupt length header before any arithmetic.
			if (MsgLen > (uint32)(TNumericLimits<int32>::Max() - (int32)sizeof(uint32)))
			{
				HandleError(TEXT("Worker sent oversized message length (corrupt header)"));
				Worker.ChangeState(EWorkerState::FailedToShutdown);
				break;
			}

			// Wait until the full body has arrived.
			if (Remaining < (int32)sizeof(uint32) + (int32)MsgLen)
			{
				break;
			}

			// Parse the JSON body directly from the buffer — no copy needed.
			const TStringView<char> JsonStr{
				reinterpret_cast<const char*>(Buf + sizeof(uint32)),
				(int32)MsgLen
			};
			TSharedRef<TJsonStringViewReader<char>> Reader =
				TJsonStringViewReader<char>::Create(JsonStr);
			TSharedPtr<FJsonValue> Result;
			if (!FJsonSerializer::Deserialize(*Reader, Result))
			{
				HandleError(TEXT("Worker sent malformed result JSON"));
				Worker.ChangeState(EWorkerState::FailedToShutdown);
				break;
			}

			// Advance past this frame before any early-out so the buffer stays consistent.
			Worker.RecvBufferReadOffset += (int32)(sizeof(uint32) + MsgLen);

			const TSharedPtr<FJsonObject>* JsonObjectPtr;
			if (!Result || !Result->TryGetObject(JsonObjectPtr))
			{
				HandleError(TEXT("Worker sent malformed result JSON"));
				Worker.ChangeState(EWorkerState::FailedToShutdown);
				break;
			}
			const FJsonObject& JsonObject = *(JsonObjectPtr->Get());

			bool bShutdown = false;
			JsonObject.TryGetBoolField(TEXT("Shutdown"), bShutdown);
			if (bShutdown)
			{
				Worker.ChangeState(EWorkerState::ShuttingDown);
				break;
			}

			// Normal per-job result.
			double JobIndex = 0.0;
			JsonObject.TryGetNumberField(TEXT("JobIndex"), JobIndex);
			FBatchJobResult& JobResult = JobResults[(int32)JobIndex];
			JobResult.State = EBatchJobState::Completed;
			JsonObject.TryGetStringField(TEXT("Output"), JobResult.Output);
			bool bError = false;
			JsonObject.TryGetBoolField(TEXT("Error"), bError);
			JobResult.bEncounteredError = bError;

			Worker.CurrentJobIndex = -1;
			Worker.LastResultTime  = FPlatformTime::Seconds();
			++CompletionCounter;

			// Recycle the worker: put it back to WaitingForWork so GiveWorkToWorkers
			// can dispatch the next job immediately (same tick).
			Worker.ChangeState(EWorkerState::WaitingForWork);
			// Do NOT break — there may be more complete frames already buffered.
		}

		// Compact the buffer: discard already-processed bytes with a single memmove
		// so the buffer doesn't grow unboundedly across ticks.
		if (Worker.RecvBufferReadOffset > 0)
		{
			const int32 Unprocessed = Worker.RecvBuffer.Num() - Worker.RecvBufferReadOffset;
			if (Unprocessed > 0)
			{
				FMemory::Memmove(
					Worker.RecvBuffer.GetData(),
					Worker.RecvBuffer.GetData() + Worker.RecvBufferReadOffset,
					Unprocessed);
			}
			Worker.RecvBuffer.SetNum(Unprocessed, EAllowShrinking::No);
			Worker.RecvBufferReadOffset = 0;
		}
	}
}

void FBatchProcessCoordinator::FailAllRemainingJobs()
{
	auto Fail = [&](int32 JobIdx)
	{
		if (JobResults[JobIdx].State != EBatchJobState::Completed)
		{
			JobResults[JobIdx].State             = EBatchJobState::Completed;
			JobResults[JobIdx].bEncounteredError = true;
			JobResults[JobIdx].Output            = TEXT("All workers failed before this job could run");
			++CompletionCounter;
		}
	};

	for (int32 Idx : RetryQueue)
	{
		Fail(Idx);
	}
	RetryQueue.Reset();

	for (int32 Idx = NextJobToDispatch; Idx < JobResults.Num(); ++Idx)
	{
		Fail(Idx);
	}
}

bool FBatchProcessCoordinator::Recv(FSocket* FromSocket, TArray<uint8>& Bytes)
{
	uint32 BufferSize = 0;
	if (FromSocket->HasPendingData(BufferSize))
	{
		Bytes.SetNumUninitialized(BufferSize);

		int32 BytesRead = 0;
		if (!FromSocket->Recv(Bytes.GetData(), Bytes.Num(), BytesRead))
		{
			HandleError(FString(TEXT("Batch worker read failed: ") + GetSocketReadableErrorCode()));
			return false;
		}
		return true;
	}

	return false;
}

FString FBatchProcessCoordinator::GetSocketReadableErrorCode()
{
	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
	const ESocketErrors LastError = SocketSubsystem->GetLastErrorCode();
	return FString::Printf(TEXT("%s (%d)"), SocketSubsystem->GetSocketError(LastError), static_cast<int32>(LastError));
}

bool FBatchProcessCoordinator::IsErrorState(EWorkerState WorkerState)
{
	return WorkerState == EWorkerState::FailedToStart ||
		WorkerState == EWorkerState::FailedToShutdown;
}

void FBatchProcessCoordinator::FWorkerStateData::ChangeState(EWorkerState NewState)
{
	WorkerState = NewState;
	LastStateChangeTime = FPlatformTime::Seconds();
}
