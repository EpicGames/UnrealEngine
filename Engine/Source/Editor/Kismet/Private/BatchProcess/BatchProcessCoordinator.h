// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Dom/JsonValue.h"
#include "HAL/PlatformProcess.h"

class FSocket;

enum class EBatchJobState
{
	Pending,
	Scheduled,
	InProgress,
	Completed
};

struct FBatchJobResult
{
	EBatchJobState State        = EBatchJobState::Pending;
	bool bEncounteredError      = false;
	FString Output;          // log output captured by the worker during this job's function call
	FString ArgumentSummary; // compact JSON of the original argument, for diagnostic display
};

struct FBatchProcessCoordinator
{
	FBatchProcessCoordinator(const FString* ClientAddress, int32 InNumWorkers = 1);
	~FBatchProcessCoordinator();

	/**
	 * Parse a job description from a file or JSON string.
	 * The JSON must contain a "function" (UFunction path) and "arguments" (array of arg objects).
	 * Call Seal() once no more jobs will be added.
	 */
	void SubmitFile(const TCHAR* Filename);
	void Submit(const FUtf8String& JsonCommand);

	/**
	 * Append a single argument object to the pending job queue.
	 * May be called at any point before Seal(), including while Tick() is running.
	 */
	void AppendJob(TSharedPtr<FJsonValue> Argument);

	/**
	 * Signal that no more jobs will be appended.
	 * The coordinator will complete once all queued jobs finish.
	 */
	void Seal();

	/** Cancel all in-flight work and shut down workers. */
	void Abort();

	/**
	 * Registers a pre-existing worker slot without launching a process.
	 * The slot accepts an incoming connection via the normal handshake.
	 * Useful for in-process testing and the "dummy coordinator" debug mode.
	 */
	void RegisterPendingWorker();

	/** @return true if still in progress, false when complete */
	bool Tick();

	bool HasAnyErrors() const;
	FString GetErrorString() const;
	int32 GetNumJobs() const;
	int32 GetNumWorkers() const;
	int32 GetNumLaunchingWorkers() const;
	/** Returns the number of workers that are Starting, WaitingForWork, or Running. */
	int32 GetNumActiveWorkers() const;
	const FBatchJobResult& GetJobResult(int32 Idx) const;
	const FString& GetFunctionAsString() const;

private:
	void HandleCleanup();
	void HandleError(const FString& Error);
	void AcceptConnections();
	void ReceiveHandshakes();
	void LaunchNewWorkers();
	void UpdateActiveWorkers();
	void CheckCompletion();

	void FindCrashedWorkers();
	void FindStuckWorkers();
	void GiveWorkToWorkers();
	void ProcessResultsFromWorkers();
	void FailAllRemainingJobs();

	bool Recv(FSocket* FromSocket, TArray<uint8>& Bytes);
	static FString GetSocketReadableErrorCode();

	FBatchProcessCoordinator(const FBatchProcessCoordinator&) = delete;
	FBatchProcessCoordinator& operator=(const FBatchProcessCoordinator&) = delete;
	FBatchProcessCoordinator(FBatchProcessCoordinator&&) = delete;
	FBatchProcessCoordinator& operator=(FBatchProcessCoordinator&&) = delete;

	TArray<FBatchJobResult>          JobResults;
	TArray<TSharedPtr<FJsonValue>>   JobArguments; // one entry per job
	FString FunctionAsString;

	enum class EWorkerState
	{
		Starting,
		WaitingForWork,
		Running,
		ShuttingDown,
		FailedToStart,
		FailedToShutdown,
	};
	static bool IsErrorState(EWorkerState WorkerState);
	struct FWorkerStateData
	{
		EWorkerState WorkerState      = EWorkerState::Starting;
		FProcHandle  Handle;
		double       LastStateChangeTime = 0.0;
		FString      LaunchString;
		FSocket*     Socket           = nullptr;

		// The global job index this worker is currently executing; -1 when idle.
		// Used for crash/timeout recovery: if the worker dies, this job is re-enqueued.
		int32  CurrentJobIndex    = -1;

		// Timestamp of the last result received from this worker.
		// Used by FindStuckWorkers to detect Running workers that have gone silent.
		double LastResultTime     = 0.0;

		// Per-worker byte buffer for incoming socket data.
		// The coordinator drains bytes non-blockingly into this buffer and
		// parses complete framed messages from it, preventing a slow/large
		// result from one worker from blocking reads of all other workers.
		TArray<uint8> RecvBuffer;
		int32 RecvBufferReadOffset = 0;

		void ChangeState(EWorkerState NewState);
	};

	TArray<FSocket*>        PendingWorkerSockets;
	TArray<FWorkerStateData> ActiveWorkers;
	FString ErrorRecord;

	int32          NumWorkers         = 1;
	int32          NextJobToDispatch  = 0;  // next sequential job index to send
	int32          CompletionCounter  = 0;
	TArray<int32>  RetryQueue;              // jobs to re-attempt after worker crash/timeout
	TArray<int8>   JobRetryCount;           // per-job retry count (parallel to JobResults)

	// socket info:
	TSharedPtr<class FInternetAddr> ListenAddr;
	FSocket* CoordinatorSocket;

	// flags:
	bool bIsCompleted;
	bool bIsSealed;  // true once no more jobs will be appended; enables completion detection
};
