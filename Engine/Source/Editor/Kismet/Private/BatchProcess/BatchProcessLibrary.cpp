// Copyright Epic Games, Inc. All Rights Reserved.

#include "BatchProcess/BatchProcessLibrary.h"

#include "BatchProcess/BatchProcessRunner.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonWriter.h"

FString UBatchProcessLibrary::RunBatch(const FString& JobJson, int32 NumWorkers)
{
	UE::Private::FBatchRunOptions Options;
	Options.NumWorkers = NumWorkers;

	const UE::Private::FBatchRunResult RunResult = UE::Private::RunBatchFromJson(JobJson, Options);

	// Serialise results to a JSON array for the Python caller.
	FString Out;
	TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> Writer =
		TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&Out);

	Writer->WriteArrayStart();
	for (int32 Idx = 0; Idx < RunResult.Jobs.Num(); ++Idx)
	{
		const FBatchJobResult& Job = RunResult.Jobs[Idx];
		Writer->WriteObjectStart();
		Writer->WriteValue(TEXT("Passed"),           !Job.bEncounteredError);
		Writer->WriteValue(TEXT("Output"),            Job.Output);
		Writer->WriteValue(TEXT("ArgumentSummary"),  Job.ArgumentSummary);
		Writer->WriteObjectEnd();
	}
	Writer->WriteArrayEnd();
	Writer->Close();

	return Out;
}
