// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "BatchProcessLibrary.generated.h"

/**
 * Blueprint/Python-callable wrapper around FBatchProcessCoordinator.
 *
 * RunBatch() lets you drive the batch processor from a single Python script:
 * discover assets, build a job description, fan out to N workers, and get
 * structured results back — all without leaving the current editor process.
 *
 * Example — stringify every Blueprint in the project with JsonObjectGraph
 * note that for snapshot_asset to be reachable you must register it in
 * init_unreal.py thusly: import snapshot_assets
 
@code{.py}
# Save in /Game/Content/Python/snapshot_assets.py
import json
import os
import unreal

# Declare a uclass with a function that we want to run, this simple function just
# takes an asset path, loads it, stringifies it, and writes the string to a file:
@unreal.uclass()
class SnapshotAsset(unreal.Object):
	@unreal.ufunction(params=[str])
	def snapshot_asset(self, asset_path: str) -> None:
		try:
			asset = unreal.load_asset(asset_path)
		except Exception as exc:
			unreal.log_error(f"FAIL: {asset_path}  (load exception: {exc})")
			return

		if asset is None:
			unreal.log_error(f"FAIL: {asset_path}  (asset not found)")
			return

		json_str = unreal.JsonObjectGraphFunctionLibrary.stringify(
			[asset.get_package()], unreal.JsonStringifyOptions(unreal.JsonStringifyFlags.FILTER_EDITOR_ONLY_DATA))

		# Build a nested output path from the package path, stripping the leading '/'.
		# e.g. "/Game/Baz/Foo" -> "Game/Baz/Foo.json" and write to that file:
		relative = asset_path.lstrip("/")
		out_path = os.path.join(
			unreal.Paths.project_saved_dir(), "SnapshotAsset", f"{relative}.json"
		)
		os.makedirs(os.path.dirname(out_path), exist_ok=True)

		with open(out_path, "w", encoding="utf-8") as f:
			f.write(json_str)

		unreal.log(f"PASS: {asset_path}  -> {out_path}")

    
# Discover all assets of the specified type, run snapshot_asset
def _run_snapshot(asset_class: str)-> None:
	registry = unreal.AssetRegistryHelpers.get_asset_registry()
	registry.wait_for_completion()
	asset_filter = unreal.ARFilter(
		class_names=[asset_class], recursive_paths=True, recursive_classes=True)
	asset_paths = [str(a.package_name)
					for a in registry.get_assets(asset_filter)]
    
	# Build the job description.
	job = {
		"Function": "/Game/Python/snapshot_assets_PY.SnapshotAsset:snapshot_asset",
		"Arguments": [{"asset_path": p} for p in asset_paths],
	}
    
	# Run the batch — blocks until all jobs complete.
	results_json = unreal.BatchProcessLibrary.run_batch(json.dumps(job), num_workers=8)
	results = json.loads(results_json)
    
	passed = [r for r in results if r["Passed"]]
	failed = [r for r in results if not r["Passed"]]
	print(f"{len(passed)}/{len(results)} passed")
	for r in failed:
		print(f"  FAIL: {r['ArgumentSummary']}\n    {r['Output']}")

def run_blueprints_snapshot() -> None:
    _run_snapshot("Blueprint")
	
if __name__ == '__main__': # critical, prevents forkbomb if init_unreal is used to import this script
    run_blueprints_snapshot()
@endcode
 */
UCLASS()
class UBatchProcessLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Run a batch of jobs across multiple worker processes and return structured results.
	 *
	 * @param JobJson    Full job description as a JSON string:
	 *                   {"Function": "<UFunction path>", "Arguments": [{...}, ...]}
	 * @param NumWorkers Number of worker processes to spawn.
	 *                   Pass 0 (default) to auto-select based on core count.
	 * @return           JSON array string — one object per job, in submission order:
	 *                   [{"Passed": bool, "Output": "...", "ArgumentSummary": "..."}, ...]
	 */
	UFUNCTION(BlueprintCallable, Category = "BatchProcess")
	static FString RunBatch(const FString& JobJson, int32 NumWorkers = 0);
};
