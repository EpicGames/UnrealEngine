// Copyright Epic Games, Inc. All Rights Reserved.

#include "BatchProcess/BatchProcessRunner.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "BatchProcess/BatchProcessLog.h"
#if PLATFORM_WINDOWS
#include "Windows/WindowsHWrapper.h"
#else
#include <unistd.h> // isatty, STDOUT_FILENO
#endif

namespace UE::Private
{

#if PLATFORM_WINDOWS
#ifndef ENABLE_VIRTUAL_TERMINAL_PROCESSING
#define ENABLE_VIRTUAL_TERMINAL_PROCESSING 0x0004
#endif

/**
 * Returns a writable console HANDLE with VT processing enabled, or INVALID_HANDLE_VALUE.
 *
 * UnrealEditor.exe is a GUI-subsystem process. When launched from a terminal the
 * inherited STD_OUTPUT_HANDLE is sometimes a pipe or has been replaced by UE's own
 * log infrastructure, so GetConsoleMode on it fails.  Falling back to CONOUT$ opens
 * the active console screen buffer directly and works as long as the process is
 * attached to an interactive terminal (cmd.exe, PowerShell, Windows Terminal, etc.).
 */
static HANDLE AcquireConsoleHandle()
{
	HANDLE hOut = ::GetStdHandle(STD_OUTPUT_HANDLE);
	DWORD Mode = 0;
	if (hOut != INVALID_HANDLE_VALUE && hOut != nullptr && ::GetConsoleMode(hOut, &Mode))
	{
		::SetConsoleMode(hOut, Mode | ENABLE_VIRTUAL_TERMINAL_PROCESSING);
		return hOut;
	}

	hOut = ::CreateFileW(L"CONOUT$", GENERIC_READ | GENERIC_WRITE,
	                     FILE_SHARE_READ | FILE_SHARE_WRITE,
	                     nullptr, OPEN_EXISTING, 0, nullptr);
	if (hOut == INVALID_HANDLE_VALUE) { return INVALID_HANDLE_VALUE; }

	if (::GetConsoleMode(hOut, &Mode))
	{
		::SetConsoleMode(hOut, Mode | ENABLE_VIRTUAL_TERMINAL_PROCESSING);
		return hOut;
	}

	::CloseHandle(hOut);
	return INVALID_HANDLE_VALUE;
}

static HANDLE GetConsoleHandle()
{
	static HANDLE hConsole = AcquireConsoleHandle();
	return hConsole;
}
#endif // PLATFORM_WINDOWS

/** Returns true if attached to an interactive terminal. Result is cached. */
static bool IsInteractiveTerminal()
{
	static int8 CachedResult = -1;
	if (CachedResult >= 0) { return CachedResult != 0; }
#if PLATFORM_WINDOWS
	CachedResult = (GetConsoleHandle() != INVALID_HANDLE_VALUE) ? 1 : 0;
#else
	CachedResult = (isatty(STDOUT_FILENO) != 0) ? 1 : 0;
#endif
	return CachedResult != 0;
}

/**
 * Writes Text to the terminal.
 * On Windows uses WriteConsoleW so \r and VT sequences reach the console even
 * when UE has redirected the C-runtime stdout.
 */
static void WriteToTerminal(const TCHAR* Text)
{
#if PLATFORM_WINDOWS
	HANDLE hConsole = GetConsoleHandle();
	if (hConsole != INVALID_HANDLE_VALUE)
	{
		DWORD Written = 0;
		::WriteConsoleW(hConsole, Text, (DWORD)FCString::Strlen(Text), &Written, nullptr);
		return;
	}
#endif
	FTCHARToUTF8 Utf8(Text);
	fwrite(Utf8.Get(), 1, Utf8.Length(), stdout);
	fflush(stdout);
}

static void PrintProgress(const FBatchProcessCoordinator& Coordinator, float JobsPerSec)
{
	const int32 Total = Coordinator.GetNumJobs();
	if (Total == 0) { return; }

	int32 Completed = 0;
	int32 Failed    = 0;
	for (int32 Idx = 0; Idx < Total; ++Idx)
	{
		const FBatchJobResult& Result = Coordinator.GetJobResult(Idx);
		if (Result.State == EBatchJobState::Completed)
		{
			++Completed;
			if (Result.bEncounteredError) { ++Failed; }
		}
	}
	const int32 Succeeded     = Completed - Failed;
	const int32 Pct           = (Completed * 100) / Total;
	const int32 ActiveWorkers = Coordinator.GetNumActiveWorkers();
	const int32 MaxWorkers    = Coordinator.GetNumWorkers();

	constexpr int32 BarWidth = 30;
	const int32 Filled = (Completed * BarWidth) / Total;
	FString Bar;
	Bar.Reserve(BarWidth);
	for (int32 i = 0;      i < Filled;   ++i) { Bar += TEXT('#'); }
	for (int32 i = Filled; i < BarWidth; ++i) { Bar += TEXT(' '); }

	if (IsInteractiveTerminal())
	{
		// \r[bar]\033[K redraws the current line.
		// The trailing \r leaves the cursor at column 0 so that any UE_LOG output
		// that fires before the next update (e.g. "Worker N ready") starts at the
		// left margin and overwrites the bar cleanly, rather than appending to it
		// mid-line.  Between log messages the bar updates in-place; when a log
		// fires, the bar scrolls one line.
		WriteToTerminal(*FString::Printf(
			TEXT("\r[%s] %4d/%-4d  %3d%%  |  ok: %d  fail: %d  w:%d/%d  %.1f/s\033[K\r"),
			*Bar, Completed, Total, Pct, Succeeded, Failed, ActiveWorkers, MaxWorkers, JobsPerSec));
		return;
	}
	UE_LOGF(LogBatchProcess, Display,
		"[%ls] %4d/%-4d  %3d%%  |  ok: %d  fail: %d  w:%d/%d  %.1f/s",
		*Bar, Completed, Total, Pct, Succeeded, Failed, ActiveWorkers, MaxWorkers, JobsPerSec);
}

FBatchRunResult RunBatchFromJson(const FString& JobJson, const FBatchRunOptions& Options)
{
	const int32 ResolvedWorkers = Options.NumWorkers > 0
		? Options.NumWorkers
		: FMath::Max(1, FPlatformMisc::NumberOfCores() / 4);

	const FString ProjectName    = FPaths::GetBaseFilename(FPaths::GetProjectFilePath());
	const FString ExecutableName = FPaths::GetCleanFilename(FPlatformProcess::ExecutablePath());

	// Persist the job JSON so the run is trivially reproducible.
	const FString TempDir  = FPaths::ProjectSavedDir() / TEXT("BatchProcess");
	const FString TempFile = TempDir / TEXT("last_run.json");
	IFileManager::Get().MakeDirectory(*TempDir, /*Tree=*/true);
	FFileHelper::SaveStringToFile(JobJson, *TempFile);

	UE_LOGF(LogBatchProcess, Display,
		"%ls %ls -run=BatchProcessCommandlet \"%ls\" -NumWorkers=%d",
		*ExecutableName, *ProjectName, *TempFile, ResolvedWorkers);

	// Set up and run the coordinator.
	const double SetupStart = FPlatformTime::Seconds();
	FBatchProcessCoordinator Coordinator(Options.ClientAddress, ResolvedWorkers);
	Coordinator.Submit(FUtf8String(TCHAR_TO_UTF8(*JobJson)));
	Coordinator.Seal();
	const double SetupElapsed = FPlatformTime::Seconds() - SetupStart;

	const int32 NumJobs = Coordinator.GetNumJobs();

	UE_LOGF(LogBatchProcess, Display,
		"Starting %d job(s) across %d worker(s)  (setup: %.2fs)",
		NumJobs, Coordinator.GetNumWorkers(), SetupElapsed);

	const double RunStart = FPlatformTime::Seconds();

	// Escape the function path once for use in per-failure reproduce commands.
	const FString EscapedFunctionPath = Coordinator.GetFunctionAsString()
		.Replace(TEXT("\\"), TEXT("\\\\"))
		.Replace(TEXT("\""), TEXT("\\\""));

	// Track which completed jobs have already been reported so we never double-print.
	TArray<bool> bReported;
	bReported.Init(false, NumJobs);
	int32 FailsPrinted = 0;
	int32 TotalFailed  = 0;

	// Scans all jobs for newly-completed failures and prints the first 10 immediately.
	auto ReportNewFailures = [&]()
	{
		for (int32 Idx = 0; Idx < NumJobs; ++Idx)
		{
			if (bReported[Idx]) { continue; }
			const FBatchJobResult& Job = Coordinator.GetJobResult(Idx);
			if (Job.State != EBatchJobState::Completed) { continue; }
			bReported[Idx] = true;
			if (!Job.bEncounteredError) { continue; }
			++TotalFailed;

			if (FailsPrinted < 10)
			{
				// Only print the first line of output — the full capture is in the
				// results file.  Dumping the entire multi-line string via a single
				// UE_LOG causes embedded newlines to leave the cursor mid-line.
				const FString FirstOutputLine = [&]() -> FString
				{
					if (Job.Output.IsEmpty())
					{
						return FString::Printf(TEXT("(no output) arg: %s"), *Job.ArgumentSummary);
					}
					const int32 NL = Job.Output.Find(TEXT("\n"));
					return NL != INDEX_NONE ? Job.Output.Left(NL) : Job.Output;
				}();
				UE_LOGF(LogBatchProcess, Error, "[FAIL job %4d]  %ls", Idx, *FirstOutputLine);

				const FString CommandJson = FString::Printf(
					TEXT("{\"Function\":\"%s\",\"Arguments\":[%s]}"),
					*EscapedFunctionPath, *Job.ArgumentSummary);
				const FString EscapedCommandJson = CommandJson.Replace(TEXT("\""), TEXT("\\\""));
				UE_LOGF(LogBatchProcess, Display,
					"Repro command:  %ls %ls -run=BatchWorkerCommandlet -singlecommand=%ls",
					*ExecutableName, *ProjectName, *EscapedCommandJson);
				++FailsPrinted;
			}
		}
	};

	double LastProgressTime    = RunStart;
	int32  LastCompletedForRate = 0;
	while (Coordinator.Tick())
	{
		FPlatformProcess::Sleep(0.016f);
		const double Now = FPlatformTime::Seconds();
		if (Now - LastProgressTime >= 0.5)
		{
			// Count completions and compute rate over the elapsed interval.
			int32 CurrentCompleted = 0;
			for (int32 Idx = 0; Idx < NumJobs; ++Idx)
			{
				if (Coordinator.GetJobResult(Idx).State == EBatchJobState::Completed)
				{
					++CurrentCompleted;
				}
			}
			const double Elapsed   = Now - LastProgressTime;
			const float  JobsPerSec = (float)((CurrentCompleted - LastCompletedForRate) / Elapsed);

			LastCompletedForRate = CurrentCompleted;
			LastProgressTime     = Now;

			ReportNewFailures();
			PrintProgress(Coordinator, JobsPerSec);
		}
	}

	// Final pass: report remaining failures, then advance past the bar line.
	ReportNewFailures();
	if (IsInteractiveTerminal()) { WriteToTerminal(TEXT("\n")); }

	if (TotalFailed > 10)
	{
		UE_LOGF(LogBatchProcess, Display, "  ... and %d more failures", TotalFailed - 10);
	}

	const double TotalElapsed = FPlatformTime::Seconds() - RunStart;
	const double JobsPerSec   = TotalElapsed > 0.0 ? NumJobs / TotalElapsed : 0.0;
	UE_LOGF(LogBatchProcess, Display,
		"Completed %d job(s) in %.1fs  (%.1f jobs/sec across %d worker(s))",
		NumJobs, TotalElapsed, JobsPerSec, Coordinator.GetNumWorkers());

	// Collect results.
	FBatchRunResult Result;
	Result.FunctionPath           = Coordinator.GetFunctionAsString();
	Result.TempFilePath           = TempFile;
	Result.bHasCoordinatorErrors  = Coordinator.HasAnyErrors();
	Result.CoordinatorErrorString = Coordinator.GetErrorString();

	Result.Jobs.Reserve(NumJobs);
	for (int32 Idx = 0; Idx < NumJobs; ++Idx)
	{
		Result.Jobs.Add(Coordinator.GetJobResult(Idx));
	}

	return Result;
}

} // namespace UE::Private
