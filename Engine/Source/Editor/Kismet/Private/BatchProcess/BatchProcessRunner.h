// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BatchProcess/BatchProcessCoordinator.h"

namespace UE::Private
{

struct FBatchRunOptions
{
	/** Number of workers to spawn. 0 = auto-select based on core count. */
	int32 NumWorkers = 0;

	/**
	 * When non-null, launches the coordinator in listen mode so a single
	 * external client can connect for debugging (-launchforclient).
	 */
	const FString* ClientAddress = nullptr;
};

struct FBatchRunResult
{
	TArray<FBatchJobResult> Jobs;        // one per submitted job, in submission order
	FString FunctionPath;                // the UFUNCTION path parsed from the job JSON
	FString TempFilePath;                // Saved/BatchProcess/last_run.json written by the runner
	bool    bHasCoordinatorErrors = false;
	FString CoordinatorErrorString;
};

/**
 * Core batch execution helper shared by BatchProcessCommandlet and BatchProcessLibrary.
 *
 * Given a raw JSON job description string this function:
 *   1. Persists the JSON to Saved/BatchProcess/last_run.json and prints a rerun command.
 *   2. Spins up an FBatchProcessCoordinator, submits all jobs, and runs the tick loop.
 *   3. Displays a live progress bar in interactive consoles.
 *   4. Prints the first 10 failures as errors, each with a -singlecommand reproduce line.
 *   5. Returns all results for the caller to handle (results file, exit code, JSON, etc.).
 */
FBatchRunResult RunBatchFromJson(const FString& JobJson, const FBatchRunOptions& Options = FBatchRunOptions());

} // namespace UE::Private
