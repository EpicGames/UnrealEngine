// Copyright Epic Games, Inc. All Rights Reserved.

#include "BatchProcess/BatchProcessSocketHelpers.h"

#include "Dom/JsonValue.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Sockets.h"

bool UE::Private::BatchProcessSocketHelpers::SendUint32(FSocket& Socket, uint32 Arg)
{
	uint8 Data[sizeof(uint32)];
	Data[0] = Arg >> 24;
	Data[1] = Arg >> 16;
	Data[2] = Arg >> 8;
	Data[3] = Arg >> 0;
	for (int32 Sent = 0; Sent != sizeof(uint32);)
	{
		int32 BytesSent;
		if (!Socket.Send(Data + Sent, sizeof(uint32) - Sent, BytesSent))
		{
			return false;
		}
		if (BytesSent == 0)
		{
			return false;
		}
		Sent += BytesSent;
	}
	return true;
}

bool UE::Private::BatchProcessSocketHelpers::RecvUint32(FSocket& Socket, uint32& Arg)
{
	uint8 Data[sizeof(uint32)];
	for (int32 Read = 0; Read != sizeof(uint32);)
	{
		int32 BytesRead;
		if (!Socket.Recv(Data + Read, sizeof(uint32) - Read, BytesRead))
		{
			return false;
		}
		if (BytesRead == 0)
		{
			return false;
		}
		Read += BytesRead;
	}
	uint32 Result = 0;
	Result |= (uint32)Data[0] << 24;
	Result |= (uint32)Data[1] << 16;
	Result |= (uint32)Data[2] << 8;
	Result |= (uint32)Data[3] << 0;
	Arg = Result;
	return true;
}

bool UE::Private::BatchProcessSocketHelpers::RecvCharArray(FSocket& Socket, uint32 Num, TUniquePtr<char[]>& OutCharArray)
{
	OutCharArray = ::MakeUnique<char[]>(Num);
	for (uint32 Read = 0; Read != Num;)
	{
		int32 BytesRead;
		if (!Socket.Recv(reinterpret_cast<uint8*>(OutCharArray.Get()) + Read, static_cast<int32>(Num - Read), BytesRead))
		{
			return false;
		}
		if (BytesRead == 0)
		{
			return false;
		}
		Read += BytesRead;
	}
	return true;
}

bool UE::Private::BatchProcessSocketHelpers::RecvString(FSocket& Socket, FString& OutString)
{
	uint32 Num = 0;
	if (!RecvUint32(Socket, Num))
	{
		return false;
	}
	if(Num > (uint32)(TNumericLimits<int32>::Max()))
	{
		return false;
	}
	TUniquePtr<char[]> CharArray;
	if (!RecvCharArray(Socket, Num, CharArray))
	{
		return false;
	}
	OutString = FString(FUtf8StringView(CharArray.Get(), (int32)Num));
	return true;
}

bool UE::Private::BatchProcessSocketHelpers::SendJsonValue(FSocket& Socket, const TSharedRef<FJsonObject>& Object)
{
	FString String;
	auto Writer = TJsonWriterFactory<>::Create(&String);
	if (!FJsonSerializer::Serialize(Object, Writer))
	{
		return false;
	}
	if (!SendString(Socket, String))
	{
		return false;
	}
	return true;
}

bool UE::Private::BatchProcessSocketHelpers::RecvJsonValue(FSocket& Socket, TSharedPtr<FJsonValue>& Value)
{
	uint32 Num = 0;
	if (!RecvUint32(Socket, Num))
	{
		return false;
	}
	if(Num > (uint32)(TNumericLimits<int32>::Max()))
	{
		return false;
	}
	TUniquePtr<char[]> Array;
	if (!RecvCharArray(Socket, Num, Array))
	{
		return false;
	}
	TStringView<char> StringView{Array.Get(), (int32)Num};
	TSharedRef<TJsonStringViewReader<char>> Reader = TJsonStringViewReader<char>::Create(StringView);
	if (!FJsonSerializer::Deserialize(*Reader, Value))
	{
		return false;
	}
	return true;
}

bool UE::Private::BatchProcessSocketHelpers::SendString(FSocket& Socket, const FString& String)
{
	auto Utf8String = ::StringCast<UTF8CHAR>(*String);
	int32 Len = FCStringUtf8::Strlen(Utf8String.Get());
	if (!SendUint32(Socket, static_cast<uint32>(Len)))
	{
		return false;
	}
	for (int32 Sent = 0; Sent != Len;)
	{
		int32 BytesSent;
		if (!Socket.Send(reinterpret_cast<const uint8*>(Utf8String.Get()) + Sent, Len - Sent, BytesSent))
		{
			return false;
		}
		if (BytesSent == 0)
		{
			return false;
		}
		Sent += BytesSent;
	}
	return true;
}
