// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Dom/JsonObject.h"

class FSocket;

namespace UE::Private::BatchProcessSocketHelpers
{
	/** Port the coordinator listens on and workers connect to. */
	constexpr int32 DefaultPortNumber = 61579;

	bool SendUint32(FSocket& Socket, uint32 Arg);
	bool RecvUint32(FSocket& Socket, uint32& Num);
	bool RecvCharArray(FSocket& Socket, uint32 Num, TUniquePtr<char[]>& OutCharArray);
	bool RecvString(FSocket& Socket, FString& OutString);
	bool SendJsonValue(FSocket& Socket, const TSharedRef<FJsonObject>& Object);
	bool RecvJsonValue(FSocket& Socket, TSharedPtr<FJsonValue>& Value);
	bool SendString(FSocket& Socket, const FString& String);
}
