// Copyright Epic Games, Inc. All Rights Reserved.

#include "BatchProcess/BatchProcessWorker.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformMemory.h"
#include "JsonObjectConverter.h"
#include "Misc/App.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Misc/Paths.h"
#include "BatchProcess/BatchProcessLog.h"
#include "BatchProcess/BatchProcessSocketHelpers.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/StructOnScope.h"
#include "UObject/UObjectGlobals.h"
#include "SocketSubsystem.h"
#include "Sockets.h"


/** Captures all UE_LOG output produced during a single job's ProcessEvent call. */
struct FJobOutputCapture : public FOutputDevice
{
	FString Buffer;
	bool    bHasError = false;

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category) override
	{
		if (Verbosity <= ELogVerbosity::Error)
		{
			bHasError = true;
		}
		if (!Buffer.IsEmpty())
		{
			Buffer += TEXT('\n');
		}
		Buffer += FString::Printf(TEXT("[%s] %s"), *Category.ToString(), V);
	}
	virtual bool CanBeUsedOnAnyThread() const override { return true; }
};

namespace UE::Private
{
	const TCHAR* const BatchWorkerName    = TEXT("BatchProcessWorker");
	// Facilities for running a single client for debugging; runs
	// a coordinator for consistency/completeness:
	const TCHAR* const DebugClientAddress = TEXT("127.0.0.1");
	const uint32 DebugClientIP            = 0x7f000001;
	const int32  DebugClientPort          = 61580;
	const int32  MaxDebugConnections      = 128;
}

FBatchProcessWorker::FBatchProcessWorker()
	: ClientState(EClientState::Starting)
{
}

FBatchProcessWorker::~FBatchProcessWorker()
{
	HandleCleanup();
}

bool FBatchProcessWorker::LaunchLocalCoordinator(const FString& Command, FString& OutCoordinatorAddr, const FString& ExtraArgs)
{
	FString ProjPath = FPaths::GetProjectFilePath();

	const FString LaunchExecutableName = FPlatformProcess::ExecutablePath();
	const FString Params = FString::Printf(TEXT("\"%s\" -run=BatchProcessCommandlet \"%s\" -launchforclient=%s %s"),
		*ProjPath, *Command, UE::Private::DebugClientAddress, *ExtraArgs);
	LocalCoordinator = FPlatformProcess::CreateProc(
		*LaunchExecutableName,
		*Params,
		false,
		true,
		true,
		nullptr,
		0,
		nullptr,
		nullptr);

	// Wait for the coordinator to start — it will connect back to us with its address,
	// then execution continues as normal.
	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get();
	check(SocketSubsystem);
	TSharedPtr<FInternetAddr> ListenAddr = SocketSubsystem->GetLocalBindAddr(*GLog);
	ListenAddr->SetIp(UE::Private::DebugClientIP);
	ListenAddr->SetPort(UE::Private::DebugClientPort);
	FSocket* LocalCoordinatorSocket = SocketSubsystem->CreateSocket(NAME_Stream, UE::Private::BatchWorkerName, ListenAddr->GetProtocolType());
	LocalCoordinatorSocket->SetReuseAddr();

	const bool bBindSuccessful = LocalCoordinatorSocket->Bind(*ListenAddr);
	if(!bBindSuccessful)
	{
		UE_LOGF(LogBatchProcess, Error, "Failed to bind debug coordinator listener");
		return false;
	}

	const bool bListenSuccessful = LocalCoordinatorSocket->Listen(UE::Private::MaxDebugConnections);
	if(!bListenSuccessful)
	{
		UE_LOGF(LogBatchProcess, Error, "Failed to listen for debug coordinator");
		return false;
	}

	// Wait for coordinator to start up — generous timeout to allow the editor to initialise.
	const float  Timeout   = 180.f; // 3 minutes
	const double StartTime = FPlatformTime::Seconds();
	while(true)
	{
		if((StartTime + Timeout) < FPlatformTime::Seconds())
		{
			UE_LOGF(LogBatchProcess, Error, "Timed out waiting for debug coordinator to start");
			return false;
		}
		else if(!FPlatformProcess::IsProcRunning(LocalCoordinator))
		{
			UE_LOGF(LogBatchProcess, Error, "Debug coordinator exited unexpectedly");
			return false;
		}

		bool bHasPendingConnection = false;
		if(LocalCoordinatorSocket->HasPendingConnection(bHasPendingConnection) && bHasPendingConnection)
		{
			FSocket* DummyCoordinatorClientSocket = LocalCoordinatorSocket->Accept(UE::Private::BatchWorkerName);
			check(DummyCoordinatorClientSocket);
			check(UE::Private::BatchProcessSocketHelpers::RecvString(*DummyCoordinatorClientSocket, OutCoordinatorAddr));
			SocketSubsystem->DestroySocket(DummyCoordinatorClientSocket);
			SocketSubsystem->DestroySocket(LocalCoordinatorSocket);
			break;
		}

		FPlatformProcess::Sleep(1.f/60.f);
	}

	return true;
}

bool FBatchProcessWorker::PerformWork(const FString& CoordinatorAddrStr, int32 WorkerId, uint32 CoordinatorPid)
{
	if (!Connect(CoordinatorAddrStr, WorkerId))
	{
		ClientState = EClientState::Error;
		return false;
	}

	// Coordinator should notice us connect and send a work description.
	while(true)
	{
		TSharedPtr<FJsonValue> Value;
		if (!UE::Private::BatchProcessSocketHelpers::RecvJsonValue(*ClientSocket, Value))
		{
			// Coordinator closed the connection.  If we were idle (finished a
			// batch and waiting for more), this is the normal completion signal —
			// the coordinator closes worker sockets once all jobs are done.
			const bool bSuccess = (ClientState == EClientState::Idle);
			HandleCleanup();
			ClientState = bSuccess ? EClientState::ShuttingDown : EClientState::Error;
			return bSuccess;
		}

		if(!Value)
		{
			HandleCleanup();
			ClientState = EClientState::Error;
			return false;
		}

		const TSharedPtr<FJsonObject>* JsonObjectPtr;
		if (!Value->TryGetObject(JsonObjectPtr))
		{
			HandleCleanup();
			checkf(false, TEXT("Failed to find work object"));
			ClientState = EClientState::Error;
			return false;
		}

		const FJsonObject& JsonObject = *(JsonObjectPtr->Get());

		bool bShutdown = false;
		if(JsonObject.TryGetBoolField(TEXT("Shutdown"), bShutdown))
		{
			ClientState = EClientState::ShuttingDown;
			break;
		}

		FString FunctionAsString;
		if(!JsonObject.TryGetStringField(TEXT("Function"), FunctionAsString))
		{
			HandleCleanup();
			checkf(false, TEXT("Failed to find function in work packet"));
			ClientState = EClientState::Error;
			return false;
		}
		const TArray<TSharedPtr<FJsonValue>>* ArgumentsField;
		if(!JsonObject.TryGetArrayField(TEXT("Arguments"), ArgumentsField))
		{
			HandleCleanup();
			checkf(false, TEXT("Failed to find arguments in work packet"));
			ClientState = EClientState::Error;
			return false;
		}

		TSoftObjectPtr<UFunction> FunctionSoftObjectPtr{FSoftObjectPath{FunctionAsString}};
		const UFunction* ResolvedFunction = FunctionSoftObjectPtr.LoadSynchronous();
		if(!ResolvedFunction)
		{
			HandleCleanup();
			checkf(false, TEXT("Failed to resolve function: %s"), *FunctionAsString);
			ClientState = EClientState::Error;
			return false;
		}

		// The coordinator tells us the global starting index for this batch so we can
		// report the correct job indices back regardless of how many batches have run.
		double StartIndex = 0.0;
		JsonObject.TryGetNumberField(TEXT("StartIndex"), StartIndex);

		ClientState = EClientState::Working;
		UObject* Obj = ResolvedFunction->GetClass()->GetDefaultObject();

		for(int32 Idx = 0; Idx < ArgumentsField->Num(); ++Idx)
		{
			const TSharedPtr<FJsonValue>& Argument = (*ArgumentsField)[Idx];
			FStructOnScope Arg(ResolvedFunction);

			FText FailReason;
			const bool bResult =
				FJsonObjectConverter::JsonObjectToUStruct(Argument->AsObject().ToSharedRef(),
					ResolvedFunction, Arg.GetStructMemory(),
					0, 0, false,
					&FailReason);

			if(!bResult)
			{
				HandleCleanup();
				checkf(false, TEXT("Failed to parse job argument: %s"), *FailReason.ToString());
				ClientState = EClientState::Error;
				return false;
			}

			// Capture all log output produced during this job so the coordinator
			// can surface it in the structured results report.
			FJobOutputCapture Capture;
			GLog->AddOutputDevice(&Capture);
			Obj->ProcessEvent(const_cast<UFunction*>(ResolvedFunction), Arg.GetStructMemory());
			// Python's unreal.log() calls may be buffered inside GLog's threaded-log
			// queue rather than dispatched immediately.  Flush before removing the
			// capture device so no output is lost.
			GLog->FlushThreadedLogs();
			GLog->RemoveOutputDevice(&Capture);

			// Report result back to coordinator, including captured output.
			TSharedRef<FJsonObject> RootObject = ::MakeShared<FJsonObject>();
			RootObject->SetNumberField(TEXT("JobIndex"), (double)((int32)StartIndex + Idx));
			RootObject->SetStringField(TEXT("Output"),   Capture.Buffer);
			RootObject->SetBoolField(  TEXT("Error"),    Capture.bHasError);
			if (!UE::Private::BatchProcessSocketHelpers::SendJsonValue(*ClientSocket, RootObject))
			{
				HandleCleanup();
				ClientState = EClientState::Error;
				return false;
			}

			// Detect coordinator death between jobs. RecvJsonValue handles the case
			// where the coordinator dies while we're waiting for the next batch, but
			// here we cover the gap where the coordinator was killed during ProcessEvent
			// and the socket send above succeeded against buffered data.
			if (CoordinatorPid != 0 && !FPlatformProcess::IsApplicationRunning(CoordinatorPid))
			{
				UE_LOGF(LogBatchProcess, Warning, "Coordinator process has exited; shutting down");
				HandleCleanup();
				ClientState = EClientState::Error;
				return false;
			}
		}

		// Batch complete.  Go back to the top of the loop and wait for the
		// coordinator to either send another batch or close the connection.
		ClientState = EClientState::Idle;

		// Trigger GC when physical memory has grown by GCThresholdBytes since the
		// last collection.  This adapts to actual workload rather than firing on a
		// fixed schedule: lightweight jobs may never trigger GC; heavy jobs that
		// allocate lots of transient assets trigger it as needed.
		static uint64 MemAtLastGC = 0;
		constexpr uint64 GCThresholdBytes = 6ull * 1024 * 1024 * 1024; // 6 GB
		const uint64 CurrentMem = FPlatformMemory::GetStats().UsedPhysical;
		if (MemAtLastGC == 0) { MemAtLastGC = CurrentMem; }
		if (CurrentMem > MemAtLastGC + GCThresholdBytes)
		{
			CollectGarbage(RF_NoFlags);
			MemAtLastGC = FPlatformMemory::GetStats().UsedPhysical;
		}
	}

	// Inform server we've shut down.
	TSharedRef<FJsonObject> RootObject = ::MakeShared<FJsonObject>();
	RootObject->SetBoolField(TEXT("Shutdown"), true);
	if (!UE::Private::BatchProcessSocketHelpers::SendJsonValue(*ClientSocket, RootObject))
	{
		UE_LOGF(LogBatchProcess, Warning, "Failed to send shutdown message to coordinator");
	}

	ClientSocket->Close();
	ISocketSubsystem::Get()->DestroySocket(ClientSocket);
	ClientSocket = nullptr; // prevent double-destroy in HandleCleanup/destructor
	return true;
}

bool FBatchProcessWorker::RunSingleCommand(const FString& JobJson)
{
	TSharedPtr<FJsonValue> Value;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(JobJson);
	if (!FJsonSerializer::Deserialize(Reader, Value) || !Value.IsValid())
	{
		UE_LOGF(LogBatchProcess, Error, "--singlecommand: failed to parse JSON");
		return false;
	}

	const TSharedPtr<FJsonObject>* RootObjPtr;
	if (!Value->TryGetObject(RootObjPtr))
	{
		UE_LOGF(LogBatchProcess, Error, "--singlecommand: JSON root must be an object");
		return false;
	}
	const FJsonObject& RootObj = **RootObjPtr;

	FString FunctionPath;
	if (!RootObj.TryGetStringField(TEXT("Function"), FunctionPath))
	{
		UE_LOGF(LogBatchProcess, Error, "--singlecommand: missing 'function' field");
		return false;
	}

	const TArray<TSharedPtr<FJsonValue>>* ArgumentsArray;
	if (!RootObj.TryGetArrayField(TEXT("Arguments"), ArgumentsArray) || ArgumentsArray->IsEmpty())
	{
		UE_LOGF(LogBatchProcess, Error, "--singlecommand: missing or empty 'arguments' array");
		return false;
	}

	TSoftObjectPtr<UFunction> FunctionSoftPtr{FSoftObjectPath{FunctionPath}};
	const UFunction* ResolvedFunction = FunctionSoftPtr.LoadSynchronous();
	if (!ResolvedFunction)
	{
		UE_LOGF(LogBatchProcess, Error, "--singlecommand: failed to resolve function: %ls", *FunctionPath);
		return false;
	}

	UObject* CDO = ResolvedFunction->GetClass()->GetDefaultObject();

	// Use a capture device solely to detect errors; output still flows to all
	// normal GLog devices so the user sees it in the console/log file.
	FJobOutputCapture Capture;
	GLog->AddOutputDevice(&Capture);
	for (const TSharedPtr<FJsonValue>& Argument : *ArgumentsArray)
	{
		FStructOnScope ArgScope(ResolvedFunction);
		FText FailReason;
		if (!FJsonObjectConverter::JsonObjectToUStruct(
			Argument->AsObject().ToSharedRef(),
			ResolvedFunction, ArgScope.GetStructMemory(),
			0, 0, false, &FailReason))
		{
			GLog->FlushThreadedLogs();
			GLog->RemoveOutputDevice(&Capture);
			UE_LOGF(LogBatchProcess, Error, "--singlecommand: failed to parse argument: %ls", *FailReason.ToString());
			return false;
		}
		CDO->ProcessEvent(const_cast<UFunction*>(ResolvedFunction), ArgScope.GetStructMemory());
	}
	GLog->FlushThreadedLogs();
	GLog->RemoveOutputDevice(&Capture);

	return !Capture.bHasError;
}

void FBatchProcessWorker::HandleCleanup()
{
	if(ClientSocket)
	{
		ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get();
		check(SocketSubsystem);
		SocketSubsystem->DestroySocket(ClientSocket);
		ClientSocket = nullptr;
	}

	if(LocalCoordinator.IsValid())
	{
		FPlatformProcess::TerminateProc(LocalCoordinator);
		LocalCoordinator = FProcHandle();
	}
}

bool FBatchProcessWorker::Connect(const FString& CoordinatorAddrStr, int32 WorkerId)
{
	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get();
	if (!SocketSubsystem)
	{
		UE_LOGF(LogBatchProcess, Error, "No socket subsystem available");
		return false;
	}

	TSharedPtr<class FInternetAddr> CoordinatorAddr = SocketSubsystem->GetLocalBindAddr(*GLog);
	bool bIpValid = false;
	// CoordinatorAddrStr carries only the IP; SetIp() does not parse a port from the
	// string, so we set it explicitly using the shared DefaultPortNumber constant.
	CoordinatorAddr->SetIp(*CoordinatorAddrStr, bIpValid);
	CoordinatorAddr->SetPort(UE::Private::BatchProcessSocketHelpers::DefaultPortNumber);
	if (!bIpValid)
	{
		UE_LOGF(LogBatchProcess, Error, "Invalid coordinator IP: %ls", *CoordinatorAddrStr);
		return false;
	}

	ClientSocket = SocketSubsystem->CreateSocket(NAME_Stream, UE::Private::BatchWorkerName, CoordinatorAddr->GetProtocolType());
	if (!ClientSocket->Connect(*CoordinatorAddr))
	{
		UE_LOGF(LogBatchProcess, Error, "Failed to connect to coordinator at %ls", *CoordinatorAddrStr);
		SocketSubsystem->DestroySocket(ClientSocket);
		ClientSocket = nullptr;
		return false;
	}

	if (!UE::Private::BatchProcessSocketHelpers::SendUint32(*ClientSocket, WorkerId))
	{
		UE_LOGF(LogBatchProcess, Error, "Failed to send handshake ID to coordinator");
		SocketSubsystem->DestroySocket(ClientSocket);
		ClientSocket = nullptr;
		return false;
	}

	return true;
}
