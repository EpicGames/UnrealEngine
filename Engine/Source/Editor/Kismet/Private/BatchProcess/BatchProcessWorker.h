// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "HAL/PlatformProcess.h"

class FSocket;

struct FBatchProcessWorker
{
	FBatchProcessWorker();
	~FBatchProcessWorker();

	bool LaunchLocalCoordinator(const FString& Command, FString& OutCoordinatorAddr, const FString& ExtraArgs = FString());
	bool PerformWork(const FString& CoordinatorAddrStr, int32 WorkerId, uint32 CoordinatorPid = 0);
	/** Returns the handle for the coordinator process started by LaunchLocalCoordinator, if any. */
	FProcHandle GetLocalCoordinatorHandle() const { return LocalCoordinator; }

	/**
	 * Run a single job inline without connecting to a coordinator.
	 * JobJson must have the form: {"function":"<path>","arguments":[{...}]}
	 * Output flows to the normal log; returns false if any error was logged.
	 */
	static bool RunSingleCommand(const FString& JobJson);

private:
	void HandleCleanup();
	bool Connect(const FString& CoordinatorAddrStr, int32 WorkerId);

	FBatchProcessWorker(const FBatchProcessWorker&) = delete;
	FBatchProcessWorker(FBatchProcessWorker&&) = delete;
	FBatchProcessWorker& operator=(const FBatchProcessWorker&) = delete;
	FBatchProcessWorker& operator=(FBatchProcessWorker&&) = delete;

	FSocket*     ClientSocket    = nullptr;
	FProcHandle  LocalCoordinator; // debug facility: client starts its own coordinator for testing

	enum class EClientState
	{
		Starting,
		Working,
		Idle,
		ShuttingDown,
		Error
	};
	EClientState ClientState;
};
