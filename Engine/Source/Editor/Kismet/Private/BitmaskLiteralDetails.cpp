// Copyright Epic Games, Inc. All Rights Reserved.

#include "BitmaskLiteralDetails.h"

#include "Containers/UnrealString.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node_BitmaskLiteral.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectIterator.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Text/STextBlock.h"

class UObject;

#define LOCTEXT_NAMESPACE "BitmaskLiteralDetails"

void FBitmaskLiteralDetails::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	const TArray<TWeakObjectPtr<UObject>>& Objects = DetailLayout.GetSelectedObjects();
	check(Objects.Num() > 0);

	if (Objects.Num() == 1)
	{
		TargetNode = CastChecked<UK2Node_BitmaskLiteral>(Objects[0].Get());

		BitmaskEnumTypeMap.Empty();
		BitmaskEnumTypeNames.Empty();
		BitmaskEnumTypeNames.Add(MakeShareable(new FString(LOCTEXT("BitmaskEnumTypeName_None", "None").ToString())));
		for (TObjectIterator<UEnum> EnumIt; EnumIt; ++EnumIt)
		{
			UEnum* CurrentEnum = *EnumIt;
			if (UEdGraphSchema_K2::IsAllowableBlueprintVariableType(CurrentEnum) && CurrentEnum->HasMetaData(TEXT("Bitflags")))
			{
				BitmaskEnumTypeMap.Add(CurrentEnum->GetFName(), CurrentEnum);
				BitmaskEnumTypeNames.Add(MakeShareable(new FString(CurrentEnum->GetName())));
			}
		}

		IDetailCategoryBuilder& Category = DetailLayout.EditCategory("Options", LOCTEXT("OptionsCategory", "Bitmask Options"));

		Category.AddCustomRow(LOCTEXT("BitmaskEnumLabel", "Bitmask Enum"))
		.NameContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("BitmaskEnumLabel", "Bitmask Enum"))
			.ToolTipText(LOCTEXT("BitmaskEnumTooltip", "Choose an optional enumeration type for the flags. Note that changing this will also reset the input pin's default value."))
		]
		.ValueContent()
		[
			SNew(STextComboBox)
			.OptionsSource(&BitmaskEnumTypeNames)
			.InitiallySelectedItem(GetBitmaskEnumTypeName())
			.OnSelectionChanged(this, &FBitmaskLiteralDetails::OnBitmaskEnumTypeChanged)
		];
	}
}

TSharedPtr<FString> FBitmaskLiteralDetails::GetBitmaskEnumTypeName() const
{
	check(BitmaskEnumTypeNames.Num() > 0);
	TSharedPtr<FString> Result = BitmaskEnumTypeNames[0];

	if (TargetNode->BitflagsEnum)
	{
		for (int32 i = 1; i < BitmaskEnumTypeNames.Num(); ++i)
		{
			if (TargetNode->BitflagsEnum->GetName() == *BitmaskEnumTypeNames[i])
			{
				Result = BitmaskEnumTypeNames[i];
				break;
			}
		}
	}

	return Result;
}

void FBitmaskLiteralDetails::OnBitmaskEnumTypeChanged(TSharedPtr<FString> ItemSelected, ESelectInfo::Type SelectInfo)
{
	if (ItemSelected == BitmaskEnumTypeNames[0])
	{
		TargetNode->BitflagsEnum = nullptr;
	}
	else if (ItemSelected.IsValid())
	{
		TargetNode->BitflagsEnum = BitmaskEnumTypeMap[FName(**ItemSelected)];
	}

	TargetNode->ReconstructNode();

	// Reset default value
	if (UEdGraphPin* InputPin = TargetNode->FindPin(TargetNode->GetBitmaskInputPinName()))
	{
		if (const UEdGraphSchema_K2* K2Schema = Cast<UEdGraphSchema_K2>(InputPin->GetSchema()))
		{
			K2Schema->SetPinAutogeneratedDefaultValueBasedOnType(InputPin);
		}
	}
}

#undef LOCTEXT_NAMESPACE
