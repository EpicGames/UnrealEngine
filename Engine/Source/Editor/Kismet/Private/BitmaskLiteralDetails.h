// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "HAL/PlatformCrt.h"
#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"

class FName;
class FString;
class IDetailLayoutBuilder;
class UEnum;
class UK2Node_BitmaskLiteral;

/** Details customization for the "Make Bitmask Literal" node */
class FBitmaskLiteralDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FBitmaskLiteralDetails);
	}

	FBitmaskLiteralDetails()
		: TargetNode(NULL)
	{
	}

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

protected:
	TSharedPtr<FString> GetBitmaskEnumTypeName() const;
	void OnBitmaskEnumTypeChanged(TSharedPtr<FString> ItemSelected, ESelectInfo::Type SelectInfo);

private:
	/** The target node */
	UK2Node_BitmaskLiteral* TargetNode;

	/** Map of enum types used as bitmasks */
	TMap<FName, UEnum*> BitmaskEnumTypeMap;

	/** Array of enum type names used by the UI */
	TArray<TSharedPtr<FString>> BitmaskEnumTypeNames;
};
