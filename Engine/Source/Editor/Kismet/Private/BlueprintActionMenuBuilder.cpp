// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintActionMenuBuilder.h"
#include "UObject/UnrealType.h"
#include "BlueprintEditorSettings.h"
#include "Settings/EditorStyleSettings.h"
#include "Engine/Blueprint.h"
#include "Editor/EditorEngine.h"
#include "BlueprintNodeBinder.h"
#include "BlueprintActionMenuItem.h"
#include "BlueprintActionFilter.h"
#include "BlueprintDragDropMenuItem.h"
#include "BlueprintActionDatabase.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintDelegateNodeSpawner.h"
#include "BlueprintVariableNodeSpawner.h"
#include "EditorCategoryUtils.h"
#include "ObjectEditorUtils.h"

#if ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING
#include "Misc/OutputDeviceFile.h"
#endif	// ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING

#define LOCTEXT_NAMESPACE "BlueprintActionMenuBuilder"
DEFINE_LOG_CATEGORY_STATIC(LogBlueprintActionMenuItemFactory, Log, All);

/*******************************************************************************
 * FBlueprintActionMenuItemFactory
 ******************************************************************************/

class FBlueprintActionMenuItemFactory
{
public:
	/** 
	 * Menu item factory constructor. Sets up the blueprint context, which
	 * is utilized when configuring blueprint menu items' names/tooltips/etc.
	 *
	 * @param  Context	The blueprint context for the menu being built.
	 */
	FBlueprintActionMenuItemFactory(FBlueprintActionContext const& Context);

	/** A root category to perpend every menu item with */
	FText RootCategory;
	/** The menu sort order to set every menu item with */
	int32 MenuGrouping;
	/** Cached context for the blueprint menu being built */
	FBlueprintActionContext const& Context;
	
	/**
	 * Spawns a new FBlueprintActionMenuItem with the node-spawner. Constructs
	 * the menu item's category, name, tooltip, etc.
	 * 
	 * @param  Action			The node-spawner that the new menu item should wrap.
	 * @return A newly allocated FBlueprintActionMenuItem (which wraps the supplied action).
	 */
	TSharedPtr<FBlueprintActionMenuItem> MakeActionMenuItem(FBlueprintActionInfo const& ActionInfo);

	/**
	 * Spawns a new FBlueprintDragDropMenuItem with the node-spawner. Constructs
	 * the menu item's category, name, tooltip, etc.
	 * 
	 * @param  SampleAction	One of the (possibly) many node-spawners that this menu item is set to represent.
	 * @return A newly allocated FBlueprintActionMenuItem (which wraps the supplied action).
	 */
	TSharedPtr<FBlueprintDragDropMenuItem> MakeDragDropMenuItem(UBlueprintNodeSpawner const* SampleAction);

	/**
	 * 
	 * 
	 * @param  BoundAction	
	 * @return 
	 */
	TSharedPtr<FBlueprintActionMenuItem> MakeBoundMenuItem(FBlueprintActionInfo const& ActionInfo);
	
private:
	/**
	 * Utility getter function that retrieves the blueprint context for the menu
	 * items being made.
	 * 
	 * @return The first blueprint out of the cached FBlueprintActionContext.
	 */
	UBlueprint* GetTargetBlueprint() const;

	/**
	 * @return 
	 */
	UEdGraph* GetTargetGraph() const;

	/**
	 * @param  ActionInfo
	 * @return
	 */
	FBlueprintActionUiSpec GetActionUiSignature(FBlueprintActionInfo const& ActionInfo);
};

//------------------------------------------------------------------------------
FBlueprintActionMenuItemFactory::FBlueprintActionMenuItemFactory(FBlueprintActionContext const& ContextIn)
	: RootCategory(FText::GetEmpty())
	, MenuGrouping(0)
	, Context(ContextIn)
{
}

TSharedPtr<FBlueprintActionMenuItem> FBlueprintActionMenuItemFactory::MakeActionMenuItem(FBlueprintActionInfo const& ActionInfo)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FBlueprintActionMenuItemFactory::MakeActionMenuItem);

	FBlueprintActionUiSpec UiSignature = GetActionUiSignature(ActionInfo);

	UBlueprintNodeSpawner const* Action = ActionInfo.NodeSpawner;
	FBlueprintActionMenuItem* NewMenuItem = new FBlueprintActionMenuItem(Action, UiSignature, IBlueprintNodeBinder::FBindingSet(), FText::FromString(RootCategory.ToString() + TEXT('|') + UiSignature.Category.ToString()), MenuGrouping);

	return MakeShareable(NewMenuItem);
}

TSharedPtr<FBlueprintDragDropMenuItem> FBlueprintActionMenuItemFactory::MakeDragDropMenuItem(UBlueprintNodeSpawner const* SampleAction)
{
	// FBlueprintDragDropMenuItem takes care of its own menu MenuDescription, etc.
	FProperty const* SampleProperty = nullptr;
	if (UBlueprintDelegateNodeSpawner const* DelegateSpawner = Cast<UBlueprintDelegateNodeSpawner const>(SampleAction))
	{
		SampleProperty = DelegateSpawner->GetDelegateProperty();
	}
	else if (UBlueprintVariableNodeSpawner const* VariableSpawner = Cast<UBlueprintVariableNodeSpawner const>(SampleAction))
	{
		SampleProperty = VariableSpawner->GetVarProperty();
	}

	FText MenuDescription;
	FText TooltipDescription;
	FText Category;
	if (SampleProperty != nullptr)
	{
		bool const bShowFriendlyNames = GetDefault<UEditorStyleSettings>()->bShowFriendlyNames;
		MenuDescription = bShowFriendlyNames ? FText::FromString(UEditorEngine::GetFriendlyName(SampleProperty)) : FText::FromName(SampleProperty->GetFName());

		TooltipDescription = SampleProperty->GetToolTipText();
		Category = FObjectEditorUtils::GetCategoryText(SampleProperty);

		bool const bIsDelegateProperty = SampleProperty->IsA<FMulticastDelegateProperty>();
		if (bIsDelegateProperty && Category.IsEmpty())
		{
			Category = FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Delegates);
		}
		else if (!bIsDelegateProperty)
		{
			check(Context.Blueprints.Num() > 0);
			UBlueprint const* Blueprint = Context.Blueprints[0];
			UClass const*     BlueprintClass = (Blueprint->SkeletonGeneratedClass != nullptr) ? Blueprint->SkeletonGeneratedClass : Blueprint->ParentClass;

			UClass const* PropertyClass = SampleProperty->GetOwnerClass();
			checkSlow(PropertyClass != nullptr);
			bool const bIsMemberProperty = BlueprintClass && BlueprintClass->IsChildOf(PropertyClass);

			FText TextCategory;
			if (Category.IsEmpty())
			{
				TextCategory = FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Variables);
			}
			else if (bIsMemberProperty)
			{
				TextCategory = FEditorCategoryUtils::BuildCategoryString(FCommonEditorCategory::Variables, Category);
			}

			if (!bIsMemberProperty)
			{
				TextCategory = FText::Format(LOCTEXT("NonMemberVarCategory", "Class|{0}|{1}"), PropertyClass->GetDisplayNameText(), TextCategory);
			}
			Category = TextCategory;
		}
	}
	else
	{
		UE_LOGF(LogBlueprintActionMenuItemFactory, Warning, "Unhandled (or invalid) spawner: '%ls'", *SampleAction->GetName());
	}

	Category = FText::FromString(RootCategory.ToString() + TEXT('|') + Category.ToString());

	FBlueprintDragDropMenuItem* NewMenuItem = new FBlueprintDragDropMenuItem(Context, SampleAction, MenuGrouping, Category, MenuDescription, TooltipDescription);
	return MakeShareable(NewMenuItem);
}

TSharedPtr<FBlueprintActionMenuItem> FBlueprintActionMenuItemFactory::MakeBoundMenuItem(FBlueprintActionInfo const& ActionInfo)
{
	FBlueprintActionUiSpec UiSignature = GetActionUiSignature(ActionInfo);

	UBlueprintNodeSpawner const* Action = ActionInfo.NodeSpawner;
	FBlueprintActionMenuItem* NewMenuItem = new FBlueprintActionMenuItem(Action, UiSignature, ActionInfo.GetBindings(), FText::FromString(RootCategory.ToString() + TEXT('|') + UiSignature.Category.ToString()), MenuGrouping);

	return MakeShareable(NewMenuItem);
}

UBlueprint* FBlueprintActionMenuItemFactory::GetTargetBlueprint() const
{
	UBlueprint* TargetBlueprint = nullptr;
	if (Context.Blueprints.Num() > 0)
	{
		TargetBlueprint = Context.Blueprints[0];
	}
	return TargetBlueprint;
}

UEdGraph* FBlueprintActionMenuItemFactory::GetTargetGraph() const
{
	UEdGraph* TargetGraph = nullptr;
	if (Context.Graphs.Num() > 0)
	{
		TargetGraph = Context.Graphs[0];
	}
	else
	{
		UBlueprint* Blueprint = GetTargetBlueprint();
		check(Blueprint != nullptr);
		
		if (Blueprint->UbergraphPages.Num() > 0)
		{
			TargetGraph = Blueprint->UbergraphPages[0];
		}
		else if (Context.EditorPtr.IsValid())
		{
			TargetGraph = Context.EditorPtr.Pin()->GetFocusedGraph();
		}
	}
	return TargetGraph;
}

FBlueprintActionUiSpec FBlueprintActionMenuItemFactory::GetActionUiSignature(FBlueprintActionInfo const& ActionInfo)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FBlueprintActionMenuItemFactory::GetActionUiSignature);

	UBlueprintNodeSpawner const* Action = ActionInfo.NodeSpawner;

	UEdGraph* TargetGraph = GetTargetGraph();
	Action->PrimeDefaultUiSpec(TargetGraph);

	return Action->GetUiSpec(Context, ActionInfo.GetBindings());
}


/*******************************************************************************
 * Static FBlueprintActionMenuBuilder Helpers
 ******************************************************************************/

namespace FBlueprintActionMenuBuilderImpl
{
	/** Defines a sub-section of the overall blueprint menu (filter, heading, etc.) */
	struct FMenuSectionDefinition
	{
	public:
		FMenuSectionDefinition(FBlueprintActionFilter const& SectionFilter, uint32 const Flags);

		/** Series of ESectionFlags, aimed at customizing how we construct this menu section */
		uint32 Flags;
		/** A filter for this section of the menu */
		FBlueprintActionFilter Filter;
		
		/** Sets the root category for menu items in this section. */
		void SetSectionHeading(FText const& RootCategory);
		/** Gets the root category for menu items in this section. */
		FText const& GetSectionHeading() const;

		/** Sets the grouping for menu items belonging to this section. */
		void SetSectionSortOrder(int32 const MenuGrouping);
		/** Gets the grouping for menu items belonging to this section. */
		int32 GetSectionSortOrder() const;
		
		/**
		 * Filters the supplied action and if it passes, spawns a new 
		 * FBlueprintActionMenuItem for the specified menu (does not add the 
		 * item to the menu-builder itself).
		 *
		 * @param  DatabaseAction	The node-spawner that the new menu item should wrap.
		 * @return An empty TSharedPtr if the action was filtered out, otherwise a newly allocated FBlueprintActionMenuItem.
		 */
		TArray< TSharedPtr<FEdGraphSchemaAction> > MakeMenuItems(FBlueprintActionInfo& DatabaseAction);

		/**
		 * 
		 * 
		 * @param  DatabaseAction	
		 * @param  Bindings	
		 * @return 
		 */
		void AddBoundMenuItems(FBlueprintActionInfo& DatabaseAction, TArray<FFieldVariant> const& Bindings, TArray< TSharedPtr<FEdGraphSchemaAction> >& MenuItemsOut);
		
		/**
		 * Clears out any consolidated properties that this may have been 
		 * tracking (so we can start a new and spawn new consolidated menu items).
		 */
		void Empty();
		
	private:
		/** In charge of spawning menu items for this section (holds category/ordering information)*/
		FBlueprintActionMenuItemFactory ItemFactory;
		/** Tracks the properties that we've already consolidated and passed (when using the ConsolidatePropertyActions flag)*/
		TMap<FProperty const*, TSharedPtr<FBlueprintDragDropMenuItem>> ConsolidatedProperties;
	};

	/** A utility for building the menu item list based on a set of action descriptors */
	struct FMenuItemListAddHelper
	{
		/** Reset for a new menu build */
		void Reset(int32 NewSize)
		{
			NextIndex = 0;
			PendingActionList.Reset(NewSize);
		}

		/** Add a new pending action */
		void AddPendingAction(FBlueprintActionInfo&& Action)
		{
			PendingActionList.Add(Forward<FBlueprintActionInfo>(Action));
		}

		/** @return the next pending action and advance */
		FBlueprintActionInfo* GetNextAction()
		{
			return PendingActionList.IsValidIndex(NextIndex) ? &PendingActionList[NextIndex++] : nullptr;
		}

		/** @return the allocated size of the pending action list */
		SIZE_T GetAllocatedSize() const
		{
			return PendingActionList.GetAllocatedSize();
		}

		/** @return the total number of actions that are still pending */
		int32 GetNumPendingActions() const
		{
			return PendingActionList.IsValidIndex(NextIndex) ? PendingActionList.Num() - NextIndex : 0;
		}

		/** @return the total number of actions that were added to the pending list */
		int32 GetNumTotalAddedActions() const
		{
			return PendingActionList.Num();
		}

	private:
		/** Keeps track of the next action list item to process */
		int32 NextIndex = 0;

		/** All actions pending menu items for the current context */
		TArray<FBlueprintActionInfo> PendingActionList;
	};

#if ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING
	static TAutoConsoleVariable<bool> CVarBPEnableActionMenuDumpToFile(
		TEXT("BP.EnableActionMenuDumpToFile"),
		false,
		TEXT("If enabled, action menu contents will be dumped to a file after building.")
	);
#endif	// ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING
}

//------------------------------------------------------------------------------
FBlueprintActionMenuBuilderImpl::FMenuSectionDefinition::FMenuSectionDefinition(FBlueprintActionFilter const& SectionFilterIn, uint32 const FlagsIn)
	: Flags(FlagsIn)
	, Filter(SectionFilterIn)
	, ItemFactory(Filter.Context)
{
}

void FBlueprintActionMenuBuilderImpl::FMenuSectionDefinition::SetSectionHeading(FText const& RootCategory)
{
	ItemFactory.RootCategory = RootCategory;
}

FText const& FBlueprintActionMenuBuilderImpl::FMenuSectionDefinition::GetSectionHeading() const
{
	return ItemFactory.RootCategory;
}

void FBlueprintActionMenuBuilderImpl::FMenuSectionDefinition::SetSectionSortOrder(int32 const MenuGrouping)
{
	ItemFactory.MenuGrouping = MenuGrouping;
}

int32 FBlueprintActionMenuBuilderImpl::FMenuSectionDefinition::GetSectionSortOrder() const
{
	return ItemFactory.MenuGrouping;
}

void FBlueprintActionMenuBuilderImpl::FMenuSectionDefinition::AddBoundMenuItems(FBlueprintActionInfo& DatabaseActionInfo, TArray<FFieldVariant> const& PerspectiveBindings, TArray< TSharedPtr<FEdGraphSchemaAction> >& MenuItemsOut)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMenuSectionDefinition::AddBoundMenuItems);

	UBlueprintNodeSpawner const* DatabaseAction = DatabaseActionInfo.NodeSpawner;

	TSharedPtr<FBlueprintActionMenuItem> LastMadeMenuItem;
	bool const bConsolidate = (Flags & FBlueprintActionMenuBuilder::ConsolidateBoundActions) != 0;
	
	IBlueprintNodeBinder::FBindingSet CompatibleBindings;
	// we don't want the blueprint database growing out of control with an entry 
	// for every object you could ever possibly bind to, so each 
	// UBlueprintNodeSpawner comes with an interface to test/bind through... 
	for (auto BindingIt = PerspectiveBindings.CreateConstIterator(); BindingIt;)
	{
		const FFieldVariant& BindingObj = *BindingIt;
		++BindingIt;
		bool const bIsLastBinding = !BindingIt;

		// check to see if this object can be bound to this action
		if (DatabaseAction->IsBindingCompatible(BindingObj))
		{
			// add bindings before filtering (in case tests accept/reject based off of this)
			CompatibleBindings.Add(BindingObj);
		}

		// if BoundAction is now "full" (meaning it can take any more 
		// bindings), or if this is the last binding to test...
		if ((CompatibleBindings.Num() > 0) && (!DatabaseAction->CanBindMultipleObjects() || bIsLastBinding || !bConsolidate))
		{
			// we don't want binding to mutate DatabaseActionInfo, so we clone  
			// the action info, and tack on some binding data
			FBlueprintActionInfo BoundActionInfo(DatabaseActionInfo, CompatibleBindings);

			// have to check IsFiltered() for every "fully bound" action (in
			// case there are tests that reject based off of this), we may 
			// test this multiple times per action (we have to make sure 
			// that every set of bound objects pass before folding them into
			// MenuItem)
			bool const bPassedFilter = !Filter.IsFiltered(BoundActionInfo);
			if (bPassedFilter)
			{
				if (!bConsolidate || !LastMadeMenuItem.IsValid())
				{
					LastMadeMenuItem = ItemFactory.MakeBoundMenuItem(BoundActionInfo);
					MenuItemsOut.Add(LastMadeMenuItem);

					if (Flags & FBlueprintActionMenuBuilder::FlattenCategoryHierarcy)
					{
						LastMadeMenuItem->CosmeticUpdateCategory( ItemFactory.RootCategory );
					}
				}
				else
				{
					// move these bindings over to the menu item (so we can 
					// test the next set)
					LastMadeMenuItem->AppendBindings(Filter.Context, CompatibleBindings);
				}
			}
			CompatibleBindings.Empty(); // do before we copy back over cached fields for DatabaseActionInfo
		}
	}
}

TArray< TSharedPtr<FEdGraphSchemaAction> > FBlueprintActionMenuBuilderImpl::FMenuSectionDefinition::MakeMenuItems(FBlueprintActionInfo& DatabaseAction)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMenuSectionDefinition::MakeMenuItems);

	TSharedPtr<FEdGraphSchemaAction> UnBoundMenuEntry;
	bool bPassedFilter = !Filter.IsFiltered(DatabaseAction);

	// if the caller wants to consolidate all property actions, then we have to 
	// check and see if this is one of those that needs consolidating (needs 
	// a FBlueprintDragDropMenuItem instead of a FBlueprintActionMenuItem)
	if (bPassedFilter && (Flags & FBlueprintActionMenuBuilder::ConsolidatePropertyActions))
	{
		FProperty const* ActionProperty = nullptr;
		if (UBlueprintVariableNodeSpawner const* VariableSpawner = Cast<UBlueprintVariableNodeSpawner>(DatabaseAction.NodeSpawner))
		{
			ActionProperty = VariableSpawner->GetVarProperty();
			bPassedFilter = (ActionProperty != nullptr);
		}
		else if (UBlueprintDelegateNodeSpawner const* DelegateSpawner = Cast<UBlueprintDelegateNodeSpawner>(DatabaseAction.NodeSpawner))
		{
			ActionProperty = DelegateSpawner->GetDelegateProperty();
			bPassedFilter = (ActionProperty != nullptr);
		}

		if (ActionProperty != nullptr)
		{
			if (TSharedPtr<FBlueprintDragDropMenuItem>* ConsolidatedMenuItem = ConsolidatedProperties.Find(ActionProperty))
			{
				(*ConsolidatedMenuItem)->AppendAction(DatabaseAction.NodeSpawner);
				// this menu entry has already been returned, don't need to 
				// create/insert a new one
				bPassedFilter = false;
			}
			else
			{
				TSharedPtr<FBlueprintDragDropMenuItem> NewMenuItem = ItemFactory.MakeDragDropMenuItem(DatabaseAction.NodeSpawner);
				ConsolidatedProperties.Add(ActionProperty, NewMenuItem);
				UnBoundMenuEntry = NewMenuItem;
			}
		}
	}

	if (!UnBoundMenuEntry.IsValid() && bPassedFilter)
	{
		UnBoundMenuEntry = ItemFactory.MakeActionMenuItem(DatabaseAction);
		if (Flags & FBlueprintActionMenuBuilder::FlattenCategoryHierarcy)
		{
			UnBoundMenuEntry->CosmeticUpdateCategory( ItemFactory.RootCategory );
		}
	}

	TArray< TSharedPtr<FEdGraphSchemaAction> > MenuItems;
	if (UnBoundMenuEntry.IsValid())
	{
		MenuItems.Add(UnBoundMenuEntry);
	}
	AddBoundMenuItems(DatabaseAction, Filter.Context.SelectedObjects, MenuItems);

	return MenuItems;
}

void FBlueprintActionMenuBuilderImpl::FMenuSectionDefinition::Empty()
{
	ConsolidatedProperties.Empty();
}

/*******************************************************************************
 * FBlueprintActionMenuBuilder
 ******************************************************************************/

FBlueprintActionMenuBuilder::FBlueprintActionMenuBuilder(EConfigFlags ConfigFlags)
{
	bUsePendingActionList = !!(ConfigFlags & EConfigFlags::UseTimeSlicing);
	MenuItemListAddHelper = MakeShared<FBlueprintActionMenuBuilderImpl::FMenuItemListAddHelper>();
}

void FBlueprintActionMenuBuilder::Empty()
{
	FGraphActionListBuilderBase::Empty();
	MenuSections.Empty();
}

void FBlueprintActionMenuBuilder::AddMenuSection(FBlueprintActionFilter const& Filter, FText const& Heading/* = FText::GetEmpty()*/, int32 MenuOrder/* = 0*/, uint32 const Flags/* = 0*/)
{
	using namespace FBlueprintActionMenuBuilderImpl;
	
	TSharedRef<FMenuSectionDefinition> SectionDescRef = MakeShareable(new FMenuSectionDefinition(Filter, Flags));
	SectionDescRef->SetSectionHeading(Heading);
	SectionDescRef->SetSectionSortOrder(MenuOrder);

	MenuSections.Add(SectionDescRef);
}

void FBlueprintActionMenuBuilder::RebuildActionList()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FBlueprintActionMenuBuilder::RebuildActionList);

	using namespace FBlueprintActionMenuBuilderImpl;

	FGraphActionListBuilderBase::Empty();
	for (TSharedRef<FMenuSectionDefinition> MenuSection : MenuSections)
	{
		// clear out intermediate actions that may have been spawned (like 
		// consolidated property actions).
		MenuSection->Empty();
	}
	
	FBlueprintActionDatabase& ActionDatabase = FBlueprintActionDatabase::Get();
	FBlueprintActionDatabase::FActionRegistry const& ActionRegistry = ActionDatabase.GetAllActions();

#if ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING
	int32 TotalActionCount = 0;
#endif	// ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING

	if (bUsePendingActionList)
	{
		MenuItemListAddHelper->Reset(ActionRegistry.Num());
	}

	for (auto Iterator(ActionRegistry.CreateConstIterator()); Iterator; ++Iterator)
	{
		const FObjectKey& ObjKey = Iterator->Key;
		const FBlueprintActionDatabase::FActionList& ActionList = Iterator->Value;

		if (UObject* ActionObject = ObjKey.ResolveObjectPtr())
		{
			for (UBlueprintNodeSpawner const* NodeSpawner : ActionList)
			{
				FBlueprintActionInfo BlueprintAction(ActionObject, NodeSpawner);

#if ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING
				++TotalActionCount;
#endif	// ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING

				if (bUsePendingActionList)
				{
					MenuItemListAddHelper->AddPendingAction(MoveTemp(BlueprintAction));
				}
				else
				{
					MakeMenuItems(BlueprintAction);
				}
			}
		}
		else
		{
			// Remove this (invalid) entry on the next tick.
			ActionDatabase.DeferredRemoveEntry(ObjKey);
		}
	}

#if ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING
	if (FBlueprintActionFilter::IsFilterTestStatsLoggingEnabled())
	{
		UE_LOGF(LogBlueprintActionMenuItemFactory, Log, "=== UNFILTERED ACTIONS: %d", TotalActionCount);
		if (MenuItemListAddHelper->GetNumPendingActions() > 0)
		{
			UE_LOGF(LogBlueprintActionMenuItemFactory, Log, "==  PENDING ACTION LIST MEMSIZE: %0.02f MB", MenuItemListAddHelper->GetAllocatedSize() / 1024.0f / 1024.0f);
		}

		// Dump detailed stats information about each filter test that was involved with building each menu section.
		for (int32 SectionIdx = 0; SectionIdx < MenuSections.Num(); ++SectionIdx)
		{
			const TSharedRef<FMenuSectionDefinition>& MenuSection = MenuSections[SectionIdx];
			FString DisplayName = FString::Printf(TEXT("MenuSection[%d:%d]"), SectionIdx, MenuSection->GetSectionSortOrder());
			const FText& SectionHeading = MenuSection->GetSectionHeading();
			if (!SectionHeading.IsEmptyOrWhitespace())
			{
				DisplayName.Appendf(TEXT(" (%s)"), *SectionHeading.ToString());
			}

			UE_LOGF(LogBlueprintActionMenuItemFactory, Log, "=== FILTER TEST PROFILE: %ls ===", *DisplayName);

			for (const FString& ProfileEntry : MenuSection->Filter.GetFilterTestProfile())
			{
				UE_LOGF(LogBlueprintActionMenuItemFactory, Log, "%ls", *ProfileEntry);
			}
		}
	}
#endif	// ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING

	if (MenuItemListAddHelper->GetNumPendingActions() == 0)
	{
		BuildCompleted();
	}
}

void FBlueprintActionMenuBuilder::MakeMenuItems(FBlueprintActionInfo& InAction)
{
	using namespace FBlueprintActionMenuBuilderImpl;

	for (TSharedRef<FMenuSectionDefinition> const& MenuSection : MenuSections)
	{
		for (TSharedPtr<FEdGraphSchemaAction> MenuEntry : MenuSection->MakeMenuItems(InAction))
		{
			AddAction(MenuEntry);
		}
	}
}

int32 FBlueprintActionMenuBuilder::GetNumPendingActions() const
{
	return MenuItemListAddHelper->GetNumPendingActions();
}

float FBlueprintActionMenuBuilder::GetPendingActionsProgress() const
{
	const float NumPendingActions = static_cast<float>(MenuItemListAddHelper->GetNumPendingActions());
	const float NumTotalAddedActions = static_cast<float>(MenuItemListAddHelper->GetNumTotalAddedActions());

	check(NumTotalAddedActions > 0.0f);
	return 1.0f - (NumPendingActions / NumTotalAddedActions);
}

bool FBlueprintActionMenuBuilder::ProcessPendingActions()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FBlueprintActionMenuBuilder::ProcessPendingActions);

	using namespace FBlueprintActionMenuBuilderImpl;

	bool bProcessedActions = false;
	const double StartTime = FPlatformTime::Seconds();
	const float MaxTimeThresholdSeconds = GetDefault<UBlueprintEditorSettings>()->ContextMenuTimeSlicingThresholdMs / 1000.0f;

	FBlueprintActionInfo* CurrentAction = MenuItemListAddHelper->GetNextAction();
	while (CurrentAction)
	{
		bProcessedActions = true;

		MakeMenuItems(*CurrentAction);

		if ((FPlatformTime::Seconds() - StartTime) < MaxTimeThresholdSeconds)
		{
			CurrentAction = MenuItemListAddHelper->GetNextAction();
		}
		else
		{
			break;
		}
	}

	if (bProcessedActions && !CurrentAction)
	{
		BuildCompleted();
	}

	return bProcessedActions;
}

void FBlueprintActionMenuBuilder::BuildCompleted()
{
#if ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING
	using namespace FBlueprintActionMenuBuilderImpl;

	if (CVarBPEnableActionMenuDumpToFile.GetValueOnGameThread())
	{
		FString DumpFilePath = FPaths::ProfilingDir() / TEXT("ActionMenu") / FString::Printf(TEXT("MenuDump_%s.txt"), *FDateTime::Now().ToString());
		FOutputDeviceFile DumpFile(*DumpFilePath, true);

		DumpFile.SetSuppressEventTag(true);

		UE_LOGF(LogBlueprintActionMenuItemFactory, Log, "Dumping menu builder action list to \"%ls\"...", *DumpFilePath);
		
		const int32 NumActions = GetNumActions();

		TArray<FString> Actions;
		TStringBuilder<512> CurrentLine;
		for (int32 I = 0; I < NumActions; ++I)
		{
			const TSharedPtr<FEdGraphSchemaAction>& Action = GetSchemaAction(I);

			CurrentLine = TEXT("");
			for (const FString& Category : Action->GetCategoryChain())
			{
				CurrentLine.Append(FString::Printf(TEXT("%s|"), *Category));
			}
			CurrentLine.Append(Action->GetMenuDescription().ToString());
			Actions.Emplace(CurrentLine);
		}
		Actions.Sort(
			[](const FString& A, const FString& B)
			{
				return FTextComparison::CompareTo(A, B) < 0;
			}
		);
		for( const FString& Line : Actions )
		{
			DumpFile.Log(Line);
		}

		CVarBPEnableActionMenuDumpToFile.AsVariable()->Set(false);
	}
#endif	// ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING
}

#undef LOCTEXT_NAMESPACE
