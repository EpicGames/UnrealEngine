// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintActionMenuItem.h"

#include "BlueprintNodeSpawner.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Set.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "HAL/PlatformCrt.h"
#include "IDocumentation.h"
#include "IDocumentationPage.h"
#include "Internationalization/Internationalization.h"
#include "K2Node.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "SNodePanel.h"
#include "ScopedTransaction.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "Textures/SlateIcon.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintActionMenuItem)

class UBlueprint;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "BlueprintActionMenuItem"

/*******************************************************************************
 * Static FBlueprintMenuActionItem Helpers
 ******************************************************************************/

namespace FBlueprintMenuActionItemImpl
{
	/**
	 * Utility function for marking blueprints dirty and recompiling them after 
	 * a node has been added.
	 * 
	 * @param  SpawnedNode	The node that was just added to the blueprint.
	 */
	static void DirtyBlueprintFromNewNode(UEdGraphNode* SpawnedNode);

	/**
	 * 
	 * 
	 * @param  Action			The action you wish to invoke.
	 * @param  ParentGraph		The graph you want the action to spawn a node into.
	 * @param  Location			The position in the graph that you want the node spawned.
	 * @param  Bindings			Any bindings you want applied after the node has been spawned.
	 * @return The spawned node (could be an existing one if the event was already placed).
	 */
	static UEdGraphNode* InvokeAction(const UBlueprintNodeSpawner* Action, UEdGraph* ParentGraph, const FVector2f& Location, IBlueprintNodeBinder::FBindingSet const& Bindings, bool& bOutNewNode);

	/**
	 * 
	 * 
	 * @param  LeadingPin    
	 * @param  Node    
	 * @return 
	 */
	static bool IsNodeLinked(UEdGraphPin* LeadingPin, UEdGraphNode& Node);

	/**
	 * 
	 * 
	 * @param  ParentGraph    
	 * @param  FromPin    
	 * @param  SpawnedNodesBeginIndex    
	 * @return 
	 */
	static UEdGraphNode* AutowireSpawnedNodes(UEdGraphPin* FromPin, const TArray<UEdGraphNode*>& GraphNodes, TArray<UEdGraphNode*>& NewNodes);
}

//------------------------------------------------------------------------------
static void FBlueprintMenuActionItemImpl::DirtyBlueprintFromNewNode(UEdGraphNode* SpawnedNode)
{
	UEdGraph const* const NodeGraph = SpawnedNode->GetGraph();
	check(NodeGraph != nullptr);
	
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(NodeGraph);
	check(Blueprint != nullptr);

	if (SpawnedNode->GetSchema()->MarkBlueprintDirtyFromNewNode(Blueprint, SpawnedNode))
	{
		return;
	}

	UK2Node* K2Node = Cast<UK2Node>(SpawnedNode);
	// see if we need to recompile skeleton after adding this node, or just mark
	// it dirty (default to rebuilding the skel, since there is no way to if
	// non-k2 nodes structurally modify the blueprint)
	if ((K2Node == nullptr) || K2Node->NodeCausesStructuralBlueprintChange())
	{
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
	}
	else
	{
		FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
	}
}

//------------------------------------------------------------------------------
static UEdGraphNode* FBlueprintMenuActionItemImpl::InvokeAction(const UBlueprintNodeSpawner* Action, UEdGraph* ParentGraph, const FVector2f& Location, IBlueprintNodeBinder::FBindingSet const& Bindings, bool& bOutNewNode)
{
	int32 const PreSpawnNodeCount = ParentGraph->Nodes.Num();
	// this could return an existing node
	UEdGraphNode* SpawnedNode = Action->Invoke(ParentGraph, Bindings, FDeprecateSlateVector2D(Location));

	// if a returned node wasn't one that previously existed in the graph
	const bool bNewNode = PreSpawnNodeCount < ParentGraph->Nodes.Num();
	bOutNewNode = bNewNode;
	if (bNewNode)
	{
		check(SpawnedNode != nullptr);
		SpawnedNode->SnapToGrid(SNodePanel::GetSnapGridSize());

		FBlueprintEditorUtils::AnalyticsTrackNewNode(SpawnedNode);
	}
	// if this node already existed, then we just want to focus on that node...
	// some node types are only allowed one instance per blueprint (like events)
	else if (SpawnedNode != nullptr)
	{
		FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(SpawnedNode);
	}

	return SpawnedNode;
}

//------------------------------------------------------------------------------
static bool FBlueprintMenuActionItemImpl::IsNodeLinked(UEdGraphPin* LeadingPin, UEdGraphNode& Node)
{
	const EEdGraphPinDirection PinDirection = LeadingPin->Direction;
	
	for (UEdGraphPin* Link : LeadingPin->LinkedTo)
	{
		UEdGraphNode* LinkNode = Link->GetOwningNode();
		if (LinkNode == &Node)
		{
			return true;
		}

		for (UEdGraphPin* NodePin : LinkNode->Pins)
		{
			if (NodePin->Direction != PinDirection)
			{
				continue;
			}

			if (IsNodeLinked(NodePin, Node))
			{
				return true;
			}
		}
	}
	return false;
}

//------------------------------------------------------------------------------
static UEdGraphNode* FBlueprintMenuActionItemImpl::AutowireSpawnedNodes(UEdGraphPin* FromPin, const TArray<UEdGraphNode*>& GraphNodes, TArray<UEdGraphNode*>& OrderedNewNodes)
{
	const EEdGraphPinDirection PinDirection = FromPin->Direction;
	// should lhs come before rhs?
	OrderedNewNodes.Sort([PinDirection](UEdGraphNode& Lhs, UEdGraphNode& Rhs)->bool
		{
			for (UEdGraphPin* NodePin : Rhs.Pins)
			{
				if (NodePin->Direction != PinDirection)
				{
					continue;
				}

				if (IsNodeLinked(NodePin, Lhs))
				{
					return false;
				}
			}
			return true;
		});
		

	int32 const PreAutowireConnectionCount = FromPin->LinkedTo.Num();

	UEdGraphPin* OldPinLink = nullptr;
	if (PreAutowireConnectionCount > 0)
	{
		OldPinLink = FromPin->LinkedTo[0];
	}

	for (UEdGraphNode* NewNode : OrderedNewNodes)
	{
		NewNode->AutowireNewNode(FromPin);

		int32 const NewConnectionCount = FromPin->LinkedTo.Num();
		if (NewConnectionCount == 0)
		{
			continue;
		}
		else if ((NewConnectionCount != PreAutowireConnectionCount) || (FromPin->LinkedTo[0] != OldPinLink))
		{
			return NewNode;
		}
	}
	return nullptr;
}

/*******************************************************************************
 * FBlueprintMenuActionItem
 ******************************************************************************/

//------------------------------------------------------------------------------
FBlueprintActionMenuItem::FBlueprintActionMenuItem(UBlueprintNodeSpawner const* NodeSpawner, FBlueprintActionUiSpec const& UiSpec, IBlueprintNodeBinder::FBindingSet const& InBindings, FText InNodeCategory, int32 InGrouping)
	: FEdGraphSchemaAction(MoveTemp(InNodeCategory), UiSpec.MenuName, UiSpec.Tooltip, InGrouping, UiSpec.Keywords)
	, Action(NodeSpawner)
	, IconTint(UiSpec.IconTint)
	, IconBrush(UiSpec.Icon.GetOptionalIcon())
	, Bindings(InBindings)
{
	check(Action != nullptr);

	DocExcerptRef.DocLink = UiSpec.DocLink;
	DocExcerptRef.DocExcerptName = UiSpec.DocExcerptTag;
	// we may fill out the UiSpec's DocLink with whitespace (so we can tell the
	// difference between an empty DocLink and one that still needs to be filled 
	// out), but this could cause troubles later with FDocumentation::GetPage()
	DocExcerptRef.DocLink.TrimStartInline();
}

//------------------------------------------------------------------------------
UEdGraphNode* FBlueprintActionMenuItem::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	using namespace FBlueprintMenuActionItemImpl;
	FScopedTransaction Transaction(LOCTEXT("AddNodeTransaction", "Add Node"));
	
	FVector2f ModifiedLocation = Location;
	if (FromPin != nullptr)
	{
		// for input pins, a new node will generally overlap the node being
		// dragged from... work out if we want add in some spacing from the connecting node
		if (FromPin->Direction == EGPD_Input)
		{
			UEdGraphNode* FromNode = FromPin->GetOwningNode();
			check(FromNode != nullptr);
			const float FromNodeX = FromNode->NodePosX;

			static const float MinNodeDistance = 60.0f; // min distance between spawned nodes (to keep them from overlapping)
			if (MinNodeDistance > FMath::Abs(FromNodeX - Location.X))
			{
				ModifiedLocation.X = FromNodeX - MinNodeDistance;
			}
		}

		// modify before the call to AutowireNewNode() below
		FromPin->Modify();
	}

	TSet<const UEdGraphNode*> NodesToFocus;
	const int32 PreSpawnNodeCount = ParentGraph->Nodes.Num();

	UEdGraphNode* LastSpawnedNode = nullptr;
	auto BoundObjIt = Bindings.CreateConstIterator();
	do
	{
		IBlueprintNodeBinder::FBindingSet BindingsSubset;
		for (; BoundObjIt && (Action->CanBindMultipleObjects() || (BindingsSubset.Num() == 0)); ++BoundObjIt)
		{
			if (BoundObjIt->IsValid())
			{
				BindingsSubset.Add(*BoundObjIt);
			}
		}

		const TSet<UEdGraphNode*> OldNodes(ParentGraph->Nodes);

		bool bNewNode = false;
		LastSpawnedNode = InvokeAction(Action, ParentGraph, ModifiedLocation, BindingsSubset, /*out*/ bNewNode);

		TArray<UEdGraphNode*> NewNodes = TSet<UEdGraphNode*>(ParentGraph->Nodes).Difference(OldNodes).Array();
		// could already be an existent node, so we have to add here (can't 
		// catch it as we go through all new nodes)
		NodesToFocus.Add(LastSpawnedNode);

		//NOTE: Between the new node is spawned and AutowireNewNode is called, the blueprint should not be compiled.

		if (FromPin != nullptr)
		{
			// make sure to auto-wire after we position the new node (in case
			// the auto-wire creates a conversion node to put between them)
			FBlueprintMenuActionItemImpl::AutowireSpawnedNodes(FromPin, ParentGraph->Nodes, NewNodes);
		}

		if (bNewNode)
		{
			DirtyBlueprintFromNewNode(LastSpawnedNode);
		}

		// Increase the node location a safe distance so follow-up nodes are not stacked
		ModifiedLocation.Y += UEdGraphSchema_K2::EstimateNodeHeight(LastSpawnedNode);

	} while (BoundObjIt);

	if (bSelectNewNode)
	{
		const int32 PostSpawnCount = ParentGraph->Nodes.Num();
		for (int32 NodeIndex = PreSpawnNodeCount; NodeIndex < PostSpawnCount; ++NodeIndex)
		{
			NodesToFocus.Add(ParentGraph->Nodes[NodeIndex]);
		}
		ParentGraph->SelectNodeSet(NodesToFocus, /*bFromUI =*/true);
	}
	// @TODO: select ALL spawned nodes
	
	return LastSpawnedNode;
}

//------------------------------------------------------------------------------
UEdGraphNode* FBlueprintActionMenuItem::PerformAction(UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	UEdGraphPin* FromPin = nullptr;
	if (FromPins.Num() > 0)
	{
		FromPin = FromPins[0];
	}
	
	UEdGraphNode* SpawnedNode = PerformAction(ParentGraph, FromPin, Location, bSelectNewNode);
	// try auto-wiring the rest of the pins (if there are any)
	for (int32 PinIndex = 1; PinIndex < FromPins.Num(); ++PinIndex)
	{
		SpawnedNode->AutowireNewNode(FromPins[PinIndex]);
	}
	
	return SpawnedNode;
}

//------------------------------------------------------------------------------
void FBlueprintActionMenuItem::AddReferencedObjects(FReferenceCollector& Collector)
{
	FEdGraphSchemaAction::AddReferencedObjects(Collector);
	
	// these don't get saved to disk, but we want to make sure the objects don't
	// get GC'd while the action array is around
	Collector.AddReferencedObject(Action);
}

//------------------------------------------------------------------------------
void FBlueprintActionMenuItem::AppendBindings(const FBlueprintActionContext& Context, IBlueprintNodeBinder::FBindingSet const& BindingSet)
{
	Bindings.Append(BindingSet);

	FBlueprintActionUiSpec UiSpec = Action->GetUiSpec(Context, Bindings);

	// ui signature could be dynamic, and change as bindings change
	// @TODO: would invalidate any category pre-pending that was done at the 
	//        MenuBuilder level
	//Category  = UiSpec.Category.ToString();
	
	UpdateSearchData(UiSpec.MenuName, UiSpec.Tooltip, FText(), UiSpec.Keywords);

	IconBrush = UiSpec.Icon.GetOptionalIcon();
	IconTint  = UiSpec.IconTint;
	DocExcerptRef.DocLink = UiSpec.DocLink;
	DocExcerptRef.DocExcerptName = UiSpec.DocExcerptTag;
	// we may fill out the UiSpec's DocLink with whitespace (so we can tell the
	// difference between an empty DocLink and one that still needs to be filled 
	// out), but this could cause troubles later with FDocumentation::GetPage()
	DocExcerptRef.DocLink.TrimStartInline();
}

//------------------------------------------------------------------------------
FSlateBrush const* FBlueprintActionMenuItem::GetMenuIcon(FSlateColor& ColorOut)
{
	ColorOut = IconTint;
	return IconBrush;
}

//------------------------------------------------------------------------------
const FBlueprintActionMenuItem::FDocExcerptRef& FBlueprintActionMenuItem::GetDocumentationExcerpt() const
{
	return DocExcerptRef;
}

//------------------------------------------------------------------------------
bool FBlueprintActionMenuItem::FDocExcerptRef::IsValid() const
{
	if (DocLink.IsEmpty())
	{
		return false;
	}

	TSharedRef<IDocumentationPage> DocumentationPage = IDocumentation::Get()->GetPage(DocLink, /*Config =*/nullptr);
	return DocumentationPage->HasExcerpt(DocExcerptName);
}

//------------------------------------------------------------------------------
UBlueprintNodeSpawner const* FBlueprintActionMenuItem::GetRawAction() const
{
	return Action;
}

#undef LOCTEXT_NAMESPACE
