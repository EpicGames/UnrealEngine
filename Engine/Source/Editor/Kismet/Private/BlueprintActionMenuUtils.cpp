// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintActionMenuUtils.h"

#include "AssetRegistry/AssetData.h"
#include "BlueprintActionFilter.h"
#include "BlueprintActionMenuBuilder.h"
#include "BlueprintActionMenuItem.h"
#include "BlueprintDragDropMenuItem.h"
#include "BlueprintEditor.h"
#include "BlueprintEditorSettings.h"
#include "BlueprintNodeBinder.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintPaletteFavorites.h"
#include "ComponentAssetBroker.h"
#include "Components/ActorComponent.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserModule.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphSchema_K2_Actions.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Engine/Blueprint.h"
#include "Engine/LevelScriptActor.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "IContentBrowserSingleton.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "K2Node_ActorBoundEvent.h"
#include "K2Node_AddComponent.h"
#include "K2Node_CallFunction.h"
#include "K2Node_ComponentBoundEvent.h"
#include "K2Node_VariableGet.h"
#include "K2Node_VariableSet.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Misc/AssertionMacros.h"
#include "Modules/ModuleManager.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Selection.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Script.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintActionMenuUtils)

class UEdGraph;

#define LOCTEXT_NAMESPACE "BlueprintActionMenuUtils"

/*******************************************************************************
* Static FBlueprintActionMenuUtils Helpers
******************************************************************************/

namespace BlueprintActionMenuUtilsImpl
{
	static int32 const FavoritesSectionGroup  = 102;
	static int32 const LevelActorSectionGroup = 101;
	static int32 const ComponentsSectionGroup = 100;
	static int32 const BoundAddComponentGroup = 002;
	static int32 const MainMenuSectionGroup   = 000;

	/**
	 * Additional filter rejection test, for menu sections that only contain 
	 * bound actions. Rejects any action that is not bound.
	 * 
	 * @param  Filter			The filter querying this rejection test.
	 * @param  BlueprintAction	The action to test.
	 * @return True if the spawner is unbound (and should be rejected), otherwise false.
	 */
	static bool IsUnBoundSpawner(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	 * Filter rejection test, for favorite menus. Rejects any actions that are 
	 * not favorited by the user.
	 * 
	 * @param  Filter			The filter querying this rejection test.
	 * @param  BlueprintAction	The action to test.
	 * @return True if the spawner is not favorited (and should be rejected), otherwise false.
	 */
	static bool IsNonFavoritedAction(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	 * 
	 * 
	 * @param  Filter	
	 * @param  BlueprintAction	
	 * @return 
	 */
	static bool IsPureNonConstAction(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	 * 
	 * 
	 * @param  Filter	
	 * @param  BlueprintAction	
	 * @return 
	 */
	static bool IsUnexposedMemberAction(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	 * 
	 * 
	 * @param  Filter	
	 * @param  BlueprintAction	
	 * @return 
	 */
	static bool IsUnexposedNonComponentAction(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	 * Utility function to find a common base class from a set of given classes.
	 * 
	 * @param  ObjectSet	The set of objects that you want a single base class for.
	 * @return A common base class for all the specified classes (falls back to UObject's class).
	 */
	static UClass* FindCommonBaseClass(TArray<UObject*> const& ObjectSet);

	/**
	 * 
	 * 
	 * @param  Pin	
	 * @return 
	 */
	static UClass* GetPinClassType(UEdGraphPin const* Pin);

	/**
	 * 
	 * 
	 * @param  MainMenuFilter    
	 * @param  ContextTargetMask    
	 * @return 
	 */
	static FBlueprintActionFilter MakeCallOnMemberFilter(FBlueprintActionFilter const& MainMenuFilter, uint32 ContextTargetMask);

	/**
	 * 
	 * 
	 * @param  ComponentsFilter	
	 * @param  MenuOut	
	 * @return 
	 */
	static void AddComponentSections(FBlueprintActionFilter const& ComponentsFilter, FBlueprintActionMenuBuilder& MenuOut);

	/**
	 * 
	 * 
	 * @param  LevelActorsFilter	
	 * @param  MenuOut	
	 * @return 
	 */
	static void AddLevelActorSections(FBlueprintActionFilter const& LevelActorsFilter, FBlueprintActionMenuBuilder& MenuOut);

	static void AddFavoritesSection(FBlueprintActionFilter const& MainMenuFilter, FBlueprintActionMenuBuilder& MenuOut);
}

//------------------------------------------------------------------------------
static bool BlueprintActionMenuUtilsImpl::IsUnBoundSpawner(FBlueprintActionFilter const& /*Filter*/, FBlueprintActionInfo& BlueprintAction)
{
	return (BlueprintAction.GetBindings().Num() <= 0);
}

//------------------------------------------------------------------------------
static bool BlueprintActionMenuUtilsImpl::IsNonFavoritedAction(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	const UEditorPerProjectUserSettings* EditorPerProjectUserSettings = GetDefault<UEditorPerProjectUserSettings>();
	// grab the user's favorites
	UBlueprintPaletteFavorites const* BlueprintFavorites = EditorPerProjectUserSettings->BlueprintFavorites;
	checkSlow(BlueprintFavorites != nullptr);

	return !BlueprintFavorites->IsFavorited(BlueprintAction);
}

//------------------------------------------------------------------------------
static bool BlueprintActionMenuUtilsImpl::IsPureNonConstAction(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	bool bIsFilteredOut = false;

	if (UFunction const* Function = BlueprintAction.GetAssociatedFunction())
	{
		bool const bIsImperative = !Function->HasAnyFunctionFlags(FUNC_BlueprintPure);
		bool const bIsConstFunc  =  Function->HasAnyFunctionFlags(FUNC_Const);
		bIsFilteredOut = !bIsImperative && !bIsConstFunc;
	}
	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static bool BlueprintActionMenuUtilsImpl::IsUnexposedMemberAction(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	bool bIsFilteredOut = false;

	if (UFunction const* Function = BlueprintAction.GetAssociatedFunction())
	{
		TArray<FString> AllExposedCategories;
		for (FBindingObject Binding : BlueprintAction.GetBindings())
		{
			if (FProperty* Property = Binding.Get<FProperty>())
			{
				const FString& ExposedCategoryMetadata = Property->GetMetaData(FBlueprintMetadata::MD_ExposeFunctionCategories);
				if (ExposedCategoryMetadata.IsEmpty())
				{
					continue;
				}

				TArray<FString> PropertyExposedCategories;
				ExposedCategoryMetadata.ParseIntoArray(PropertyExposedCategories, TEXT(","), true);
				AllExposedCategories.Append(PropertyExposedCategories);
			}
		}

		const FString& FunctionCategory = Function->GetMetaData(FBlueprintMetadata::MD_FunctionCategory);
		bIsFilteredOut = !AllExposedCategories.Contains(FunctionCategory);
	}
	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static bool BlueprintActionMenuUtilsImpl::IsUnexposedNonComponentAction(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	bool bIsFilteredOut = false;

	for (FBindingObject Binding : BlueprintAction.GetBindings())
	{
		if (FObjectProperty* ObjectProperty = Binding.Get<FObjectProperty>())
		{
			bool const bIsComponent = ObjectProperty->PropertyClass->IsChildOf<UActorComponent>();
			// ignoring components for this rejection test
			if (bIsComponent)
			{
				continue;
			}
		}
		// else, it's not a component... let's do this!

		bIsFilteredOut = IsUnexposedMemberAction(Filter, BlueprintAction);
		break;
	}

	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static UClass* BlueprintActionMenuUtilsImpl::FindCommonBaseClass(TArray<UObject*> const& ObjectSet)
{
	UClass* CommonClass = UObject::StaticClass();
	if (ObjectSet.Num() > 0)
	{
		CommonClass = ObjectSet[0]->GetClass();
		for (UObject const* Object : ObjectSet)
		{
			UClass* Class = Object->GetClass();
			while (!Class->IsChildOf(CommonClass))
			{
				CommonClass = CommonClass->GetSuperClass();
			}
		}
	}	
	return CommonClass;
}

//------------------------------------------------------------------------------
static UClass* BlueprintActionMenuUtilsImpl::GetPinClassType(UEdGraphPin const* Pin)
{
	UClass* PinObjClass = nullptr;

	FEdGraphPinType const& PinType = Pin->PinType;
	if ((PinType.PinCategory == UEdGraphSchema_K2::PC_Object) || 
		(PinType.PinCategory == UEdGraphSchema_K2::PC_Interface))
	{
		bool const bIsSelfPin = !PinType.PinSubCategoryObject.IsValid();
		if (bIsSelfPin)
		{
			PinObjClass = CastChecked<UK2Node>(Pin->GetOwningNode())->GetBlueprint()->SkeletonGeneratedClass;
		}
		else
		{
			PinObjClass = Cast<UClass>(PinType.PinSubCategoryObject.Get());
		}
	}

	if (PinObjClass != nullptr)
	{
		if (UBlueprint* ClassBlueprint = Cast<UBlueprint>(PinObjClass->ClassGeneratedBy))
		{
			if (ClassBlueprint->SkeletonGeneratedClass != nullptr)
			{
				PinObjClass = ClassBlueprint->SkeletonGeneratedClass;
			}
		}
	}

	return PinObjClass;
}

//------------------------------------------------------------------------------
static FBlueprintActionFilter BlueprintActionMenuUtilsImpl::MakeCallOnMemberFilter(FBlueprintActionFilter const& MainMenuFilter, uint32 ContextTargetMask)
{
	FBlueprintActionFilter CallOnMemberFilter;
	CallOnMemberFilter.Context = MainMenuFilter.Context;
	CallOnMemberFilter.PermittedNodeTypes.Add(UK2Node_CallFunction::StaticClass());
	CallOnMemberFilter.AddRejectionTest(FBlueprintActionFilter::FRejectionTestDelegate::CreateStatic(IsUnBoundSpawner));
	
	const UBlueprintEditorSettings* BlueprintSettings = GetDefault<UBlueprintEditorSettings>();
	// instead of looking for "ExposeFunctionCategories" on component properties,
	// we just expose functions for all components, but we still need to check
	// for "ExposeFunctionCategories" on any non-component properties... 
	if (BlueprintSettings->bExposeAllMemberComponentFunctions)
	{
		CallOnMemberFilter.AddRejectionTest(FBlueprintActionFilter::FRejectionTestDelegate::CreateStatic(IsUnexposedNonComponentAction));
	}
	else
	{
		CallOnMemberFilter.AddRejectionTest(FBlueprintActionFilter::FRejectionTestDelegate::CreateStatic(IsUnexposedMemberAction));
	}

	bool bForceAddComponents = ((ContextTargetMask & EContextTargetFlags::TARGET_SubComponents) != 0);

	auto TargetClasses = MainMenuFilter.TargetClasses;
	if (bForceAddComponents && (TargetClasses.Num() == 0))
	{
		for (UBlueprint const* TargetBlueprint : MainMenuFilter.Context.Blueprints)
		{
			UClass* BpClass = TargetBlueprint->SkeletonGeneratedClass;
			if (BpClass != nullptr)
			{
				FBlueprintActionFilter::AddUnique(TargetClasses, BpClass);
			}
		}
	}

	for ( const auto& ClassData : TargetClasses)
	{
		UClass const* TargetClass = ClassData.TargetClass;
		for (TFieldIterator<FObjectProperty> PropertyIt(TargetClass, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
		{
 			FObjectProperty* ObjectProperty = *PropertyIt;
			if (!ObjectProperty->HasAnyPropertyFlags(CPF_BlueprintVisible))
			{
				continue;
			}

			if ( ObjectProperty->HasMetaData(FBlueprintMetadata::MD_ExposeFunctionCategories) || 
				(bForceAddComponents && FBlueprintEditorUtils::IsSCSComponentProperty(ObjectProperty)) )
			{
				CallOnMemberFilter.Context.SelectedObjects.Add(ObjectProperty);
			}			
		}
	}

	return CallOnMemberFilter;
}

//------------------------------------------------------------------------------
static void BlueprintActionMenuUtilsImpl::AddComponentSections(FBlueprintActionFilter const& ComponentsFilter, FBlueprintActionMenuBuilder& MenuOut)
{
	FText EventSectionHeading = LOCTEXT("ComponentsEventCategory", "Add Event for Selected Components");
	FText FuncSectionHeading  = LOCTEXT("ComponentsFuncCategory",  "Call Function on Selected Components");

	if (ComponentsFilter.Context.SelectedObjects.Num() == 1)
	{
		FText const ComponentName = FText::FromName(ComponentsFilter.Context.SelectedObjects.Last().GetFName());
		FuncSectionHeading  = FText::Format(LOCTEXT("SingleComponentFuncCategory", "Call Function on {0}"), ComponentName);
		EventSectionHeading = FText::Format(LOCTEXT("SingleComponentEventCategory", "Add Event for {0}"), ComponentName);
	}

	FBlueprintActionFilter ComponentFunctionsFilter = ComponentsFilter;
	ComponentFunctionsFilter.PermittedNodeTypes.Add(UK2Node_CallFunction::StaticClass());
	MenuOut.AddMenuSection(ComponentFunctionsFilter, FuncSectionHeading, ComponentsSectionGroup, FBlueprintActionMenuBuilder::ConsolidateBoundActions);

	FBlueprintActionFilter ComponentEventsFilter = ComponentsFilter;
	ComponentEventsFilter.PermittedNodeTypes.Add(UK2Node_ComponentBoundEvent::StaticClass());
	MenuOut.AddMenuSection(ComponentEventsFilter, EventSectionHeading, ComponentsSectionGroup, FBlueprintActionMenuBuilder::ConsolidateBoundActions);
}

//------------------------------------------------------------------------------
static void BlueprintActionMenuUtilsImpl::AddLevelActorSections(FBlueprintActionFilter const& LevelActorsFilter, FBlueprintActionMenuBuilder& MenuOut)
{
	FText EventSectionHeading = LOCTEXT("ActorsEventCategory", "Add Event for Selected Actors");
	FText FuncSectionHeading  = LOCTEXT("ActorsFuncCategory",  "Call Function on Selected Actors");

	if (LevelActorsFilter.Context.SelectedObjects.Num() == 1)
	{
		auto GetDisplayNameText = [](const FFieldVariant& Element) -> const FText
		{
			// If we have an actor, then use it's name
			if (const AActor* const Actor = Element.Get<AActor>())
			{
				return FText::FromString(Actor->GetActorLabel());
			}

			// otherwise, fall back to just using its FName
			return FText::FromName(Element.GetFName());
		};
		
		FText const ActorName = GetDisplayNameText(LevelActorsFilter.Context.SelectedObjects.Last());
		
		FuncSectionHeading  = FText::Format(LOCTEXT("SingleActorFuncCategory", "Call Function on {0}"), ActorName);
		EventSectionHeading = FText::Format(LOCTEXT("SingleActorEventCategory", "Add Event for {0}"), ActorName);
	}

	FBlueprintActionFilter ActorFunctionsFilter = LevelActorsFilter;
	ActorFunctionsFilter.PermittedNodeTypes.Add(UK2Node_CallFunction::StaticClass());
	MenuOut.AddMenuSection(ActorFunctionsFilter, FuncSectionHeading, LevelActorSectionGroup, FBlueprintActionMenuBuilder::ConsolidateBoundActions);

	FBlueprintActionFilter ActorEventsFilter = LevelActorsFilter;
	ActorEventsFilter.PermittedNodeTypes.Add(UK2Node_ActorBoundEvent::StaticClass());
	MenuOut.AddMenuSection(ActorEventsFilter, EventSectionHeading, LevelActorSectionGroup, FBlueprintActionMenuBuilder::ConsolidateBoundActions);

	FBlueprintActionFilter ActorReferencesFilter = LevelActorsFilter;
	ActorReferencesFilter.RejectedNodeTypes.Append(ActorFunctionsFilter.PermittedNodeTypes);
	ActorReferencesFilter.RejectedNodeTypes.Append(ActorEventsFilter.PermittedNodeTypes);
	MenuOut.AddMenuSection(ActorReferencesFilter, FText::GetEmpty(), LevelActorSectionGroup, FBlueprintActionMenuBuilder::ConsolidateBoundActions);
}

//------------------------------------------------------------------------------
static void BlueprintActionMenuUtilsImpl::AddFavoritesSection(FBlueprintActionFilter const& MainMenuFilter, FBlueprintActionMenuBuilder& MenuOut)
{
	const UBlueprintEditorSettings* BlueprintSettings = GetDefault<UBlueprintEditorSettings>();
	if (BlueprintSettings->bShowContextualFavorites)
	{
		FBlueprintActionFilter FavoritesFilter = MainMenuFilter;

		FavoritesFilter.AddRejectionTest(FBlueprintActionFilter::FRejectionTestDelegate::CreateStatic(IsNonFavoritedAction));
		
		uint32 SectionFlags = 0x00;
		FText  SectionHeading = LOCTEXT("ContextMenuFavoritesTitle", "Favorites");

		if (BlueprintSettings->bFlattenFavoritesMenus)
		{
			SectionFlags |= FBlueprintActionMenuBuilder::FlattenCategoryHierarcy;
			SectionHeading = FText::GetEmpty();
		}

		MenuOut.AddMenuSection(FavoritesFilter, SectionHeading, FavoritesSectionGroup, SectionFlags);
	}
}

/*******************************************************************************
 * FBlueprintActionMenuUtils
 ******************************************************************************/

//------------------------------------------------------------------------------
void FBlueprintActionMenuUtils::MakePaletteMenu(FBlueprintActionContext const& Context, UClass* FilterClass, FBlueprintActionMenuBuilder& MenuOut)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FBlueprintActionMenuUtils::MakePaletteMenu);

	MenuOut.Empty();
	
	FBlueprintActionFilter::EFlags FilterFlags = FBlueprintActionFilter::BPFILTER_NoFlags;
	if (FilterClass != nullptr)
	{
		// make sure we exclude global and static library actions
		FilterFlags |= FBlueprintActionFilter::BPFILTER_RejectGlobalFields;
	}
	
	FBlueprintActionFilter MenuFilter(FilterFlags);
	MenuFilter.Context = Context;
	
	// self member variables can be accessed through the MyBluprint panel (even
	// inherited ones)... external variables can be accessed through the context
	// menu (don't want to clutter the palette, I guess?)
	MenuFilter.RejectedNodeTypes.Add(UK2Node_VariableGet::StaticClass());
	MenuFilter.RejectedNodeTypes.Add(UK2Node_VariableSet::StaticClass());
	
	if (FilterClass != nullptr)
	{
		FBlueprintActionFilter::Add(MenuFilter.TargetClasses, FilterClass);
	}

	MenuOut.AddMenuSection(MenuFilter, LOCTEXT("PaletteRoot", "Library"), /*MenuOrder =*/0, FBlueprintActionMenuBuilder::ConsolidatePropertyActions);
	MenuOut.RebuildActionList();
}

//------------------------------------------------------------------------------
void FBlueprintActionMenuUtils::MakeContextMenu(FBlueprintActionContext const& Context, bool bIsContextSensitive, uint32 ClassTargetMask, FBlueprintActionMenuBuilder& MenuOut)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FBlueprintActionMenuUtils::MakeContextMenu);

	using namespace BlueprintActionMenuUtilsImpl;

	//--------------------------------------
	// Composing Filters
	//--------------------------------------

	FBlueprintActionFilter::EFlags FilterFlags = FBlueprintActionFilter::BPFILTER_NoFlags;
	if ( bIsContextSensitive && ((ClassTargetMask & EContextTargetFlags::TARGET_BlueprintLibraries) == 0) )
	{
		FilterFlags |= FBlueprintActionFilter::BPFILTER_RejectGlobalFields;
	}

	if ( bIsContextSensitive && ((ClassTargetMask & EContextTargetFlags::TARGET_NonImportedTypes) == 0) )
	{
		FilterFlags |= FBlueprintActionFilter::BPFILTER_RejectNonImportedFields;
	}

	if(bIsContextSensitive)
	{
		FilterFlags |= FBlueprintActionFilter::BPFILTER_RejectIncompatibleThreadSafety;
	}

	FBlueprintActionFilter MainMenuFilter(FilterFlags);
	MainMenuFilter.Context = Context;
	MainMenuFilter.Context.SelectedObjects.Empty();

	FBlueprintActionFilter ComponentsFilter;
	ComponentsFilter.Context = Context;
	// only want bound actions for this menu section
	ComponentsFilter.AddRejectionTest(FBlueprintActionFilter::FRejectionTestDelegate::CreateStatic(IsUnBoundSpawner));
	// @TODO: don't know exactly why we can only bind non-pure/const functions;
	//        this is mirrored after FK2ActionMenuBuilder::GetFunctionCallsOnSelectedActors()
	//        and FK2ActionMenuBuilder::GetFunctionCallsOnSelectedComponents(),
	//        where we make the same stipulation
	ComponentsFilter.AddRejectionTest(FBlueprintActionFilter::FRejectionTestDelegate::CreateStatic(IsPureNonConstAction));
	

	FBlueprintActionFilter LevelActorsFilter;
	LevelActorsFilter.Context = Context;
	// only want bound actions for this menu section
	LevelActorsFilter.AddRejectionTest(FBlueprintActionFilter::FRejectionTestDelegate::CreateStatic(IsUnBoundSpawner));

	// Build asset reference filter
	FAssetReferenceFilterContext AssetReferenceFilterContext;
	for (UBlueprint* Blueprint : Context.Blueprints)
	{
		AssetReferenceFilterContext.AddReferencingAsset(FAssetData(Blueprint));
	}

	MainMenuFilter.AssetReferenceFilter = GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext);

	const UBlueprintEditorSettings* BlueprintSettings = GetDefault<UBlueprintEditorSettings>();
	bool bCanOperateOnLevelActors = bIsContextSensitive && (Context.Pins.Num() == 0);
	bool bCanHaveActorComponents  = bIsContextSensitive;
	// determine if we can operate on certain object selections (level actors, 
	// components, etc.)
	for (UBlueprint* Blueprint : Context.Blueprints)
	{
		UClass* BlueprintClass = Blueprint->SkeletonGeneratedClass;
		if (BlueprintClass != nullptr)
		{
			bCanOperateOnLevelActors &= BlueprintClass->IsChildOf<ALevelScriptActor>();
			if (bIsContextSensitive && (ClassTargetMask & EContextTargetFlags::TARGET_Blueprint))
			{
				FBlueprintActionFilter::AddUnique(MainMenuFilter.TargetClasses, BlueprintClass);
			}
		}
		bCanHaveActorComponents &= FBlueprintEditorUtils::DoesSupportComponents(Blueprint);
	}

	UEdGraphSchema_K2 const* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// make sure the bound menu sections have the proper OwnerClasses specified
	for (FFieldVariant Selection : Context.SelectedObjects)
	{
		if (FObjectProperty* ObjProperty = CastField<FObjectProperty>(Selection.ToField()))
		{
			LevelActorsFilter.Context.SelectedObjects.Remove(Selection);
		}
		else if (AActor* LevelActor = Cast<AActor>(Selection.ToUObject()))
		{
			ComponentsFilter.Context.SelectedObjects.Remove(Selection);
			if (!bCanOperateOnLevelActors || (!LevelActor->NeedsLoadForClient() && !LevelActor->NeedsLoadForServer()))
			{
				// don't want to let the level script operate on actors that won't be loaded in game
				LevelActorsFilter.Context.SelectedObjects.Remove(Selection);
			}
			else
			{
				// Make sure every blueprint is in the same level as this actor
				for (UBlueprint* Blueprint : Context.Blueprints)
				{
					if (!K2Schema->IsActorValidForLevelScriptRefs(LevelActor, Blueprint))
					{
						LevelActorsFilter.Context.SelectedObjects.Remove(Selection);
						break;
					}
				}
			}
		}
		else
		{
			ComponentsFilter.Context.SelectedObjects.Remove(Selection);
			LevelActorsFilter.Context.SelectedObjects.Remove(Selection);
		}
	}

	// make sure all selected level actors are accounted for (in case the caller
	// did not include them in the context)
	for (FSelectionIterator LvlActorIt(*GEditor->GetSelectedActors()); LvlActorIt; ++LvlActorIt)
	{
		AActor* LevelActor = Cast<AActor>(*LvlActorIt);
		// don't want to let the level script operate on actors that won't be loaded in game
		if (bCanOperateOnLevelActors && (LevelActor->NeedsLoadForClient() || LevelActor->NeedsLoadForServer()))
		{
			bool bAddActor = true;
			// Make sure every blueprint is in the same level as this actor
			for (UBlueprint* Blueprint : Context.Blueprints)
			{
				if (!K2Schema->IsActorValidForLevelScriptRefs(LevelActor, Blueprint))
				{
					bAddActor = false;
					break;
				}
			}
			if (bAddActor)
			{
				LevelActorsFilter.Context.SelectedObjects.AddUnique(LevelActor);
			}
		}
	}

	if(bCanHaveActorComponents)
	{
		// Don't allow actor components in static function graphs
		for (UEdGraph* Graph : Context.Graphs)
		{
			bCanHaveActorComponents &= !K2Schema->IsStaticFunctionGraph(Graph);
		}
	}

	if (bIsContextSensitive)
	{
		// if we're dragging from a pin, we further extend the context to cover
		// that pin and any other pins it sits beside 
		for (UEdGraphPin* ContextPin : Context.Pins)
		{
			if (UClass* PinObjClass = GetPinClassType(ContextPin))
			{
				if (ClassTargetMask & EContextTargetFlags::TARGET_PinObject)
				{
					FBlueprintActionFilter::AddUnique(MainMenuFilter.TargetClasses, PinObjClass);
				}
			}

			UEdGraphNode* OwningNode = ContextPin->GetOwningNodeUnchecked();
			if ((OwningNode != nullptr) && (ClassTargetMask & EContextTargetFlags::TARGET_NodeTarget))
			{
				// @TODO: should we search instead by name/DefaultToSelf
				if (UEdGraphPin* TargetPin = K2Schema->FindSelfPin(*OwningNode, EGPD_Input))
				{
					if (UClass* TargetClass = GetPinClassType(TargetPin))
					{
						FBlueprintActionFilter::AddUnique(MainMenuFilter.TargetClasses, TargetClass);
					}
				}
			}

			if ((ClassTargetMask & EContextTargetFlags::TARGET_SiblingPinObjects) == 0)
			{
				continue;
			}

			for (UEdGraphPin* NodePin : ContextPin->GetOwningNode()->Pins)
			{
				if ((NodePin->Direction == EGPD_Output))
				{
					if (UClass* PinClass = GetPinClassType(NodePin))
					{
						FBlueprintActionFilter::AddUnique(MainMenuFilter.TargetClasses, PinClass);
					}
				}
			}
		}
	}

	// should be called AFTER the MainMenuFilter if fully constructed
	FBlueprintActionFilter CallOnMemberFilter = MakeCallOnMemberFilter(MainMenuFilter, ClassTargetMask);

	FBlueprintActionFilter AddComponentFilter;
	AddComponentFilter.Context = MainMenuFilter.Context;
	AddComponentFilter.PermittedNodeTypes.Add(UK2Node_AddComponent::StaticClass());
	AddComponentFilter.AddRejectionTest(FBlueprintActionFilter::FRejectionTestDelegate::CreateStatic(IsUnBoundSpawner));

	if (BlueprintSettings->bIncludeActionsForSelectedAssetsInContextMenu)
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		TArray<FAssetData> SelectedAssets;
		ContentBrowserModule.Get().GetSelectedAssets(SelectedAssets);

		if (SelectedAssets.Num() <= 1 || !BlueprintSettings->bLimitAssetActionBindingToSingleSelectionOnly)
		{
			for (FAssetData& Asset : SelectedAssets)
			{
				UClass* AssetClass = Asset.GetClass();
				// filter here (rather than in FBlueprintActionFilter) so we only load
				// assets that we can use
				if ((AssetClass == nullptr) || (FComponentAssetBrokerage::GetPrimaryComponentForAsset(AssetClass) == nullptr))
				{
					continue;
				}

				// loading assets here may be slow (but we need a UObject to properly bind to)
				if (Asset.IsAssetLoaded() || BlueprintSettings->bLoadSelectedAssetsForContextMenuActionBinding)
				{
					UObject* AssetObj = Asset.GetAsset();
					AddComponentFilter.Context.SelectedObjects.Add(AssetObj);
				}
			}
		}
	}

	//--------------------------------------
	// Defining Menu Sections
	//--------------------------------------	

	MenuOut.Empty();

	if (!bIsContextSensitive)
	{
		MainMenuFilter.Context.Pins.Empty();
	}
	// for legacy purposes, we have to add the main menu section first (when 
	// reconstructing the legacy menu, we pull the first menu system)
	MenuOut.AddMenuSection(MainMenuFilter, FText::GetEmpty(), MainMenuSectionGroup);

	bool const bAddComponentsSection = bIsContextSensitive && bCanHaveActorComponents && (ComponentsFilter.Context.SelectedObjects.Num() > 0);
	// add the components section to the menu (if we don't have any components
	// selected, then inform the user through a dummy menu entry)
	if (bAddComponentsSection)
	{
		AddComponentSections(ComponentsFilter, MenuOut);
	}

	bool const bAddLevelActorsSection = bIsContextSensitive && bCanOperateOnLevelActors && (LevelActorsFilter.Context.SelectedObjects.Num() > 0);
	// add the level actor section to the menu
	if (bAddLevelActorsSection)
	{
		AddLevelActorSections(LevelActorsFilter, MenuOut);
	}

	if (bIsContextSensitive)
	{
		AddFavoritesSection(MainMenuFilter, MenuOut);
		MenuOut.AddMenuSection(CallOnMemberFilter, FText::GetEmpty(), MainMenuSectionGroup);
		MenuOut.AddMenuSection(AddComponentFilter, FText::GetEmpty(), BoundAddComponentGroup);
	}	

	//--------------------------------------
	// Building the Menu
	//--------------------------------------

	MenuOut.RebuildActionList();

	for (UEdGraph const* Graph : Context.Graphs)
	{
		if (FKismetEditorUtilities::CanPasteNodes(Graph))
		{
			// @TODO: Grey out menu option with tooltip if one of the nodes cannot paste into this graph
			TSharedPtr<FEdGraphSchemaAction> PasteHereAction(new FEdGraphSchemaAction_K2PasteHere(FText::GetEmpty(), LOCTEXT("PasteHereMenuName", "Paste here"), FText::GetEmpty(), MainMenuSectionGroup));
			MenuOut.AddAction(PasteHereAction);
			break;
		}
	}

	if (bIsContextSensitive && bCanHaveActorComponents && !bAddComponentsSection)
	{
		FText SelectComponentMsg = LOCTEXT("SelectComponentForEvents", "Select a Component to see available Events & Functions");
		FText SelectComponentToolTip = LOCTEXT("SelectComponentForEventsTooltip", "Select a Component in the MyBlueprint tab to see available Events and Functions in this menu.");
		TSharedPtr<FEdGraphSchemaAction> MsgAction = TSharedPtr<FEdGraphSchemaAction>(new FEdGraphSchemaAction_Dummy(FText::GetEmpty(), SelectComponentMsg, SelectComponentToolTip, ComponentsSectionGroup));
		MenuOut.AddAction(MsgAction);
	}

	if (bIsContextSensitive && bCanOperateOnLevelActors && !bAddLevelActorsSection)
	{
		FText SelectActorsMsg = LOCTEXT("SelectActorForEvents", "Select Actor(s) to see available Events & Functions");
		FText SelectActorsToolTip = LOCTEXT("SelectActorForEventsTooltip", "Select Actor(s) in the level to see available Events and Functions in this menu.");
		TSharedPtr<FEdGraphSchemaAction> MsgAction = TSharedPtr<FEdGraphSchemaAction>(new FEdGraphSchemaAction_Dummy(FText::GetEmpty(), SelectActorsMsg, SelectActorsToolTip, LevelActorSectionGroup));
		MenuOut.AddAction(MsgAction);
	}
}

//------------------------------------------------------------------------------
void FBlueprintActionMenuUtils::MakeFavoritesMenu(FBlueprintActionContext const& Context, FBlueprintActionMenuBuilder& MenuOut)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FBlueprintActionMenuUtils::MakeFavoritesMenu);

	MenuOut.Empty();

	FBlueprintActionFilter MenuFilter;
	MenuFilter.Context = Context;
	MenuFilter.AddRejectionTest(FBlueprintActionFilter::FRejectionTestDelegate::CreateStatic(BlueprintActionMenuUtilsImpl::IsNonFavoritedAction));

	uint32 SectionFlags = 0x00;
	const UBlueprintEditorSettings* BlueprintSettings = GetDefault<UBlueprintEditorSettings>();
	if (BlueprintSettings->bFlattenFavoritesMenus)
	{
		SectionFlags = FBlueprintActionMenuBuilder::FlattenCategoryHierarcy;
	}

	MenuOut.AddMenuSection(MenuFilter, FText::GetEmpty(), BlueprintActionMenuUtilsImpl::MainMenuSectionGroup, SectionFlags);
	MenuOut.RebuildActionList();
}

//------------------------------------------------------------------------------
const UK2Node* FBlueprintActionMenuUtils::ExtractNodeTemplateFromAction(const TSharedPtr<FEdGraphSchemaAction>& PaletteAction)
{
	if (PaletteAction.IsValid())
	{
		return ExtractNodeTemplateFromAction(*PaletteAction);
	}

	return nullptr;
}

const UK2Node* FBlueprintActionMenuUtils::ExtractNodeTemplateFromAction(const FEdGraphSchemaAction& PaletteAction)
{
	UK2Node const* TemplateNode = nullptr;
	FName const ActionId = PaletteAction.GetTypeId();
	if (ActionId == FBlueprintActionMenuItem::StaticGetTypeId())
	{
		const FBlueprintActionMenuItem& NewNodeActionMenuItem = (const FBlueprintActionMenuItem&)PaletteAction;
		TemplateNode = Cast<UK2Node>(NewNodeActionMenuItem.GetRawAction()->GetTemplateNode());
	}
	else if (ActionId == FBlueprintDragDropMenuItem::StaticGetTypeId())
	{
		// 			const FBlueprintDragDropMenuItem& DragDropActionMenuItem = (const FBlueprintDragDropMenuItem&)PaletteAction.Get();
		// 			TemplateNode = Cast<UK2Node>(DragDropActionMenuItem.GetSampleAction()->GetTemplateNode());
	}
	// if this action inherits from FEdGraphSchemaAction_K2NewNode
	else if (ActionId == FEdGraphSchemaAction_K2NewNode::StaticGetTypeId() ||
		ActionId == FEdGraphSchemaAction_K2AssignDelegate::StaticGetTypeId() ||
		ActionId == FEdGraphSchemaAction_K2AddComponent::StaticGetTypeId() ||
		ActionId == FEdGraphSchemaAction_K2AddCustomEvent::StaticGetTypeId() ||
		ActionId == FEdGraphSchemaAction_K2AddCallOnActor::StaticGetTypeId() ||
		ActionId == FEdGraphSchemaAction_K2TargetNode::StaticGetTypeId() ||
		ActionId == FEdGraphSchemaAction_K2Event::StaticGetTypeId() ||
		ActionId == FEdGraphSchemaAction_K2AddEvent::StaticGetTypeId() ||
		ActionId == FEdGraphSchemaAction_K2InputAction::StaticGetTypeId())
	{
		const FEdGraphSchemaAction_K2NewNode& NewNodeAction = (const FEdGraphSchemaAction_K2NewNode&)PaletteAction;
		TemplateNode = NewNodeAction.NodeTemplate;
	}
	else if (ActionId == FEdGraphSchemaAction_K2ViewNode::StaticGetTypeId())
	{
		const FEdGraphSchemaAction_K2ViewNode& FocusNodeAction = (const FEdGraphSchemaAction_K2ViewNode&)PaletteAction;
		TemplateNode = FocusNodeAction.NodePtr;
	}

	return TemplateNode;
}

#undef LOCTEXT_NAMESPACE
