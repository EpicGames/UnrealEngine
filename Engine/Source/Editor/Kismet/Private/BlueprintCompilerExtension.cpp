// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintCompilerExtension.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintCompilerExtension)

class FKismetCompilerContext;

UBlueprintCompilerExtension::UBlueprintCompilerExtension(const FObjectInitializer& ObjectInitializer)
	: UObject(ObjectInitializer)
{
}

void UBlueprintCompilerExtension::BlueprintCompiled(const FKismetCompilerContext& CompilationContext, const FBlueprintCompiledData& Data)
{
	// common entry point in case we need to add logging, profiling, etc
	ProcessBlueprintCompiled(CompilationContext, Data);
}
