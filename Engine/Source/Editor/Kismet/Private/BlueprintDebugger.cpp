// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintDebugger.h"

#include "BlueprintEditorTabs.h"
#include "CallStackViewer.h"
#include "CoreGlobals.h"
#include "Debugging/SKismetDebuggingView.h"
#include "Delegates/Delegate.h"
#include "Engine/Blueprint.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/LayoutService.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

class SWidget;
class SWindow;

#define LOCTEXT_NAMESPACE "BlueprintDebugger"

struct FBlueprintDebuggerCommands : public TCommands<FBlueprintDebuggerCommands>
{
	FBlueprintDebuggerCommands()
		: TCommands<FBlueprintDebuggerCommands>(
			TEXT("BlueprintDebugger"), // Context name for fast lookup
			LOCTEXT("BlueprintDebugger", "Blueprint Debugger"), // Localized context name for displaying
			NAME_None, // Parent
			FCoreStyle::Get().GetStyleSetName() // Icon Style Set
		)
	{
	}

	// TCommand<> interface
	virtual void RegisterCommands() override;
	// End of TCommand<> interface

	TSharedPtr<FUICommandInfo> ShowCallStackViewer;
	TSharedPtr<FUICommandInfo> ShowExecutionTrace;
};

void FBlueprintDebuggerCommands::RegisterCommands()
{
	UI_COMMAND(ShowCallStackViewer, "Call Stack", "Toggles visibility of the Call Stack window", EUserInterfaceActionType::Check, FInputChord());
	UI_COMMAND(ShowExecutionTrace, "Data Flow", "Toggles visibility of the Data Flow window", EUserInterfaceActionType::Check, FInputChord());
}

struct FBlueprintDebuggerImpl
{
	FBlueprintDebuggerImpl();
	~FBlueprintDebuggerImpl();

	/** Function registered with tab manager to create the bluepring debugger */
	TSharedRef<SDockTab> CreateBluprintDebuggerTab(const FSpawnTabArgs& Args);

	/** Sets the debugged blueprint in the debugger */
	void SetDebuggedBlueprint(UBlueprint* InBlueprint);

	TSharedPtr<FTabManager> DebuggingToolsTabManager;
	TSharedPtr<FTabManager::FLayout> BlueprintDebuggerLayout;

private:
	// prevent copying:
	FBlueprintDebuggerImpl(const FBlueprintDebuggerImpl&);
	FBlueprintDebuggerImpl(FBlueprintDebuggerImpl&&);
	FBlueprintDebuggerImpl& operator=(FBlueprintDebuggerImpl const&);
	FBlueprintDebuggerImpl& operator=(FBlueprintDebuggerImpl&&);
};

FBlueprintDebuggerImpl::FBlueprintDebuggerImpl()
{
	const IWorkspaceMenuStructure& MenuStructure = WorkspaceMenu::GetMenuStructure();

	FBlueprintDebuggerCommands::Register();
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(FBlueprintEditorTabs::BlueprintDebuggerID, FOnSpawnTab::CreateRaw(this, &FBlueprintDebuggerImpl::CreateBluprintDebuggerTab))
		.SetDisplayName(NSLOCTEXT("BlueprintDebugger", "TabTitle", "Blueprint Debugger"))
		.SetTooltipText(NSLOCTEXT("BlueprintDebugger", "TooltipText", "Open the Blueprint Debugger tab."))
		.SetGroup(MenuStructure.GetDeveloperToolsDebugCategory())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintDebugger.TabIcon"));
}

FBlueprintDebuggerImpl::~FBlueprintDebuggerImpl()
{
	if (FSlateApplication::IsInitialized())
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(FBlueprintEditorTabs::BlueprintDebuggerID);
	}

	if (DebuggingToolsTabManager.IsValid())
	{
		FGlobalTabmanager::Get()->UnregisterTabSpawner(FBlueprintEditorTabs::BlueprintDebuggerID);
		BlueprintDebuggerLayout = TSharedPtr<FTabManager::FLayout>();
		DebuggingToolsTabManager = TSharedPtr<FTabManager>();
	}
	FBlueprintDebuggerCommands::Unregister();
}

TSharedRef<SDockTab> FBlueprintDebuggerImpl::CreateBluprintDebuggerTab(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> NomadTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		.Label(NSLOCTEXT("BlueprintDebugger", "TabTitle", "Blueprint Debugger"));

	DebuggingToolsTabManager = FGlobalTabmanager::Get()->NewTabManager(NomadTab);
	// on persist layout will handle saving layout if the editor is shut down:
	DebuggingToolsTabManager->SetOnPersistLayout(
		FTabManager::FOnPersistLayout::CreateStatic(
			[](const TSharedRef<FTabManager::FLayout>& InLayout)
			{
				if (InLayout->GetPrimaryArea().Pin().IsValid())
				{
					FLayoutSaveRestore::SaveToConfig(GEditorLayoutIni, InLayout);
				}
			}
		)
	);

	// Register Toolbar
	SKismetDebuggingView::TryRegisterDebugToolbar();

	const FName ExecutionFlowTabName = FName(TEXT("ExecutionFlowApp"));
	const FName CallStackTabName = CallStackViewer::GetTabName();

	TWeakPtr<FTabManager> DebuggingToolsTabManagerWeak = DebuggingToolsTabManager;
	// On tab close will save the layout if the debugging window itself is closed,
	// this handler also cleans up any floating debugging controls. If we don't close
	// all areas we need to add some logic to the tab manager to reuse existing tabs:
	NomadTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateStatic(
		[](TSharedRef<SDockTab> Self, TWeakPtr<FTabManager> TabManager)
		{
			TSharedPtr<FTabManager> OwningTabManager = TabManager.Pin();
			if (OwningTabManager.IsValid())
			{
				FLayoutSaveRestore::SaveToConfig(GEditorLayoutIni, OwningTabManager->PersistLayout());
				OwningTabManager->CloseAllAreas();
			}
		}
		, DebuggingToolsTabManagerWeak
	));

	DebuggingToolsTabManager->RegisterTabSpawner(
		ExecutionFlowTabName,
		FOnSpawnTab::CreateStatic(
			[](const FSpawnTabArgs&)->TSharedRef<SDockTab>
			{
			const TSharedPtr<SKismetDebuggingView> KismetDebuggingView = SNew(SKismetDebuggingView)
				.BlueprintToWatch(nullptr);
			return SNew(SDockTab)
				.TabRole(ETabRole::PanelTab)
				.Label_Raw(KismetDebuggingView.Get(), &SKismetDebuggingView::GetTabLabel)
				[
					KismetDebuggingView.ToSharedRef()
				];
			}
		)
	)
	.SetDisplayName(NSLOCTEXT("BlueprintDebugger", "ExecutionFlowTabTitle", "Blueprint Data Flow"))
	.SetTooltipText(NSLOCTEXT("BlueprintDebugger", "ExecutionFlowTooltipText", "Open the Blueprint Data Flow tab."));

	CallStackViewer::RegisterTabSpawner(*DebuggingToolsTabManager);

	BlueprintDebuggerLayout = FTabManager::NewLayout("Standalone_BlueprintDebugger_Layout_v2")
	->AddArea
	(
		FTabManager::NewPrimaryArea()
		->SetOrientation(Orient_Vertical)
		->Split
		(
			FTabManager::NewStack()
			->SetSizeCoefficient(.4f)
			->SetHideTabWell(true)
			->AddTab(CallStackTabName, ETabState::OpenedTab)
			->AddTab(ExecutionFlowTabName, ETabState::OpenedTab)
			->SetForegroundTab(CallStackTabName)
		)
	);

	BlueprintDebuggerLayout = FLayoutSaveRestore::LoadFromConfig(GEditorLayoutIni, BlueprintDebuggerLayout.ToSharedRef());

	TSharedRef<SWidget> TabContents = DebuggingToolsTabManager->RestoreFrom(BlueprintDebuggerLayout.ToSharedRef(), TSharedPtr<SWindow>()).ToSharedRef();

	// build command list for tab restoration menu:
	TSharedPtr<FUICommandList> CommandList = MakeShareable(new FUICommandList());

	TWeakPtr<FTabManager> DebuggingToolsManagerWeak = DebuggingToolsTabManager;

	const auto ToggleTabVisibility = [](TWeakPtr<FTabManager> InDebuggingToolsManagerWeak, FName InTabName)
	{
		TSharedPtr<FTabManager> InDebuggingToolsManager = InDebuggingToolsManagerWeak.Pin();
		if (InDebuggingToolsManager.IsValid())
		{
			TSharedPtr<SDockTab> ExistingTab = InDebuggingToolsManager->FindExistingLiveTab(InTabName);
			if (ExistingTab.IsValid())
			{
				ExistingTab->RequestCloseTab();
			}
			else
			{
				InDebuggingToolsManager->TryInvokeTab(InTabName);
			}
		}
	};

	const auto IsTabVisible = [](TWeakPtr<FTabManager> InDebuggingToolsManagerWeak, FName InTabName)
	{
		TSharedPtr<FTabManager> InDebuggingToolsManager = InDebuggingToolsManagerWeak.Pin();
		if (InDebuggingToolsManager.IsValid())
		{
			return InDebuggingToolsManager->FindExistingLiveTab(InTabName).IsValid();
		}
		return false;
	};

	CommandList->MapAction(
		FBlueprintDebuggerCommands::Get().ShowCallStackViewer,
		FExecuteAction::CreateStatic(
			ToggleTabVisibility,
			DebuggingToolsManagerWeak,
			CallStackTabName
		),
		FCanExecuteAction::CreateStatic(
			[]() { return true; }
		),
		FIsActionChecked::CreateStatic(
			IsTabVisible,
			DebuggingToolsManagerWeak,
			CallStackTabName
		)
	);

	CommandList->MapAction(
		FBlueprintDebuggerCommands::Get().ShowExecutionTrace,
		FExecuteAction::CreateStatic(
			ToggleTabVisibility,
			DebuggingToolsManagerWeak,
			ExecutionFlowTabName
		),
		FCanExecuteAction::CreateStatic(
			[]() { return true; }
		),
		FIsActionChecked::CreateStatic(
			IsTabVisible,
			DebuggingToolsManagerWeak,
			ExecutionFlowTabName
		)
	);

	FMenuBarBuilder MenuBarBuilder(CommandList);
	MenuBarBuilder.AddPullDownMenu(
		LOCTEXT("WindowMenuLabel", "Window"),
		FText::GetEmpty(),
		FNewMenuDelegate::CreateLambda([](FMenuBuilder& Builder) {
			Builder.AddMenuEntry(FBlueprintDebuggerCommands::Get().ShowCallStackViewer);
			Builder.AddMenuEntry(FBlueprintDebuggerCommands::Get().ShowExecutionTrace);
			})
	);
	

	TSharedRef<SWidget> MenuBarWidget = MenuBarBuilder.MakeWidget();

	NomadTab->SetContent(
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			MenuBarWidget
		]
		+SVerticalBox::Slot()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
			.Padding(FMargin(0.f, 2.f))
			[
				TabContents
			]
		]
	);
	

	// Tell tab-manager about the multi-box for platforms with a global menu bar
	DebuggingToolsTabManager->SetMenuMultiBox(MenuBarBuilder.GetMultiBox(), MenuBarWidget);

	return NomadTab;
}

void FBlueprintDebuggerImpl::SetDebuggedBlueprint(UBlueprint* InBlueprint)
{
	static const FName ExecutionFlowTabName(TEXT("ExecutionFlowApp"));
	TSharedPtr<SDockTab> DebuggingViewTab = DebuggingToolsTabManager->TryInvokeTab(ExecutionFlowTabName);
	if (DebuggingViewTab.IsValid())
	{
		TSharedRef<SKismetDebuggingView> DebuggingViewWidget = StaticCastSharedRef<SKismetDebuggingView>(DebuggingViewTab->GetContent());
		DebuggingViewWidget->SetBlueprintToWatch(InBlueprint);
	}
}

FBlueprintDebugger::FBlueprintDebugger()
	: Impl(MakeUnique<FBlueprintDebuggerImpl>())
{
}

FBlueprintDebugger::~FBlueprintDebugger()
{
}

void FBlueprintDebugger::SetDebuggedBlueprint(UBlueprint* InBlueprint)
{
	Impl->SetDebuggedBlueprint(InBlueprint);
}

#undef LOCTEXT_NAMESPACE 
