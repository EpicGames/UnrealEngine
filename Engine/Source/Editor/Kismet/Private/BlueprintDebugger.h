// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UniquePtr.h"

class UBlueprint;

struct FBlueprintDebugger
{
	// Initializes the global state of the debugger (commands, tab spawners, etc):
	FBlueprintDebugger();

	// Destructor declaration purely so that we can pimpl:
	~FBlueprintDebugger();

	/** Sets the current debugged blueprint in the debugger */
	void SetDebuggedBlueprint(UBlueprint* InBlueprint);

private:
	TUniquePtr< struct FBlueprintDebuggerImpl > Impl;

	// prevent copying:
	FBlueprintDebugger(const FBlueprintDebugger&);
	FBlueprintDebugger(FBlueprintDebugger&&);
	FBlueprintDebugger& operator=(FBlueprintDebugger const&);
	FBlueprintDebugger& operator=(FBlueprintDebugger&&);
};

