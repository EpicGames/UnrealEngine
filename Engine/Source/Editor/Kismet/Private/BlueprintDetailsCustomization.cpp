// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintDetailsCustomization.h"

#include "AssetRegistry/AssetData.h"
#include "BlueprintEditor.h"
#include "BlueprintEditorModes.h"
#include "BlueprintEditorModule.h"
#include "BlueprintEditorSettings.h"
#include "BlueprintNamespaceRegistry.h"
#include "BlueprintNamespaceUtilities.h"
#include "ClassViewerModule.h"
#include "Components/ActorComponent.h"
#include "Components/ChildActorComponent.h"
#include "Components/SceneComponent.h"
#include "Components/TimelineComponent.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNode_Documentation.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphNode_Comment.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphSchema_K2_Actions.h"
#include "EdMode.h"
#include "Editor/EditorEngine.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Engine.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/MemberReference.h"
#include "Engine/SCS_Node.h"
#include "Engine/SimpleConstructionScript.h"
#include "StructUtils/UserDefinedStruct.h"
#include "EngineLogs.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/ITypedTableView.h"
#include "GameFramework/Actor.h"
#include "GenericPlatform/GenericApplication.h"
#include "GenericPlatform/ICursor.h"
#include "GraphEditor.h"
#include "HAL/PlatformMath.h"
#include "HAL/PlatformMisc.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailDragDropHandler.h"
#include "IDetailPropertyRow.h"
#include "IDetailsView.h"
#include "IDocumentation.h"
#include "IDocumentationPage.h"
#include "IFieldNotificationClassDescriptor.h"
#include "INotifyFieldValueChanged.h"
#include "ISequencerModule.h"
#include "Input/DragAndDrop.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "K2Node.h"
#include "K2Node_CallFunction.h"
#include "K2Node_ComponentBoundEvent.h"
#include "K2Node_Composite.h"
#include "K2Node_CustomEvent.h"
#include "K2Node_Event.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_FunctionResult.h"
#include "K2Node_FunctionTerminator.h"
#include "K2Node_MacroInstance.h"
#include "K2Node_MathExpression.h"
#include "K2Node_Tunnel.h"
#include "K2Node_Variable.h"
#include "K2Node_VariableGet.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/ChildActorComponentEditorUtils.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Layout/Margin.h"
#include "Layout/WidgetPath.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/UnitConversion.h"
#include "Math/Vector2D.h"
#include "Math/Vector4.h"
#include "Misc/Attribute.h"
#include "Misc/CString.h"
#include "Misc/Guid.h"
#include "Misc/MessageDialog.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "NodeFactory.h"
#include "ObjectEditorUtils.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorDelegates.h"
#include "PropertyHandle.h"
#include "PropertyRestriction.h"
#include "SBlueprintNamespaceEntry.h"
#include "SFieldNotificationCheckList.h"
#include "SGraphPin.h"
#include "SKismetInspector.h"
#include "SPinTypeSelector.h"
#include "SSubobjectBlueprintEditor.h"
#include "SSubobjectEditor.h"
#include "ScopedTransaction.h"
#include "Serialization/Archive.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "SSocketChooser.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "SubobjectData.h"
#include "SubobjectDataSubsystem.h"
#include "SupportedRangeTypes.h"	// StructsSupportingRangeVisibility
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Textures/SlateIcon.h"
#include "Tools/LegacyEdModeWidgetHelpers.h"
#include "Trace/Detail/Channel.h"
#include "Types/ISlateMetaData.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/CoreNetTypes.h"
#include "UObject/EnumProperty.h"
#include "UObject/Field.h"
#include "UObject/Interface.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/ReflectedTypeAccessors.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/StructOnScope.h"
#include "UObject/TextProperty.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/UObjectBaseUtility.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealType.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SToolTip.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"

class ITableRow;
class STableViewBase;
struct FGeometry;

#define LOCTEXT_NAMESPACE "BlueprintDetailsCustomization"

namespace BlueprintDocumentationDetailDefs
{
	/** Minimum size of the details title panel */
	static const float DetailsTitleMinWidth = 125.f;
	/** Maximum size of the details title panel */
	static const float DetailsTitleMaxWidth = 300.f;
	/** magic number retrieved from SGraphNodeComment::GetWrapAt() */
	static const float DetailsTitleWrapPadding = 32.0f;
	/** label of the option to reset the drop-down value function name */
	static const FString EmptyDropDownOptionName = TEXT("None");
};

void FBlueprintDetails::AddEventsCategory(IDetailLayoutBuilder& DetailBuilder, FName PropertyName, UClass* PropertyClass)
{
	UBlueprint* BlueprintObj = GetBlueprintObj();
	check(BlueprintObj);

	// Check for Ed Graph vars that can generate events
	if ( PropertyClass && BlueprintObj->AllowsDynamicBinding() )
	{
		// If the object property can't be resolved for the property, than we can't use it's events.
		FObjectProperty* VariableProperty = FindFProperty<FObjectProperty>(BlueprintObj->SkeletonGeneratedClass, PropertyName);

		if ( FBlueprintEditorUtils::CanClassGenerateEvents(PropertyClass) && VariableProperty )
		{
			for ( TFieldIterator<FMulticastDelegateProperty> PropertyIt(PropertyClass, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt )
			{
				FMulticastDelegateProperty* Property = *PropertyIt;

				static const FName HideInDetailPanelName("HideInDetailPanel");
				// Check for multicast delegates that we can safely assign
				if ( !Property->HasAnyPropertyFlags(CPF_Parm) && Property->HasAllPropertyFlags(CPF_BlueprintAssignable) &&
					!Property->HasMetaData(HideInDetailPanelName) )
				{
					FName EventName = Property->GetFName();
					FText EventText = Property->GetDisplayNameText();

					IDetailCategoryBuilder& EventCategory = DetailBuilder.EditCategory(TEXT("Events"), LOCTEXT("Events", "Events"), ECategoryPriority::Uncommon);

					EventCategory.AddCustomRow(EventText)
					.WholeRowContent()
					[
						SNew(SHorizontalBox)
						.ToolTipText(Property->GetToolTipText())

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(0.0f, 0.0f, 5.0f, 0.0f)
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("GraphEditor.Event_16x"))
						]

						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.Text(EventText)
						]

						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						.Padding(0.0f)
						[
							SNew(SButton)
							.ContentPadding(FMargin(3.0, 2.0))
							.OnClicked(this, &FBlueprintVarActionDetails::HandleAddOrViewEventForVariable, EventName, PropertyName, MakeWeakObjectPtr(PropertyClass))
							[
								SNew(SWidgetSwitcher)
								.WidgetIndex(this, &FBlueprintVarActionDetails::HandleAddOrViewIndexForButton, EventName, PropertyName)

								+ SWidgetSwitcher::Slot()
								[
									SNew(SImage)
									.ColorAndOpacity(FSlateColor::UseForeground())
									.Image(FAppStyle::Get().GetBrush("Icons.SelectInViewport"))
								]

								+ SWidgetSwitcher::Slot()
								[
									SNew(SImage)
									.ColorAndOpacity(FSlateColor::UseForeground())
									.Image(FAppStyle::Get().GetBrush("Icons.Plus"))
								]
							]
						]
					];
				}
			}
		}
	}
}

FReply FBlueprintDetails::HandleAddOrViewEventForVariable(const FName EventName, FName PropertyName, TWeakObjectPtr<UClass> PropertyClass)
{
	UBlueprint* BlueprintObj = GetBlueprintObj();

	// Find the corresponding variable property in the Blueprint
	FObjectProperty* VariableProperty = FindFProperty<FObjectProperty>(BlueprintObj->SkeletonGeneratedClass, PropertyName);

	if ( VariableProperty )
	{
		if ( !FKismetEditorUtilities::FindBoundEventForComponent(BlueprintObj, EventName, VariableProperty->GetFName()) )
		{
			FKismetEditorUtilities::CreateNewBoundEventForClass(PropertyClass.Get(), EventName, BlueprintObj, VariableProperty);
		}
		else
		{
			const UK2Node_ComponentBoundEvent* ExistingNode = FKismetEditorUtilities::FindBoundEventForComponent(BlueprintObj, EventName, VariableProperty->GetFName());
			if ( ExistingNode )
			{
				FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(ExistingNode);
			}
		}
	}

	return FReply::Handled();
}

int32 FBlueprintDetails::HandleAddOrViewIndexForButton(const FName EventName, FName PropertyName) const
{
	UBlueprint* BlueprintObj = GetBlueprintObj();

	if ( FKismetEditorUtilities::FindBoundEventForComponent(BlueprintObj, EventName, PropertyName) )
	{
		return 0; // View
	}

	return 1; // Add
}

FBlueprintVarActionDetails::~FBlueprintVarActionDetails()
{
	if(MyBlueprint.IsValid())
	{
		// Remove the callback delegate we registered for
		TWeakPtr<FBlueprintEditor> BlueprintEditor = MyBlueprint.Pin()->GetBlueprintEditor();
		if( BlueprintEditor.IsValid() )
		{
			BlueprintEditor.Pin()->OnRefresh().RemoveAll(this);
		}
	}
}

// FProperty Detail Customization
BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FBlueprintVarActionDetails::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{
	CachedVariableProperty = SelectionAsProperty();

	if(!CachedVariableProperty.IsValid())
	{
		return;
	}

	CachedVariableName = GetVariableName();

	TWeakPtr<FBlueprintEditor> BlueprintEditor = MyBlueprint.Pin()->GetBlueprintEditor();
	if( BlueprintEditor.IsValid() )
	{
		BlueprintEditor.Pin()->OnRefresh().AddSP(this, &FBlueprintVarActionDetails::OnPostEditorRefresh);
	}

	UBlueprint* BlueprintPtr = GetBlueprintObj();
	
	// Get an appropiate name validator
	TSharedPtr<INameValidatorInterface> NameValidator = nullptr;
	{
		const UEdGraphSchema* Schema = nullptr;		
		if (BlueprintPtr)
		{
			TArray<UEdGraph*> Graphs;
			BlueprintPtr->GetAllGraphs(Graphs);
			if (Graphs.Num() > 0)
			{
				Schema = Graphs[0]->GetSchema();
			}
		}			
	
		if (Schema)
		{
			NameValidator = Schema->GetNameValidator(BlueprintPtr, GetVariableName(), nullptr, FEdGraphSchemaAction_K2Var::StaticGetTypeId());	
		}
	}

	FProperty* VariableProperty = CachedVariableProperty.Get();

	// Cache the Blueprint which owns this VariableProperty
	if (UBlueprintGeneratedClass* GeneratedClass = Cast<UBlueprintGeneratedClass>(VariableProperty->GetOwnerClass()))
	{
		PropertyOwnerBlueprint = Cast<UBlueprint>(GeneratedClass->ClassGeneratedBy);
	}

	IDetailCategoryBuilder& Category = DetailLayout.EditCategory("Variable", LOCTEXT("VariableDetailsCategory", "Variable"));
	const FSlateFontInfo DetailFontInfo = IDetailLayoutBuilder::GetDetailFont();
	
	const FString DocLink = TEXT("Shared/Editors/BlueprintEditor/VariableDetails");

	TSharedPtr<SToolTip> VarNameTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("VarNameTooltip", "The name of the variable."), NULL, DocLink, TEXT("VariableName"));

	Category.AddCustomRow( LOCTEXT("BlueprintVarActionDetails_VariableNameLabel", "Variable Name") )
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("BlueprintVarActionDetails_VariableNameLabel", "Variable Name"))
		.ToolTip(VarNameTooltip)
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	.MaxDesiredWidth(250.0f)
	[
		SAssignNew(VarNameEditableTextBox, SEditableTextBox)
		.Text(this, &FBlueprintVarActionDetails::OnGetVarName)
		.ToolTip(VarNameTooltip)
		.OnTextChanged(this, &FBlueprintVarActionDetails::OnVarNameChanged)
		.OnTextCommitted(this, &FBlueprintVarActionDetails::OnVarNameCommitted)
		.OnVerifyTextChanged_Lambda([this, NameValidator](const FText& InNewText, FText& OutErrorMessage) -> bool
		{	
			if (NameValidator.IsValid())
			{
				EValidatorResult ValidatorResult = NameValidator->IsValid(InNewText.ToString());
				switch (ValidatorResult)
				{
					case EValidatorResult::Ok:
					case EValidatorResult::ExistingName:
						// These are fine, don't need to surface to the user, the rename can 'proceed' even if the name is the existing one
						return true;
					default:
						OutErrorMessage = INameValidatorInterface::GetErrorText(InNewText.ToString(), ValidatorResult);
						return false;
				}
			}
			
			return true;
		})
		.IsReadOnly(this, &FBlueprintVarActionDetails::GetVariableNameChangeEnabled)
		.Font(IDetailLayoutBuilder::GetDetailFont())
	];

	TSharedPtr<SToolTip> VarTypeTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("VarTypeTooltip", "The type of the variable."), NULL, DocLink, TEXT("VariableType"));

	TArray<TSharedPtr<IPinTypeSelectorFilter>> CustomPinTypeFilters;
	if (BlueprintEditor.IsValid())
	{
		BlueprintEditor.Pin()->GetPinTypeSelectorFilters(CustomPinTypeFilters);
	}
	
	const UEdGraphSchema* Schema = GetDefault<UEdGraphSchema_K2>();
	if (BlueprintEditor.IsValid())
	{
		if (BlueprintEditor.Pin()->GetFocusedGraph())
		{
			Schema = BlueprintEditor.Pin()->GetFocusedGraph()->GetSchema();
		}
	}

	Category.AddCustomRow(LOCTEXT("VariableTypeLabel", "Variable Type"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("VariableTypeLabel", "Variable Type"))
			.ToolTip(VarTypeTooltip)
			.Font(DetailFontInfo)
		]
		.ValueContent()
		.MaxDesiredWidth(980.f)
		[
			SNew(SPinTypeSelector, FGetPinTypeTree::CreateUObject(GetDefault<UEdGraphSchema_K2>(), &UEdGraphSchema_K2::GetVariableTypeTree))
			.TargetPinType(this, &FBlueprintVarActionDetails::OnGetVarType)
			.OnPinTypeChanged(this, &FBlueprintVarActionDetails::OnVarTypeChanged)
			.IsEnabled(this, &FBlueprintVarActionDetails::GetVariableTypeChangeEnabled)
			.Schema(Schema)
			.TypeTreeFilter(ETypeTreeFilter::None)
			.Font(DetailFontInfo)
			.ToolTip(VarTypeTooltip)
			.CustomFilters(CustomPinTypeFilters)
		]
		.AddCustomContextMenuAction(FUIAction(
			FExecuteAction::CreateRaw(this, &FBlueprintVarActionDetails::OnBrowseToVarType),
			FCanExecuteAction::CreateRaw(this, &FBlueprintVarActionDetails::CanBrowseToVarType)
			),
			LOCTEXT("BrowseToType", "Browse to Type"),
			LOCTEXT("BrowseToTypeToolTip", "Browse to this variable type in the Content Browser."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.BrowseContent")
		);

	TSharedPtr<SToolTip> ToolTipTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("VarToolTipTooltip", "Extra information about this variable, shown when cursor is over it."), NULL, DocLink, TEXT("Description"));

	Category.AddCustomRow( LOCTEXT("IsVariableToolTipLabel", "Description") )
	.Visibility(TAttribute<EVisibility>(this, &FBlueprintVarActionDetails::IsTooltipEditVisible))
	.NameContent()
	[
		SNew(STextBlock)
		.Text( LOCTEXT("IsVariableToolTipLabel", "Description") )
		.ToolTip(ToolTipTooltip)
		.Font( DetailFontInfo )
	]
	.ValueContent()
	.MinDesiredWidth(250.f)
	.MaxDesiredWidth(250.f)
	[
		SNew(SMultiLineEditableTextBox)
		.Text( this, &FBlueprintVarActionDetails::OnGetTooltipText )
		.ToolTipText( this, &FBlueprintVarActionDetails::OnGetTooltipText )
		.OnTextCommitted( this, &FBlueprintVarActionDetails::OnTooltipTextCommitted, CachedVariableName )
		.IsEnabled(IsVariableInBlueprint())
		.Font( DetailFontInfo )
		.ModiferKeyForNewLine(EModifierKey::Shift)
	];

	TSharedPtr<SToolTip> EditableTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("VarEditableTooltip", "Whether this variable is publicly editable on instances of this Blueprint."), NULL, DocLink, TEXT("Editable"));

	Category.AddCustomRow( LOCTEXT("IsVariableEditableLabel", "Instance Editable") )
	.Visibility(TAttribute<EVisibility>(this, &FBlueprintVarActionDetails::ShowEditableCheckboxVisibility))
	.NameContent()
	[
		SNew(STextBlock)
		.Text( LOCTEXT("IsVariableEditableLabel", "Instance Editable") )
		.ToolTip(EditableTooltip)
		.Font( IDetailLayoutBuilder::GetDetailFont() )
	]
	.ValueContent()
	[
		SNew(SCheckBox)
		.IsChecked( this, &FBlueprintVarActionDetails::OnEditableCheckboxState )
		.OnCheckStateChanged( this, &FBlueprintVarActionDetails::OnEditableChanged )
		.IsEnabled(IsVariableInBlueprint())
		.ToolTip(EditableTooltip)
	];
	
	TSharedPtr<SToolTip> ReadOnlyTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("VarReadOnlyTooltip", "Whether this variable can be set by Blueprint nodes or if it is read-only."), NULL, DocLink, TEXT("ReadOnly"));

	Category.AddCustomRow(LOCTEXT("IsVariableReadOnlyLabel", "Blueprint Read Only"))
	.Visibility(TAttribute<EVisibility>(this, &FBlueprintVarActionDetails::ShowReadOnlyCheckboxVisibility))
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("IsVariableReadOnlyLabel", "Blueprint Read Only"))
		.ToolTip(ReadOnlyTooltip)
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	[
		SNew(SCheckBox)
		.IsChecked(this, &FBlueprintVarActionDetails::OnReadyOnlyCheckboxState)
		.OnCheckStateChanged(this, &FBlueprintVarActionDetails::OnReadyOnlyChanged)
		.IsEnabled(IsVariableInBlueprint())
		.ToolTip(ReadOnlyTooltip)
	];
	
	if (BlueprintPtr && FBlueprintEditorUtils::ImplementsInterface(BlueprintPtr, true, UNotifyFieldValueChanged::StaticClass()) && !MyBlueprint.Pin()->SelectionAsDelegate())
	{
		// Show the flag if the class implement the interface but only allow the flag to be changed if the variable is defined in BP
		const FText ToolTip = LOCTEXT("FieldNotifyToolTip", "Generate a field entry for the Field Notification system.");
		TSharedPtr<SToolTip> FieldNotificationTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("FieldNotifyToolTip", "Generate a field entry for the Field Notification system."), NULL, DocLink, TEXT("FieldNotify"));

		Category.AddCustomRow(LOCTEXT("IsVariableFieldNotifyLabel", "Field Notify"))
			.NameContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("IsVariableFieldNotifyLabel", "Field Notify"))
				.ToolTip(FieldNotificationTooltip)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SCheckBox)
					.IsChecked(this, &FBlueprintVarActionDetails::OnFieldNotifyCheckboxState)
					.OnCheckStateChanged(this, &FBlueprintVarActionDetails::OnFieldNotifyChanged)
					.IsEnabled(IsVariableInBlueprint() && GetPropertyOwnerBlueprint() && IsABlueprintVariable(VariableProperty) && IsAUserVariable(VariableProperty))
					.ToolTip(FieldNotificationTooltip)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(UE::FieldNotification::SFieldNotificationCheckList)
					.FieldName(CachedVariableName)
					.BlueprintPtr(BlueprintPtr)
					.Visibility(this, &FBlueprintVarActionDetails::GetFieldNotifyCheckboxListVisibility)
				]

			];
	}

	TSharedPtr<SToolTip> Widget3DTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("VariableWidget3D_Tooltip", "When true, allows the user to tweak the vector variable by using a 3D transform widget in the viewport (usable when varible is public/enabled)."), NULL, DocLink, TEXT("Widget3D"));

	Category.AddCustomRow( LOCTEXT("VariableWidget3D_Prompt", "Show 3D Widget") )
	.Visibility(TAttribute<EVisibility>(this, &FBlueprintVarActionDetails::Show3DWidgetVisibility))
	.NameContent()
	[
		SNew(STextBlock)
		.ToolTip(Widget3DTooltip)
		.Text(LOCTEXT("VariableWidget3D_Prompt", "Show 3D Widget"))
		.Font( DetailFontInfo )
		.IsEnabled(Is3DWidgetEnabled())
	]
	.ValueContent()
	[
		SNew(SCheckBox)
		.IsChecked( this, &FBlueprintVarActionDetails::OnCreateWidgetCheckboxState )
		.OnCheckStateChanged( this, &FBlueprintVarActionDetails::OnCreateWidgetChanged )
		.IsEnabled(Is3DWidgetEnabled() && IsVariableInBlueprint())
		.ToolTip(Widget3DTooltip)
	];

	TSharedPtr<SToolTip> ExposeOnSpawnTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("VariableExposeToSpawn_Tooltip", "Should this variable be exposed as a pin when spawning this Blueprint?"), NULL, DocLink, TEXT("ExposeOnSpawn"));

	Category.AddCustomRow( LOCTEXT("VariableExposeToSpawnLabel", "Expose on Spawn") )
	.Visibility(TAttribute<EVisibility>(this, &FBlueprintVarActionDetails::ExposeOnSpawnVisibility))
	.NameContent()
	[
		SNew(STextBlock)
		.ToolTip(ExposeOnSpawnTooltip)
		.Text( LOCTEXT("VariableExposeToSpawnLabel", "Expose on Spawn") )
		.Font( DetailFontInfo )
	]
	.ValueContent()
	[
		SNew(SCheckBox)
		.IsChecked( this, &FBlueprintVarActionDetails::OnGetExposedToSpawnCheckboxState )
		.OnCheckStateChanged( this, &FBlueprintVarActionDetails::OnExposedToSpawnChanged )
		.IsEnabled(IsVariableInBlueprint())
		.ToolTip(ExposeOnSpawnTooltip)
	];

	TSharedPtr<SToolTip> PrivateTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("VariablePrivate_Tooltip", "Should this variable be private (derived blueprints cannot modify it)?"), NULL, DocLink, TEXT("Private"));

	Category.AddCustomRow(LOCTEXT("VariablePrivate", "Private"))
	.Visibility(TAttribute<EVisibility>(this, &FBlueprintVarActionDetails::ExposePrivateVisibility))
	.NameContent()
	[
		SNew(STextBlock)
		.ToolTip(PrivateTooltip)
		.Text( LOCTEXT("VariablePrivate", "Private") )
		.Font( DetailFontInfo )
	]
	.ValueContent()
	[
		SNew(SCheckBox)
		.IsChecked( this, &FBlueprintVarActionDetails::OnGetPrivateCheckboxState )
		.OnCheckStateChanged( this, &FBlueprintVarActionDetails::OnPrivateChanged )
		.IsEnabled(IsVariableInBlueprint())
		.ToolTip(PrivateTooltip)
	];

	TSharedPtr<SToolTip> ExposeToCinematicsTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("VariableExposeToCinematics_Tooltip", "Should this variable be exposed for Sequencer to modify?"), NULL, DocLink, TEXT("ExposeToCinematics"));

	Category.AddCustomRow( LOCTEXT("VariableExposeToCinematics", "Expose to Cinematics") )
	.Visibility(TAttribute<EVisibility>(this, &FBlueprintVarActionDetails::ExposeToCinematicsVisibility))
	.NameContent()
	[
		SNew(STextBlock)
		.ToolTip(ExposeToCinematicsTooltip)
		.Text( LOCTEXT("VariableExposeToCinematics", "Expose to Cinematics") )
		.Font( DetailFontInfo )
	]
	.ValueContent()
	[
		SNew(SCheckBox)
		.IsChecked( this, &FBlueprintVarActionDetails::OnGetExposedToCinematicsCheckboxState )
		.OnCheckStateChanged( this, &FBlueprintVarActionDetails::OnExposedToCinematicsChanged )
		.IsEnabled(IsVariableInBlueprint())
		.ToolTip(ExposeToCinematicsTooltip)
	];

	FText LocalizedTooltip;
	if (IsConfigCheckBoxEnabled())
	{
		// Build the property specific config variable tool tip
		FFormatNamedArguments ConfigTooltipArgs;
		if (UClass* OwnerClass = VariableProperty->GetOwnerClass())
		{
			const UClass* RealClass = OwnerClass->GetAuthoritativeClass();
			ConfigTooltipArgs.Add(TEXT("ConfigName"), FText::FromName(RealClass->ClassConfigName));
			ConfigTooltipArgs.Add(TEXT("ConfigSection"), FText::FromString(RealClass->GetPathName()));
		}
		LocalizedTooltip = FText::Format(LOCTEXT("VariableExposeToConfig_Tooltip", "Should this variable read its default value from a config file if it is present?\r\n\r\nThis is used for customizing variable default values and behavior between different projects and configurations.\r\n\r\nConfig file [{ConfigName}]\r\nConfig section [{ConfigSection}]"), ConfigTooltipArgs);
	}
	else if (IsVariableInBlueprint())
	{
		// mimics the error that UHT would throw
		LocalizedTooltip = LOCTEXT("ObjectVariableConfig_Tooltip", "Not allowed to use 'config' with object variables");
	}
	TSharedPtr<SToolTip> ExposeToConfigTooltip = IDocumentation::Get()->CreateToolTip(LocalizedTooltip, NULL, DocLink, TEXT("ExposeToConfig"));

	Category.AddCustomRow( LOCTEXT("VariableExposeToConfig", "Config Variable"), true )
	.Visibility(TAttribute<EVisibility>(this, &FBlueprintVarActionDetails::ExposeConfigVisibility))
	.NameContent()
	[
		SNew(STextBlock)
		.ToolTip( ExposeToConfigTooltip )
		.Text( LOCTEXT("ExposeToConfigLabel", "Config Variable") )
		.Font( DetailFontInfo )
	]
	.ValueContent()
	[
		SNew(SCheckBox)
		.ToolTip( ExposeToConfigTooltip )
		.IsChecked( this, &FBlueprintVarActionDetails::OnGetConfigVariableCheckboxState )
		.OnCheckStateChanged( this, &FBlueprintVarActionDetails::OnSetConfigVariableState )
		.IsEnabled(this, &FBlueprintVarActionDetails::IsConfigCheckBoxEnabled)
	];

	PopulateCategories(MyBlueprint.Pin().Get(), CategorySource);
	TSharedPtr<SComboButton> NewComboButton;
	TSharedPtr<SListView<TSharedPtr<FText>>> NewListView;

	TSharedPtr<SToolTip> CategoryTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("EditCategoryName_Tooltip", "The category of the variable; editing this will place the variable into another category or create a new one."), NULL, DocLink, TEXT("Category"));

	Category.AddCustomRow( LOCTEXT("CategoryLabel", "Category") )
		.Visibility(GetPropertyOwnerBlueprint()? EVisibility::Visible : EVisibility::Hidden)
	.NameContent()
	[
		SNew(STextBlock)
		.Text( LOCTEXT("CategoryLabel", "Category") )
		.ToolTip(CategoryTooltip)
		.Font( DetailFontInfo )
	]
	.ValueContent()
	[
		SAssignNew(NewComboButton, SComboButton)
		.ContentPadding(FMargin(0.0f, 0.0f, 5.0f, 0.0f))
		.IsEnabled(this, &FBlueprintVarActionDetails::GetVariableCategoryChangeEnabled)
		.ToolTip(CategoryTooltip)
		.ButtonContent()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("NoBorder") )
			.Padding(FMargin(0, 0, 5, 0))
			[
				SNew(SEditableTextBox)
					.Text(this, &FBlueprintVarActionDetails::OnGetCategoryText)
					.OnTextCommitted(this, &FBlueprintVarActionDetails::OnCategoryTextCommitted, CachedVariableName )
					.OnVerifyTextChanged_Lambda([&](const FText& InNewText, FText& OutErrorMessage) -> bool
					{
						if (InNewText.IsEmpty())
						{
							OutErrorMessage = LOCTEXT("CategoryEmpty", "Cannot add a category with an empty string.");
							return false;
						}
						if (InNewText.EqualTo(FText::FromString(GetBlueprintObj()->GetName())))
						{
							OutErrorMessage = LOCTEXT("CategoryEqualsBlueprintName", "Cannot add a category with the same name as the blueprint.");
							return false;
						}
						return true;
					})
					.ToolTip(CategoryTooltip)
					.SelectAllTextWhenFocused(true)
					.RevertTextOnEscape(true)
					.Font( DetailFontInfo )
			]
		]
		.MenuContent()
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.MaxHeight(400.0f)
			[
				SAssignNew(NewListView, SListView<TSharedPtr<FText>>)
					.ListItemsSource(&CategorySource)
					.OnGenerateRow(this, &FBlueprintVarActionDetails::MakeCategoryViewWidget)
					.OnSelectionChanged(this, &FBlueprintVarActionDetails::OnCategorySelectionChanged)
			]
		]
	];

	CategoryComboButton = NewComboButton;
	CategoryListView = NewListView;

	TSharedPtr<SToolTip> SliderRangeTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("SliderRange_Tooltip", "Allows setting the minimum and maximum values for the UI slider for this variable."), NULL, DocLink, TEXT("SliderRange"));

	FName UIMin = TEXT("UIMin");
	FName UIMax = TEXT("UIMax");
	Category.AddCustomRow( LOCTEXT("SliderRangeLabel", "Slider Range") )
	.Visibility(TAttribute<EVisibility>(this, &FBlueprintVarActionDetails::RangeVisibility))
	.NameContent()
	[
		SNew(STextBlock)
		.Text( LOCTEXT("SliderRangeLabel", "Slider Range") )
		.ToolTip(SliderRangeTooltip)
		.Font( DetailFontInfo )
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		.ToolTip(SliderRangeTooltip)
		+SHorizontalBox::Slot()
		.FillWidth(1)
		[
			SNew(SEditableTextBox)
			.Text(this, &FBlueprintVarActionDetails::OnGetMetaKeyValue, UIMin)
			.OnTextCommitted(this, &FBlueprintVarActionDetails::OnMetaKeyValueChanged, UIMin)
			.IsEnabled(IsVariableInBlueprint())
			.Font( DetailFontInfo )
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew( STextBlock )
			.Text( LOCTEXT("Min .. Max Separator", " .. ") )
			.Font(DetailFontInfo)
		]
		+SHorizontalBox::Slot()
		.FillWidth(1)
		[
			SNew(SEditableTextBox)
			.Text(this, &FBlueprintVarActionDetails::OnGetMetaKeyValue, UIMax)
			.OnTextCommitted(this, &FBlueprintVarActionDetails::OnMetaKeyValueChanged, UIMax)
			.IsEnabled(IsVariableInBlueprint())
			.Font( DetailFontInfo )
		]
	];

	TSharedPtr<SToolTip> ValueRangeTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("ValueRangeLabel_Tooltip", "The range of values allowed by this variable. Values outside of this will be clamped to the range."), NULL, DocLink, TEXT("ValueRange"));

	FName ClampMin = TEXT("ClampMin");
	FName ClampMax = TEXT("ClampMax");
	Category.AddCustomRow(LOCTEXT("ValueRangeLabel", "Value Range"))
	.Visibility(TAttribute<EVisibility>(this, &FBlueprintVarActionDetails::RangeVisibility))
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("ValueRangeLabel", "Value Range"))
		.ToolTipText(LOCTEXT("ValueRangeLabel_Tooltip", "The range of values allowed by this variable. Values outside of this will be clamped to the range."))
		.ToolTip(ValueRangeTooltip)
		.Font(DetailFontInfo)
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1)
		[
			SNew(SEditableTextBox)
			.Text(this, &FBlueprintVarActionDetails::OnGetMetaKeyValue, ClampMin)
			.OnTextCommitted(this, &FBlueprintVarActionDetails::OnMetaKeyValueChanged, ClampMin)
			.IsEnabled(IsVariableInBlueprint())
			.Font(DetailFontInfo)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Min .. Max Separator", " .. "))
			.Font(DetailFontInfo)
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1)
		[
			SNew(SEditableTextBox)
			.Text(this, &FBlueprintVarActionDetails::OnGetMetaKeyValue, ClampMax)
			.OnTextCommitted(this, &FBlueprintVarActionDetails::OnMetaKeyValueChanged, ClampMax)
			.IsEnabled(IsVariableInBlueprint())
			.Font(DetailFontInfo)
		]
	];
	
	TSharedPtr<SToolTip> UnitsTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("VarUnitsTooltip", "Units of this variable."), NULL, DocLink, TEXT("Units"));

	UnitsOptions.Empty();
	UnitsOptions.Add(MakeShareable(new FString("None")));
	for (const TCHAR* UnitsName : FUnitConversion::GetSupportedUnits())
	{
		UnitsOptions.Add(MakeShareable(new FString(UnitsName)));
	}
	
	Category.AddCustomRow(LOCTEXT("VariableUnitsLabel", "Units"))
	.Visibility(TAttribute<EVisibility>(this, &FBlueprintVarActionDetails::GetVariableUnitsVisibility))
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("VariableUnitsLabel", "Units"))
		.ToolTip(UnitsTooltip)
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	[
		SNew(STextComboBox)
			.OptionsSource( &UnitsOptions )
			.InitiallySelectedItem(GetVariableUnits())
			.OnSelectionChanged( this, &FBlueprintVarActionDetails::OnVariableUnitsChanged )
			.ToolTip(UnitsTooltip)
			.Font( DetailFontInfo )
	];

	TSharedPtr<SToolTip> BitmaskTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("VarBitmaskTooltip", "Whether or not to treat this variable as a bitmask."), nullptr, DocLink, TEXT("Bitmask"));

	Category.AddCustomRow(LOCTEXT("IsVariableBitmaskLabel", "Bitmask"))
	.Visibility(TAttribute<EVisibility>(this, &FBlueprintVarActionDetails::BitmaskVisibility))
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("IsVariableBitmaskLabel", "Bitmask"))
		.ToolTip(BitmaskTooltip)
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	[
		SNew(SCheckBox)
		.IsChecked(this, &FBlueprintVarActionDetails::OnBitmaskCheckboxState)
		.OnCheckStateChanged(this, &FBlueprintVarActionDetails::OnBitmaskChanged)
		.IsEnabled(IsVariableInBlueprint())
		.ToolTip(BitmaskTooltip)
	];

	BitmaskEnumTypePaths.Empty();
	BitmaskEnumTypePaths.Add(MakeShareable(new FTopLevelAssetPath())); // option to set the bitmask to None
	for (TObjectIterator<UEnum> EnumIt; EnumIt; ++EnumIt)
	{
		UEnum* CurrentEnum = *EnumIt;
		if (UEdGraphSchema_K2::IsAllowableBlueprintVariableType(CurrentEnum) && CurrentEnum->HasMetaData(TEXT("Bitflags")))
		{
			BitmaskEnumTypePaths.Add(MakeShareable(new FTopLevelAssetPath(CurrentEnum->GetPackage()->GetFName(), CurrentEnum->GetFName())));
		}
	}

	TSharedPtr<SToolTip> BitmaskEnumTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("VarBitmaskEnumTooltip", "If this is a bitmask, choose an optional enumeration type for the flags. Note that changing this will also reset the default value."), nullptr, DocLink, TEXT("Bitmask Flags"));
	
	Category.AddCustomRow(LOCTEXT("BitmaskEnumLabel", "Bitmask Enum"))
	.Visibility(TAttribute<EVisibility>(this, &FBlueprintVarActionDetails::BitmaskVisibility))
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("BitmaskEnumLabel", "Bitmask Enum"))
		.ToolTip(BitmaskEnumTooltip)
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	[
		SNew(SComboBox<TSharedPtr<FTopLevelAssetPath>>)
		.OptionsSource(&BitmaskEnumTypePaths)
		.InitiallySelectedItem(GetBitmaskEnumTypePath())
		.OnSelectionChanged(this, &FBlueprintVarActionDetails::OnBitmaskEnumTypeChanged)
		.OnGenerateWidget(this, &FBlueprintVarActionDetails::GenerateBitmaskEnumTypeWidget)
		.IsEnabled(IsVariableInBlueprint() && OnBitmaskCheckboxState() == ECheckBoxState::Checked)
		[
			SNew(STextBlock)
				.Text(this, &FBlueprintVarActionDetails::GetBitmaskEnumTypeName)
		]
	];

	ReplicationOptions.Empty();
	ReplicationOptions.Add(MakeShareable(new FString("None")));
	ReplicationOptions.Add(MakeShareable(new FString("Replicated")));
	ReplicationOptions.Add(MakeShareable(new FString("RepNotify")));

	TSharedPtr<SToolTip> ReplicationTooltip = IDocumentation::Get()->CreateToolTip( TAttribute<FText>::Create( TAttribute<FText>::FGetter::CreateRaw( this, &FBlueprintVarActionDetails::ReplicationTooltip ) ), NULL, DocLink, TEXT("Replication"));

	Category.AddCustomRow( LOCTEXT("VariableReplicationLabel", "Replication") )
	.Visibility(TAttribute<EVisibility>(this, &FBlueprintVarActionDetails::ReplicationVisibility))
	.NameContent()
	[
		SNew(STextBlock)
		.ToolTip(ReplicationTooltip)
		.Text( LOCTEXT("VariableReplicationLabel", "Replication") )
		.Font( DetailFontInfo )
	]
	.ValueContent()
	[
		SNew(STextComboBox)
		.OptionsSource( &ReplicationOptions )
		.InitiallySelectedItem(GetVariableReplicationType())
		.OnSelectionChanged( this, &FBlueprintVarActionDetails::OnChangeReplication )
		.IsEnabled(this, &FBlueprintVarActionDetails::ReplicationEnabled)
		.ToolTip(ReplicationTooltip)
	];

	ReplicationConditionEnumTypeNames.Empty();
	UEnum* Enum = StaticEnum<ELifetimeCondition>();
	check(Enum);
	
	for (int32 i = 0; i < Enum->NumEnums(); i++)
	{
		if (!Enum->HasMetaData(TEXT("Hidden"), i))
		{
			ReplicationConditionEnumTypeNames.Add(MakeShareable(new FString(Enum->GetDisplayNameTextByIndex(i).ToString())));
		}
	}

	Category.AddCustomRow(LOCTEXT("VariableReplicationConditionsLabel", "Replication Condition"))
	.Visibility(TAttribute<EVisibility>(this, &FBlueprintVarActionDetails::ReplicationVisibility))
	.NameContent()
	[
		SNew(STextBlock)
		.ToolTip(ReplicationTooltip)
		.Text(LOCTEXT("VariableReplicationConditionsLabel", "Replication Condition"))
		.Font(DetailFontInfo)
	]
	.ValueContent()
	[
		SNew(STextComboBox)
		.OptionsSource(&ReplicationConditionEnumTypeNames)
		.InitiallySelectedItem(GetVariableReplicationCondition())
		.OnSelectionChanged(this, &FBlueprintVarActionDetails::OnChangeReplicationCondition)
		.IsEnabled(this, &FBlueprintVarActionDetails::ReplicationConditionEnabled)
	];


	UBlueprint* BlueprintObj = GetBlueprintObj();

	// Handle event generation
	if ( FBlueprintEditorUtils::DoesSupportEventGraphs(BlueprintObj) )
	{	
		if (FObjectProperty* ComponentProperty = CastField<FObjectProperty>(VariableProperty))
		{
			AddEventsCategory(DetailLayout, ComponentProperty->GetFName(), ComponentProperty->PropertyClass);
		}		
	}

	// Add in default value editing for properties that can be edited, local properties cannot be edited
	if ((BlueprintObj != nullptr) && (BlueprintObj->GeneratedClass != nullptr))
	{
		bool bVariableRenamed = false;
		if (VariableProperty != nullptr && IsVariableInBlueprint())
		{
			// Determine the current property name on the CDO is stale
			if (PropertyOwnerBlueprint.IsValid() && VariableProperty)
			{
				UBlueprint* PropertyBlueprint = PropertyOwnerBlueprint.Get();
				const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(PropertyBlueprint, CachedVariableName);
				if (VarIndex != INDEX_NONE)
				{
					const FGuid VarGuid = PropertyBlueprint->NewVariables[VarIndex].VarGuid;
					if (UBlueprintGeneratedClass* AuthoritiveBPGC = Cast<UBlueprintGeneratedClass>(PropertyBlueprint->GeneratedClass))
					{
						if (const FName* OldName = AuthoritiveBPGC->PropertyGuids.FindKey(VarGuid))
						{
							bVariableRenamed = CachedVariableName != *OldName;
						}
					}
				}
			}
			const FProperty* OriginalProperty = nullptr;
		
			if(!IsALocalVariable(VariableProperty))
			{
				OriginalProperty = FindFProperty<FProperty>(BlueprintObj->GeneratedClass, VariableProperty->GetFName());
			}
			else
			{
				OriginalProperty = VariableProperty;
			}

			if (OriginalProperty == nullptr || bVariableRenamed)
			{
				// Prevent editing the default value of a skeleton property
				VariableProperty = nullptr;
			}
			else if (const FStructProperty* StructProperty = CastField<const FStructProperty>(OriginalProperty))
			{
				// Prevent editing the default value of a stale struct
				const UUserDefinedStruct* BGStruct = Cast<const UUserDefinedStruct>(StructProperty->Struct);
				if (BGStruct && (EUserDefinedStructureStatus::UDSS_UpToDate != BGStruct->Status))
				{
					VariableProperty = nullptr;
				}
			}
		}

		// Find the class containing the variable
		UClass* VariableClass = (VariableProperty ? VariableProperty->GetTypedOwner<UClass>() : nullptr);

		FText ErrorMessage;
		IDetailCategoryBuilder& DefaultValueCategory = DetailLayout.EditCategory(TEXT("DefaultValueCategory"), LOCTEXT("DefaultValueCategoryHeading", "Default Value"));

		if (VariableProperty == nullptr)
		{
			if (BlueprintObj->Status != BS_UpToDate)
			{
				ErrorMessage = LOCTEXT("VariableMissing_DirtyBlueprint", "Please compile the blueprint");
			}
			else
			{
				ErrorMessage = LOCTEXT("VariableMissing_CleanBlueprint", "Failed to find variable property");
			}
		}

		// Show the error message if something went wrong
		if (!ErrorMessage.IsEmpty())
		{
			DefaultValueCategory.AddCustomRow( ErrorMessage )
			.WholeRowWidget
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.ToolTipText(ErrorMessage)
				.Text(ErrorMessage)
				.Font(DetailFontInfo)
			];
		}
		else 
		{
			TSharedPtr<IDetailsView> DetailsView;
			if (BlueprintEditor.IsValid())
			{
				DetailsView = BlueprintEditor.Pin()->GetInspector()->GetPropertyView();
			}

			if(IsALocalVariable(VariableProperty))
			{
				UFunction* StructScope = VariableProperty->GetOwner<UFunction>();
				check(StructScope);

				UEdGraph* Graph = FBlueprintEditorUtils::FindScopeGraph(GetBlueprintObj(), (UFunction*)StructScope);

				// Find the function entry nodes in the current graph
				TArray<UK2Node_FunctionEntry*> EntryNodes;
				Graph->GetNodesOfClass(EntryNodes);

				// There should always be an entry node in the function graph
				check(EntryNodes.Num() > 0);
				
				UK2Node_FunctionEntry* FuncEntry = EntryNodes[0];

				TSharedPtr<FStructOnScope> StructData = MakeShareable(new FStructOnScope((UFunction*)StructScope));
				StructData->SetPackage(BlueprintObj->GetPackage());

				for (const FBPVariableDescription& LocalVar : FuncEntry->LocalVariables)
				{
					if (LocalVar.VarName == VariableProperty->GetFName())
					{
						// Only set the default value if there is one
						if (!LocalVar.DefaultValue.IsEmpty())
						{
							FBlueprintEditorUtils::PropertyValueFromString(VariableProperty, LocalVar.DefaultValue, StructData->GetStructMemory());
						}
						break;
					}
				}

				if (DetailsView.IsValid())
				{
					TWeakObjectPtr<UK2Node_EditablePinBase> EntryNode = FuncEntry;
					DetailsView->OnFinishedChangingProperties().AddSP(this, &FBlueprintVarActionDetails::OnFinishedChangingLocalVariable, StructData, EntryNode);
				}

				IDetailPropertyRow* Row = DefaultValueCategory.AddExternalStructureProperty(StructData, VariableProperty->GetFName());
			}
			else
			{
				UBlueprint* CurrPropertyOwnerBlueprint = IsVariableInheritedByBlueprint() ? GetBlueprintObj() : GetPropertyOwnerBlueprint();
				UObject* TargetBlueprintDefaultObject = nullptr;
				if (CurrPropertyOwnerBlueprint && CurrPropertyOwnerBlueprint->GeneratedClass)
				{
					TargetBlueprintDefaultObject = CurrPropertyOwnerBlueprint->GeneratedClass->GetDefaultObject();
				}
				else if (UBlueprint* PropertyOwnerBP = GetPropertyOwnerBlueprint())
				{
					TargetBlueprintDefaultObject = PropertyOwnerBP->GeneratedClass->GetDefaultObject();
				}
				else if (CachedVariableProperty.IsValid())
				{
					// Capture the non-BP class CDO so we can show the default value
					UClass* OwnerClass = CachedVariableProperty->GetOwnerClass();
					TargetBlueprintDefaultObject = OwnerClass ? OwnerClass->GetDefaultObject() : nullptr;
				}

				if (TargetBlueprintDefaultObject != nullptr)
				{
					// Things are in order, show the property and allow it to be edited
					TArray<UObject*> ObjectList;
					ObjectList.Add(TargetBlueprintDefaultObject);
					IDetailPropertyRow* Row = DefaultValueCategory.AddExternalObjectProperty(ObjectList, VariableProperty->GetFName());
					if (Row != nullptr)
					{
						Row->IsEnabled(IsVariableInheritedByBlueprint());
					}

					if (DetailsView.IsValid())
					{
						DetailsView->OnFinishedChangingProperties().AddSP(this, &FBlueprintVarActionDetails::OnFinishedChangingVariable);
					}
				}
			}
		}

		TSharedPtr<SToolTip> TransientTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("VariableTransient_Tooltip", "Should this variable not serialize and be zero-filled at load?"), NULL, DocLink, TEXT("Transient"));

		Category.AddCustomRow(LOCTEXT("VariableTransient", "Transient"), true)
			.Visibility(TAttribute<EVisibility>(this, &FBlueprintVarActionDetails::GetTransientVisibility))
			.NameContent()
		[
			SNew(STextBlock)
			.ToolTip(TransientTooltip)
			.Text( LOCTEXT("VariableTransient", "Transient") )
			.Font( DetailFontInfo )
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked( this, &FBlueprintVarActionDetails::OnGetTransientCheckboxState )
			.OnCheckStateChanged( this, &FBlueprintVarActionDetails::OnTransientChanged )
			.IsEnabled(IsVariableInBlueprint())
			.ToolTip(TransientTooltip)
		];

		TSharedPtr<SToolTip> SaveGameTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("VariableSaveGame_Tooltip", "Should this variable be serialized for saved games?"), NULL, DocLink, TEXT("SaveGame"));

		Category.AddCustomRow(LOCTEXT("VariableSaveGame", "SaveGame"), true)
		.Visibility(TAttribute<EVisibility>(this, &FBlueprintVarActionDetails::GetSaveGameVisibility))
		.NameContent()
		[
			SNew(STextBlock)
			.ToolTip(SaveGameTooltip)
			.Text( LOCTEXT("VariableSaveGame", "SaveGame") )
			.Font( DetailFontInfo )
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked( this, &FBlueprintVarActionDetails::OnGetSaveGameCheckboxState )
			.OnCheckStateChanged( this, &FBlueprintVarActionDetails::OnSaveGameChanged )
			.IsEnabled(IsVariableInBlueprint())
			.ToolTip(SaveGameTooltip)
		];

		TSharedPtr<SToolTip> AdvancedDisplayTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("VariableAdvancedDisplay_Tooltip", "Hide this variable in Class Defaults windows by default"), NULL, DocLink, TEXT("AdvancedDisplay"));

		Category.AddCustomRow(LOCTEXT("VariableAdvancedDisplay", "Advanced Display"), true)
			.Visibility(TAttribute<EVisibility>(this, &FBlueprintVarActionDetails::GetAdvancedDisplayVisibility))
			.NameContent()
			[
				SNew(STextBlock)
				.ToolTip(AdvancedDisplayTooltip)
				.Text(LOCTEXT("VariableAdvancedDisplay", "Advanced Display"))
				.Font(DetailFontInfo)
			]
		.ValueContent()
			[
				SNew(SCheckBox)
				.IsChecked(this, &FBlueprintVarActionDetails::OnGetAdvancedDisplayCheckboxState)
				.OnCheckStateChanged(this, &FBlueprintVarActionDetails::OnAdvancedDisplayChanged)
				.IsEnabled(IsVariableInBlueprint())
				.ToolTip(AdvancedDisplayTooltip)
			];

		TSharedPtr<SToolTip> MultilineTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("VariableMultilineTooltip_Tooltip", "Allow the value of this variable to have newlines (use Shift+Enter to add one while editing)"), NULL, DocLink, TEXT("Multiline"));

		Category.AddCustomRow(LOCTEXT("VariableMultilineTooltip", "Multi line"), true)
			.Visibility(TAttribute<EVisibility>(this, &FBlueprintVarActionDetails::GetMultilineVisibility))
			.NameContent()
			[
				SNew(STextBlock)
				.ToolTip(MultilineTooltip)
				.Text(LOCTEXT("VariableMultiline", "Multi line"))
				.Font(DetailFontInfo)
			]
		.ValueContent()
			[
				SNew(SCheckBox)
				.IsChecked(this, &FBlueprintVarActionDetails::OnGetMultilineCheckboxState)
				.OnCheckStateChanged(this, &FBlueprintVarActionDetails::OnMultilineChanged)
				.IsEnabled(IsVariableInBlueprint())
				.ToolTip(MultilineTooltip)
			];

		TSharedPtr<SToolTip> DeprecatedTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("VariableDeprecated_Tooltip", "Deprecate usage of this variable. Any nodes that reference it will produce a compiler warning indicating that it should be removed or replaced."), nullptr, DocLink, TEXT("Deprecated"));

		Category.AddCustomRow(LOCTEXT("VariableDeprecated", "Deprecated"), true)
			.Visibility(TAttribute<EVisibility>(this, &FBlueprintVarActionDetails::GetDeprecatedVisibility))
			.NameContent()
			[
				SNew(STextBlock)
				.ToolTip(DeprecatedTooltip)
				.Text(LOCTEXT("VariableDeprecated", "Deprecated"))
				.Font(DetailFontInfo)
			]
			.ValueContent()
			[
				SNew(SCheckBox)
				.IsChecked(this, &FBlueprintVarActionDetails::OnGetDeprecatedCheckboxState)
				.OnCheckStateChanged(this, &FBlueprintVarActionDetails::OnDeprecatedChanged)
				.IsEnabled(IsVariableInBlueprint())
				.ToolTip(DeprecatedTooltip)
			];

		TSharedPtr<SToolTip> DeprecationMessageTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("VariableDeprecationMessage_Tooltip", "Optional message to include with the deprecation compiler warning. For example: \'X is no longer being used. Please replace with Y.\'"), nullptr, DocLink, TEXT("DeprecationMessage"));

		Category.AddCustomRow(LOCTEXT("VariableDeprecationMessageLabel", "Deprecation Message"), true)
			.Visibility(TAttribute<EVisibility>(this, &FBlueprintVarActionDetails::GetDeprecatedVisibility))
			.IsEnabled(TAttribute<bool>(this, &FBlueprintVarActionDetails::IsVariableDeprecated))
			.NameContent()
			[
				SNew(STextBlock)
				.ToolTip(DeprecationMessageTooltip)
				.Text(LOCTEXT("VariableDeprecationMessageLabel", "Deprecation Message"))
				.Font(DetailFontInfo)
			]
			.ValueContent()
			[
				SNew(SEditableTextBox)
				.Text(this, &FBlueprintVarActionDetails::GetDeprecationMessageText)
				.OnTextCommitted(this, &FBlueprintVarActionDetails::OnDeprecationMessageTextCommitted, CachedVariableName)
				.IsEnabled(IsVariableInBlueprint())
				.ToolTipText(this, &FBlueprintVarActionDetails::GetDeprecationMessageText)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			];

		CollectDropDownOptions();

		TSharedPtr<SToolTip> GetOptionsTooltip = IDocumentation::Get()->CreateToolTip(
			LOCTEXT(
				"VariableGetOptions_Tooltip",
				"The name of the function which will populate a list of options the user can select from for the value of this variable."
			),
			nullptr, DocLink, TEXT("GetOptions")
		);

		TSharedPtr<SToolTip> GetOptionsInputTooltip = IDocumentation::Get()->CreateToolTip(
			LOCTEXT(
				"VariableGetOptions_InputTooltip",
				"List of functions that return an array of names or strings.\n"
				"You can also enter the name of a global static function, in the form of 'ClassName.FunctionName'."
			),
			nullptr, DocLink, TEXT("GetOptions")
		);

		Category.AddCustomRow(LOCTEXT("VariableGetOptions", "Drop-down Options"), true)
		.Visibility(TAttribute<EVisibility>(this, &FBlueprintVarActionDetails::GetDropDownOptionsVisibility))
		.NameContent()
		[
			SNew(STextBlock)
			.ToolTip(GetOptionsTooltip)
			.Text(LOCTEXT("VariableGetOptions", "Drop-down Options"))
			.Font(DetailFontInfo)
		]
		.ValueContent()
		[
			SNew(SSearchableComboBox)
			.ToolTip(GetOptionsInputTooltip)
			.OptionsSource(&DropDownFunctionOptions)
			.OnGenerateWidget(this, &FBlueprintVarActionDetails::GenerateDropDownOptionWidget)
			.OnSelectionChanged(this, &FBlueprintVarActionDetails::OnDropDownOptionSelectionChanged)
			.Content()
			[
				SNew(SEditableTextBox)
				.ToolTip(GetOptionsInputTooltip)
				.SelectAllTextWhenFocused(true)
				.RevertTextOnEscape(true)
				.Font(DetailFontInfo)
				.Text(this, &FBlueprintVarActionDetails::GetDropDownOptionDisplayText)
				.OnTextCommitted(this, &FBlueprintVarActionDetails::OnDropDownOptionTextChanged)
			]
		];

		TSharedPtr<SToolTip> PropertyFlagsTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("DefinedPropertyFlags_Tooltip", "List of defined flags for this property"), NULL, DocLink, TEXT("PropertyFlags"));

		Category.AddCustomRow(LOCTEXT("DefinedPropertyFlags", "Defined Property Flags"), true)
		.WholeRowWidget
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.ToolTip(PropertyFlagsTooltip)
			.Text( LOCTEXT("DefinedPropertyFlags", "Defined Property Flags") )
			.Font( IDetailLayoutBuilder::GetDetailFontBold() )
		];

		Category.AddCustomRow(FText::GetEmpty(), true)
		.WholeRowWidget
		[
			SAssignNew(PropertyFlagWidget, SListView< TSharedPtr< FString > >)
				.OnGenerateRow(this, &FBlueprintVarActionDetails::OnGenerateWidgetForPropertyList)
				.ListItemsSource(&PropertyFlags)
				.SelectionMode(ESelectionMode::None)
				.ScrollbarVisibility(EVisibility::Collapsed)
				.ToolTip(PropertyFlagsTooltip)
		];

		RefreshPropertyFlags();
	}

	// See if anything else wants to customize our details
	FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::GetModuleChecked<FBlueprintEditorModule>("Kismet");
	TArray<TSharedPtr<IDetailCustomization>> Customizations = BlueprintEditorModule.CustomizeVariable(CachedVariableProperty->GetClass(), BlueprintEditor.Pin());
	ExternalDetailCustomizations.Append(Customizations);
	for (TSharedPtr<IDetailCustomization> ExternalDetailCustomization : ExternalDetailCustomizations)
	{
		ExternalDetailCustomization->CustomizeDetails(DetailLayout);
	}
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void FBlueprintVarActionDetails::RefreshPropertyFlags()
{
	FProperty* VariableProperty = CachedVariableProperty.Get();
	if(VariableProperty)
	{
		PropertyFlags.Empty();
		for( const TCHAR* PropertyFlag : ParsePropertyFlags(VariableProperty->PropertyFlags) )
		{
			PropertyFlags.Add(MakeShareable<FString>(new FString(PropertyFlag)));
		}

		PropertyFlagWidget.Pin()->RequestListRefresh();
	}
}

TSharedRef<ITableRow> FBlueprintVarActionDetails::OnGenerateWidgetForPropertyList( TSharedPtr< FString > Item, const TSharedRef<STableViewBase>& OwnerTable )
{
	return SNew(STableRow< TSharedPtr< FString > >, OwnerTable)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			[
				SNew(STextBlock)
					.Text(FText::FromString(*Item.Get()))
					.ToolTipText(FText::FromString(*Item.Get()))
					.Font( IDetailLayoutBuilder::GetDetailFont() )
			]

			+SHorizontalBox::Slot()
				.AutoWidth()
			[
				SNew(SCheckBox)
					.IsChecked(ECheckBoxState::Checked)
					.IsEnabled(false)
			]
		];
}

bool FBlueprintVarActionDetails::IsAUserVariable(FProperty* VariableProperty) const
{
	FObjectProperty* VariableObjProp = VariableProperty ? CastField<FObjectProperty>(VariableProperty) : NULL;

	if (VariableObjProp != NULL && VariableObjProp->PropertyClass != NULL)
	{
		return FBlueprintEditorUtils::IsVariableCreatedByBlueprint(GetBlueprintObj(), VariableObjProp);
	}
	return true;
}

bool FBlueprintVarActionDetails::IsASCSVariable(FProperty* VariableProperty) const
{
	FObjectProperty* VariableObjProp = VariableProperty ? CastField<FObjectProperty>(VariableProperty) : NULL;
	if (VariableObjProp != NULL && VariableObjProp->PropertyClass != NULL)
	{
		return (!IsAUserVariable(VariableProperty) && VariableObjProp->PropertyClass->IsChildOf(UActorComponent::StaticClass()));
	}
	return false;
}

bool FBlueprintVarActionDetails::IsABlueprintVariable(FProperty* VariableProperty) const
{
	UClass* VarSourceClass = VariableProperty ? VariableProperty->GetOwner<UClass>() : NULL;
	if(VarSourceClass)
	{
		return (VarSourceClass->ClassGeneratedBy != NULL);
	}
	return false;
}

bool FBlueprintVarActionDetails::IsALocalVariable(FProperty* VariableProperty) const
{
	return VariableProperty && (VariableProperty->GetOwner<UFunction>() != NULL);
}

UStruct* FBlueprintVarActionDetails::GetLocalVariableScope(FProperty* VariableProperty) const
{
	if(IsALocalVariable(VariableProperty))
	{
		return VariableProperty->GetOwner<UFunction>();
	}

	return NULL;
}

bool FBlueprintVarActionDetails::GetVariableNameChangeEnabled() const
{
	bool bIsReadOnly = true;

	UBlueprint* BlueprintObj = GetBlueprintObj();
	check(BlueprintObj != nullptr);

	FProperty* VariableProperty = CachedVariableProperty.Get();
	if(VariableProperty != nullptr && IsVariableInBlueprint())
	{
		if(FBlueprintEditorUtils::FindNewVariableIndex(BlueprintObj, CachedVariableName) != INDEX_NONE)
		{
			bIsReadOnly = false;
		}
		else if(BlueprintObj->FindTimelineTemplateByVariableName(CachedVariableName))
		{
			bIsReadOnly = false;
		}
		else if(IsASCSVariable(VariableProperty) && BlueprintObj->SimpleConstructionScript != nullptr)
		{
			if (USCS_Node* Node = BlueprintObj->SimpleConstructionScript->FindSCSNode(CachedVariableName))
			{
				bIsReadOnly = !FComponentEditorUtils::IsValidVariableNameString(Node->ComponentTemplate, Node->GetVariableName().ToString());
			}
		}
		else if(IsALocalVariable(VariableProperty))
		{
			bIsReadOnly = false;
		}
	}

	return bIsReadOnly;
}

FText FBlueprintVarActionDetails::OnGetVarName() const
{
	return FText::FromName(CachedVariableName);
}

void FBlueprintVarActionDetails::OnVarNameChanged(const FText& InNewText)
{
	bIsVarNameInvalid = true;

	UBlueprint* BlueprintObj = GetBlueprintObj();
	check(BlueprintObj != nullptr);

	FProperty* VariableProperty = CachedVariableProperty.Get();
	if(VariableProperty && IsASCSVariable(VariableProperty) && BlueprintObj->SimpleConstructionScript != nullptr)
	{
		for (USCS_Node* Node : BlueprintObj->SimpleConstructionScript->GetAllNodes())
		{
			if (Node && Node->GetVariableName() == CachedVariableName && !FComponentEditorUtils::IsValidVariableNameString(Node->ComponentTemplate, InNewText.ToString()))
			{
				VarNameEditableTextBox->SetError(LOCTEXT("ComponentVariableRenameFailed_NotValid", "This name is reserved for engine use."));
				return;
			}
		}
	}

	TSharedPtr<INameValidatorInterface> NameValidator = MakeShareable(new FKismetNameValidator(BlueprintObj, CachedVariableName, GetLocalVariableScope(VariableProperty)));

	EValidatorResult ValidatorResult = NameValidator->IsValid(InNewText.ToString());
	if(ValidatorResult == EValidatorResult::AlreadyInUse)
	{
		VarNameEditableTextBox->SetError(FText::Format(LOCTEXT("RenameFailed_InUse", "{0} is in use by another variable or function!"), InNewText));
	}
	else if(ValidatorResult == EValidatorResult::EmptyName)
	{
		VarNameEditableTextBox->SetError(LOCTEXT("RenameFailed_LeftBlank", "Names cannot be left blank!"));
	}
	else if(ValidatorResult == EValidatorResult::TooLong)
	{
		VarNameEditableTextBox->SetError(FText::Format( LOCTEXT("RenameFailed_NameTooLong", "Names must have fewer than {0} characters!"), FText::AsNumber( FKismetNameValidator::GetMaximumNameLength())));
	}
	else if(ValidatorResult == EValidatorResult::LocallyInUse)
	{
		VarNameEditableTextBox->SetError(LOCTEXT("ConflictsWithProperty", "Conflicts with another local variable or function parameter!"));
	}
	else
	{
		bIsVarNameInvalid = false;
		VarNameEditableTextBox->SetError(FText::GetEmpty());
	}
}

void FBlueprintVarActionDetails::OnVarNameCommitted(const FText& InNewText, ETextCommit::Type InTextCommit)
{
	if(InTextCommit != ETextCommit::OnCleared && !bIsVarNameInvalid)
	{
		const FScopedTransaction Transaction( LOCTEXT( "RenameVariable", "Rename Variable" ) );

		FName NewVarName = FName(*InNewText.ToString());

		// Double check we're not renaming a timeline disguised as a variable
		bool bIsTimeline = false;

		FProperty* VariableProperty = CachedVariableProperty.Get();
		if (VariableProperty != NULL)
		{
			// Don't allow removal of timeline properties - you need to remove the timeline node for that
			FObjectProperty* ObjProperty = CastField<FObjectProperty>(VariableProperty);
			if(ObjProperty != NULL && ObjProperty->PropertyClass == UTimelineComponent::StaticClass())
			{
				bIsTimeline = true;
			}

			// Rename as a timeline if required
			if (bIsTimeline)
			{
				FBlueprintEditorUtils::RenameTimeline(GetBlueprintObj(), CachedVariableName, NewVarName);
			}
			else if(IsALocalVariable(VariableProperty))
			{
				UFunction* LocalVarScope = VariableProperty->GetOwner<UFunction>();
				FBlueprintEditorUtils::RenameLocalVariable(GetBlueprintObj(), LocalVarScope, CachedVariableName, NewVarName);
			}
			else
			{
				FBlueprintEditorUtils::RenameMemberVariable(GetBlueprintObj(), CachedVariableName, NewVarName);
			}

			check(MyBlueprint.IsValid());
			MyBlueprint.Pin()->SelectItemByName(NewVarName, ESelectInfo::OnMouseClick);
		}
	}

	bIsVarNameInvalid = false;
	VarNameEditableTextBox->SetError(FText::GetEmpty());
}

bool FBlueprintVarActionDetails::GetVariableTypeChangeEnabled() const
{
	FProperty* VariableProperty = CachedVariableProperty.Get();
	if(VariableProperty && !VariableProperty->IsA<FMulticastDelegateProperty>() && IsVariableInBlueprint())
	{
		if (!IsALocalVariable(VariableProperty))
		{
			if(GetBlueprintObj()->SkeletonGeneratedClass->GetAuthoritativeClass() != VariableProperty->GetOwnerClass()->GetAuthoritativeClass())
			{
				return false;
			}
			// If the variable belongs to this class and cannot be found in the member variable list, it is not editable (it may be a component)
			if (FBlueprintEditorUtils::FindNewVariableIndex(GetBlueprintObj(), CachedVariableName) == INDEX_NONE)
			{
				return false;
			}
		}
		return true;
	}
	return false;
}

bool FBlueprintVarActionDetails::GetVariableCategoryChangeEnabled() const
{
	FProperty* VariableProperty = CachedVariableProperty.Get();
	if(VariableProperty && IsVariableInBlueprint())
	{
		if(UClass* VarSourceClass = VariableProperty->GetOwner<UClass>())
		{
			// If the variable's source class is the same as the current blueprint's class then it was created in this blueprint and it's category can be changed.
			return VarSourceClass == GetBlueprintObj()->SkeletonGeneratedClass;
		}
		else if(IsALocalVariable(VariableProperty))
		{
			return true;
		}
	}

	return false;
}

FEdGraphPinType FBlueprintVarActionDetails::OnGetVarType() const
{
	FProperty* VariableProperty = CachedVariableProperty.Get();
	if (VariableProperty)
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		FEdGraphPinType Type;
		K2Schema->ConvertPropertyToPinType(VariableProperty, Type);
		return Type;
	}
	return FEdGraphPinType();
}

void FBlueprintVarActionDetails::OnVarTypeChanged(const FEdGraphPinType& NewPinType)
{
	if (FBlueprintEditorUtils::IsPinTypeValid(NewPinType))
	{
		FName VarName = CachedVariableName;

		if (VarName != NAME_None)
		{
			// Set the MyBP tab's last pin type used as this, for adding lots of variables of the same type
			MyBlueprint.Pin()->GetLastPinTypeUsed() = NewPinType;

			FProperty* VariableProperty = CachedVariableProperty.Get();
			if(VariableProperty)
			{
				if(IsALocalVariable(VariableProperty))
				{
					FBlueprintEditorUtils::ChangeLocalVariableType(GetBlueprintObj(), GetLocalVariableScope(VariableProperty), VarName, NewPinType);
				}
				else
				{
					FBlueprintEditorUtils::ChangeMemberVariableType(GetBlueprintObj(), VarName, NewPinType);
				}

				TSharedPtr<FBlueprintEditor> BlueprintEditor = MyBlueprint.Pin()->GetBlueprintEditor().Pin();
					
				// Auto-import the underlying type object's default namespace set into the current editor context.
				const UObject* PinSubCategoryObject = NewPinType.PinSubCategoryObject.Get();
				if (PinSubCategoryObject && BlueprintEditor.IsValid())
				{
					FBlueprintEditor::FImportNamespaceExParameters Params;
					FBlueprintNamespaceUtilities::GetDefaultImportsForObject(PinSubCategoryObject, Params.NamespacesToImport);
					BlueprintEditor->ImportNamespaceEx(Params);
				}
			}
		}
	}
}

FText FBlueprintVarActionDetails::OnGetTooltipText() const
{
	FName VarName = CachedVariableName;
	if (VarName != NAME_None)
	{
		if ( UBlueprint* OwnerBlueprint = GetPropertyOwnerBlueprint() )
		{
			FString Result;
			FBlueprintEditorUtils::GetBlueprintVariableMetaData(GetPropertyOwnerBlueprint(), VarName, GetLocalVariableScope(CachedVariableProperty.Get()), TEXT("tooltip"), Result);
			return FText::FromString(Result);
		}
	}
	return FText();
}

void FBlueprintVarActionDetails::OnTooltipTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit, FName VarName)
{
	FBlueprintEditorUtils::SetBlueprintVariableMetaData(GetBlueprintObj(), VarName, GetLocalVariableScope(CachedVariableProperty.Get()), TEXT("tooltip"), NewText.ToString() );
}

void FBlueprintVarActionDetails::PopulateCategories(SMyBlueprint* MyBlueprint, TArray<TSharedPtr<FText>>& CategorySource)
{
	auto IsNewCategorySource = [&CategorySource](const FText& NewCategory)
	{
		return !CategorySource.ContainsByPredicate([&NewCategory](const TSharedPtr<FText>& ExistingCategory)
		{
			return ExistingCategory->ToString().Equals(NewCategory.ToString(), ESearchCase::CaseSensitive);
		});
	};

	bool bShowUserVarsOnly = MyBlueprint->ShowUserVarsOnly();
	UBlueprint* Blueprint = MyBlueprint->GetBlueprintObj();
	check(Blueprint != NULL);
	if (Blueprint->SkeletonGeneratedClass == NULL)
	{
		UE_LOGF(LogBlueprint, Error, "Blueprint %ls has NULL SkeletonGeneratedClass in FBlueprintVarActionDetails::PopulateCategories().  Cannot Populate Categories.", *GetNameSafe(Blueprint));
		return;
	}

	check(Blueprint->SkeletonGeneratedClass != NULL);
	EFieldIteratorFlags::SuperClassFlags SuperClassFlag = EFieldIteratorFlags::ExcludeSuper;
	if(!bShowUserVarsOnly)
	{
		SuperClassFlag = EFieldIteratorFlags::IncludeSuper;
	}

	TArray<FName> VisibleVariables;
	for (TFieldIterator<FProperty> PropertyIt(Blueprint->SkeletonGeneratedClass, SuperClassFlag); PropertyIt; ++PropertyIt)
	{
		FProperty* Property = *PropertyIt;

		if ((!Property->HasAnyPropertyFlags(CPF_Parm) && Property->HasAllPropertyFlags(CPF_BlueprintVisible)))
		{
			VisibleVariables.Add(Property->GetFName());
		}
	}

	CategorySource.Reset();
	CategorySource.Add(MakeShared<FText>(UEdGraphSchema_K2::VR_DefaultCategory));
	FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>("Kismet");
	for (const UE::BlueprintEditor::FAdditionalCategory& Category : BlueprintEditorModule.GetAdditionalBlueprintCategories(Blueprint))
	{
		if (IsNewCategorySource(Category.Category))
		{
			CategorySource.Add(MakeShared<FText>(Category.Category));
		}
	}

	for (const FName& VariableName : VisibleVariables)
	{
		FText Category = FBlueprintEditorUtils::GetBlueprintVariableCategory(Blueprint, VariableName, nullptr);
		if (!Category.IsEmpty() && !Category.EqualTo(FText::FromString(Blueprint->GetName())))
		{
			if (IsNewCategorySource(Category))
			{
				CategorySource.Add(MakeShared<FText>(Category));
			}
		}
	}

	// Search through all function graphs for entry nodes to search for local variables to pull their categories
	for (UEdGraph* FunctionGraph : Blueprint->FunctionGraphs)
	{
		if(UFunction* Function = Blueprint->SkeletonGeneratedClass->FindFunctionByName(FunctionGraph->GetFName()))
		{
			FText FunctionCategory = FObjectEditorUtils::GetCategoryText(Function);

			if(!FunctionCategory.IsEmpty())
			{
				if (IsNewCategorySource(FunctionCategory))
				{
					CategorySource.Add(MakeShared<FText>(FunctionCategory));
				}
			}
		}

		UK2Node_EditablePinBase* EntryNode = FBlueprintEditorUtils::GetEntryNode(FunctionGraph);
		if (UK2Node_FunctionEntry* FunctionEntryNode = Cast<UK2Node_FunctionEntry>(EntryNode))
		{
			for (FBPVariableDescription& Variable : FunctionEntryNode->LocalVariables)
			{
				if (IsNewCategorySource(Variable.Category))
				{
					CategorySource.Add(MakeShared<FText>(Variable.Category));
				}
			}
		}
	}

	for (UEdGraph* MacroGraph : Blueprint->MacroGraphs)
	{
		UK2Node_EditablePinBase* EntryNode = FBlueprintEditorUtils::GetEntryNode(MacroGraph);
		if (UK2Node_Tunnel* TypedEntryNode = ExactCast<UK2Node_Tunnel>(EntryNode))
		{
			if (!TypedEntryNode->MetaData.Category.IsEmpty())
			{
				if (IsNewCategorySource(TypedEntryNode->MetaData.Category))
				{
					CategorySource.Add(MakeShared<FText>(TypedEntryNode->MetaData.Category));
				}
			}
		}
	}

	// Pull categories from overridable functions
	for (TFieldIterator<UFunction> FunctionIt(Blueprint->ParentClass, EFieldIteratorFlags::IncludeSuper); FunctionIt; ++FunctionIt)
	{
		const UFunction* Function = *FunctionIt;
		const FName FunctionName = Function->GetFName();

		if (UEdGraphSchema_K2::CanKismetOverrideFunction(Function) && !UEdGraphSchema_K2::FunctionCanBePlacedAsEvent(Function))
		{
			FText FunctionCategory = FObjectEditorUtils::GetCategoryText(Function);

			if (!FunctionCategory.IsEmpty())
			{
				if (IsNewCategorySource(FunctionCategory))
				{
					CategorySource.Add(MakeShared<FText>(FunctionCategory));
				}
			}
		}
	}

	// Sort categories, but keep the default category listed first
	CategorySource.Sort([](const TSharedPtr <FText> &LHS, const TSharedPtr <FText> &RHS)
	{
		if (LHS.IsValid() && RHS.IsValid())
		{
			return (LHS->EqualTo(UEdGraphSchema_K2::VR_DefaultCategory) || LHS->CompareToCaseIgnored(*RHS) <= 0);
		}
		return false;
	});
}

UK2Node_Variable* FBlueprintVarActionDetails::EdGraphSelectionAsVar() const
{
	TWeakPtr<FBlueprintEditor> BlueprintEditor = MyBlueprint.Pin()->GetBlueprintEditor();

	if( BlueprintEditor.IsValid() )
	{
		/** Get the currently selected set of nodes */
		FGraphPanelSelectionSet Objects = BlueprintEditor.Pin()->GetSelectedNodes();

		if (Objects.Num() == 1)
		{
			FGraphPanelSelectionSet::TIterator Iter(Objects);
			UObject* Object = *Iter;

			if (Object && Object->IsA<UK2Node_Variable>())
			{
				return Cast<UK2Node_Variable>(Object);
			}
		}
	}
	return nullptr;
}

FProperty* FBlueprintVarActionDetails::SelectionAsProperty() const
{
	if (FEdGraphSchemaAction_BlueprintVariableBase* BPVar = MyBlueprint.Pin()->SelectionAsBlueprintVariable())
	{
		return BPVar->GetProperty();
	}
	else if (UK2Node_Variable* GraphVar = EdGraphSelectionAsVar())
	{
		return GraphVar->GetPropertyForVariable();
	}

	return nullptr;
}

FName FBlueprintVarActionDetails::GetVariableName() const
{
	if (FEdGraphSchemaAction_BlueprintVariableBase* BPVar = MyBlueprint.Pin()->SelectionAsBlueprintVariable())
	{
		return BPVar->GetVariableName();
	}
	else if (UK2Node_Variable* GraphVar = EdGraphSelectionAsVar())
	{
		return GraphVar->GetVarName();
	}

	return NAME_None;
}

FText FBlueprintVarActionDetails::OnGetCategoryText() const
{
	FName VarName = CachedVariableName;
	if (VarName != NAME_None)
	{
		if ( UBlueprint* OwnerBlueprint = GetPropertyOwnerBlueprint() )
		{
			FText Category = FBlueprintEditorUtils::GetBlueprintVariableCategory(OwnerBlueprint, VarName, GetLocalVariableScope(CachedVariableProperty.Get()));

			// Older blueprints will have their name as the default category and whenever it is the same as the default category, display localized text
			if ( Category.EqualTo(FText::FromString(OwnerBlueprint->GetName())) || Category.EqualTo(UEdGraphSchema_K2::VR_DefaultCategory) )
			{
				return UEdGraphSchema_K2::VR_DefaultCategory;
			}
			else
			{
				return Category;
			}
		}

		return FText::FromName(VarName);
	}
	return FText();
}

void FBlueprintVarActionDetails::OnCategoryTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit, FName VarName)
{
	if (InTextCommit == ETextCommit::OnEnter || InTextCommit == ETextCommit::OnUserMovedFocus)
	{
		FBlueprintEditorUtils::SetBlueprintVariableCategory(GetBlueprintObj(), VarName, GetLocalVariableScope(CachedVariableProperty.Get()), NewText);
		check(MyBlueprint.IsValid());
		PopulateCategories(MyBlueprint.Pin().Get(), CategorySource);
		MyBlueprint.Pin()->ExpandCategory(NewText);
	}
}

TSharedRef< ITableRow > FBlueprintVarActionDetails::MakeCategoryViewWidget( TSharedPtr<FText> Item, const TSharedRef< STableViewBase >& OwnerTable )
{
	return SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
		[
			SNew(STextBlock) .Text(*Item.Get())
		];
}

void FBlueprintVarActionDetails::OnCategorySelectionChanged( TSharedPtr<FText> ProposedSelection, ESelectInfo::Type /*SelectInfo*/ )
{
	FName VarName = CachedVariableName;
	if (ProposedSelection.IsValid() && VarName != NAME_None)
	{
		FText NewCategory = *ProposedSelection.Get();

		FBlueprintEditorUtils::SetBlueprintVariableCategory(GetBlueprintObj(), VarName, GetLocalVariableScope(CachedVariableProperty.Get()), NewCategory );
		CategoryListView.Pin()->ClearSelection();
		CategoryComboButton.Pin()->SetIsOpen(false);
		MyBlueprint.Pin()->ExpandCategory(NewCategory);
	}
}

EVisibility FBlueprintVarActionDetails::ShowEditableCheckboxVisibility() const
{
	FProperty* VariableProperty = CachedVariableProperty.Get();
	if (VariableProperty && GetPropertyOwnerBlueprint())
	{
		if (IsABlueprintVariable(VariableProperty) && IsAUserVariable(VariableProperty))
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}

ECheckBoxState FBlueprintVarActionDetails::OnEditableCheckboxState() const
{
	FProperty* VariableProperty = CachedVariableProperty.Get();
	if (VariableProperty)
	{
		return VariableProperty->HasAnyPropertyFlags(CPF_DisableEditOnInstance) ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
	}
	return ECheckBoxState::Unchecked;
}

void FBlueprintVarActionDetails::OnEditableChanged(ECheckBoxState InNewState)
{
	FName VarName = CachedVariableName;

	// Toggle the flag on the blueprint's version of the variable description, based on state
	const bool bVariableIsExposed = InNewState == ECheckBoxState::Checked;

	UBlueprint* BlueprintObj = MyBlueprint.Pin()->GetBlueprintObj();
	FBlueprintEditorUtils::SetBlueprintOnlyEditableFlag(BlueprintObj, VarName, !bVariableIsExposed);
}

EVisibility FBlueprintVarActionDetails::ShowReadOnlyCheckboxVisibility() const
{
	FProperty* VariableProperty = CachedVariableProperty.Get();
	if (VariableProperty && GetPropertyOwnerBlueprint())
	{
		if (IsABlueprintVariable(VariableProperty) && IsAUserVariable(VariableProperty))
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}

ECheckBoxState FBlueprintVarActionDetails::OnReadyOnlyCheckboxState() const
{
	FProperty* VariableProperty = CachedVariableProperty.Get();
	if (VariableProperty)
	{
		return VariableProperty->HasAnyPropertyFlags(CPF_BlueprintReadOnly) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Unchecked;
}

void FBlueprintVarActionDetails::OnReadyOnlyChanged(ECheckBoxState InNewState)
{
	FName VarName = CachedVariableName;

	// Toggle the flag on the blueprint's version of the variable description, based on state
	const bool bVariableIsReadOnly = InNewState == ECheckBoxState::Checked;

	UBlueprint* BlueprintObj = MyBlueprint.Pin()->GetBlueprintObj();
	FBlueprintEditorUtils::SetBlueprintPropertyReadOnlyFlag(BlueprintObj, VarName, bVariableIsReadOnly);
}

EVisibility FBlueprintVarActionDetails::GetVariableUnitsVisibility() const
{
	FProperty* VariableProperty = CachedVariableProperty.Get();
	if (VariableProperty)
	{
		const bool bIsInteger = VariableProperty->IsA(FIntProperty::StaticClass()) || VariableProperty->IsA(FInt64Property::StaticClass());
		const bool bIsReal = VariableProperty->IsA(FFloatProperty::StaticClass()) || VariableProperty->IsA(FDoubleProperty::StaticClass());

		if (IsABlueprintVariable(VariableProperty) && !IsALocalVariable(VariableProperty) && (bIsInteger || bIsReal))
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Hidden;
}

TSharedPtr<FString> FBlueprintVarActionDetails::GetVariableUnits() const
{
	if (CachedVariableName != NAME_None)
	{
		if (const UBlueprint* BlueprintObj = GetPropertyOwnerBlueprint() )
		{
			FString Result;
			if (FBlueprintEditorUtils::GetBlueprintVariableMetaData(BlueprintObj, CachedVariableName, GetLocalVariableScope(CachedVariableProperty.Get()), "ForceUnits", /*out*/ Result))
			{
				for (const TSharedPtr<FString>& UnitOption : UnitsOptions)
				{
					if (*UnitOption == Result)
					{
						return UnitOption;
					}
				}
			}
		}
	}
	// Return none;
	return UnitsOptions.IsEmpty() ? MakeShareable(new FString("None")) : UnitsOptions[0];
}

void FBlueprintVarActionDetails::OnVariableUnitsChanged(TSharedPtr<FString> UnitsSelected, ESelectInfo::Type SelectInfo)
{
	if (CachedVariableName != NAME_None)
	{
		if ( UBlueprint* BlueprintObj = GetPropertyOwnerBlueprint() )
		{
			if (UnitsSelected && !UnitsOptions.IsEmpty() && UnitsSelected != UnitsOptions[0] )
			{
				FBlueprintEditorUtils::SetBlueprintVariableMetaData(BlueprintObj, CachedVariableName, GetLocalVariableScope(CachedVariableProperty.Get()), "ForceUnits", *UnitsSelected);
			}
			else
			{
				FBlueprintEditorUtils::RemoveBlueprintVariableMetaData(BlueprintObj, CachedVariableName, GetLocalVariableScope(CachedVariableProperty.Get()), "ForceUnits");
			}
		}
	}
}

ECheckBoxState FBlueprintVarActionDetails::OnFieldNotifyCheckboxState() const
{
	UBlueprint* const BlueprintObj = GetPropertyOwnerBlueprint();
	const FName VarName = CachedVariableName;

	if (!VarName.IsNone())
	{
		if (BlueprintObj)
		{
			const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(BlueprintObj, VarName);
			if (VarIndex != INDEX_NONE)
			{
				return BlueprintObj->NewVariables[VarIndex].HasMetaData(FBlueprintMetadata::MD_FieldNotify) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
		}
		
		const UClass* VarSourceClass = BlueprintObj ? *BlueprintObj->GeneratedClass : nullptr;
		if (VarSourceClass == nullptr && CachedVariableProperty.IsValid())
		{
			VarSourceClass = CachedVariableProperty->GetOwner<UClass>();
		}

		if (VarSourceClass && VarSourceClass->ImplementsInterface(UNotifyFieldValueChanged::StaticClass()) && VarSourceClass->GetDefaultObject())
		{
			TScriptInterface<INotifyFieldValueChanged> DefaultObject = VarSourceClass->GetDefaultObject();
			return DefaultObject->GetFieldNotificationDescriptor().GetField(VarSourceClass, VarName).IsValid() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
	}
	
	return ECheckBoxState::Unchecked;
}

void FBlueprintVarActionDetails::OnFieldNotifyChanged(ECheckBoxState InNewState)
{
	FName VarName = CachedVariableName;

	// Toggle the flag on the blueprint's version of the variable description, based on state
	const bool bVariableIsFieldNotify = InNewState == ECheckBoxState::Checked;

	UBlueprint* BlueprintObj = MyBlueprint.Pin()->GetBlueprintObj();
	if (bVariableIsFieldNotify)
	{
		// todo look through graph and check if the variable is used in a FieldNotify function, add that info to the metadata
		//maybe do that in PreCompileFunction
		FBlueprintEditorUtils::SetBlueprintVariableMetaData(BlueprintObj, VarName, GetLocalVariableScope(CachedVariableProperty.Get()), FBlueprintMetadata::MD_FieldNotify, TEXT(""));
	}
	else
	{
		FBlueprintEditorUtils::RemoveFieldNotifyFromAllMetadata(BlueprintObj, VarName);
		FBlueprintEditorUtils::RemoveBlueprintVariableMetaData(GetBlueprintObj(), VarName, GetLocalVariableScope(CachedVariableProperty.Get()), FBlueprintMetadata::MD_FieldNotify);
	}
}

EVisibility FBlueprintVarActionDetails::GetFieldNotifyCheckboxListVisibility() const
{
	UBlueprint* const BlueprintObj = GetBlueprintObj();
	const FName VarName = CachedVariableName;

	if (BlueprintObj && !VarName.IsNone())
	{
		// Only show the list of checkboxes in the details panel when this variable is field notify.
		const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(BlueprintObj, VarName);
		if (VarIndex != INDEX_NONE)
		{
			return BlueprintObj->NewVariables[VarIndex].HasMetaData(FBlueprintMetadata::MD_FieldNotify) ? EVisibility::Visible : EVisibility::Collapsed;
		}
	}
	return EVisibility::Collapsed;
}

ECheckBoxState FBlueprintVarActionDetails::OnCreateWidgetCheckboxState() const
{
	FProperty* Property = CachedVariableProperty.Get();
	if (Property)
	{
		bool bMakingWidget = FEdMode::ShouldCreateWidgetForProperty(Property);

		return bMakingWidget ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Unchecked;
}

void FBlueprintVarActionDetails::OnCreateWidgetChanged(ECheckBoxState InNewState)
{
	const FName VarName = CachedVariableName;
	if (VarName != NAME_None)
	{
		if (InNewState == ECheckBoxState::Checked)
		{
			FBlueprintEditorUtils::SetBlueprintVariableMetaData(GetBlueprintObj(), VarName, GetLocalVariableScope(CachedVariableProperty.Get()), FEdMode::MD_MakeEditWidget, TEXT("true"));
		}
		else
		{
			FBlueprintEditorUtils::RemoveBlueprintVariableMetaData(GetBlueprintObj(), VarName, GetLocalVariableScope(CachedVariableProperty.Get()), FEdMode::MD_MakeEditWidget);
		}
	}
}

EVisibility FBlueprintVarActionDetails::Show3DWidgetVisibility() const
{
	FProperty* VariableProperty = CachedVariableProperty.Get();
	if (VariableProperty && GetPropertyOwnerBlueprint())
	{
		if (IsABlueprintVariable(VariableProperty) && FEdMode::CanCreateWidgetForProperty(VariableProperty))
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}

bool FBlueprintVarActionDetails::Is3DWidgetEnabled()
{
	FProperty* VariableProperty = CachedVariableProperty.Get();
	if (VariableProperty)
	{
		return ( VariableProperty && !VariableProperty->HasAnyPropertyFlags(CPF_DisableEditOnInstance) ) ;
	}
	return false;
}

ECheckBoxState FBlueprintVarActionDetails::OnGetExposedToSpawnCheckboxState() const
{
	FProperty* Property = CachedVariableProperty.Get();
	if (Property)
	{
		return (Property && Property->GetBoolMetaData(FBlueprintMetadata::MD_ExposeOnSpawn) != false) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Unchecked;
}

void FBlueprintVarActionDetails::OnExposedToSpawnChanged(ECheckBoxState InNewState)
{
	const FName VarName = CachedVariableName;
	if (VarName != NAME_None)
	{
		const bool bExposeOnSpawn = (InNewState == ECheckBoxState::Checked);
		if(bExposeOnSpawn)
		{
			FBlueprintEditorUtils::SetBlueprintVariableMetaData(GetBlueprintObj(), VarName, NULL, FBlueprintMetadata::MD_ExposeOnSpawn, TEXT("true"));
		}
		else
		{
			FBlueprintEditorUtils::RemoveBlueprintVariableMetaData(GetBlueprintObj(), VarName, NULL, FBlueprintMetadata::MD_ExposeOnSpawn);
		} 
	}
}

EVisibility FBlueprintVarActionDetails::ExposeOnSpawnVisibility() const
{
	FProperty* VariableProperty = CachedVariableProperty.Get();
	if (VariableProperty && GetPropertyOwnerBlueprint())
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		FEdGraphPinType VariablePinType;
		K2Schema->ConvertPropertyToPinType(VariableProperty, VariablePinType);

		const bool bShowPrivacySetting = IsABlueprintVariable(VariableProperty) && IsAUserVariable(VariableProperty);
		if (bShowPrivacySetting && (K2Schema->FindSetVariableByNameFunction(VariablePinType) != NULL))
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}

ECheckBoxState FBlueprintVarActionDetails::OnGetPrivateCheckboxState() const
{
	FProperty* Property = CachedVariableProperty.Get();
	if (Property)
	{
		return (Property && Property->GetBoolMetaData(FBlueprintMetadata::MD_Private) != false) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Unchecked;
}

void FBlueprintVarActionDetails::OnPrivateChanged(ECheckBoxState InNewState)
{
	const FName VarName = CachedVariableName;
	if (VarName != NAME_None)
	{
		const bool bExposeOnSpawn = (InNewState == ECheckBoxState::Checked);
		if(bExposeOnSpawn)
		{
			FBlueprintEditorUtils::SetBlueprintVariableMetaData(GetBlueprintObj(), VarName, NULL, FBlueprintMetadata::MD_Private, TEXT("true"));
		}
		else
		{
			FBlueprintEditorUtils::RemoveBlueprintVariableMetaData(GetBlueprintObj(), VarName, NULL, FBlueprintMetadata::MD_Private);
		}
	}
}

EVisibility FBlueprintVarActionDetails::ExposePrivateVisibility() const
{
	FProperty* Property = CachedVariableProperty.Get();
	if (Property && GetPropertyOwnerBlueprint())
	{
		if (IsABlueprintVariable(Property) && IsAUserVariable(Property))
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}

ECheckBoxState FBlueprintVarActionDetails::OnGetExposedToCinematicsCheckboxState() const
{
	FProperty* Property = CachedVariableProperty.Get();
	if (Property)
	{
		return Property && Property->HasAnyPropertyFlags(CPF_Interp) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Unchecked;
}

void FBlueprintVarActionDetails::OnExposedToCinematicsChanged(ECheckBoxState InNewState)
{
	// Toggle the flag on the blueprint's version of the variable description, based on state
	const bool bExposeToCinematics = (InNewState == ECheckBoxState::Checked);
	
	const FName VarName = CachedVariableName;
	if (VarName != NAME_None)
	{
		FBlueprintEditorUtils::SetInterpFlag(GetBlueprintObj(), VarName, bExposeToCinematics);
	}
}

EVisibility FBlueprintVarActionDetails::ExposeToCinematicsVisibility() const
{
	FProperty* VariableProperty = CachedVariableProperty.Get();
	if (VariableProperty && !IsALocalVariable(VariableProperty))
	{
		const bool bIsInteger = VariableProperty->IsA(FIntProperty::StaticClass());
		const bool bIsByte = VariableProperty->IsA(FByteProperty::StaticClass());
		const bool bIsEnum = VariableProperty->IsA(FEnumProperty::StaticClass());
		const bool bIsFloat = VariableProperty->IsA(FFloatProperty::StaticClass());
		const bool bIsBool = VariableProperty->IsA(FBoolProperty::StaticClass());
		const bool bIsStr = VariableProperty->IsA(FStrProperty::StaticClass());
		
		const FStructProperty* AsStructProperty = CastField<FStructProperty>(VariableProperty);
		const bool bIsVectorStruct = AsStructProperty != nullptr && AsStructProperty->Struct->GetFName() == NAME_Vector;
		const bool bIsTransformStruct = AsStructProperty != nullptr && AsStructProperty->Struct->GetFName() == NAME_Transform;
		const bool bIsColorStruct = AsStructProperty != nullptr && AsStructProperty->Struct->GetFName() == NAME_Color;
		const bool bIsLinearColorStruct = AsStructProperty != nullptr && AsStructProperty->Struct->GetFName() == NAME_LinearColor;
		
		const FObjectProperty* AsObjectProperty = CastField<FObjectProperty>(VariableProperty);
		const bool bIsActorProperty = AsObjectProperty != nullptr && AsObjectProperty->PropertyClass && AsObjectProperty->PropertyClass->IsChildOf(AActor::StaticClass());

		if (bIsInteger || bIsByte || bIsEnum || bIsFloat || bIsBool || bIsStr || bIsVectorStruct || bIsTransformStruct || bIsColorStruct || bIsLinearColorStruct || bIsActorProperty)
		{
			return EVisibility::Visible;
		}
		else
		{
			ISequencerModule* SequencerModule = FModuleManager::Get().GetModulePtr<ISequencerModule>("Sequencer");
			if (SequencerModule->CanAnimateProperty(VariableProperty))
			{
				return EVisibility::Visible;
			}
		}
	}
	return EVisibility::Collapsed;
}

TSharedPtr<FString> FBlueprintVarActionDetails::GetVariableReplicationCondition() const
{
	ELifetimeCondition VariableRepCondition = COND_None;
		
	const FProperty* const Property = CachedVariableProperty.Get();

	if (Property)
	{
		VariableRepCondition = Property->GetBlueprintReplicationCondition();
	}

	return ReplicationConditionEnumTypeNames[(uint8)VariableRepCondition];
}

void FBlueprintVarActionDetails::OnChangeReplicationCondition(TSharedPtr<FString> ItemSelected, ESelectInfo::Type SelectInfo)
{
	int32 NewSelection;
	const bool bFound = ReplicationConditionEnumTypeNames.Find(ItemSelected, NewSelection);
	check(bFound && NewSelection != INDEX_NONE);

	const ELifetimeCondition NewRepCondition = (ELifetimeCondition)NewSelection;

	UBlueprint* const BlueprintObj = GetBlueprintObj();
	const FName VarName = CachedVariableName;

	if (BlueprintObj && VarName != NAME_None)
	{
		const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(BlueprintObj, VarName);

		if (VarIndex != INDEX_NONE)
		{
			BlueprintObj->NewVariables[VarIndex].ReplicationCondition = NewRepCondition;
		
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(BlueprintObj);
		}
	}

}

bool FBlueprintVarActionDetails::ReplicationConditionEnabled() const
{
	const FProperty* const VariableProperty = CachedVariableProperty.Get();
	if (VariableProperty)
	{
		const uint64 *PropFlagPtr = FBlueprintEditorUtils::GetBlueprintVariablePropertyFlags(GetBlueprintObj(), VariableProperty->GetFName());
		uint64 PropFlags = 0;

		if (PropFlagPtr != nullptr)
		{
			PropFlags = *PropFlagPtr;
			return (PropFlags & CPF_Net) > 0;
			
		}
	}

	return false;
}

bool FBlueprintVarActionDetails::ReplicationEnabled() const
{
	// Update FBlueprintVarActionDetails::ReplicationTooltip if you alter this function
	// so that users can understand why replication settings are disabled!
	bool bVariableCanBeReplicated = true;
	const FProperty* const VariableProperty = CachedVariableProperty.Get();
	if (VariableProperty)
	{
		// sets and maps cannot yet be replicated, neither can Event Dispatchers:
		bVariableCanBeReplicated =
			CastField<FSetProperty>(VariableProperty) == nullptr &&
			CastField<FMapProperty>(VariableProperty) == nullptr &&
			CastField<FMulticastInlineDelegateProperty>(VariableProperty) == nullptr;
	}
	return bVariableCanBeReplicated && IsVariableInBlueprint();
}

FText FBlueprintVarActionDetails::ReplicationTooltip() const
{
	if (ReplicationEnabled())
	{
		return LOCTEXT("VariableReplicate_Tooltip", "Should this Variable be replicated over the network?");
	}
	else
	{
		const FProperty* const VariableProperty = CachedVariableProperty.Get();
		if (CastField<FSetProperty>(VariableProperty) || CastField<FMapProperty>(VariableProperty))
		{
			return LOCTEXT("VariableReplicateDisabledSetsAndMaps_Tooltip", "Set and Map properties cannot be replicated");
		}
		else if (CastField<FMulticastInlineDelegateProperty>(VariableProperty))
		{
			return LOCTEXT("VariableReplicateDisabledEventDispatchers_Tooltip", "Event Dispatcher properties cannot be replicated");
		}
		else
		{
			return LOCTEXT("VariableReplicateDisabledDefault_Tooltip", "This property type cannot be replicated");
		}
	}
}

ECheckBoxState FBlueprintVarActionDetails::OnGetConfigVariableCheckboxState() const
{
	UBlueprint* BlueprintObj = GetPropertyOwnerBlueprint();
	const FName VarName = CachedVariableName;
	ECheckBoxState CheckboxValue = ECheckBoxState::Unchecked;

	if( BlueprintObj && VarName != NAME_None )
	{
		const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex( BlueprintObj, VarName );

		if( VarIndex != INDEX_NONE && BlueprintObj->NewVariables[ VarIndex ].PropertyFlags & CPF_Config )
		{
			CheckboxValue = ECheckBoxState::Checked;
		}
	}
	return CheckboxValue;
}

void FBlueprintVarActionDetails::OnSetConfigVariableState( ECheckBoxState InNewState )
{
	UBlueprint* BlueprintObj = GetBlueprintObj();
	const FName VarName = CachedVariableName;

	if( BlueprintObj && VarName != NAME_None )
	{
		const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex( BlueprintObj, VarName );

		if( VarIndex != INDEX_NONE )
		{
			if( InNewState == ECheckBoxState::Checked )
			{
				BlueprintObj->NewVariables[ VarIndex ].PropertyFlags |= CPF_Config;
			}
			else
			{
				BlueprintObj->NewVariables[ VarIndex ].PropertyFlags &= ~CPF_Config;
			}
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified( BlueprintObj );
		}
	}
}

EVisibility FBlueprintVarActionDetails::ExposeConfigVisibility() const
{
	FProperty* Property = CachedVariableProperty.Get();
	if (Property)
	{
		if (IsABlueprintVariable(Property) && IsAUserVariable(Property))
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}

bool FBlueprintVarActionDetails::IsConfigCheckBoxEnabled() const
{
	bool bEnabled = IsVariableInBlueprint();
	if (bEnabled && CachedVariableProperty.IsValid())
	{
		if (FProperty* VariableProperty = CachedVariableProperty.Get())
		{
			// meant to match up with UHT's FPropertyBase::IsObject(), which it uses to block object properties from being marked with CPF_Config
			bEnabled = VariableProperty->IsA<FClassProperty>() || VariableProperty->IsA<FSoftClassProperty>() || VariableProperty->IsA<FSoftObjectProperty>() ||
				(!VariableProperty->IsA<FObjectPropertyBase>() && !VariableProperty->IsA<FInterfaceProperty>());
		}
	}
	return bEnabled;
}

FText FBlueprintVarActionDetails::OnGetMetaKeyValue(FName Key) const
{
	FName VarName = CachedVariableName;
	if (VarName != NAME_None)
	{
		if ( UBlueprint* BlueprintObj = GetPropertyOwnerBlueprint() )
		{
			FString Result;
			FBlueprintEditorUtils::GetBlueprintVariableMetaData(BlueprintObj, VarName, GetLocalVariableScope(CachedVariableProperty.Get()), Key, /*out*/ Result);

			return FText::FromString(Result);
		}
	}
	return FText();
}

void FBlueprintVarActionDetails::OnMetaKeyValueChanged(const FText& NewMinValue, ETextCommit::Type CommitInfo, FName Key)
{
	FName VarName = CachedVariableName;
	if (VarName != NAME_None)
	{
		if ((CommitInfo == ETextCommit::OnEnter) || (CommitInfo == ETextCommit::OnUserMovedFocus))
		{
			FBlueprintEditorUtils::SetBlueprintVariableMetaData(GetBlueprintObj(), VarName, GetLocalVariableScope(CachedVariableProperty.Get()), Key, NewMinValue.ToString());
		}
	}
}

EVisibility FBlueprintVarActionDetails::RangeVisibility() const
{
	FProperty* VariableProperty = CachedVariableProperty.Get();
	if (VariableProperty)
	{
		const bool bIsInteger = VariableProperty->IsA(FIntProperty::StaticClass());
		const bool bIsNonEnumByte = (VariableProperty->IsA(FByteProperty::StaticClass()) && CastField<const FByteProperty>(VariableProperty)->Enum == nullptr);
		const bool bIsReal = (VariableProperty->IsA(FFloatProperty::StaticClass()) || VariableProperty->IsA(FDoubleProperty::StaticClass()));

		// If this is a struct property than we must check the name of the struct it points to, so we can check
		// if it supports the editing of the UIMin/UIMax metadata
		const FStructProperty* StructProp = CastField<FStructProperty>(VariableProperty);
		const UStruct* InnerStruct = StructProp ? StructProp->Struct.Get() : nullptr;
		const bool bIsSupportedStruct = InnerStruct ? RangeVisibilityUtils::StructsSupportingRangeVisibility.Contains(InnerStruct->GetFName()) : false;

		if (IsABlueprintVariable(VariableProperty) && (bIsInteger || bIsNonEnumByte || bIsReal || bIsSupportedStruct))
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}

EVisibility FBlueprintVarActionDetails::BitmaskVisibility() const
{
	FProperty* VariableProperty = CachedVariableProperty.Get();
	if (VariableProperty && IsABlueprintVariable(VariableProperty) && VariableProperty->IsA(FIntProperty::StaticClass()))
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

ECheckBoxState FBlueprintVarActionDetails::OnBitmaskCheckboxState() const
{
	FProperty* Property = CachedVariableProperty.Get();
	if (Property)
	{
		return (Property && Property->HasMetaData(FBlueprintMetadata::MD_Bitmask)) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Unchecked;
}

void FBlueprintVarActionDetails::OnBitmaskChanged(ECheckBoxState InNewState)
{
	const FName VarName = CachedVariableName;
	if (VarName != NAME_None)
	{
		UBlueprint* LocalBlueprint = GetBlueprintObj();

		const bool bIsBitmask = (InNewState == ECheckBoxState::Checked);
		if (bIsBitmask)
		{
			FBlueprintEditorUtils::SetBlueprintVariableMetaData(LocalBlueprint, VarName, nullptr, FBlueprintMetadata::MD_Bitmask, TEXT(""));
		}
		else
		{
			FBlueprintEditorUtils::RemoveBlueprintVariableMetaData(LocalBlueprint, VarName, nullptr, FBlueprintMetadata::MD_Bitmask);
		}

		// Reset default value
		if (LocalBlueprint->GeneratedClass)
		{
			UObject* CDO = LocalBlueprint->GeneratedClass->GetDefaultObject(false);
			FProperty* VarProperty = FindFProperty<FProperty>(LocalBlueprint->GeneratedClass, VarName);

			if (CDO != nullptr && VarProperty != nullptr)
			{
				VarProperty->InitializeValue_InContainer(CDO);
			}
		}

		TArray<UK2Node_Variable*> VariableNodes;
		FBlueprintEditorUtils::GetAllNodesOfClass<UK2Node_Variable>(GetBlueprintObj(), VariableNodes);

		for (TArray<UK2Node_Variable*>::TConstIterator NodeIt(VariableNodes); NodeIt; ++NodeIt)
		{
			UK2Node_Variable* CurrentNode = *NodeIt;
			if (VarName == CurrentNode->GetVarName())
			{
				CurrentNode->ReconstructNode();
			}
		}
	}
}

TSharedPtr<FTopLevelAssetPath> FBlueprintVarActionDetails::GetBitmaskEnumTypePath() const
{
	TSharedPtr<FTopLevelAssetPath> Result;
	const FName VarName = CachedVariableName;

	if (BitmaskEnumTypePaths.Num() > 0 && VarName != NAME_None)
	{
		Result = BitmaskEnumTypePaths[0];

		FString OutValue;
		FBlueprintEditorUtils::GetBlueprintVariableMetaData(GetBlueprintObj(), VarName, nullptr, FBlueprintMetadata::MD_BitmaskEnum, OutValue);
		
		for (int32 i = 1; i < BitmaskEnumTypePaths.Num(); ++i)
		{
			if (OutValue == BitmaskEnumTypePaths[i]->ToString())
			{
				Result = BitmaskEnumTypePaths[i];
				break;
			}
		}
	}

	return Result;
}

void FBlueprintVarActionDetails::OnBitmaskEnumTypeChanged(TSharedPtr<FTopLevelAssetPath> ItemSelected, ESelectInfo::Type SelectInfo)
{
	const FName VarName = CachedVariableName;
	if (VarName != NAME_None)
	{
		UBlueprint* LocalBlueprint = GetBlueprintObj();

		if (ItemSelected == BitmaskEnumTypePaths[0])
		{
			FBlueprintEditorUtils::RemoveBlueprintVariableMetaData(LocalBlueprint, VarName, nullptr, FBlueprintMetadata::MD_BitmaskEnum);
		}
		else if(ItemSelected.IsValid())
		{
			FBlueprintEditorUtils::SetBlueprintVariableMetaData(LocalBlueprint, VarName, nullptr, FBlueprintMetadata::MD_BitmaskEnum, ItemSelected->ToString());
		}

		// Reset default value
		if (LocalBlueprint->GeneratedClass)
		{
			UObject* CDO = LocalBlueprint->GeneratedClass->GetDefaultObject(false);
			FProperty* VarProperty = FindFProperty<FProperty>(LocalBlueprint->GeneratedClass, VarName);

			if (CDO != nullptr && VarProperty != nullptr)
			{
				VarProperty->InitializeValue_InContainer(CDO);
			}
		}

		TArray<UK2Node_Variable*> VariableNodes;
		FBlueprintEditorUtils::GetAllNodesOfClass<UK2Node_Variable>(GetBlueprintObj(), VariableNodes);

		for (TArray<UK2Node_Variable*>::TConstIterator NodeIt(VariableNodes); NodeIt; ++NodeIt)
		{
			UK2Node_Variable* CurrentNode = *NodeIt;
			if (VarName == CurrentNode->GetVarName())
			{
				CurrentNode->ReconstructNode();
			}
		}
	}
}

TSharedRef<SWidget> FBlueprintVarActionDetails::GenerateBitmaskEnumTypeWidget(TSharedPtr<FTopLevelAssetPath> Item)
{
	check(Item.IsValid());

	return SNew(STextBlock)
		.Text(FText::FromName(Item->GetAssetName()));
}

FText FBlueprintVarActionDetails::GetBitmaskEnumTypeName() const
{
	const TSharedPtr<FTopLevelAssetPath> BitmaskEnumTypePath = GetBitmaskEnumTypePath();
	return BitmaskEnumTypePath? FText::FromName(BitmaskEnumTypePath->GetAssetName()) : FText();
}

TSharedPtr<FString> FBlueprintVarActionDetails::GetVariableReplicationType() const
{
	EVariableReplication::Type VariableReplication = EVariableReplication::None;
	
	uint64 PropFlags = 0;
	FProperty* VariableProperty = CachedVariableProperty.Get();

	if (VariableProperty && (IsVariableInBlueprint() || IsVariableInheritedByBlueprint()))
	{
		UBlueprint* BlueprintObj = GetPropertyOwnerBlueprint();
		if (BlueprintObj != nullptr)
		{
			uint64 *PropFlagPtr = FBlueprintEditorUtils::GetBlueprintVariablePropertyFlags(BlueprintObj, VariableProperty->GetFName());

			if (PropFlagPtr != NULL)
			{
				PropFlags = *PropFlagPtr;
				bool IsReplicated = (PropFlags & CPF_Net) > 0;
				bool bHasRepNotify = FBlueprintEditorUtils::GetBlueprintVariableRepNotifyFunc(BlueprintObj, VariableProperty->GetFName()) != NAME_None;
				if (bHasRepNotify)
				{
					// Verify they actually have a valid rep notify function still
					UClass* GenClass = GetPropertyOwnerBlueprint()->SkeletonGeneratedClass;
					UFunction* OnRepFunc = GenClass->FindFunctionByName(FBlueprintEditorUtils::GetBlueprintVariableRepNotifyFunc(BlueprintObj, VariableProperty->GetFName()));
					if (OnRepFunc == NULL || OnRepFunc->NumParms != 0 || OnRepFunc->GetReturnProperty() != NULL)
					{
						bHasRepNotify = false;
						ReplicationOnRepFuncChanged(FName(NAME_None).ToString());
					}
				}

				VariableReplication = !IsReplicated ? EVariableReplication::None :
					bHasRepNotify ? EVariableReplication::RepNotify : EVariableReplication::Replicated;
			}
		}
	}

	return ReplicationOptions[(int32)VariableReplication];
}

void FBlueprintVarActionDetails::OnChangeReplication(TSharedPtr<FString> ItemSelected, ESelectInfo::Type SelectInfo)
{
	int32 NewSelection;
	bool bFound = ReplicationOptions.Find(ItemSelected, NewSelection);
	check(bFound && NewSelection != INDEX_NONE);

	EVariableReplication::Type VariableReplication = (EVariableReplication::Type)NewSelection;
	
	FProperty* VariableProperty = CachedVariableProperty.Get();

	UBlueprint* const BlueprintObj = GetBlueprintObj();
	const FName VarName = CachedVariableName;
	int32 VarIndex = INDEX_NONE;
	if (BlueprintObj && VarName != NAME_None)
	{
		VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(BlueprintObj, VarName);
	}

	if (VariableProperty)
	{
		uint64 *PropFlagPtr = FBlueprintEditorUtils::GetBlueprintVariablePropertyFlags(GetBlueprintObj(), VariableProperty->GetFName());
		if (PropFlagPtr != NULL)
		{
			switch(VariableReplication)
			{
			case EVariableReplication::None:
				*PropFlagPtr &= ~CPF_Net;
				ReplicationOnRepFuncChanged(FName(NAME_None).ToString());

				//set replication condition to none:
				if (VarIndex != INDEX_NONE)
				{
					BlueprintObj->NewVariables[VarIndex].ReplicationCondition = COND_None;
				}
				
				break;
				
			case EVariableReplication::Replicated:
				*PropFlagPtr |= CPF_Net;
				ReplicationOnRepFuncChanged(FName(NAME_None).ToString());	
				break;

			case EVariableReplication::RepNotify:
				*PropFlagPtr |= CPF_Net;
				FString NewFuncName = FString::Printf(TEXT("OnRep_%s"), *VariableProperty->GetName());
				UEdGraph* FuncGraph = FindObject<UEdGraph>(BlueprintObj, *NewFuncName);
				if (!FuncGraph)
				{
					FuncGraph = FBlueprintEditorUtils::CreateNewGraph(BlueprintObj, FName(*NewFuncName), UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());
					FBlueprintEditorUtils::AddFunctionGraph<UClass>(BlueprintObj, FuncGraph, false, NULL);
				}

				if (FuncGraph)
				{
					ReplicationOnRepFuncChanged(NewFuncName);
				}
				break;
			}

			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(BlueprintObj);
		}
	}
}

void FBlueprintVarActionDetails::ReplicationOnRepFuncChanged(const FString& NewOnRepFunc) const
{
	FName NewFuncName = FName(*NewOnRepFunc);
	FProperty* VariableProperty = CachedVariableProperty.Get();

	if (VariableProperty)
	{
		FBlueprintEditorUtils::SetBlueprintVariableRepNotifyFunc(GetBlueprintObj(), VariableProperty->GetFName(), NewFuncName);
		uint64 *PropFlagPtr = FBlueprintEditorUtils::GetBlueprintVariablePropertyFlags(GetBlueprintObj(), VariableProperty->GetFName());
		if (PropFlagPtr != NULL)
		{
			if (NewFuncName != NAME_None)
			{
				*PropFlagPtr |= CPF_RepNotify;
				*PropFlagPtr |= CPF_Net;
			}
			else
			{
				*PropFlagPtr &= ~CPF_RepNotify;
			}
		}
	}
}

EVisibility FBlueprintVarActionDetails::ReplicationVisibility() const
{
	FProperty* VariableProperty = CachedVariableProperty.Get();
	if(VariableProperty)
	{
		if (IsAUserVariable(VariableProperty) && IsABlueprintVariable(VariableProperty))
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}

TSharedRef<SWidget> FBlueprintVarActionDetails::BuildEventsMenuForVariable() const
{
	if(MyBlueprint.IsValid())
	{
		TSharedPtr<SMyBlueprint> MyBlueprintPtr = MyBlueprint.Pin();
		FEdGraphSchemaAction_K2Var* Variable = MyBlueprintPtr->SelectionAsVar();
		FObjectProperty* ComponentProperty = Variable ? CastField<FObjectProperty>(Variable->GetProperty()) : NULL;
		TWeakPtr<FBlueprintEditor> BlueprintEditorPtr = MyBlueprintPtr->GetBlueprintEditor();
		if( BlueprintEditorPtr.IsValid() && ComponentProperty )
		{
			TSharedPtr<SSubobjectBlueprintEditor> Editor = StaticCastSharedPtr<SSubobjectBlueprintEditor>(BlueprintEditorPtr.Pin()->GetSubobjectEditor());
			FMenuBuilder MenuBuilder(true, nullptr);
			Editor->BuildMenuEventsSection( MenuBuilder, BlueprintEditorPtr.Pin()->GetBlueprintObj(), ComponentProperty->PropertyClass, 
											FCanExecuteAction::CreateSP(BlueprintEditorPtr.Pin().Get(), &FBlueprintEditor::InEditingMode),
											FGetSelectedObjectsDelegate::CreateSP(MyBlueprintPtr.Get(), &SMyBlueprint::GetSelectedItemsForContextMenu));
			return MenuBuilder.MakeWidget();
		}
	}
	return SNullWidget::NullWidget;
}

void FBlueprintVarActionDetails::OnPostEditorRefresh()
{
	CachedVariableProperty = SelectionAsProperty();
	CachedVariableName = GetVariableName();
}

EVisibility FBlueprintVarActionDetails::GetTransientVisibility() const
{
	FProperty* VariableProperty = CachedVariableProperty.Get();
	if (VariableProperty)
	{
		if (IsABlueprintVariable(VariableProperty) && IsAUserVariable(VariableProperty))
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}

ECheckBoxState FBlueprintVarActionDetails::OnGetTransientCheckboxState() const
{
	FProperty* Property = CachedVariableProperty.Get();
	if (Property)
	{
		return (Property && Property->HasAnyPropertyFlags(CPF_Transient)) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Unchecked;
}

void FBlueprintVarActionDetails::OnTransientChanged(ECheckBoxState InNewState)
{
	FProperty* Property = CachedVariableProperty.Get();
	if (Property)
	{
		const bool bTransientFlag = (InNewState == ECheckBoxState::Checked);
		FBlueprintEditorUtils::SetVariableTransientFlag(GetBlueprintObj(), Property->GetFName(), bTransientFlag);
	}
}

EVisibility FBlueprintVarActionDetails::GetSaveGameVisibility() const
{
	FProperty* VariableProperty = CachedVariableProperty.Get();
	if (VariableProperty)
	{
		if (IsABlueprintVariable(VariableProperty) && IsAUserVariable(VariableProperty))
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}

ECheckBoxState FBlueprintVarActionDetails::OnGetSaveGameCheckboxState() const
{
	FProperty* Property = CachedVariableProperty.Get();
	if (Property)
	{
		return (Property && Property->HasAnyPropertyFlags(CPF_SaveGame)) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Unchecked;
}

void FBlueprintVarActionDetails::OnSaveGameChanged(ECheckBoxState InNewState)
{
	FProperty* Property = CachedVariableProperty.Get();
	if (Property)
	{
		const bool bSaveGameFlag = (InNewState == ECheckBoxState::Checked);
		FBlueprintEditorUtils::SetVariableSaveGameFlag(GetBlueprintObj(), Property->GetFName(), bSaveGameFlag);
	}
}

EVisibility FBlueprintVarActionDetails::GetAdvancedDisplayVisibility() const
{
	FProperty* VariableProperty = CachedVariableProperty.Get();
	if (VariableProperty)
	{
		if (IsABlueprintVariable(VariableProperty) && IsAUserVariable(VariableProperty))
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}

ECheckBoxState FBlueprintVarActionDetails::OnGetAdvancedDisplayCheckboxState() const
{
	FProperty* Property = CachedVariableProperty.Get();
	if (Property)
	{
		return (Property && Property->HasAnyPropertyFlags(CPF_AdvancedDisplay)) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Unchecked;
}

void FBlueprintVarActionDetails::OnAdvancedDisplayChanged(ECheckBoxState InNewState)
{
	if (FProperty* Property = CachedVariableProperty.Get())
	{
		const bool bAdvancedFlag = (InNewState == ECheckBoxState::Checked);
		FBlueprintEditorUtils::SetVariableAdvancedDisplayFlag(GetBlueprintObj(), Property->GetFName(), bAdvancedFlag);
	}
}

EVisibility FBlueprintVarActionDetails::GetMultilineVisibility() const
{
	FProperty* VariableProperty = nullptr;
	if (FProperty* RawVariableProperty = CachedVariableProperty.Get())
	{
		if (IsABlueprintVariable(RawVariableProperty))
		{
			if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(RawVariableProperty))
			{
				VariableProperty = ArrayProperty->Inner;
			}
			else if (const FSetProperty* SetProperty = CastField<FSetProperty>(RawVariableProperty))
			{
				VariableProperty = SetProperty->ElementProp;
			}
			else if (const FMapProperty* MapProperty = CastField<FMapProperty>(RawVariableProperty))
			{
				VariableProperty = MapProperty->ValueProp;
			}
			else
			{
				VariableProperty = RawVariableProperty;
			}
		}
	}

	const bool bCanBeMultiline = (VariableProperty != nullptr) && (VariableProperty->IsA(FTextProperty::StaticClass()) || VariableProperty->IsA(FStrProperty::StaticClass()));
	return bCanBeMultiline ? EVisibility::Visible : EVisibility::Collapsed;
}

ECheckBoxState FBlueprintVarActionDetails::OnGetMultilineCheckboxState() const
{
	FProperty* Property = CachedVariableProperty.Get();
	if (Property)
	{
		return (Property && Property->GetBoolMetaData(TEXT("MultiLine"))) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Unchecked;
}

void FBlueprintVarActionDetails::OnMultilineChanged(ECheckBoxState InNewState)
{
	FProperty* Property = CachedVariableProperty.Get();
	if (Property)
	{
		const bool bMultiline = (InNewState == ECheckBoxState::Checked);
		if (bMultiline)
		{
			FBlueprintEditorUtils::SetBlueprintVariableMetaData(GetBlueprintObj(), Property->GetFName(), GetLocalVariableScope(CachedVariableProperty.Get()), TEXT("MultiLine"), TEXT("true"));
		}
		else
		{
			FBlueprintEditorUtils::RemoveBlueprintVariableMetaData(GetBlueprintObj(), Property->GetFName(), GetLocalVariableScope(CachedVariableProperty.Get()), TEXT("MultiLine"));
		}
	}
}

EVisibility FBlueprintVarActionDetails::GetDeprecatedVisibility() const
{
	if (FProperty* VariableProperty = CachedVariableProperty.Get())
	{
		if (IsABlueprintVariable(VariableProperty) && IsAUserVariable(VariableProperty))
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}

ECheckBoxState FBlueprintVarActionDetails::OnGetDeprecatedCheckboxState() const
{
	return IsVariableDeprecated() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FBlueprintVarActionDetails::OnDeprecatedChanged(ECheckBoxState InNewState)
{
	FProperty* Property = CachedVariableProperty.Get();
	if (Property)
	{
		const bool bDeprecatedFlag = (InNewState == ECheckBoxState::Checked);
		FBlueprintEditorUtils::SetVariableDeprecatedFlag(GetBlueprintObj(), Property->GetFName(), bDeprecatedFlag);
	}
}

EVisibility FBlueprintVarActionDetails::GetDropDownOptionsVisibility() const
{
	if (FProperty* VariableProperty = CachedVariableProperty.Get())
	{
		const bool bMatchingType = VariableProperty->IsA(FNameProperty::StaticClass())
			|| VariableProperty->IsA(FStrProperty::StaticClass());

		if (bMatchingType && IsABlueprintVariable(VariableProperty) && IsAUserVariable(VariableProperty))
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

void FBlueprintVarActionDetails::OnDropDownOptionSelectionChanged(TSharedPtr<FString> InString,
	ESelectInfo::Type)
{
	SetDropDownOptionsFunctionName(*InString);
}

void FBlueprintVarActionDetails::OnDropDownOptionTextChanged(const FText& Text, ETextCommit::Type)
{
	SetDropDownOptionsFunctionName(Text.ToString());
}

TSharedRef<SWidget> FBlueprintVarActionDetails::GenerateDropDownOptionWidget(TSharedPtr<FString> InItem) const
{
	return SNew(STextBlock)
		.Text(FText::FromString(*InItem.Get()))
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

void FBlueprintVarActionDetails::CollectDropDownOptions()
{
	DropDownFunctionOptions.Empty();

	const FProperty* VariableProperty = CachedVariableProperty.Get();
	if (!VariableProperty)
	{
		return;
	}

	for (TFieldIterator<UFunction> It(VariableProperty->GetOwner<UClass>(), EFieldIteratorFlags::IncludeSuper); It; ++It)
	{
		const UFunction* Func = *It;

		// Is the method valid and not latent?
		if (Func && !Func->HasMetaData(FBlueprintMetadata::MD_Latent))
		{
			bool bSignatureValid = false;
			for (TFieldIterator<FProperty> PropertyId(Func); PropertyId; ++PropertyId)
			{
				FProperty* Property = *PropertyId;
				if (!(Property->PropertyFlags & CPF_Parm))
				{
					continue;
				}

				// If we've already had a valid signature, and found one more param, then it's not valid anymore
				if (bSignatureValid)
				{
					bSignatureValid = false;
					break;
				}

				// If there is an input parameter, then it's not matching
				if (!(Property->PropertyFlags & (CPF_ReturnParm | CPF_OutParm)))
				{
					break;
				}

				// Only accept array outputs
				const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property);
				if (!ArrayProperty)
				{
					break;
				}

				// Only accept matching signatures
				if (!ArrayProperty->Inner->SameType(VariableProperty))
				{
					break;
				}

				// It is the first valid function parameter!
				bSignatureValid = true;
			}

			if (bSignatureValid)
			{
				DropDownFunctionOptions.Add(MakeShared<FString>(Func->GetName()));
			}
		}
	}

	// Sort functions by name
	DropDownFunctionOptions.Sort([](const TSharedPtr<FString>& A, const TSharedPtr<FString>& B)
	{
		return A->Compare(*B) < 0;
	});

	DropDownFunctionOptions.Insert(MakeShared<FString>(BlueprintDocumentationDetailDefs::EmptyDropDownOptionName), 0);
}

FText FBlueprintVarActionDetails::GetDropDownOptionDisplayText() const
{
	return FText::FromString(GetDropDownOptionsFunctionName());
}

FString FBlueprintVarActionDetails::GetDropDownOptionsFunctionName() const
{
	const UBlueprint* PropertyBlueprint = GetPropertyOwnerBlueprint();
	if (!PropertyBlueprint)
	{
		return { };
	}

	FString Value;
	FBlueprintEditorUtils::GetBlueprintVariableMetaData(
		PropertyBlueprint,
		CachedVariableName,
		GetLocalVariableScope(CachedVariableProperty.Get()),
		FBlueprintMetadata::MD_GetOptions,
		Value
	);

	return Value;
}

void FBlueprintVarActionDetails::SetDropDownOptionsFunctionName(const FString& InFunctionName)
{
	UBlueprint* PropertyBlueprint = GetPropertyOwnerBlueprint();
	if (!PropertyBlueprint)
	{
		return;
	}

	if (!GetDropDownOptionsFunctionName().Equals(InFunctionName))
	{
		if (InFunctionName.IsEmpty() || InFunctionName == BlueprintDocumentationDetailDefs::EmptyDropDownOptionName)
		{
			FBlueprintEditorUtils::RemoveBlueprintVariableMetaData(
				PropertyBlueprint,
				CachedVariableName,
				GetLocalVariableScope(CachedVariableProperty.Get()),
				FBlueprintMetadata::MD_GetOptions
			);
		}
		else
		{
			FBlueprintEditorUtils::SetBlueprintVariableMetaData(
				PropertyBlueprint,
				CachedVariableName,
				GetLocalVariableScope(CachedVariableProperty.Get()),
				FBlueprintMetadata::MD_GetOptions,
				*InFunctionName
			);
		}
	}
}

FText FBlueprintVarActionDetails::GetDeprecationMessageText() const
{
	FName VarName = CachedVariableName;
	if (VarName != NAME_None)
	{
		if (UBlueprint* OwnerBlueprint = GetPropertyOwnerBlueprint())
		{
			FString Result;
			FBlueprintEditorUtils::GetBlueprintVariableMetaData(GetPropertyOwnerBlueprint(), VarName, GetLocalVariableScope(CachedVariableProperty.Get()), FBlueprintMetadata::MD_DeprecationMessage, Result);
			return FText::FromString(Result);
		}
	}
	return FText();
}

void FBlueprintVarActionDetails::OnDeprecationMessageTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit, FName VarName)
{
	if (NewText.IsEmpty())
	{
		FBlueprintEditorUtils::RemoveBlueprintVariableMetaData(GetBlueprintObj(), VarName, GetLocalVariableScope(CachedVariableProperty.Get()), FBlueprintMetadata::MD_DeprecationMessage);
	}
	else
	{
		FBlueprintEditorUtils::SetBlueprintVariableMetaData(GetBlueprintObj(), VarName, GetLocalVariableScope(CachedVariableProperty.Get()), FBlueprintMetadata::MD_DeprecationMessage, NewText.ToString());
	}
}

EVisibility FBlueprintVarActionDetails::IsTooltipEditVisible() const
{
	FProperty* VariableProperty = CachedVariableProperty.Get();
	if (VariableProperty)
	{
		if ((IsABlueprintVariable(VariableProperty) && IsAUserVariable(VariableProperty)) || IsALocalVariable(VariableProperty))
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}

void FBlueprintVarActionDetails::OnBrowseToVarType() const
{
	FEdGraphPinType PinType = OnGetVarType();
	if (const UObject* Object = PinType.PinSubCategoryObject.Get())
	{
		if (Object->IsAsset())
		{
			FAssetData AssetData(Object, false);
			if (AssetData.IsValid())
			{
				TArray<FAssetData> AssetDataList = { AssetData };
				GEditor->SyncBrowserToObjects(AssetDataList);
			}
		}
	}
}

bool FBlueprintVarActionDetails::CanBrowseToVarType() const
{
	FEdGraphPinType PinType = OnGetVarType();
	if (const UObject* Object = PinType.PinSubCategoryObject.Get())
	{
		if (Object->IsAsset())
		{
			FAssetData AssetData(Object, false);
			if (AssetData.IsValid())
			{
				return true;
			}
		}
	}

	return false;
}

void FBlueprintVarActionDetails::OnFinishedChangingVariable(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (InPropertyChangedEvent.GetNumObjectsBeingEdited() == 0)
	{
		return;
	}

	ImportNamespacesForPropertyValue(InPropertyChangedEvent.MemberProperty, InPropertyChangedEvent.GetObjectBeingEdited(0));
}

void FBlueprintVarActionDetails::OnFinishedChangingLocalVariable(const FPropertyChangedEvent& InPropertyChangedEvent, TSharedPtr<FStructOnScope> InStructData, TWeakObjectPtr<UK2Node_EditablePinBase> InEntryNode)
{
	if (!InPropertyChangedEvent.MemberProperty ||
		!InPropertyChangedEvent.MemberProperty->GetOwnerStruct() ||
		!InPropertyChangedEvent.MemberProperty->GetOwnerStruct()->IsA<UFunction>())
	{
		return;
	}

	// Find the top level property that was modified within the UFunction
	const FProperty* DirectProperty = InPropertyChangedEvent.MemberProperty;
	while (!DirectProperty->GetOwner<const UFunction>())
	{
		DirectProperty = DirectProperty->GetOwnerChecked<const FProperty>();
	}

	FString DefaultValueString;
	bool bDefaultValueSet = false;

	if (InStructData.IsValid())
	{
		UK2Node_FunctionEntry* FuncEntry = Cast<UK2Node_FunctionEntry>(InEntryNode.Get());

		bDefaultValueSet = FBlueprintEditorUtils::PropertyValueToString(DirectProperty, InStructData->GetStructMemory(), DefaultValueString, FuncEntry);

		if (bDefaultValueSet)
		{
			// Search out the correct local variable in the Function Entry Node and set the default value
			for (FBPVariableDescription& LocalVar : FuncEntry->LocalVariables)
			{
				if (LocalVar.VarName == DirectProperty->GetFName() && LocalVar.DefaultValue != DefaultValueString)
				{
					const FScopedTransaction Transaction(LOCTEXT("ChangeDefaults", "Change Defaults"));

					FuncEntry->Modify();
					GetBlueprintObj()->Modify();
					LocalVar.DefaultValue = DefaultValueString;
					FuncEntry->RefreshFunctionVariableCache();
					FBlueprintEditorUtils::MarkBlueprintAsModified(GetBlueprintObj());
					break;
				}
			}
		}

		ImportNamespacesForPropertyValue(DirectProperty, InStructData->GetStructMemory());
	}
}

void FBlueprintVarActionDetails::ImportNamespacesForPropertyValue(const FProperty* InProperty, const void* InContainer)
{
	// Auto-import any namespace(s) associated with the property's value into the current editor context.
	TSharedPtr<SMyBlueprint> MyBlueprintPtr = MyBlueprint.Pin();
	if (MyBlueprintPtr.IsValid())
	{
		TSharedPtr<FBlueprintEditor> BlueprintEditor = MyBlueprintPtr->GetBlueprintEditor().Pin();
		if (BlueprintEditor.IsValid())
		{
			FBlueprintEditor::FImportNamespaceExParameters Params;
			FBlueprintNamespaceUtilities::GetPropertyValueNamespaces(InProperty, InContainer, Params.NamespacesToImport);
			BlueprintEditor->ImportNamespaceEx(Params);
		}
	}
}

bool FBlueprintVarActionDetails::IsVariableInheritedByBlueprint() const
{
	UClass* PropertyOwnerClass = nullptr;
	if (UBlueprint* PropertyOwnerBP = GetPropertyOwnerBlueprint())
	{
		PropertyOwnerClass = PropertyOwnerBP->SkeletonGeneratedClass;
	}
	else if (CachedVariableProperty.IsValid())
	{
		PropertyOwnerClass = CachedVariableProperty->GetOwnerClass();
	}
	const UClass* SkeletonGeneratedClass = GetBlueprintObj()->SkeletonGeneratedClass;
	return SkeletonGeneratedClass && SkeletonGeneratedClass->IsChildOf(PropertyOwnerClass);
}

bool FBlueprintVarActionDetails::IsVariableDeprecated() const
{
	FProperty* Property = CachedVariableProperty.Get();

	return Property && Property->HasAnyPropertyFlags(CPF_Deprecated);
}

static TArray<UK2Node_EditablePinBase*> GatherAllResultNodes(UK2Node_EditablePinBase* TargetNode)
{
	if (UK2Node_FunctionResult* ResultNode = Cast<UK2Node_FunctionResult>(TargetNode))
	{
		return (TArray<UK2Node_EditablePinBase*>)ResultNode->GetAllResultNodes();
	}
	TArray<UK2Node_EditablePinBase*> Result;
	if (TargetNode)
	{
		Result.Add(TargetNode);
	}
	return Result;
}

/** Drag-and-drop operation that stores data about the function parameter pin being dragged */
class FBlueprintGraphArgumentDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FBlueprintGraphArgumentDragDropOp, FDecoratedDragDropOp);

	FBlueprintGraphArgumentDragDropOp(UK2Node_EditablePinBase* InTargetNode, TWeakPtr<FUserPinInfo> InParamItemPtr)
		: TargetNode(InTargetNode)
		, ParamItemPtr(InParamItemPtr)
	{
		MouseCursor = EMouseCursor::GrabHandClosed;
	}

	void Init()
	{
		SetValidTarget(false);
		SetupDefaults();
		Construct();
	}

	void SetValidTarget(bool IsValidTarget)
	{
		FText PinName = FText::FromName(ParamItemPtr.IsValid() ? ParamItemPtr.Pin()->PinName : NAME_None);
		FFormatNamedArguments Args;
		Args.Add(TEXT("PinName"), PinName);

		if (IsValidTarget)
		{
			CurrentHoverText = FText::Format(LOCTEXT("MovePinHere", "Move '{PinName}' Here"), Args);
			CurrentIconBrush = FAppStyle::Get().GetBrush("Graph.ConnectorFeedback.OK");
		}
		else
		{
			CurrentHoverText = FText::Format(LOCTEXT("CannotMovePinHere", "Cannot Move '{PinName}' Here"), Args);
			CurrentIconBrush = FAppStyle::Get().GetBrush("Graph.ConnectorFeedback.Error");
		}
	}

	UK2Node_EditablePinBase* GetTargetNode() const
	{
		return TargetNode;
	}

	TWeakPtr<FUserPinInfo> GetParamItem() const
	{
		return ParamItemPtr;
	}

private:
	UK2Node_EditablePinBase* TargetNode;
	TWeakPtr<FUserPinInfo> ParamItemPtr;
};

/** Handler for customizing the drag-and-drop behavior for function entry/result pins, allowing parameters to be reordered */
class FBlueprintGraphArgumentDragDropHandler : public IDetailDragDropHandler
{
public:
	FBlueprintGraphArgumentDragDropHandler(
		TWeakPtr<FBaseBlueprintGraphActionDetails> InGraphActionDetailsPtr,
		UK2Node_EditablePinBase* InTargetNode,
		TWeakPtr<FUserPinInfo> InParamItemPtr)
		: GraphActionDetailsPtr(InGraphActionDetailsPtr)
		, TargetNode(InTargetNode)
		, ParamItemPtr(InParamItemPtr)
	{
	}

	virtual TSharedPtr<FDragDropOperation> CreateDragDropOperation() const override
	{
		TSharedPtr<FBlueprintGraphArgumentDragDropOp> DragOp = MakeShared<FBlueprintGraphArgumentDragDropOp>(TargetNode, ParamItemPtr);
		DragOp->Init();
		return DragOp;
	}

	/** Compute new target index for use with AcceptDrop/CanAcceptDrop based on drop zone (above vs below) */
	static int32 ComputeNewIndex(int32 OriginalIndex, int32 DropOntoIndex, EItemDropZone DropZone)
	{
		check(DropZone != EItemDropZone::OntoItem);

		int32 NewIndex = DropOntoIndex;
		if (DropZone == EItemDropZone::BelowItem)
		{
			// If the drop zone is below, then we actually move it to the next item's index
			NewIndex++;
		}
		if (OriginalIndex < NewIndex)
		{
			// If the item is moved down the list, then all the other elements below it are shifted up one
			NewIndex--;
		}

		return ensure(NewIndex >= 0) ? NewIndex : 0;
	}

	virtual bool AcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone) const override
	{
		const TSharedPtr<FBlueprintGraphArgumentDragDropOp> DragOp = DragDropEvent.GetOperationAs<FBlueprintGraphArgumentDragDropOp>();
		if (!DragOp.IsValid() || DragOp->GetTargetNode() != TargetNode)
		{
			return false;
		}

		if (!ensure(ParamItemPtr.IsValid()) || !ensure(DragOp->GetParamItem().IsValid()))
		{
			return false;
		}

		// Check that the original and new indices are valid, and that they aren't the same (we're actually moving something)
		const int32 OriginalParamIndex = DragOp->GetTargetNode()->UserDefinedPins.Find(DragOp->GetParamItem().Pin());
		const int32 OntoParamIndex = TargetNode->UserDefinedPins.Find(ParamItemPtr.Pin());
		const int32 NewParamIndex = ComputeNewIndex(OriginalParamIndex, OntoParamIndex, DropZone);
		if (OriginalParamIndex == INDEX_NONE || OriginalParamIndex == NewParamIndex || NewParamIndex < 0 || NewParamIndex >= TargetNode->UserDefinedPins.Num())
		{
			return false;
		}

		const FScopedTransaction Transaction(LOCTEXT("K2_MovePin", "Move Pin"));
		TArray<UK2Node_EditablePinBase*> TargetNodes = GatherAllResultNodes(TargetNode);
		for (UK2Node_EditablePinBase* Node : TargetNodes)
		{
			Node->Modify();

			TSharedPtr<FUserPinInfo> ParamToMove = Node->UserDefinedPins[OriginalParamIndex];
			Node->UserDefinedPins.RemoveAt(OriginalParamIndex);
			Node->UserDefinedPins.Insert(ParamToMove, NewParamIndex);

			TSharedPtr<FBaseBlueprintGraphActionDetails> GraphActionDetails = GraphActionDetailsPtr.Pin();
			if (GraphActionDetails.IsValid())
			{
				GraphActionDetails->OnParamsChanged(Node, true);
			}
		}

		return true;
	}

	virtual TOptional<EItemDropZone> CanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone) const override
	{
		const TSharedPtr<FBlueprintGraphArgumentDragDropOp> DragOp = DragDropEvent.GetOperationAs<FBlueprintGraphArgumentDragDropOp>();
		if (!DragOp.IsValid() || DragOp->GetTargetNode() != TargetNode)
		{
			return TOptional<EItemDropZone>();
		}

		// We're reordering, so there's no logical interpretation for dropping directly onto another parameter.
		// Just change it to a drop-above in this case.
		const EItemDropZone OverrideZone = (DropZone == EItemDropZone::BelowItem) ? EItemDropZone::BelowItem : EItemDropZone::AboveItem;

		// Check that the original and new indices are valid, and that they aren't the same (we're actually moving something)
		const int32 OriginalParamIndex = DragOp->GetTargetNode()->UserDefinedPins.Find(DragOp->GetParamItem().Pin());
		const int32 OntoParamIndex = TargetNode->UserDefinedPins.Find(ParamItemPtr.Pin());
		const int32 NewParamIndex = ComputeNewIndex(OriginalParamIndex, OntoParamIndex, OverrideZone);
		if (OriginalParamIndex == INDEX_NONE || OriginalParamIndex == NewParamIndex || NewParamIndex < 0 || NewParamIndex >= TargetNode->UserDefinedPins.Num())
		{
			return TOptional<EItemDropZone>();
		}

		DragOp->SetValidTarget(true);
		return OverrideZone;
	}

private:
	/** The parent graph action details customization */
	TWeakPtr<FBaseBlueprintGraphActionDetails> GraphActionDetailsPtr;

	/** The target node that the argument pin is on */
	UK2Node_EditablePinBase* TargetNode;

	/** The argument pin that this drag handler reflects */
	TWeakPtr<FUserPinInfo> ParamItemPtr;
};

void FBlueprintGraphArgumentGroupLayout::SetOnRebuildChildren( FSimpleDelegate InOnRegenerateChildren )
{
	GraphActionDetailsPtr.Pin()->SetRefreshDelegate(InOnRegenerateChildren, TargetNode == GraphActionDetailsPtr.Pin()->GetFunctionEntryNode().Get());
}

void FBlueprintGraphArgumentGroupLayout::GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder )
{
	bool WasContentAdded = false;
	if(TargetNode.IsValid())
	{
		TArray<TSharedPtr<FUserPinInfo>> Pins = TargetNode->UserDefinedPins;

		if(Pins.Num() > 0)
		{
			bool bIsInputNode = TargetNode == GraphActionDetailsPtr.Pin()->GetFunctionEntryNode().Get();
			for (int32 i = 0; i < Pins.Num(); ++i)
			{
				// If possible, use stable guids for the argument names since the path names are used to store
				// expansion state. Using guids means that the expansion state travels with the row when
				// reordering arguments.
				// Fall back to the old style of using the pin index for the name if we can't find the pin.
				FString ArgumentName;
				if (UEdGraphPin* Pin = TargetNode->FindPin(Pins[i]->PinName, Pins[i]->DesiredPinDirection))
				{
					ArgumentName = Pin->PinId.ToString();
				}
				else
				{
					ArgumentName = bIsInputNode ? FString::Printf(TEXT("InputArgument%i"), i) : FString::Printf(TEXT("OutputArgument%i"), i);
				}

				TSharedRef<class FBlueprintGraphArgumentLayout> BlueprintArgumentLayout = MakeShareable(new FBlueprintGraphArgumentLayout(
					TWeakPtr<FUserPinInfo>(Pins[i]),
					TargetNode.Get(),
					GraphActionDetailsPtr,
					FName(*ArgumentName),
					bIsInputNode));
				ChildrenBuilder.AddCustomBuilder(BlueprintArgumentLayout);
				WasContentAdded = true;
			}
		}
	}
	if (!WasContentAdded)
	{
		// Add a text widget to let the user know to hit the + icon to add parameters.
		ChildrenBuilder.AddCustomRow(FText::GetEmpty()).WholeRowContent()
			.MaxDesiredWidth(980.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(0.0f, 0.0f, 4.0f, 0.0f)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("NoArgumentsAddedForBlueprint", "Please press the + icon above to add parameters"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
			];
	}
}

// Internal
static bool ShouldAllowWildcard(UK2Node_EditablePinBase* TargetNode)
{
	// allow wildcards for tunnel nodes in macro graphs
	if ( TargetNode->IsA(UK2Node_Tunnel::StaticClass()) )
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		return ( K2Schema->GetGraphType( TargetNode->GetGraph() ) == GT_Macro );
	}

	return false;
}

FBlueprintGraphArgumentLayout::~FBlueprintGraphArgumentLayout()
{
	if (IsValid(TargetNode) && TargetNode->HasValidBlueprint() && GraphChangedHandler != FDelegateHandle())
	{
		TargetNode->GetGraph()->RemoveOnGraphChangedHandler(GraphChangedHandler);
	}
}

void FBlueprintGraphArgumentLayout::GenerateHeaderRowContent( FDetailWidgetRow& NodeRow )
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	ETypeTreeFilter TypeTreeFilter = ETypeTreeFilter::None;
	if (TargetNode->CanModifyExecutionWires())
	{
		TypeTreeFilter |= ETypeTreeFilter::AllowExec;
	}

	if (ShouldAllowWildcard(TargetNode))
	{
		TypeTreeFilter |= ETypeTreeFilter::AllowWildcard;
	}

	TArray<TSharedPtr<IPinTypeSelectorFilter>> CustomPinTypeFilters;
	if (GraphActionDetailsPtr.IsValid())
	{
		TSharedPtr<SMyBlueprint> MyBlueprintPtr = GraphActionDetailsPtr.Pin()->GetMyBlueprint().Pin();
		if (MyBlueprintPtr.IsValid())
		{
			TSharedPtr<FBlueprintEditor> BlueprintEditorPtr = MyBlueprintPtr->GetBlueprintEditor().Pin();
			if (BlueprintEditorPtr.IsValid())
			{
				BlueprintEditorPtr->GetPinTypeSelectorFilters(CustomPinTypeFilters);
			}
		}
	}

	NodeRow
	.NameContent()
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1)
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.MinDesiredWidth(125.f)
			[
				SAssignNew(ArgumentNameWidget, SEditableTextBox)
				.Text( this, &FBlueprintGraphArgumentLayout::OnGetArgNameText )
				.OnTextChanged(this, &FBlueprintGraphArgumentLayout::OnArgNameChange)
				.OnTextCommitted(this, &FBlueprintGraphArgumentLayout::OnArgNameTextCommitted)
				.ToolTipText(this, &FBlueprintGraphArgumentLayout::OnGetArgToolTipText)
				.Font( IDetailLayoutBuilder::GetDetailFont() )
				.IsEnabled(!ShouldPinBeReadOnly())
			]
		]
	]
	.ValueContent()
	.MaxDesiredWidth(980.f)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(0.0f)
		.FillWidth(1.0f)
		[
			SNew(SPinTypeSelector, FGetPinTypeTree::CreateUObject(K2Schema, &UEdGraphSchema_K2::GetVariableTypeTree))
				.TargetPinType(this, &FBlueprintGraphArgumentLayout::OnGetPinInfo)
				.OnPinTypePreChanged(this, &FBlueprintGraphArgumentLayout::OnPrePinInfoChange)
				.OnPinTypeChanged(this, &FBlueprintGraphArgumentLayout::PinInfoChanged)
				.Schema(K2Schema)
				.TypeTreeFilter(TypeTreeFilter)
				.bAllowArrays(!ShouldPinBeReadOnly())
				.IsEnabled(!ShouldPinBeReadOnly(true))
				.Font( IDetailLayoutBuilder::GetDetailFont() )
				.CustomFilters(CustomPinTypeFilters)
		]
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(10, 0, 0, 0)
		.AutoWidth()
		[
			PropertyCustomizationHelpers::MakeClearButton(FSimpleDelegate::CreateSP(this, &FBlueprintGraphArgumentLayout::OnRemoveClicked), LOCTEXT("FunctionArgDetailsClearTooltip", "Remove this parameter."), !IsPinEditingReadOnly())
		]

	]
	.DragDropHandler(MakeShared<FBlueprintGraphArgumentDragDropHandler>(GraphActionDetailsPtr, TargetNode, ParamItemPtr));
}

void FBlueprintGraphArgumentLayout::GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder )
{
	bool bMadePinWidgets = false;
	if (bHasDefaultValue)
	{
		UEdGraphPin* FoundPin = GetPin();
		if (FoundPin)
		{
			// Certain types are outlawed at the compiler level, or to keep consistency with variable rules for actors
			const UClass* ClassObject = Cast<UClass>(FoundPin->PinType.PinSubCategoryObject.Get());
			const bool bTypeWithNoDefaults = (FoundPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object) || (FoundPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Class) || (FoundPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Interface) 
				|| (FoundPin->PinType.PinCategory == UEdGraphSchema_K2::PC_SoftObject && ClassObject && ClassObject->IsChildOf(AActor::StaticClass()))
				|| UEdGraphSchema_K2::IsExecPin(*FoundPin)
				|| FoundPin->PinType.IsContainer();

			if (!FoundPin->PinType.bIsReference && !bTypeWithNoDefaults)
			{
				bMadePinWidgets = true;
				DefaultValuePinWidget = FNodeFactory::CreatePinWidget(FoundPin);
				DefaultValuePinWidget->SetOnlyShowDefaultValue(true);
				TSharedRef<SWidget> DefaultValueWidget = DefaultValuePinWidget->GetDefaultValueWidget();

				if (DefaultValueWidget != SNullWidget::NullWidget)
				{
					ChildrenBuilder.AddCustomRow(LOCTEXT("FunctionArgDetailsDefaultValue", "Default Value"))
						.NameContent()
						[
							SNew(STextBlock)
							.Text(LOCTEXT("FunctionArgDetailsDefaultValue", "Default Value"))
							.ToolTipText(LOCTEXT("FunctionArgDetailsDefaultValueParamTooltip", "The default value of the parameter."))
							.Font(IDetailLayoutBuilder::GetDetailFont())
						]
						.ValueContent()
						.MaxDesiredWidth(512)
						[
							DefaultValueWidget
						];
				}
				else
				{
					DefaultValuePinWidget.Reset();
				}
			}
		}

		bool bMacroGraph = false;
		if (TargetNode && TargetNode->HasValidBlueprint())
		{
			if (const UBlueprint* Blueprint = TargetNode->GetBlueprint())
			{
				if (Blueprint->BlueprintType == BPTYPE_MacroLibrary)
				{
					bMacroGraph = true;
				}
				else if (const UEdGraph* Graph = TargetNode->GetGraph()) 
				{
					bMacroGraph = Blueprint->MacroGraphs.Contains(Graph);
				}
			}
		}

		// Exec pins can't be passed by reference
		if (FoundPin && !UEdGraphSchema_K2::IsExecPin(*FoundPin) && !bMacroGraph)
		{
			auto ShouldPassByRefBeReadOnly = [this]
			{
				// Array types will always be implicitly passed by reference, regardless of
				// the checkbox setting so make it readonly.
				return OnGetPinInfo().IsArray() || ShouldPinBeReadOnly();
			};

			ChildrenBuilder.AddCustomRow(LOCTEXT("FunctionArgDetailsPassByReference", "Pass-by-Reference"))
				.NameContent()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("FunctionArgDetailsPassByReference", "Pass-by-Reference"))
					.ToolTipText(LOCTEXT("FunctionArgDetailsPassByReferenceTooltip", "Pass this parameter by reference?"))
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			.ValueContent()
				[
					SNew(SCheckBox)
					.IsChecked(this, &FBlueprintGraphArgumentLayout::IsRefChecked)
					.OnCheckStateChanged(this, &FBlueprintGraphArgumentLayout::OnRefCheckStateChanged)
					.IsEnabled(!ShouldPassByRefBeReadOnly())
				];

			ChildrenBuilder.AddCustomRow(LOCTEXT("FunctionArgDetailsConst", "Const"))
				.NameContent()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("FunctionArgDetailsConst", "Const"))
					.ToolTipText(LOCTEXT("FunctionArgDetailsConstTooltip", "When passing by reference a parameter can be specified as const (not writable)"))
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			.ValueContent()
				[
					SNew(SCheckBox)
					.IsChecked(this, &FBlueprintGraphArgumentLayout::IsConstChecked)
					.OnCheckStateChanged(this, &FBlueprintGraphArgumentLayout::OnConstCheckStateChanged)
					.IsEnabled(this, &FBlueprintGraphArgumentLayout::CanChangeConst)
				];
		}
	}
		
	if(bMadePinWidgets)
	{
		// add a listener to the owning graph so we can clean up our pin widgets:
		if (TargetNode && TargetNode->HasValidBlueprint())
		{
			GraphChangedHandler = TargetNode->GetGraph()->AddOnGraphChangedHandler(
				FOnGraphChanged::FDelegate::CreateSP(this, &FBlueprintGraphArgumentLayout::OnArgumentGraphChanged));
		}
	}
}

void FBlueprintGraphArgumentLayout::OnRemoveClicked()
{
	TSharedPtr<FUserPinInfo> ParamItem = ParamItemPtr.Pin();
	if (ParamItem.IsValid())
	{
		const FScopedTransaction Transaction( LOCTEXT( "RemoveParam", "Remove Parameter" ) );

		TSharedPtr<FBaseBlueprintGraphActionDetails> GraphActionDetails = GraphActionDetailsPtr.Pin();
		TArray<UK2Node_EditablePinBase*> TargetNodes = GatherAllResultNodes(TargetNode);
		for (UK2Node_EditablePinBase* Node : TargetNodes)
		{
			Node->Modify();
			Node->RemoveUserDefinedPinByName(ParamItem->PinName);

			if (GraphActionDetails.IsValid())
			{
				GraphActionDetails->OnParamsChanged(Node, true);
			}
		}
	}
}

void FBlueprintGraphArgumentLayout::OnArgumentGraphChanged(const FEdGraphEditAction& Action)
{
	TSharedPtr<FBaseBlueprintGraphActionDetails> GraphActionDetails = GraphActionDetailsPtr.Pin();
	if (GraphActionDetails.IsValid())
	{
		GraphActionDetails->OnPostEditorRefresh();
		DefaultValuePinWidget.Reset();
	}
}

bool FBlueprintGraphArgumentLayout::ShouldPinBeReadOnly(bool bIsEditingPinType/* = false*/) const
{
	if (TargetNode && ParamItemPtr.IsValid())
	{
		// Right now, we only care that the user is unable to edit the auto-generated "then" pin
		if ((ParamItemPtr.Pin()->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec) && (!TargetNode->CanModifyExecutionWires()))
		{
			return true;
		}
		else
		{
			// Check if pin editing is read only
			return IsPinEditingReadOnly(bIsEditingPinType);
		}
	}
	
	return false;
}

bool FBlueprintGraphArgumentLayout::IsPinEditingReadOnly(bool bIsEditingPinType/* = false*/) const
{
	if(UEdGraph* NodeGraph = TargetNode->GetGraph())
	{
		// Math expression should not be modified directly (except for the pin type), do not let the user tweak the parameters
		if (!bIsEditingPinType && Cast<UK2Node_MathExpression>(NodeGraph->GetOuter()) )
		{
			return true;
		}
	}
	return false;
}

FText FBlueprintGraphArgumentLayout::OnGetArgNameText() const
{
	if (ParamItemPtr.IsValid())
	{
		return FText::FromName(ParamItemPtr.Pin()->PinName);
	}
	return FText();
}

FText FBlueprintGraphArgumentLayout::OnGetArgToolTipText() const
{
	if (ParamItemPtr.IsValid())
	{
		FText PinTypeText = UEdGraphSchema_K2::TypeToText(ParamItemPtr.Pin()->PinType);
		return FText::Format(LOCTEXT("BlueprintArgToolTipText", "Name: {0}\nType: {1}"), FText::FromName(ParamItemPtr.Pin()->PinName), PinTypeText);
	}
	return FText::GetEmpty();
}

void FBlueprintGraphArgumentLayout::OnArgNameChange(const FText& InNewText)
{
	bool bVerified = true;

	FText ErrorMessage;

	if (!ParamItemPtr.IsValid())
	{
		return;
	}

	if (InNewText.IsEmpty())
	{
		ErrorMessage = LOCTEXT("EmptyArgument", "Name cannot be empty!");
		bVerified = false;
	}
	else
	{
		bVerified = GraphActionDetailsPtr.Pin()->OnVerifyPinRename(TargetNode, ParamItemPtr.Pin()->PinName, InNewText.ToString(), ErrorMessage);
	}

	if(!bVerified)
	{
		ArgumentNameWidget.Pin()->SetError(ErrorMessage);
	}
	else
	{
		ArgumentNameWidget.Pin()->SetError(FText::GetEmpty());
	}
}

void FBlueprintGraphArgumentLayout::OnArgNameTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit)
{
	if (!NewText.IsEmpty() && TargetNode && ParamItemPtr.IsValid() && GraphActionDetailsPtr.IsValid() && !ShouldPinBeReadOnly())
	{
		const FName OldName = ParamItemPtr.Pin()->PinName;
		const FString& NewName = NewText.ToString();
		if (!OldName.ToString().Equals(NewName))
		{
			GraphActionDetailsPtr.Pin()->OnPinRenamed(TargetNode, OldName, NewName);
		}
	}
}

FEdGraphPinType FBlueprintGraphArgumentLayout::OnGetPinInfo() const
{
	if (ParamItemPtr.IsValid())
	{
		return ParamItemPtr.Pin()->PinType;
	}
	return FEdGraphPinType();
}

UEdGraphPin* FBlueprintGraphArgumentLayout::GetPin() const
{
	if (ParamItemPtr.IsValid() && TargetNode)
	{
		return TargetNode->FindPin(ParamItemPtr.Pin()->PinName, ParamItemPtr.Pin()->DesiredPinDirection);
	}
	return nullptr;
}

ECheckBoxState FBlueprintGraphArgumentLayout::IsRefChecked() const
{
	const FEdGraphPinType PinType = OnGetPinInfo();

	// Array types will always be implicitly passed by reference, regardless of
	// the checkbox setting so show it as checked
	return (PinType.bIsReference || PinType.IsArray())  ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

ECheckBoxState FBlueprintGraphArgumentLayout::IsConstChecked() const
{
	return OnGetPinInfo().bIsConst ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FBlueprintGraphArgumentLayout::OnRefCheckStateChanged(ECheckBoxState InState)
{
	const FScopedTransaction Transaction(LOCTEXT("ChangeByRef", "Change Pass By Reference"));

	const bool bIsReference = (InState == ECheckBoxState::Checked);
	FEdGraphPinType PinType = OnGetPinInfo();
	PinType.bIsReference = bIsReference;
	if(!bIsReference)
	{
		// constness is not meaningful for non reference types,
		// reset constness when refness is toggled off:
		PinType.bIsConst = ShouldBeForceConst();
	}

	PinInfoChanged(PinType);
}

void FBlueprintGraphArgumentLayout::OnConstCheckStateChanged(ECheckBoxState InState)
{
	FEdGraphPinType PinType = OnGetPinInfo();

	PinType.bIsConst = ShouldBeForceConst() || (InState == ECheckBoxState::Checked);

	PinInfoChanged(PinType);
}

bool FBlueprintGraphArgumentLayout::ShouldBeForceConst() const
{
	// Const-ness is not meaningful unless we're passing by reference, including all arrays 
	// which are implicitly passed by reference. If we have a reference pin type (implicit or
	// otherwise) we want to honor ShouldUseConstRefParams:
	FEdGraphPinType PinType = OnGetPinInfo();
	return (PinType.bIsReference || PinType.IsArray()) && TargetNode && TargetNode->ShouldUseConstRefParams();
}

bool FBlueprintGraphArgumentLayout::CanChangeConst() const
{
	return !ShouldBeForceConst() && IsRefChecked() == ECheckBoxState::Checked;
}

void FBlueprintGraphArgumentLayout::PinInfoChanged(const FEdGraphPinType& PinType)
{
	if (ParamItemPtr.IsValid() && FBlueprintEditorUtils::IsPinTypeValid(PinType))
	{
		const FName PinName = ParamItemPtr.Pin()->PinName;
		TSharedPtr<class FBaseBlueprintGraphActionDetails> GraphActionDetailsPinned = GraphActionDetailsPtr.Pin();
		if (GraphActionDetailsPinned.IsValid())
		{
			TSharedPtr<SMyBlueprint> MyBPPinned = GraphActionDetailsPinned->GetMyBlueprint().Pin();
			if (MyBPPinned.IsValid())
			{
				MyBPPinned->GetLastFunctionPinTypeUsed() = PinType;
			}
			if( !ShouldPinBeReadOnly(true) )
			{
				TArray<UK2Node_EditablePinBase*> TargetNodes = GatherAllResultNodes(TargetNode);
				for (UK2Node_EditablePinBase* Node : TargetNodes)
				{
					if (Node)
					{
						TSharedPtr<FUserPinInfo>* UDPinPtr = Node->UserDefinedPins.FindByPredicate([PinName](TSharedPtr<FUserPinInfo>& UDPin)
						{
							return UDPin.IsValid() && (UDPin->PinName == PinName);
						});
						if (UDPinPtr)
						{
							Node->Modify();
							(*UDPinPtr)->PinType = PinType;

							// Inputs flagged as pass-by-reference will also be flagged as 'const' here to conform to the expected native C++
							// declaration of 'const Type&' for input reference parameters on functions with no outputs (i.e. events). Array
							// types are also flagged as 'const' here since they will always be implicitly passed by reference, regardless of
							// the checkbox setting. See UEditablePinBase::PostLoad() for more details.
							if(!PinType.bIsConst && Node->ShouldUseConstRefParams())
							{
								(*UDPinPtr)->PinType.bIsConst = PinType.IsArray() || PinType.bIsReference;
							}

							// Reset default value, it probably doesn't match
							(*UDPinPtr)->PinDefaultValue.Reset();

							TSharedPtr<FBlueprintEditor> BlueprintEditor;
							if(MyBPPinned.IsValid())
							{
								BlueprintEditor = MyBPPinned->GetBlueprintEditor().Pin();
							}

							// Auto-import the underlying type object's default namespace set into the current editor context.
							const UObject* PinSubCategoryObject = PinType.PinSubCategoryObject.Get();
							if (PinSubCategoryObject && BlueprintEditor.IsValid())
							{
								FBlueprintEditor::FImportNamespaceExParameters Params;
								FBlueprintNamespaceUtilities::GetDefaultImportsForObject(PinSubCategoryObject, Params.NamespacesToImport);
								BlueprintEditor->ImportNamespaceEx(Params);
							}
						}
						GraphActionDetailsPinned->OnParamsChanged(Node);
					}
				}
			}
		}
	}
}

void FBlueprintGraphArgumentLayout::OnPrePinInfoChange(const FEdGraphPinType& PinType)
{
	if (!ShouldPinBeReadOnly(true))
	{
		TArray<UK2Node_EditablePinBase*> TargetNodes = GatherAllResultNodes(TargetNode);
		for (UK2Node_EditablePinBase* Node : TargetNodes)
		{
			if (Node)
			{
				Node->Modify();
			}
		}
	}
}

void FBlueprintGraphLocalVariableGroupLayout::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	bool WasContentAdded = false;
	if(TargetGraph.IsValid())
	{
		if(const UEdGraph* TopLevelGraph = FBlueprintEditorUtils::GetTopLevelGraph(TargetGraph.Get()))
		{
			bool bSchemaImplementsGetLocalVariables = false;
		
			// grab the parent graph's name
			if (UEdGraphSchema const* Schema = TopLevelGraph->GetSchema())
			{
				FGraphDisplayInfo EdGraphDisplayInfo;
				Schema->GetGraphDisplayInformation(*TopLevelGraph, EdGraphDisplayInfo);

				// Try to get the local variables from the schema
				TArray<FBPVariableDescription> LocalVariables;
				bSchemaImplementsGetLocalVariables = Schema->GetLocalVariables(TargetGraph.Get(), LocalVariables);
				for (const FBPVariableDescription& LocalVariable : LocalVariables)
				{
					TSharedRef<class FBlueprintGraphLocalVariableLayout> BlueprintLocalVariableLayout = MakeShareable(new FBlueprintGraphLocalVariableLayout(OwningFunction, LocalVariable));
					ChildrenBuilder.AddCustomBuilder(BlueprintLocalVariableLayout);
					WasContentAdded = true;
				}
			}
			// If the schema did not return any local variables, try to get them from the function entry
			if (!bSchemaImplementsGetLocalVariables)
			{
				TArray<UK2Node_FunctionEntry*> FunctionEntryNodes;
				TopLevelGraph->GetNodesOfClass<UK2Node_FunctionEntry>(FunctionEntryNodes);
				if (!FunctionEntryNodes.IsEmpty())
				{
					TArray<FBPVariableDescription>& LocalVariables = FunctionEntryNodes[0]->LocalVariables;
					
					// Search in all FunctionEntry nodes for their local variables
					FText ActionCategory;
					for (int I = 0; I < LocalVariables.Num(); ++I)
					{
						TSharedPtr<class FBlueprintGraphLocalVariableLayout> BlueprintLocalVariableLayout = nullptr;
						if (PropertyHandle)
						{
							BlueprintLocalVariableLayout = MakeShareable(new FBlueprintGraphLocalVariableLayout(OwningFunction, PropertyHandle->GetChildHandle(I)));
						}
						else
						{
							BlueprintLocalVariableLayout = MakeShareable(new FBlueprintGraphLocalVariableLayout(OwningFunction, LocalVariables[I]));
						}
						ChildrenBuilder.AddCustomBuilder(BlueprintLocalVariableLayout.ToSharedRef());
						WasContentAdded = true;		
					}
				}
			}
		}
	}
	if (!WasContentAdded)
	{
		// Add a text widget to let the user know to hit the + icon to add parameters.
		ChildrenBuilder.AddCustomRow(FText::GetEmpty()).WholeRowContent()
			.MaxDesiredWidth(980.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(0.0f, 0.0f, 4.0f, 0.0f)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("NoLocalVariablesAddedForBlueprint", "No Local Variables"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
			];
	}
}

TSharedPtr<IPropertyHandle> FBlueprintGraphLocalVariableLayout::GetPropertyHandle() const
{
	return Data.IsType<TSharedPtr<IPropertyHandle>>() ? Data.Get<TSharedPtr<IPropertyHandle>>() : nullptr;
}

const FBPVariableDescription& FBlueprintGraphLocalVariableLayout::GetVariable() const
{
	// if stored by property handle, extract value from there
	if (const TSharedPtr<IPropertyHandle> PropertyHandle = GetPropertyHandle())
	{
		void* Address;
		PropertyHandle->GetValueData(Address);
		return *(FBPVariableDescription*)Address;
	}
	// if stored as variable description, return it directly
	return Data.Get<FBPVariableDescription>();
}

void FBlueprintGraphLocalVariableLayout::SetVariable(const FBPVariableDescription& NewValue)
{
	const TSharedPtr<IPropertyHandle> PropertyHandle = GetPropertyHandle();
	checkf(PropertyHandle != nullptr, TEXT("Tried to call a setter on a read-only local variable layout"));
	
	void* Address;
	PropertyHandle->GetValueData(Address);
	*(FBPVariableDescription*)Address = NewValue;
}

void FBlueprintGraphLocalVariableLayout::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	TSharedPtr<IPropertyHandle> PropHandle = GetPropertyHandle();
	
	NodeRow
	.NameContent()
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1)
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.MinDesiredWidth(125.f)
			[
				SAssignNew(VariableNameWidget, SEditableTextBox)
				.Text( this, &FBlueprintGraphLocalVariableLayout::OnGetVarNameText )
#if UE_BP_LOCAL_VAR_LAYOUT_SETTERS_IMPLEMENTED
				.OnTextChanged(this, &FBlueprintGraphLocalVariableLayout::OnVarNameChange)
				.OnTextCommitted(this, &FBlueprintGraphLocalVariableLayout::OnVarNameTextCommitted)
#endif
				.ToolTipText(this, &FBlueprintGraphLocalVariableLayout::OnGetVarToolTipText)
				.Font( IDetailLayoutBuilder::GetDetailFont() )
				.IsEnabled(!ShouldVarBeReadOnly())
			]
		]
	]
	.ValueContent()
	.MaxDesiredWidth(980.f)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(0.0f)
		.FillWidth(1.0f)
		[
			SNew(SPinTypeSelector, FGetPinTypeTree::CreateUObject(K2Schema, &UEdGraphSchema_K2::GetVariableTypeTree))
				.TargetPinType(this, &FBlueprintGraphLocalVariableLayout::OnGetVariableType)
#if UE_BP_LOCAL_VAR_LAYOUT_SETTERS_IMPLEMENTED
				.OnPinTypePreChanged(this, &FBlueprintGraphLocalVariableLayout::OnPreVariableTypeChange)
				.OnPinTypeChanged(this, &FBlueprintGraphLocalVariableLayout::VariableTypeChanged)
#endif
				.Schema(K2Schema)
				.bAllowArrays(!ShouldVarBeReadOnly())
				.IsEnabled(!ShouldVarBeReadOnly(true))
				.Font( IDetailLayoutBuilder::GetDetailFont() )
		]
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(10, 0, 0, 0)
		.AutoWidth()
#if UE_BP_LOCAL_VAR_LAYOUT_SETTERS_IMPLEMENTED
		[
			PropertyCustomizationHelpers::MakeClearButton(FSimpleDelegate::CreateSP(this, &FBlueprintGraphLocalVariableLayout::OnRemoveClicked), LOCTEXT("LocalVariableDetailsClearTooltip", "Remove this variable."), !IsVariableEditingReadOnly())
		]
#endif

	]
	.PropertyHandleList({GetPropertyHandle()})
#if UE_BP_LOCAL_VAR_LAYOUT_SETTERS_IMPLEMENTED
	.DragDropHandler(/* implement drag drop handler and add it here */);
	static_assert(false)
#endif
	;
}

void FBlueprintGraphLocalVariableLayout::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	if (const UFunction* Function = OwningFunction.Get())
	{
		const TSharedPtr<FStructOnScope> StructData = MakeShareable(new FStructOnScope(Function));

		// ensure that the default value is up to date inside property
		for (TFieldIterator<FProperty> PropertyIterator(Function); PropertyIterator; ++PropertyIterator)
		{
			const FProperty *VariableProperty = *PropertyIterator;
			if (VariableProperty->GetFName() == GetName())
			{
				FBlueprintEditorUtils::PropertyValueFromString(VariableProperty, GetVariable().DefaultValue, StructData->GetStructMemory());
				break;
			}
		}
		if (IDetailPropertyRow* Row = ChildrenBuilder.AddExternalStructureProperty(StructData.ToSharedRef(), GetName()))
		{
			Row->DisplayName(LOCTEXT("LocalVariableDefaultValue", "Default Value"));
		}
	}
}

bool FBlueprintGraphLocalVariableLayout::ShouldVarBeReadOnly(bool bIsEditingPinType) const
{
#if UE_BP_LOCAL_VAR_LAYOUT_SETTERS_IMPLEMENTED
	// if this is ever made non-const, implement this method
	static_assert(false);
#endif
	return true;
}

bool FBlueprintGraphLocalVariableLayout::IsVariableEditingReadOnly(bool bIsEditingPinType) const
{
#if UE_BP_LOCAL_VAR_LAYOUT_SETTERS_IMPLEMENTED
	// if this is ever made non-const, implement this method
	static_assert(false)
#endif
	return true;
}

FText FBlueprintGraphLocalVariableLayout::OnGetVarNameText() const
{
	return FText::FromName(GetName());
}

FText FBlueprintGraphLocalVariableLayout::OnGetVarToolTipText() const
{
	const FBPVariableDescription& Variable = GetVariable();
	const FText PinTypeText = UEdGraphSchema_K2::TypeToText(Variable.VarType);
	return FText::Format(LOCTEXT("BlueprintArgToolTipText", "Name: {0}\nType: {1}"), FText::FromName(Variable.VarName), PinTypeText);
}

FEdGraphPinType FBlueprintGraphLocalVariableLayout::OnGetVariableType() const
{
	return GetVariable().VarType;
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FBlueprintGraphActionDetails::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{
	DetailsLayoutPtr = &DetailLayout;
	ObjectsBeingEdited = DetailsLayoutPtr->GetSelectedObjects();

	SetEntryAndResultNodes();

	UK2Node_EditablePinBase* FunctionEntryNode = FunctionEntryNodePtr.Get();
	UK2Node_EditablePinBase* FunctionResultNode = FunctionResultNodePtr.Get();

	// Fill Access specifiers list
	AccessSpecifierLabels.Empty(3);
	AccessSpecifierLabels.Add( MakeShareable( new FAccessSpecifierLabel( AccessSpecifierProperName(FUNC_Public), FUNC_Public )));
	AccessSpecifierLabels.Add( MakeShareable( new FAccessSpecifierLabel( AccessSpecifierProperName(FUNC_Protected), FUNC_Protected )));
	AccessSpecifierLabels.Add( MakeShareable( new FAccessSpecifierLabel( AccessSpecifierProperName(FUNC_Private), FUNC_Private )));

	const bool bHasAGraph = (GetGraph() != NULL);

	if (FunctionEntryNode && FunctionEntryNode->IsEditable())
	{
		const bool bIsCustomEvent = IsCustomEvent();
		const bool bIsFunctionGraph = FunctionEntryNode->IsA<UK2Node_FunctionEntry>();

		IDetailCategoryBuilder& Category = DetailLayout.EditCategory("Graph", LOCTEXT("FunctionDetailsGraph", "Graph"));
		if (bHasAGraph)
		{
			Category.AddCustomRow( LOCTEXT( "DefaultTooltip", "Description" ) )
			.NameContent()
			[
				SNew(STextBlock)
					.Text( LOCTEXT( "DefaultTooltip", "Description" ) )
					.ToolTipText(LOCTEXT("FunctionTooltipTooltip", "Enter a short message describing the purpose and operation of this graph"))
					.Font( IDetailLayoutBuilder::GetDetailFont() )
			]
			.ValueContent()
			[
				SNew(SMultiLineEditableTextBox)
					.Text( this, &FBlueprintGraphActionDetails::OnGetTooltipText )
					.OnTextCommitted( this, &FBlueprintGraphActionDetails::OnTooltipTextCommitted )
					.Font( IDetailLayoutBuilder::GetDetailFont() )
					.ModiferKeyForNewLine(EModifierKey::Shift)
			];

			// Composite graphs are auto-categorized into their parent graph
			if(!GetGraph()->GetOuter()->GetClass()->IsChildOf(UK2Node_Composite::StaticClass()))
			{
				FBlueprintVarActionDetails::PopulateCategories(MyBlueprint.Pin().Get(), CategorySource);
				TSharedPtr<SComboButton> NewComboButton;
				TSharedPtr<SListView<TSharedPtr<FText>>> NewListView;

				const FString DocLink = TEXT("Shared/Editors/BlueprintEditor/GraphDetails");
				TSharedPtr<SToolTip> CategoryTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("EditGraphCategoryName_Tooltip", "The category of the graph; editing this will place the graph into another category or create a new one."), NULL, DocLink, TEXT("Category"));

				Category.AddCustomRow( LOCTEXT("CategoryLabel", "Category") )
					.NameContent()
					[
						SNew(STextBlock)
						.Text( LOCTEXT("CategoryLabel", "Category") )
						.ToolTip(CategoryTooltip)
						.Font( IDetailLayoutBuilder::GetDetailFont() )
					]
				.ValueContent()
					[
						SAssignNew(NewComboButton, SComboButton)
						.ContentPadding(FMargin(0,0,5,0))
						.ToolTip(CategoryTooltip)
						.ButtonContent()
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::Get().GetBrush("NoBorder") )
							.Padding(FMargin(0, 0, 5, 0))
							[
								SNew(SEditableTextBox)
								.Text(this, &FBlueprintGraphActionDetails::OnGetCategoryText)
								.OnTextCommitted(this, &FBlueprintGraphActionDetails::OnCategoryTextCommitted )
								.ToolTip(CategoryTooltip)
								.SelectAllTextWhenFocused(true)
								.RevertTextOnEscape(true)
								.Font( IDetailLayoutBuilder::GetDetailFont() )
							]
						]
						.MenuContent()
							[
								SNew(SVerticalBox)
								+SVerticalBox::Slot()
								.AutoHeight()
								.MaxHeight(400.0f)
								[
									SAssignNew(NewListView, SListView<TSharedPtr<FText>>)
									.ListItemsSource(&CategorySource)
									.OnGenerateRow(this, &FBlueprintGraphActionDetails::MakeCategoryViewWidget)
									.OnSelectionChanged(this, &FBlueprintGraphActionDetails::OnCategorySelectionChanged)
								]
							]
					];

				CategoryComboButton = NewComboButton;
				CategoryListView = NewListView;

				TSharedPtr<SToolTip> KeywordsTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("EditKeywords_Tooltip", "Keywords for searching for the function or macro."), NULL, DocLink, TEXT("Keywords"));
				Category.AddCustomRow( LOCTEXT("KeywordsLabel", "Keywords") )
					.NameContent()
					[
						SNew(STextBlock)
						.Text( LOCTEXT("KeywordsLabel", "Keywords") )
						.ToolTip(KeywordsTooltip)
						.Font( IDetailLayoutBuilder::GetDetailFont() )
					]
				.ValueContent()
					[
						SNew(SEditableTextBox)
						.Text(this, &FBlueprintGraphActionDetails::OnGetKeywordsText)
						.OnTextCommitted(this, &FBlueprintGraphActionDetails::OnKeywordsTextCommitted )
						.ToolTip(KeywordsTooltip)
						.RevertTextOnEscape(true)
						.Font( IDetailLayoutBuilder::GetDetailFont() )
					];

				TSharedPtr<SToolTip> CompactNodeTitleTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("EditCompactNodeTitle_Tooltip", "Sets the compact node title for calls to this function or macro. Compact node titles convert a node to display as a compact node and are used as a keyword for searching."), NULL, DocLink, TEXT("Compact Node Title"));
				Category.AddCustomRow( LOCTEXT("CompactNodeTitleLabel", "Compact Node Title") )
					.NameContent()
					[
						SNew(STextBlock)
						.Text( LOCTEXT("CompactNodeTitleLabel", "Compact Node Title") )
						.ToolTip(CompactNodeTitleTooltip)
						.Font( IDetailLayoutBuilder::GetDetailFont() )
					]
				.ValueContent()
					[
						SNew(SEditableTextBox)
						.Text(this, &FBlueprintGraphActionDetails::OnGetCompactNodeTitleText)
						.OnTextCommitted(this, &FBlueprintGraphActionDetails::OnCompactNodeTitleTextCommitted )
						.ToolTip(CompactNodeTitleTooltip)
						.RevertTextOnEscape(true)
						.Font( IDetailLayoutBuilder::GetDetailFont() )
					];
			}
			 
			UBlueprint* BlueprintPtr = GetBlueprintObj();
			
			if (BlueprintPtr && IsFieldNotifyCheckVisible())
			{
				const FText ToolTip = LOCTEXT("FieldNotifyToolTip", "Generate a field entry for the Field Notification system.");
				const FString DocLink = TEXT("Shared/Editors/BlueprintEditor/GraphDetails");
				TSharedPtr<SToolTip> FieldNotificationTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("FieldNotifyToolTip", "Generate a field entry for the Field Notification system."), NULL, DocLink, TEXT("FieldNotify"));

				Category.AddCustomRow(LOCTEXT("IsFunctionFieldNotifyLabel", "Field Notify"))
					.NameContent()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("IsFunctionFieldNotifyLabel", "Field Notify"))
						.ToolTip(FieldNotificationTooltip)
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
					.ValueContent()
					[
						SNew(SCheckBox)
						.IsChecked(this, &FBlueprintGraphActionDetails::OnFieldNotifyCheckboxState)
						.OnCheckStateChanged(this, &FBlueprintGraphActionDetails::OnFieldNotifyChanged)
						.IsEnabled(this, &FBlueprintGraphActionDetails::GetIsFieldNotfyEnabled)
						.ToolTip(FieldNotificationTooltip)
					];
			}

			if (GetInstanceColorVisibility())
			{
				Category.AddCustomRow( LOCTEXT( "InstanceColor", "Instance Color" ) )
				.NameContent()
				[
					SNew(STextBlock)
						.Text( LOCTEXT( "InstanceColor", "Instance Color" ) )
						.ToolTipText( LOCTEXT("FunctionColorTooltip", "Choose a title bar color for references of this graph") )
						.Font( IDetailLayoutBuilder::GetDetailFont() )
				]
				.ValueContent()
				[
					SAssignNew( ColorBlock, SColorBlock )
						.Color( this, &FBlueprintGraphActionDetails::GetNodeTitleColor )
						.CornerRadius(FVector4(4.0f, 4.0f, 4.0f, 4.0f))
						.Size(FVector2D(70.0f, 22.0f))
						.AlphaDisplayMode(EColorBlockAlphaDisplayMode::Ignore)
						.OnMouseButtonDown( this, &FBlueprintGraphActionDetails::ColorBlock_OnMouseButtonDown )
				];
			}
			if (IsPureFunctionVisible())
			{
				Category.AddCustomRow( LOCTEXT( "FunctionPure_Tooltip", "Pure" ) )
				.NameContent()
				[
					SNew(STextBlock)
						.Text( LOCTEXT( "FunctionPure_Tooltip", "Pure" ) )
						.ToolTipText( LOCTEXT("FunctionIsPure_Tooltip", "Force this to be a pure function?") )
						.Font( IDetailLayoutBuilder::GetDetailFont() )
				]
				.ValueContent()
				[
					SNew( SCheckBox )
						.IsChecked( this, &FBlueprintGraphActionDetails::GetIsPureFunction )
						.OnCheckStateChanged( this, &FBlueprintGraphActionDetails::OnIsPureFunctionModified )
				];
			}
			if (IsConstFunctionVisible())
			{
				Category.AddCustomRow( LOCTEXT( "FunctionConst_Tooltip", "Const" ), true )
				.NameContent()
				[
					SNew(STextBlock)
					.Text( LOCTEXT( "FunctionConst_Tooltip", "Const" ) )
					.ToolTipText( LOCTEXT("FunctionIsConst_Tooltip", "Force this to be a const function?") )
					.Font( IDetailLayoutBuilder::GetDetailFont() )
				]
				.ValueContent()
				[
					SNew( SCheckBox )
					.IsChecked( this, &FBlueprintGraphActionDetails::GetIsConstFunction )
					.OnCheckStateChanged( this, &FBlueprintGraphActionDetails::OnIsConstFunctionModified )
				];
			}
			if (IsExecFunctionVisible())
			{
				Category.AddCustomRow( LOCTEXT( "FunctionExec_Tooltip", "Exec" ), true )
				.NameContent()
				[
					SNew(STextBlock)
					.Text( LOCTEXT( "FunctionExec_Tooltip", "Exec" ) )
					.ToolTipText( LOCTEXT("FunctionIsExec_Tooltip", "Cause this function to be able to process console commands?") )
					.Font( IDetailLayoutBuilder::GetDetailFont() )
				]
				.ValueContent()
				[
					SNew( SCheckBox )
					.IsChecked( this, &FBlueprintGraphActionDetails::GetIsExecFunction )
					.OnCheckStateChanged( this, &FBlueprintGraphActionDetails::OnIsExecFunctionModified )
				];
			}
			if (IsThreadSafeFunctionVisible())
			{
				Category.AddCustomRow( LOCTEXT( "FunctionThreadSafe_Tooltip", "Thread Safe" ), true )
				.NameContent()
				[
					SNew(STextBlock)
					.Text( LOCTEXT( "FunctionThreadSafe_Tooltip", "Thread Safe" ) )
				 	.ToolTipText( LOCTEXT("FunctionIsThreadSafe_Tooltip", "Enable thread-safety checks on this function. Only thread-safe functions and operations are allowed in this function.") )
					.Font( IDetailLayoutBuilder::GetDetailFont() )
				]
				.ValueContent()
				[
					SNew( SCheckBox )
					.IsChecked( this, &FBlueprintGraphActionDetails::GetIsThreadSafeFunction )
					.OnCheckStateChanged( this, &FBlueprintGraphActionDetails::OnIsThreadSafeFunctionModified )
				];
			}
			if (IsUnsafeDuringActorConstructionVisible())
			{
				Category.AddCustomRow(LOCTEXT("FunctionUnsafeDuringActorConstruction_Tooltip", "Unsafe During Actor Construction"), true)
				.NameContent()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("FunctionUnsafeDuringActorConstruction_Tooltip", "Unsafe During Actor Construction"))
					.ToolTipText(LOCTEXT("FunctionIsUnsafeDuringActorConstruction_Tooltip", "Mark this function as unsafe during actor construction so that a warning is generated when it is called by a Construction Script - useful when calling native functions that are also unsafe during construction"))
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
				.ValueContent()
				[
					SNew(SCheckBox)
					.IsChecked(this, &FBlueprintGraphActionDetails::GetIsUnsafeDuringActorConstruction)
					.OnCheckStateChanged(this, &FBlueprintGraphActionDetails::OnIsUnsafeDuringActorConstructionModified)
				];
			}
		}

		if (bIsCustomEvent)
		{
			/** A collection of static utility callbacks to provide the custom-event details ui with */
			struct LocalCustomEventUtils
			{
				/** Checks to see if the selected node is NOT an override */
				static bool IsNotCustomEventOverride(TWeakObjectPtr<UK2Node_EditablePinBase> SelectedNode)
				{
					bool bIsOverride = false;
					if (SelectedNode.IsValid())
					{
						UK2Node_CustomEvent const* SelectedCustomEvent = Cast<UK2Node_CustomEvent const>(SelectedNode.Get());
						check(SelectedCustomEvent != nullptr);

						bIsOverride = SelectedCustomEvent->IsOverride();
					}

					return !bIsOverride;
				}

				/** If the selected node represent an override, this returns tooltip text explaining why you can't alter the replication settings */
				static FText GetDisabledTooltip(TWeakObjectPtr<UK2Node_EditablePinBase> SelectedNode)
				{
					FText ToolTipOut = FText::GetEmpty();
					if (!IsNotCustomEventOverride(SelectedNode))
					{
						ToolTipOut = LOCTEXT("CannotChangeOverrideReplication", "Cannot alter a custom-event's replication settings when it overrides an event declared in a parent.");
					}
					return ToolTipOut;
				}

				/** Determines if the selected node's "Reliable" net setting should be enabled for the user */
				static bool CanSetReliabilityProperty(TWeakObjectPtr<UK2Node_EditablePinBase> SelectedNode)
				{
					bool bIsReliabilitySettingEnabled = false;
					if (IsNotCustomEventOverride(SelectedNode) && SelectedNode.IsValid())
					{
						UK2Node_CustomEvent const* SelectedCustomEvent = Cast<UK2Node_CustomEvent const>(SelectedNode.Get());
						check(SelectedCustomEvent != nullptr);

						bIsReliabilitySettingEnabled = ((SelectedCustomEvent->GetNetFlags() & FUNC_Net) != 0);
					}
					return bIsReliabilitySettingEnabled;
				}
			};
			FCanExecuteAction CanExecuteDelegate = FCanExecuteAction::CreateStatic(&LocalCustomEventUtils::IsNotCustomEventOverride, FunctionEntryNodePtr);

			FMenuBuilder RepComboMenu( true, NULL );
			RepComboMenu.AddMenuEntry( 	ReplicationSpecifierProperName(0), 
										LOCTEXT("NotReplicatedToolTip", "This event is not replicated to anyone."),
										FSlateIcon(),
										FUIAction(FExecuteAction::CreateStatic( &FBlueprintGraphActionDetails::SetNetFlags, FunctionEntryNodePtr, 0U ), CanExecuteDelegate));
			RepComboMenu.AddMenuEntry(	ReplicationSpecifierProperName(FUNC_NetMulticast), 
										LOCTEXT("MulticastToolTip", "Replicate this event from the server to everyone else. Server executes this event locally too. Only call this from the server."),
										FSlateIcon(),
										FUIAction(FExecuteAction::CreateStatic( &FBlueprintGraphActionDetails::SetNetFlags, FunctionEntryNodePtr, static_cast<uint32>(FUNC_NetMulticast) ), CanExecuteDelegate));
			RepComboMenu.AddMenuEntry(	ReplicationSpecifierProperName(FUNC_NetServer), 
										LOCTEXT("ServerToolTip", "Replicate this event from net owning client to server."),
										FSlateIcon(),
										FUIAction(FExecuteAction::CreateStatic( &FBlueprintGraphActionDetails::SetNetFlags, FunctionEntryNodePtr, static_cast<uint32>(FUNC_NetServer) ), CanExecuteDelegate));
			RepComboMenu.AddMenuEntry(	ReplicationSpecifierProperName(FUNC_NetClient), 
										LOCTEXT("ClientToolTip", "Replicate this event from the server to owning client."),
										FSlateIcon(),
										FUIAction(FExecuteAction::CreateStatic( &FBlueprintGraphActionDetails::SetNetFlags, FunctionEntryNodePtr, static_cast<uint32>(FUNC_NetClient) ), CanExecuteDelegate));

			const FString DocLink = TEXT("Shared/Editors/BlueprintEditor/GraphDetails");
			TSharedPtr<SToolTip> KeywordsTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("EditEventKeywords_Tooltip", "Keywords for searching for the event."), nullptr, DocLink, TEXT("Keywords"));
			Category.AddCustomRow(LOCTEXT("EventsKeywordsLabel", "Keywords"))
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("KeywordsLabel", "Keywords"))
					.ToolTip(KeywordsTooltip)
					.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SNew(SEditableTextBox)
					.Text(this, &FBlueprintGraphActionDetails::OnGetKeywordsText)
					.OnTextCommitted(this, &FBlueprintGraphActionDetails::OnKeywordsTextCommitted)
					.ToolTip(KeywordsTooltip)
					.RevertTextOnEscape(true)
					.Font(IDetailLayoutBuilder::GetDetailFont())
			];


			Category.AddCustomRow( LOCTEXT( "FunctionReplicate", "Replicates" ) )
			.NameContent()
			[
				SNew(STextBlock)
					.Text( LOCTEXT( "FunctionReplicate", "Replicates" ) )
					.ToolTipText( LOCTEXT("FunctionReplicate_Tooltip", "Should this Event be replicated to all clients when called on the server?") )
					.Font( IDetailLayoutBuilder::GetDetailFont() )
			]
			.ValueContent()
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				[
					SNew(SComboButton)
						.ContentPadding(0.0f)
						.IsEnabled_Static(&LocalCustomEventUtils::IsNotCustomEventOverride, FunctionEntryNodePtr)
						.ToolTipText_Static(&LocalCustomEventUtils::GetDisabledTooltip, FunctionEntryNodePtr)
						.ButtonContent()
						[
							SNew(STextBlock)
								.Text(this, &FBlueprintGraphActionDetails::GetCurrentReplicatedEventString)
								.Font( IDetailLayoutBuilder::GetDetailFont() )
						]
						.MenuContent()
						[
							SNew(SVerticalBox)
							+SVerticalBox::Slot()
							[
								SNew(SVerticalBox)
								+SVerticalBox::Slot()
									.AutoHeight()
									.MaxHeight(400.0f)
								[
									RepComboMenu.MakeWidget()
								]
							]
						]
				]

				+SVerticalBox::Slot()
					.AutoHeight()
					.MaxHeight(400.0f)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
						.AutoWidth()
					[
						SNew( SCheckBox )
							.IsChecked( this, &FBlueprintGraphActionDetails::GetIsReliableReplicatedFunction )
							.IsEnabled_Static(&LocalCustomEventUtils::CanSetReliabilityProperty, FunctionEntryNodePtr)
							.ToolTipText_Static(&LocalCustomEventUtils::GetDisabledTooltip, FunctionEntryNodePtr)
							.OnCheckStateChanged( this, &FBlueprintGraphActionDetails::OnIsReliableReplicationFunctionModified )
						[
							SNew(STextBlock)
								.Text( LOCTEXT( "FunctionReplicateReliable", "Reliable" ) )
						]
					]
				]
			];
		}
		
		const bool bShowCallInEditor = bIsCustomEvent || bIsFunctionGraph;
		if( bShowCallInEditor )
		{
			Category.AddCustomRow( LOCTEXT( "EditorCallable", "Call In Editor" ) )
			.NameContent()
			[
				SNew(STextBlock)
					.Text( LOCTEXT( "EditorCallable", "Call In Editor" ) )
					.ToolTipText( LOCTEXT("EditorCallable_Tooltip", "Enable this event to be called from within the editor") )
					.Font( IDetailLayoutBuilder::GetDetailFont() )
			]
			.ValueContent()
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew( SCheckBox )
						.IsChecked( this, &FBlueprintGraphActionDetails::GetIsEditorCallableEvent )
						.ToolTipText( LOCTEXT("EditorCallable_Tooltip", "Enable this event to be called from within the editor" ))
						.OnCheckStateChanged( this, &FBlueprintGraphActionDetails::OnEditorCallableEventModified )
					]
				]
			];
		}

		const bool bShowDeprecated = bIsCustomEvent || bIsFunctionGraph;
		if (bShowDeprecated)
		{
			FFormatNamedArguments DeprecationTooltipFormatArgs;
			if (bIsFunctionGraph)
			{
				DeprecationTooltipFormatArgs.Add(TEXT("FunctionOrCustomEvent"), LOCTEXT("FunctionOrEvent_Function", "function"));
			}
			else
			{
				DeprecationTooltipFormatArgs.Add(TEXT("FunctionOrCustomEvent"), LOCTEXT("FunctionOrEvent_CustomEvent", "custom event"));
			}

			bool bIsOverride = false;
			FFunctionFromNodeHelper FunctionFromNode(FunctionEntryNode);
			if (FunctionFromNode.Function)
			{
				bIsOverride = (UEdGraphSchema_K2::GetCallableParentFunction(FunctionFromNode.Function) != nullptr);
			}

			const FText DeprecatedTooltipText = FText::Format(LOCTEXT("DeprecatedFunction_Tooltip", "Deprecate usage of this {FunctionOrCustomEvent}. Any nodes that reference it will produce a compiler warning indicating that it should be removed or replaced."), DeprecationTooltipFormatArgs);

			Category.AddCustomRow(LOCTEXT("DeprecatedFunction", "Deprecated"), true)
			.NameContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("DeprecatedFunction", "Deprecated"))
				.ToolTipText(DeprecatedTooltipText)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SNew(SCheckBox)
				.IsChecked(this, &FBlueprintGraphActionDetails::OnGetDeprecatedCheckboxState)
				.ToolTipText(DeprecatedTooltipText)
				.OnCheckStateChanged(this, &FBlueprintGraphActionDetails::OnDeprecatedChanged)
				.IsEnabled(!bIsOverride)
			];

			const FText DeprecationMessageTooltipText = LOCTEXT("DeprecationMessage_Tooltip", "Optional message to include with the deprecation compiler warning. For example: \'X is no longer being used. Please replace with Y.\'");

			Category.AddCustomRow(LOCTEXT("DeprecationMessage", "Deprecation Message"), true)
			.IsEnabled(TAttribute<bool>(this, &FBlueprintGraphActionDetails::IsFunctionDeprecated))
			.NameContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("DeprecationMessage", "Deprecation Message"))
				.ToolTipText(DeprecationMessageTooltipText)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SNew(SEditableTextBox)
				.Text(this, &FBlueprintGraphActionDetails::GetDeprecationMessageText)
				.OnTextCommitted(this, &FBlueprintGraphActionDetails::OnDeprecationMessageTextCommitted)
				.IsEnabled(!bIsOverride)
				.ToolTipText(this, &FBlueprintGraphActionDetails::GetDeprecationMessageText)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			];
		}

		const bool bShowAccessSpecifiers = bIsCustomEvent || bIsFunctionGraph;
		if (IsAccessSpecifierVisible())
		{
			Category.AddCustomRow(LOCTEXT("AccessSpecifier", "Access Specifier"))
				.NameContent()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AccessSpecifier", "Access Specifier"))
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
				.ValueContent()
				[
					SAssignNew(AccessSpecifierComboButton, SComboButton)
					.ContentPadding(0.0f)
					.ButtonContent()
					[
						SNew(STextBlock)
						.Text(this, &FBlueprintGraphActionDetails::GetCurrentAccessSpecifierName)
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
					.MenuContent()
					[
						SNew(SListView<TSharedPtr<FAccessSpecifierLabel> >)
						.ListItemsSource(&AccessSpecifierLabels)
						.OnGenerateRow(this, &FBlueprintGraphActionDetails::HandleGenerateRowAccessSpecifier)
						.OnSelectionChanged(this, &FBlueprintGraphActionDetails::OnAccessSpecifierSelected)
					]
				];
		}

		IDetailCategoryBuilder& InputsCategory = DetailLayout.EditCategory("Inputs", LOCTEXT("FunctionDetailsInputs", "Inputs"));
		
		TSharedRef<FBlueprintGraphArgumentGroupLayout> InputArgumentGroup =
			MakeShareable(new FBlueprintGraphArgumentGroupLayout(SharedThis(this), FunctionEntryNode));
		InputsCategory.AddCustomBuilder(InputArgumentGroup);

		TSharedRef<SHorizontalBox> InputsHeaderContentWidget = SNew(SHorizontalBox);
		TWeakPtr<SWidget> WeakInputsHeaderWidget = InputsHeaderContentWidget;

		InputsHeaderContentWidget->AddSlot()
			.HAlign(HAlign_Right)
			[

				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ContentPadding(FMargin(1, 0))
				.OnClicked(this, &FBlueprintGraphActionDetails::OnAddNewInputClicked)
				.Visibility(this, &FBlueprintGraphActionDetails::GetAddNewInputOutputVisibility)
				.HAlign(HAlign_Right)
				.ToolTipText(LOCTEXT("FunctionNewInputArgTooltip", "Create a new input argument"))
				.VAlign(VAlign_Center)
				.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("FunctionNewInputArg")))
				.IsEnabled(this, &FBlueprintGraphActionDetails::IsAddNewInputOutputEnabled)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			];
		InputsCategory.HeaderContent(InputsHeaderContentWidget);

		if (bHasAGraph)
		{
			IDetailCategoryBuilder& OutputsCategory = DetailLayout.EditCategory("Outputs", LOCTEXT("FunctionDetailsOutputs", "Outputs"));
		
			TSharedRef<FBlueprintGraphArgumentGroupLayout> OutputArgumentGroup =
				MakeShareable(new FBlueprintGraphArgumentGroupLayout(SharedThis(this), FunctionResultNode));
			OutputsCategory.AddCustomBuilder(OutputArgumentGroup);
		
			TSharedRef<SHorizontalBox> OutputsHeaderContentWidget = SNew(SHorizontalBox);
			TWeakPtr<SWidget> WeakOutputsHeaderWidget = OutputsHeaderContentWidget;

			OutputsHeaderContentWidget->AddSlot()
				.HAlign(HAlign_Right)
				.Padding(FMargin(0,0,2,0))
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.ContentPadding(FMargin(1, 0))
					.OnClicked(this, &FBlueprintGraphActionDetails::OnAddNewOutputClicked)
					.Visibility(this, &FBlueprintGraphActionDetails::GetAddNewInputOutputVisibility)
					.HAlign(HAlign_Right)
					.ToolTipText(LOCTEXT("FunctionNewOutputArgTooltip", "Create a new output argument"))
					.VAlign(VAlign_Center)
					.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("FunctionNewOutputArg")))
					.IsEnabled(this, &FBlueprintGraphActionDetails::IsAddNewInputOutputEnabled)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				];
			OutputsCategory.HeaderContent(OutputsHeaderContentWidget);
		}

		if (bShowLocalVariables)
		{
			const UEdGraph* TopLevelGraph = FBlueprintEditorUtils::GetTopLevelGraph(GetGraph());
			TSharedPtr<IPropertyHandle> LocalVariablesProperty = nullptr;
			if (TopLevelGraph)
			{
				LocalVariablesProperty = DetailLayout.AddObjectPropertyData({FunctionEntryNode}, TEXT("LocalVariables"));
			}
		
			IDetailCategoryBuilder& LocalVarsCategory = DetailLayout.EditCategory("Local Variables", LOCTEXT("FunctionDetailsLocalVariables", "Local Variables"));
			TSharedRef<FBlueprintGraphLocalVariableGroupLayout> LocalVarsArgumentGroup =
					MakeShareable(new FBlueprintGraphLocalVariableGroupLayout(SharedThis(this), GetGraph(), GetBlueprintObj(), FindFunction(), LocalVariablesProperty));
			LocalVarsCategory.AddCustomBuilder(LocalVarsArgumentGroup);
		
			TSharedRef<SHorizontalBox> LocalVarsHeaderContentWidget = SNew(SHorizontalBox);

			LocalVarsCategory.HeaderContent(LocalVarsHeaderContentWidget);
		}
		
		// See if anything else wants to customize our details
		TWeakPtr<FBlueprintEditor> BlueprintEditor = MyBlueprint.Pin()->GetBlueprintEditor();
		FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::GetModuleChecked<FBlueprintEditorModule>("Kismet");
		TArray<TSharedPtr<IDetailCustomization>> Customizations = BlueprintEditorModule.CustomizeFunction(FunctionEntryNode->GetClass(), BlueprintEditor.Pin());
		ExternalDetailCustomizations.Append(Customizations);
		if (ExternalDetailCustomizations.Num() > 0)
		{
			for (TSharedPtr<IDetailCustomization> ExternalDetailCustomization : ExternalDetailCustomizations)
			{
				ExternalDetailCustomization->CustomizeDetails(DetailLayout);
			}
		}
	}
	else if (bHasAGraph)
	{
		// See if anything else wants to customize our details
		TWeakPtr<FBlueprintEditor> BlueprintEditor = MyBlueprint.Pin()->GetBlueprintEditor();
		FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::GetModuleChecked<FBlueprintEditorModule>("Kismet");
		TArray<TSharedPtr<IDetailCustomization>> Customizations = BlueprintEditorModule.CustomizeGraph(GetGraph()->GetSchema(), BlueprintEditor.Pin());
		ExternalDetailCustomizations.Append(Customizations);
		if(ExternalDetailCustomizations.Num() > 0)
		{
			for (TSharedPtr<IDetailCustomization> ExternalDetailCustomization : ExternalDetailCustomizations)
			{
				ExternalDetailCustomization->CustomizeDetails(DetailLayout);
			}
		}
		else
		{
			IDetailCategoryBuilder& Category = DetailLayout.EditCategory("Graph", LOCTEXT("FunctionDetailsGraph", "Graph"));
			Category.AddCustomRow( FText::GetEmpty() )
			.WholeRowContent()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text( LOCTEXT("GraphPresentButNotEditable", "Graph is not editable.") )
			];
		}
	}
	
	if (MyBlueprint.IsValid())
	{
		TWeakPtr<FBlueprintEditor> BlueprintEditor = MyBlueprint.Pin()->GetBlueprintEditor();
		if (BlueprintEditor.IsValid())
		{
			BlueprintEditorRefreshDelegateHandle = BlueprintEditor.Pin()->OnRefresh().AddSP(this, &FBlueprintGraphActionDetails::OnPostEditorRefresh);
		}
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

bool FBlueprintGraphActionDetails::IsFieldNotifyCheckVisible() const
{
	bool bSupportedType = false;
	bool bIsEditable = false;
	bool bImplementsFieldNotify = false;
	UK2Node_EditablePinBase* FunctionEntryNode = FunctionEntryNodePtr.Get();
	UFunction* Function = FindFunction();

	if (FunctionEntryNode && Function)
	{
		UBlueprint* Blueprint = FunctionEntryNode->GetBlueprint();
		const bool bIsInterface = FBlueprintEditorUtils::IsInterfaceBlueprint(Blueprint);
		bImplementsFieldNotify = FBlueprintEditorUtils::ImplementsInterface(Blueprint, true, UNotifyFieldValueChanged::StaticClass());

		bSupportedType = !bIsInterface && FunctionEntryNode->IsA<UK2Node_FunctionEntry>();
		bIsEditable = FunctionEntryNode->IsEditable();
	}
	return bSupportedType && bIsEditable && bImplementsFieldNotify;
}

bool FBlueprintGraphActionDetails::GetIsFieldNotfyEnabled() const
{
	UK2Node_EditablePinBase* FunctionEntryNode = FunctionEntryNodePtr.Get();
	UK2Node_EditablePinBase* FunctionResultNode = FunctionResultNodePtr.Get();

	if (FunctionEntryNode && FunctionResultNode)
	{
		return GetIsConstFunction() == ECheckBoxState::Checked && GetIsPureFunction() == ECheckBoxState::Checked && FunctionEntryNode->GetAllPins().Num() == 1 && FunctionResultNode->GetAllPins().Num() == 2;
	}
	return false;
}

ECheckBoxState FBlueprintGraphActionDetails::OnFieldNotifyCheckboxState() const
{
	UBlueprint* const BlueprintObj = GetBlueprintObj();

	if (UFunction* Function = FindFunction())
	{
		const FName FuncName = Function->GetFName();

		if (BlueprintObj && GetIsFieldNotfyEnabled())
		{
			if (!FuncName.IsNone())
			{
				return GetMetadataBlock()->HasMetaData(FBlueprintMetadata::MD_FieldNotify) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
			else if (BlueprintObj->GeneratedClass && BlueprintObj->GeneratedClass->ImplementsInterface(UNotifyFieldValueChanged::StaticClass()) && BlueprintObj->GeneratedClass->GetDefaultObject())
			{
				TScriptInterface<INotifyFieldValueChanged> DefaultObject = BlueprintObj->GeneratedClass->GetDefaultObject();
				return DefaultObject->GetFieldNotificationDescriptor().GetField(BlueprintObj->GeneratedClass, FuncName).IsValid() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
		}

		FBlueprintEditorUtils::RemoveFieldNotifyFromAllMetadata(BlueprintObj, FuncName);
		GetMetadataBlock()->RemoveMetaData(FBlueprintMetadata::MD_FieldNotify);
	}
	return ECheckBoxState::Unchecked;
}

void FBlueprintGraphActionDetails::OnFieldNotifyChanged(ECheckBoxState InNewState)
{
	UBlueprint* const BlueprintObj = GetBlueprintObj();
	if (UFunction* Function = FindFunction())
	{
		const FName FuncName = Function->GetFName();
		const bool bFuncIsFieldNotify = InNewState == ECheckBoxState::Checked;

		if (BlueprintObj)
		{
			if (bFuncIsFieldNotify)
			{
				GetMetadataBlock()->SetMetaData(FBlueprintMetadata::MD_FieldNotify, FString());
			}
			else
			{
				FBlueprintEditorUtils::RemoveFieldNotifyFromAllMetadata(BlueprintObj, FuncName);
				GetMetadataBlock()->RemoveMetaData(FBlueprintMetadata::MD_FieldNotify);
			}
		}
	}
}

TSharedRef<ITableRow> FBlueprintGraphActionDetails::OnGenerateReplicationComboWidget( TSharedPtr<FReplicationSpecifierLabel> InNetFlag, const TSharedRef<STableViewBase>& OwnerTable )
{
	return
		SNew(STableRow< TSharedPtr<FString> >, OwnerTable)
		[
			SNew( STextBlock )
			.Text( InNetFlag.IsValid() ? InNetFlag.Get()->LocalizedName : FText::GetEmpty() )
			.ToolTipText( InNetFlag.IsValid() ? InNetFlag.Get()->LocalizedToolTip : FText::GetEmpty() )
		];
}

void FBlueprintGraphActionDetails::SetNetFlags( TWeakObjectPtr<UK2Node_EditablePinBase> FunctionEntryNode, uint32 NetFlags)
{
	if( FunctionEntryNode.IsValid() )
	{
		const int32 FlagsToSet = NetFlags ? FUNC_Net|NetFlags : 0;
		const int32 FlagsToClear = FUNC_Net|FUNC_NetMulticast|FUNC_NetServer|FUNC_NetClient;
		// Clear all net flags before setting
		if( FlagsToSet != FlagsToClear )
		{
			const FScopedTransaction Transaction( LOCTEXT("GraphSetNetFlags", "Change Replication") );
			FunctionEntryNode->Modify();

			bool bBlueprintModified = false;

			if (UK2Node_FunctionEntry* TypedEntryNode = Cast<UK2Node_FunctionEntry>(FunctionEntryNode.Get()))
			{
				int32 ExtraFlags = TypedEntryNode->GetExtraFlags();
				ExtraFlags &= ~FlagsToClear;
				ExtraFlags |= FlagsToSet;
				TypedEntryNode->SetExtraFlags(ExtraFlags);
				bBlueprintModified = true;
			}
			if (UK2Node_CustomEvent* CustomEventNode = Cast<UK2Node_CustomEvent>(FunctionEntryNode.Get()))
			{
				CustomEventNode->FunctionFlags &= ~FlagsToClear;
				CustomEventNode->FunctionFlags |= FlagsToSet;
				bBlueprintModified = true;
			}

			if( bBlueprintModified )
			{
				FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified( FunctionEntryNode->GetBlueprint() );
			}
		}
	}
}

FText FBlueprintGraphActionDetails::GetCurrentReplicatedEventString() const
{
	const UK2Node_EditablePinBase* FunctionEntryNode = FunctionEntryNodePtr.Get();
	const UK2Node_CustomEvent* CustomEvent = Cast<const UK2Node_CustomEvent>(FunctionEntryNode);

	uint32 const ReplicatedNetMask = (FUNC_NetMulticast | FUNC_NetServer | FUNC_NetClient);

	FText ReplicationText;

	if(CustomEvent)
	{
		uint32 NetFlags = CustomEvent->FunctionFlags & ReplicatedNetMask;
		if (CustomEvent->IsOverride())
		{
			UFunction* SuperFunction = FindUField<UFunction>(CustomEvent->GetBlueprint()->ParentClass, CustomEvent->CustomFunctionName);
			check(SuperFunction != NULL);

			NetFlags = SuperFunction->FunctionFlags & ReplicatedNetMask;
		}
		ReplicationText = ReplicationSpecifierProperName(NetFlags);
	}
	return ReplicationText;
}

bool FBaseBlueprintGraphActionDetails::AttemptToCreateResultNode()
{
	if (!FunctionResultNodePtr.IsValid())
	{
		FunctionResultNodePtr = FBlueprintEditorUtils::FindOrCreateFunctionResultNode(FunctionEntryNodePtr.Get());
	}
	return FunctionResultNodePtr.IsValid();
}

FBaseBlueprintGraphActionDetails::~FBaseBlueprintGraphActionDetails()
{
	if (BlueprintEditorRefreshDelegateHandle.IsValid() && MyBlueprint.IsValid())
	{
		// Remove the callback delegate we registered for
		TWeakPtr<FBlueprintEditor> BlueprintEditor = MyBlueprint.Pin()->GetBlueprintEditor();
		if (BlueprintEditor.IsValid())
		{
			BlueprintEditor.Pin()->OnRefresh().Remove(BlueprintEditorRefreshDelegateHandle);
		}
	}
}

void FBaseBlueprintGraphActionDetails::OnPostEditorRefresh()
{
	/** Blueprint changed, need to refresh inputs in case pin UI changed */
	RegenerateInputsChildrenDelegate.ExecuteIfBound();
	RegenerateOutputsChildrenDelegate.ExecuteIfBound();
}

void FBaseBlueprintGraphActionDetails::SetRefreshDelegate(FSimpleDelegate RefreshDelegate, bool bForInputs)
{
	((bForInputs) ? RegenerateInputsChildrenDelegate : RegenerateOutputsChildrenDelegate) = RefreshDelegate;
}

ECheckBoxState FBlueprintGraphActionDetails::GetIsEditorCallableEvent() const
{
	ECheckBoxState Result = ECheckBoxState::Unchecked;

	if( FunctionEntryNodePtr.IsValid() )
	{
		if( UK2Node_CustomEvent* CustomEventNode = Cast<UK2Node_CustomEvent>(FunctionEntryNodePtr.Get()))
		{
			if( CustomEventNode->bCallInEditor  )
			{
				Result = ECheckBoxState::Checked;
			}
		}
		else if( UK2Node_FunctionEntry* EntryPoint = Cast<UK2Node_FunctionEntry>(FunctionEntryNodePtr.Get()) )
		{
			if( EntryPoint->MetaData.bCallInEditor )
			{
				Result = ECheckBoxState::Checked;
			}
		}
	}
	return Result;
}

void FBlueprintGraphActionDetails::OnEditorCallableEventModified( const ECheckBoxState NewCheckedState ) const
{
	if( FunctionEntryNodePtr.IsValid() )
	{
		const bool bCallInEditor = NewCheckedState == ECheckBoxState::Checked;
		const FText TransactionType = bCallInEditor ?	LOCTEXT( "DisableCallInEditor", "Disable Call In Editor " ) : 
														LOCTEXT( "EnableCallInEditor", "Enable Call In Editor" );

		if( UK2Node_CustomEvent* CustomEventNode = Cast<UK2Node_CustomEvent>(FunctionEntryNodePtr.Get()) )
		{
			if( UBlueprint* Blueprint = FunctionEntryNodePtr->GetBlueprint() )
			{
				const FScopedTransaction Transaction( TransactionType );
				CustomEventNode->bCallInEditor = bCallInEditor;
				FBlueprintEditorUtils::MarkBlueprintAsModified( CustomEventNode->GetBlueprint() );
			}
		}
		else if( UK2Node_FunctionEntry* EntryPoint = Cast<UK2Node_FunctionEntry>(FunctionEntryNodePtr.Get()) )
		{
			const FScopedTransaction Transaction( TransactionType );
			EntryPoint->MetaData.bCallInEditor = bCallInEditor;
			FBlueprintEditorUtils::MarkBlueprintAsModified( EntryPoint->GetBlueprint() );
		}
		else
		{
			checkf(false, TEXT("Only Events and Functions are Callable In Editor"));
		}
	}
}

bool FBlueprintGraphActionDetails::IsFunctionDeprecated() const
{
	bool bIsDeprecated = false;

	UK2Node_EditablePinBase* Node = FunctionEntryNodePtr.Get();
	if (Node != nullptr)
	{
		UK2Node_CustomEvent* CustomEventNode = Cast<UK2Node_CustomEvent>(Node);
		if (CustomEventNode != nullptr)
		{
			bIsDeprecated = CustomEventNode->bIsDeprecated;
		}
		else
		{
			UK2Node_FunctionEntry* FunctionEntryNode = Cast<UK2Node_FunctionEntry>(Node);
			if (FunctionEntryNode != nullptr)
			{
				bIsDeprecated = FunctionEntryNode->MetaData.bIsDeprecated;
			}
		}
	}

	return bIsDeprecated;
}

ECheckBoxState FBlueprintGraphActionDetails::OnGetDeprecatedCheckboxState() const
{
	return IsFunctionDeprecated() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FBlueprintGraphActionDetails::OnDeprecatedChanged(ECheckBoxState InNewState)
{
	const bool bIsDeprecated = (InNewState == ECheckBoxState::Checked);

	UK2Node_EditablePinBase* Node = FunctionEntryNodePtr.Get();
	if (Node != nullptr)
	{
		UK2Node_CustomEvent* CustomEventNode = Cast<UK2Node_CustomEvent>(Node);
		if (CustomEventNode != nullptr)
		{
			CustomEventNode->bIsDeprecated = bIsDeprecated;
		}
		else
		{
			CastChecked<UK2Node_FunctionEntry>(Node)->MetaData.bIsDeprecated = bIsDeprecated;
		}

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Node->GetBlueprint());
	}
}

FText FBlueprintGraphActionDetails::GetDeprecationMessageText() const
{
	FText DeprecationMessage;

	UK2Node_EditablePinBase* Node = FunctionEntryNodePtr.Get();
	if (Node != nullptr)
	{
		UK2Node_CustomEvent* CustomEventNode = Cast<UK2Node_CustomEvent>(Node);
		if (CustomEventNode != nullptr)
		{
			DeprecationMessage = FText::FromString(CustomEventNode->DeprecationMessage);
		}
		else
		{
			UK2Node_FunctionEntry* FunctionEntryNode = Cast<UK2Node_FunctionEntry>(Node);
			if (FunctionEntryNode != nullptr)
			{
				DeprecationMessage = FText::FromString(FunctionEntryNode->MetaData.DeprecationMessage);
			}
		}
	}

	return DeprecationMessage;
}

void FBlueprintGraphActionDetails::OnDeprecationMessageTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit)
{
	const FString DeprecationMessage = NewText.ToString();

	UK2Node_EditablePinBase* Node = FunctionEntryNodePtr.Get();
	if (Node != nullptr)
	{
		UK2Node_CustomEvent* CustomEventNode = Cast<UK2Node_CustomEvent>(Node);
		if (CustomEventNode != nullptr)
		{
			CustomEventNode->DeprecationMessage = DeprecationMessage;
		}
		else
		{
			CastChecked<UK2Node_FunctionEntry>(Node)->MetaData.DeprecationMessage = DeprecationMessage;
		}

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Node->GetBlueprint());
	}
}

FMulticastDelegateProperty* FBlueprintDelegateActionDetails::GetDelegateProperty() const
{
	if (MyBlueprint.IsValid())
	{
		if (const FEdGraphSchemaAction_K2Delegate* DelegateVar = MyBlueprint.Pin()->SelectionAsDelegate())
		{
			return DelegateVar->GetDelegateProperty();
		}
	}
	return NULL;
}

bool FBlueprintDelegateActionDetails::IsBlueprintProperty() const
{
	const FMulticastDelegateProperty* Property = GetDelegateProperty();
	const UBlueprint* Blueprint = GetBlueprintObj();
	if(Property && Blueprint)
	{
		return (Property->GetOwner<UObject>() == Blueprint->SkeletonGeneratedClass);
	}

	return false;
}

void FBlueprintDelegateActionDetails::SetEntryNode()
{
	if (UEdGraph* NewTargetGraph = GetGraph())
	{
		TArray<UK2Node_FunctionEntry*> EntryNodes;
		NewTargetGraph->GetNodesOfClass(EntryNodes);

		if ((EntryNodes.Num() > 0) && EntryNodes[0]->IsEditable())
		{
			FunctionEntryNodePtr = EntryNodes[0];
		}
	}
}

UEdGraph* FBlueprintDelegateActionDetails::GetGraph() const
{
	if (MyBlueprint.IsValid())
	{
		if (const FEdGraphSchemaAction_K2Delegate* DelegateVar = MyBlueprint.Pin()->SelectionAsDelegate())
		{
			return DelegateVar->EdGraph;
		}
	}
	return NULL;
}

void FBlueprintDelegateActionDetails::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{
	DetailsLayoutPtr = &DetailLayout;
	ObjectsBeingEdited = DetailsLayoutPtr->GetSelectedObjects();

	SetEntryNode();

	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	const FSlateFontInfo DetailFontInfo = IDetailLayoutBuilder::GetDetailFont();

	if (UK2Node_EditablePinBase* FunctionEntryNode = FunctionEntryNodePtr.Get())
	{
		IDetailCategoryBuilder& InputsCategory = DetailLayout.EditCategory("DelegateInputs", LOCTEXT("DelegateDetailsInputs", "Inputs"));
		TSharedRef<FBlueprintGraphArgumentGroupLayout> InputArgumentGroup = MakeShareable(new FBlueprintGraphArgumentGroupLayout(SharedThis(this), FunctionEntryNode));
		InputsCategory.AddCustomBuilder(InputArgumentGroup);

		TSharedRef<SHorizontalBox> InputsHeaderContentWidget = SNew(SHorizontalBox);
		TWeakPtr<SWidget> WeakInputsHeaderWidget = InputsHeaderContentWidget;
		InputsHeaderContentWidget->AddSlot()
		[
			SNew(SHorizontalBox)
		];
		InputsHeaderContentWidget->AddSlot()
		.AutoWidth()
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "RoundButton")
			.ForegroundColor(FAppStyle::Get().GetSlateColor("DefaultForeground"))
			.ContentPadding(FMargin(2, 0))
			.OnClicked(this, &FBlueprintDelegateActionDetails::OnAddNewInputClicked)
			.HAlign(HAlign_Right)
			.ToolTipText(LOCTEXT("DelegateNewOutputArgTooltip", "Create a new input argument"))
			.VAlign(VAlign_Center)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("DelegateNewInputArg")))
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(0, 1))
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Plus"))
				]
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(FMargin(2, 0, 0, 0))
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFontBold())
					.Text(LOCTEXT("DelegateNewParameterInputArg", "New Parameter"))
					.Visibility(this, &FBlueprintDelegateActionDetails::OnGetSectionTextVisibility, WeakInputsHeaderWidget)
					.ShadowOffset(FVector2D(1, 1))
				]
			]
		];
		InputsCategory.HeaderContent(InputsHeaderContentWidget);

		CollectAvailibleSignatures();

		InputsCategory.AddCustomRow( LOCTEXT("CopySignatureFrom", "Copy signature from") )
		.NameContent()
		[
			SNew(STextBlock)
				.Text(LOCTEXT("CopySignatureFrom", "Copy signature from"))
				.Font( DetailFontInfo )
		]
		.ValueContent()
		[
			SAssignNew(CopySignatureComboButton, STextComboBox)
				.OptionsSource(&FunctionsToCopySignatureFrom)
				.OnSelectionChanged(this, &FBlueprintDelegateActionDetails::OnFunctionSelected)
		];
	}
}

void FBlueprintDelegateActionDetails::CollectAvailibleSignatures()
{
	FunctionsToCopySignatureFrom.Empty();
	if (FMulticastDelegateProperty* Property = GetDelegateProperty())
	{
		if (UClass* ScopeClass = Property->GetOwner<UClass>())
		{
			for(TFieldIterator<UFunction> It(ScopeClass, EFieldIteratorFlags::IncludeSuper); It; ++It)
			{
				UFunction* Func = *It;
				if (UEdGraphSchema_K2::FunctionCanBeUsedInDelegate(Func) && !UEdGraphSchema_K2::HasFunctionAnyOutputParameter(Func))
				{
					TSharedPtr<FString> ItemData = MakeShareable(new FString(Func->GetName()));
					FunctionsToCopySignatureFrom.Add(ItemData);
				}
			}

			// Sort the function list
			FunctionsToCopySignatureFrom.Sort([](const TSharedPtr<FString>& ElementA, const TSharedPtr<FString>& ElementB) -> bool
			{
				return *ElementA < *ElementB;
			});
		}
	}
}

void FBlueprintDelegateActionDetails::OnFunctionSelected(TSharedPtr<FString> FunctionName, ESelectInfo::Type SelectInfo)
{
	UK2Node_EditablePinBase* FunctionEntryNode = FunctionEntryNodePtr.Get();
	FMulticastDelegateProperty* Property = GetDelegateProperty();
	UClass* ScopeClass = Property ? Property->GetOwner<UClass>() : NULL;
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	if (FunctionEntryNode && FunctionName.IsValid() && ScopeClass)
	{
		const FName Name( *(*FunctionName) );
		if (UFunction* NewSignature = ScopeClass->FindFunctionByName(Name))
		{
			const FScopedTransaction Transaction(LOCTEXT("CopySignature", "Copy Signature"));

			while (FunctionEntryNode->UserDefinedPins.Num())
			{
				TSharedPtr<FUserPinInfo> Pin = FunctionEntryNode->UserDefinedPins[0];
				FunctionEntryNode->RemoveUserDefinedPin(Pin);
			}

			for (TFieldIterator<FProperty> PropIt(NewSignature); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
			{
				FProperty* FuncParam = *PropIt;
				FEdGraphPinType TypeOut;
				Schema->ConvertPropertyToPinType(FuncParam, TypeOut);
				UEdGraphPin* EdGraphPin = FunctionEntryNode->CreateUserDefinedPin(FuncParam->GetFName(), TypeOut, EGPD_Output);
				ensure(EdGraphPin);
			}

			OnParamsChanged(FunctionEntryNode);
		}
	}
}

void FBaseBlueprintGraphActionDetails::OnParamsChanged(UK2Node_EditablePinBase* TargetNode, bool bForceRefresh)
{
	UEdGraph* Graph = GetGraph();

	// TargetNode can be null, if we just removed the result node because there are no more out params
	if (TargetNode)
	{
		RegenerateInputsChildrenDelegate.ExecuteIfBound();
		RegenerateOutputsChildrenDelegate.ExecuteIfBound();

		// Reconstruct the entry/exit definition and recompile the blueprint to make sure the signature has changed before any fixups
		{
			const bool bCurDisableOrphanSaving = TargetNode->bDisableOrphanPinSaving;
			TargetNode->bDisableOrphanPinSaving = true;

			TargetNode->ReconstructNode();

			TargetNode->bDisableOrphanPinSaving = bCurDisableOrphanSaving;
		}

		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

		K2Schema->HandleParameterDefaultValueChanged(TargetNode);
	}
}

EVisibility FBlueprintDelegateActionDetails::OnGetSectionTextVisibility(TWeakPtr<SWidget> RowWidget) const
{
	bool ShowText = RowWidget.Pin()->IsHovered();

	// If the row is currently hovered, or a menu is being displayed for a button, keep the button expanded.
	if (ShowText)
	{
		return EVisibility::SelfHitTestInvisible;
	}
	else
	{
		return EVisibility::Collapsed;
	}
}

struct FPinRenamedHelper : public FBasePinChangeHelper
{
	TSet<UBlueprint*> ModifiedBlueprints;
	TSet<UK2Node*> NodesToRename;

	virtual void EditMacroInstance(UK2Node_MacroInstance* MacroInstance, UBlueprint* Blueprint) override
	{
		NodesToRename.Add(MacroInstance);
		if (Blueprint)
		{
			ModifiedBlueprints.Add(Blueprint);
		}
	}

	virtual void EditCallSite(UK2Node_CallFunction* CallSite, UBlueprint* Blueprint) override
	{
		NodesToRename.Add(CallSite);
		if (Blueprint)
		{
			ModifiedBlueprints.Add(Blueprint);
		}
	}
};

bool FBaseBlueprintGraphActionDetails::OnVerifyPinRename(UK2Node_EditablePinBase* InTargetNode, const FName InOldName, const FString& InNewName, FText& OutErrorMessage)
{
	// If the name is unchanged, allow the name
	if (InOldName.ToString() == InNewName)
	{
		return true;
	}

	if (InNewName.Len() >= NAME_SIZE)
	{
		OutErrorMessage = FText::Format( LOCTEXT("PinNameTooLong", "The name you entered is too long. Names must be less than {0} characters"), FText::AsNumber( NAME_SIZE ) );
		return false;
	}

	static const TArray<FString> ReservedParamNames =
	{
		TEXT("None"),
		TEXT("Self")
	};

	for(const FString& ReservedName : ReservedParamNames)
	{
		if (!FCString::Stricmp(*InNewName, *ReservedName))
		{			
			OutErrorMessage = FText::Format(LOCTEXT("PinNameIsReserved", "'{0}' is a reserved name"), FText::FromString(ReservedName));
			return false;
		}
	}

	if (InTargetNode)
	{
		UK2Node_EditablePinBase* EntryNode = FunctionEntryNodePtr.Get();
		UK2Node_EditablePinBase* ResultNode = FunctionResultNodePtr.Get();
		const FName NewFName = *InNewName;

		ERenamePinResult RenameResult = InTargetNode->RenameUserDefinedPin(InOldName, NewFName, true);

		if (RenameResult != ERenamePinResult_NameCollision)
		{
			UK2Node_EditablePinBase* OtherNode = (InTargetNode == EntryNode) ? ResultNode : EntryNode;

			// OtherNode can be null if the function, macro, etc. doesn't return a value.
			if (OtherNode)
			{
				RenameResult = OtherNode->RenameUserDefinedPin(InOldName, NewFName, true);
			}
		}

		if (RenameResult == ERenamePinResult_NameCollision)
		{
			OutErrorMessage = LOCTEXT("ConflictsWithProperty", "Conflicts with another local variable or function parameter!");
			return false;
		}
	}
	return true;
}

bool FBaseBlueprintGraphActionDetails::OnPinRenamed(UK2Node_EditablePinBase* TargetNode, const FName OldName, const FString& NewName)
{
	// Before changing the name, verify the name
	FText ErrorMessage;
	if (!OnVerifyPinRename(TargetNode, OldName, NewName, ErrorMessage))
	{
		return false;
	}

	UEdGraph* Graph = GetGraph();

	if (TargetNode)
	{
		FPinRenamedHelper PinRenamedHelper;

		const FScopedTransaction Transaction(LOCTEXT("RenameParam", "Rename Parameter"));

		TArray<UK2Node_EditablePinBase*> TerminalNodes = GatherAllResultNodes(FunctionResultNodePtr.Get());
		if (UK2Node_EditablePinBase* EntryNode = FunctionEntryNodePtr.Get())
		{
			TerminalNodes.Add(EntryNode);
		}

		bool bHasFunctionEntryNode = false;
		bool bHasFunctionResultNode = false;
		for (UK2Node_EditablePinBase* TerminalNode : TerminalNodes)
		{
			TerminalNode->Modify();
			PinRenamedHelper.NodesToRename.Add(TerminalNode);

			// Since function terminator node pins map to generated function properties, we need to
			// regenerate the referenced function so that dependent pins can be reconstructed properly.
			bHasFunctionEntryNode |= TerminalNode->IsA<UK2Node_FunctionEntry>();
			bHasFunctionResultNode |= TerminalNode->IsA<UK2Node_FunctionResult>();
		}

		UBlueprint* TargetBlueprint = GetBlueprintObj();
		PinRenamedHelper.ModifiedBlueprints.Add(TargetBlueprint);

		// GATHER 
		PinRenamedHelper.Broadcast(TargetBlueprint, TargetNode, Graph);

		const FName NewFName = *NewName;

		// TEST
		for (UK2Node* NodeToRename : PinRenamedHelper.NodesToRename)
		{
			if (ERenamePinResult::ERenamePinResult_NameCollision == NodeToRename->RenameUserDefinedPin(OldName, NewFName, /*bTest =*/ true))
			{
				return false;
			}
		}

		// UPDATE
		for (UK2Node* NodeToRename : PinRenamedHelper.NodesToRename)
		{
			// Note: This will internally call Modify() on any matching pin(s).
			NodeToRename->RenameUserDefinedPin(OldName, NewFName, /*bTest =*/ false);
		}

		// Update the corresponding UserDefinedPins entry for each terminal node.
		// Note: This array is not serialized, so a Modify() here isn't necessary.
		for (UK2Node_EditablePinBase* TerminalNode : TerminalNodes)
		{
			TSharedPtr<FUserPinInfo>* UDPinPtr = TerminalNode->UserDefinedPins.FindByPredicate([&](TSharedPtr<FUserPinInfo>& Pin)
			{
				return Pin.IsValid() && (Pin->PinName == OldName);
			});
			if (UDPinPtr)
			{
				(*UDPinPtr)->PinName = NewFName;
			}
		}

		// A function signature change requires recompilation to update the underlying property chain.
		// However, if we changed the function inputs at all, then we need to update any getter nodes that referenced the old name.
		if (bHasFunctionEntryNode)
		{
			if (!Graph)
			{
				// Function graph could be in the middle of being deleted
				return false;
			}

			TArray<UK2Node_VariableGet*> GetterNodes;
			Graph->GetNodesOfClass<UK2Node_VariableGet>(GetterNodes);

			for (UK2Node_VariableGet* GetterNode : GetterNodes)
			{
				check(GetterNode);
				if (GetterNode->ReferencesVariable(OldName, nullptr))
				{
					GetterNode->HandleVariableRenamed(TargetBlueprint, TargetBlueprint->GeneratedClass, Graph, OldName, NewFName);
				}
			}

			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(TargetBlueprint);
		}
		else if (bHasFunctionResultNode)
		{
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(TargetBlueprint);
		}

		// Trigger a change notification for Blueprints that were updated, in case there's anything we need to refresh.
		for (UBlueprint* ModifiedBlueprint : PinRenamedHelper.ModifiedBlueprints)
		{
			ModifiedBlueprint->BroadcastChanged();
		}
	}
	return true;
}

void FBlueprintGraphActionDetails::SetEntryAndResultNodes()
{
	// Clear the entry and exit nodes to the graph
	FunctionEntryNodePtr = nullptr;
	FunctionResultNodePtr = nullptr;

	if (UEdGraph* NewTargetGraph = GetGraph())
	{
		FBlueprintEditorUtils::GetEntryAndResultNodes(NewTargetGraph, FunctionEntryNodePtr, FunctionResultNodePtr);
	}
	else if (UK2Node_EditablePinBase* Node = GetEditableNode())
	{
		FunctionEntryNodePtr = Node;
	}
}

UEdGraph* FBaseBlueprintGraphActionDetails::GetGraph() const
{
	check(ObjectsBeingEdited.Num() > 0);

	if (ObjectsBeingEdited.Num() == 1)
	{
		UObject* const Object = ObjectsBeingEdited[0].Get();
		if (!Object)
		{
			return nullptr;
		}

		if (Object->IsA<UK2Node_Composite>())
		{
			return Cast<UK2Node_Composite>(Object)->BoundGraph;
		}
		else if (!Object->IsA<UK2Node_MacroInstance>() && (Object->IsA<UK2Node_Tunnel>() || Object->IsA<UK2Node_FunctionTerminator>()))
		{
			return Cast<UK2Node>(Object)->GetGraph();
		}
		else if (UK2Node_CallFunction* FunctionCall = Cast<UK2Node_CallFunction>(Object))
		{
			return FindObject<UEdGraph>(FunctionCall->GetBlueprint(), *(FunctionCall->FunctionReference.GetMemberName().ToString()));
		}
		else if (Object->IsA<UEdGraph>())
		{
			return Cast<UEdGraph>(Object);
		}
	}

	return nullptr;
}

UK2Node_EditablePinBase* FBlueprintGraphActionDetails::GetEditableNode() const
{
	check(ObjectsBeingEdited.Num() > 0);

	if (ObjectsBeingEdited.Num() == 1)
	{
		UObject* const Object = ObjectsBeingEdited[0].Get();
		if (!Object)
		{
			return nullptr;
		}

		if (Object->IsA<UK2Node_CustomEvent>())
		{
			return Cast<UK2Node_CustomEvent>(Object);
		}
	}

	return nullptr;
}

UFunction* FBlueprintGraphActionDetails::FindFunction() const
{
	if (UK2Node_CustomEvent* EventNode = Cast<UK2Node_CustomEvent>(FunctionEntryNodePtr.Get()))
	{
		return FFunctionFromNodeHelper::FunctionFromNode(EventNode);
	}
	else if (UEdGraph* Graph = GetGraph())
	{
		if (UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(Graph))
		{
			UClass* Class = Blueprint->SkeletonGeneratedClass;

			for (TFieldIterator<UFunction> FunctionIt(Class, EFieldIteratorFlags::IncludeSuper); FunctionIt; ++FunctionIt)
			{
				UFunction* Function = *FunctionIt;
				if (Function->GetName() == Graph->GetName())
				{
					return Function;
				}
			}
		}
	}
	return nullptr;
}

FKismetUserDeclaredFunctionMetadata* FBlueprintGraphActionDetails::GetMetadataBlock() const
{
	UK2Node_EditablePinBase* FunctionEntryNode = FunctionEntryNodePtr.Get();
	if (UK2Node_FunctionEntry* TypedEntryNode = Cast<UK2Node_FunctionEntry>(FunctionEntryNode))
	{
		return &(TypedEntryNode->MetaData);
	}
	else if (UK2Node_Tunnel* TunnelNode = ExactCast<UK2Node_Tunnel>(FunctionEntryNode))
	{
		// Must be exactly a tunnel, not a macro instance
		return &(TunnelNode->MetaData);
	}
	else if (UK2Node_CustomEvent* EventNode = Cast<UK2Node_CustomEvent>(FunctionEntryNode))
	{
		return &(EventNode->GetUserDefinedMetaData());
	}

	return nullptr;
}

FText FBlueprintGraphActionDetails::OnGetTooltipText() const
{
	if (FKismetUserDeclaredFunctionMetadata* Metadata = GetMetadataBlock())
	{
		return Metadata->ToolTip;
	}
	else
	{
		return LOCTEXT( "NoTooltip", "(None)" );
	}
}

void FBlueprintGraphActionDetails::OnTooltipTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit)
{
	if (FKismetUserDeclaredFunctionMetadata* Metadata = GetMetadataBlock())
	{
		Metadata->ToolTip = NewText;
		if (UFunction* Function = FindFunction())
		{
			Function->Modify();
			Function->SetMetaData(FBlueprintMetadata::MD_Tooltip, *NewText.ToString());
		}
	}
}

FText FBlueprintGraphActionDetails::OnGetCategoryText() const
{
	if (FKismetUserDeclaredFunctionMetadata* Metadata = GetMetadataBlock())
	{
		if( Metadata->Category.IsEmpty() || Metadata->Category.EqualTo(UEdGraphSchema_K2::VR_DefaultCategory) )
		{
			return LOCTEXT("DefaultCategory", "Default");
		}
		
		return Metadata->Category;
	}
	else
	{
		return LOCTEXT( "NoFunctionCategory", "(None)" );
	}
}

void FBlueprintGraphActionDetails::OnCategoryTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit)
{
	if (InTextCommit == ETextCommit::OnEnter || InTextCommit == ETextCommit::OnUserMovedFocus)
	{
		// Remove excess whitespace and prevent categories with just spaces
		FText CategoryName = FText::TrimPrecedingAndTrailing(NewText);

		FBlueprintEditorUtils::SetBlueprintFunctionOrMacroCategory(GetGraph(), CategoryName);
		MyBlueprint.Pin()->Refresh();
	}
}

void FBlueprintGraphActionDetails::OnCategorySelectionChanged( TSharedPtr<FText> ProposedSelection, ESelectInfo::Type /*SelectInfo*/ )
{
	if(ProposedSelection.IsValid())
	{
		if (FKismetUserDeclaredFunctionMetadata* Metadata = GetMetadataBlock())
		{
			FBlueprintEditorUtils::SetBlueprintFunctionOrMacroCategory(GetGraph(), *ProposedSelection.Get());
			MyBlueprint.Pin()->Refresh();

			CategoryListView.Pin()->ClearSelection();
			CategoryComboButton.Pin()->SetIsOpen(false);
			MyBlueprint.Pin()->ExpandCategory(*ProposedSelection.Get());
		}
	}
}

TSharedRef< ITableRow > FBlueprintGraphActionDetails::MakeCategoryViewWidget( TSharedPtr<FText> Item, const TSharedRef< STableViewBase >& OwnerTable )
{
	return SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
		[
			SNew(STextBlock) .Text(*Item.Get())
		];
}

FText FBlueprintGraphActionDetails::OnGetKeywordsText() const
{
	FText ResultKeywords;
	if (FKismetUserDeclaredFunctionMetadata* Metadata = GetMetadataBlock())
	{
		ResultKeywords = Metadata->Keywords;
	}
	return ResultKeywords;
}

void FBlueprintGraphActionDetails::OnKeywordsTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit)
{
	if (InTextCommit == ETextCommit::OnEnter || InTextCommit == ETextCommit::OnUserMovedFocus)
	{
		if (FKismetUserDeclaredFunctionMetadata* Metadata = GetMetadataBlock())
		{
			// Remove excess whitespace and prevent keywords with just spaces
			FText Keywords = FText::TrimPrecedingAndTrailing(NewText);

			if (!Keywords.EqualTo(Metadata->Keywords))
			{
				Metadata->Keywords = Keywords;

				if (UFunction* Function = FindFunction())
				{
					Function->Modify();
					Function->SetMetaData(FBlueprintMetadata::MD_FunctionKeywords, *Keywords.ToString());
				}
				OnParamsChanged(GetFunctionEntryNode().Get(), true);
				FBlueprintEditorUtils::MarkBlueprintAsModified(GetBlueprintObj());
			}
		}
	}
}

FText FBlueprintGraphActionDetails::OnGetCompactNodeTitleText() const
{
	FText ResultKeywords;
	if (FKismetUserDeclaredFunctionMetadata* Metadata = GetMetadataBlock())
	{
		ResultKeywords = Metadata->CompactNodeTitle;
	}
	return ResultKeywords;
}

void FBlueprintGraphActionDetails::OnCompactNodeTitleTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit)
{
	if (InTextCommit == ETextCommit::OnEnter || InTextCommit == ETextCommit::OnUserMovedFocus)
	{
		if (FKismetUserDeclaredFunctionMetadata* Metadata = GetMetadataBlock())
		{
			// Remove excess whitespace and prevent CompactNodeTitle with just spaces
			FText CompactNodeTitle = FText::TrimPrecedingAndTrailing(NewText);

			if (!CompactNodeTitle.EqualTo(Metadata->CompactNodeTitle))
			{
				Metadata->CompactNodeTitle = CompactNodeTitle;

				if (UFunction* Function = FindFunction())
				{
					Function->Modify();

					if (CompactNodeTitle.IsEmpty())
					{
						// Remove the metadata from the function, empty listings will still display the node as Compact
						Function->RemoveMetaData(FBlueprintMetadata::MD_FunctionKeywords);
					}
					else
					{
						Function->SetMetaData(FBlueprintMetadata::MD_CompactNodeTitle, *CompactNodeTitle.ToString());
					}
				}
				OnParamsChanged(GetFunctionEntryNode().Get(), true);
				FBlueprintEditorUtils::MarkBlueprintAsModified(GetBlueprintObj());
			}
		}
	}
}

FText FBlueprintGraphActionDetails::AccessSpecifierProperName( uint32 AccessSpecifierFlag ) const
{
	switch(AccessSpecifierFlag)
	{
	case FUNC_Public:
		return LOCTEXT( "Public", "Public" );
	case FUNC_Private:
		return LOCTEXT( "Private", "Private" );
	case FUNC_Protected:
		return LOCTEXT( "Protected", "Protected" );
	case 0:
		return LOCTEXT( "Unknown", "Unknown" ); // Default?
	}
	return LOCTEXT( "Error", "Error" );
}

FText FBlueprintGraphActionDetails::ReplicationSpecifierProperName( uint32 ReplicationSpecifierFlag ) const
{
	switch(ReplicationSpecifierFlag)
	{
	case FUNC_NetMulticast:
		return LOCTEXT( "MulticastDropDown", "Multicast" );
	case FUNC_NetServer:
		return LOCTEXT( "ServerDropDown", "Run on Server" );
	case FUNC_NetClient:
		return LOCTEXT( "ClientDropDown", "Run on owning Client" );
	case 0:
		return LOCTEXT( "NotReplicatedDropDown", "Not Replicated" );
	}
	return LOCTEXT( "Error", "Error" );
}

TSharedRef<ITableRow> FBlueprintGraphActionDetails::HandleGenerateRowAccessSpecifier( TSharedPtr<FAccessSpecifierLabel> SpecifierName, const TSharedRef<STableViewBase>& OwnerTable )
{
	return SNew(STableRow< TSharedPtr<FAccessSpecifierLabel> >, OwnerTable)
		.Content()
		[
			SNew( STextBlock ) 
				.Text( SpecifierName.IsValid() ? SpecifierName->LocalizedName : FText::GetEmpty() )
		];
}

FText FBlueprintGraphActionDetails::GetCurrentAccessSpecifierName() const
{
	uint32 AccessSpecifierFlag = 0;
	UK2Node_EditablePinBase* FunctionEntryNode = FunctionEntryNodePtr.Get();
	if(UK2Node_FunctionEntry* EntryNode = Cast<UK2Node_FunctionEntry>(FunctionEntryNode))
	{
		AccessSpecifierFlag = FUNC_AccessSpecifiers & EntryNode->GetFunctionFlags();
	}
	else if(UK2Node_CustomEvent* CustomEventNode = Cast<UK2Node_CustomEvent>(FunctionEntryNode))
	{
		AccessSpecifierFlag = FUNC_AccessSpecifiers & CustomEventNode->FunctionFlags;
	}
	return AccessSpecifierProperName( AccessSpecifierFlag );
}

bool FBlueprintGraphActionDetails::IsAccessSpecifierVisible() const
{
	bool bSupportedType = false;
	bool bIsEditable = false;
	UK2Node_EditablePinBase* FunctionEntryNode = FunctionEntryNodePtr.Get();
	if(FunctionEntryNode)
	{
		UBlueprint* Blueprint = FunctionEntryNode->GetBlueprint();
		const bool bIsInterface = FBlueprintEditorUtils::IsInterfaceBlueprint(Blueprint);

		bSupportedType = !bIsInterface && (FunctionEntryNode->IsA<UK2Node_FunctionEntry>() || FunctionEntryNode->IsA<UK2Node_Event>());
		bIsEditable = FunctionEntryNode->IsEditable();
	}
	return bSupportedType && bIsEditable;
}

void FBlueprintGraphActionDetails::OnAccessSpecifierSelected( TSharedPtr<FAccessSpecifierLabel> SpecifierName, ESelectInfo::Type SelectInfo )
{
	if(AccessSpecifierComboButton.IsValid())
	{
		AccessSpecifierComboButton->SetIsOpen(false);
	}

	UK2Node_EditablePinBase* FunctionEntryNode = FunctionEntryNodePtr.Get();
	if(FunctionEntryNode && SpecifierName.IsValid())
	{
		const FScopedTransaction Transaction( LOCTEXT( "ChangeAccessSpecifier", "Change Access Specifier" ) );

		FunctionEntryNode->Modify();
		UFunction* Function = FindFunction();
		if(Function)
		{
			Function->Modify();
		}

		const EFunctionFlags ClearAccessSpecifierMask = ~FUNC_AccessSpecifiers;
		if(UK2Node_FunctionEntry* EntryNode = Cast<UK2Node_FunctionEntry>(FunctionEntryNode))
		{
			int32 ExtraFlags = EntryNode->GetExtraFlags();
			ExtraFlags &= ClearAccessSpecifierMask;
			ExtraFlags |= SpecifierName->SpecifierFlag;
			EntryNode->SetExtraFlags(ExtraFlags);
		}
		else if(UK2Node_Event* EventNode = Cast<UK2Node_Event>(FunctionEntryNode))
		{
			EventNode->FunctionFlags &= ClearAccessSpecifierMask;
			EventNode->FunctionFlags |= SpecifierName->SpecifierFlag;
		}
		if(Function)
		{
			Function->FunctionFlags &= ClearAccessSpecifierMask;
			Function->FunctionFlags |= SpecifierName->SpecifierFlag;
		}

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprintObj());
	}
}

bool FBlueprintGraphActionDetails::GetInstanceColorVisibility() const
{
	// Hide the color editor if it's a top level function declaration.
	// Show it if we're editing a collapsed graph or macro
	UEdGraph* Graph = GetGraph();
	if (Graph)
	{
		const UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(Graph);
		if (Blueprint)
		{
			const bool bIsTopLevelFunctionGraph = Blueprint->FunctionGraphs.Contains(Graph);
			const bool bIsTopLevelMacroGraph = Blueprint->MacroGraphs.Contains(Graph);
			const bool bIsMacroGraph = Blueprint->BlueprintType == BPTYPE_MacroLibrary;
			return ((bIsMacroGraph || bIsTopLevelMacroGraph) || !bIsTopLevelFunctionGraph);
		}

	}
	
	return false;
}

FLinearColor FBlueprintGraphActionDetails::GetNodeTitleColor() const
{
	if (FKismetUserDeclaredFunctionMetadata* Metadata = GetMetadataBlock())
	{
		return Metadata->InstanceTitleColor;
	}
	else
	{
		return FLinearColor::White;
	}
}

FReply FBlueprintGraphActionDetails::ColorBlock_OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (FKismetUserDeclaredFunctionMetadata* Metadata = GetMetadataBlock())
		{
			TWeakPtr<FBlueprintGraphActionDetails> WeakSelf = SharedThis(this);

			FColorPickerArgs PickerArgs;
			PickerArgs.bIsModal = true;
			PickerArgs.ParentWidget = ColorBlock;
			PickerArgs.DisplayGamma = TAttribute<float>::Create( TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma) );
			PickerArgs.InitialColor = Metadata->InstanceTitleColor;
			PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateLambda([WeakSelf](FLinearColor NewValue)
			{
				if (TSharedPtr<FBlueprintGraphActionDetails> Self = WeakSelf.Pin())
				{
					if (FKismetUserDeclaredFunctionMetadata* Metadata = Self->GetMetadataBlock())
					{
						Metadata->InstanceTitleColor = NewValue;
					}
				}
			});

			OpenColorPicker(PickerArgs);
		}

		return FReply::Handled();
	}
	else
	{
		return FReply::Unhandled();
	}
}

bool FBlueprintGraphActionDetails::IsCustomEvent() const
{
	return (NULL != Cast<UK2Node_CustomEvent>(FunctionEntryNodePtr.Get()));
}

void FBlueprintGraphActionDetails::OnIsReliableReplicationFunctionModified(const ECheckBoxState NewCheckedState)
{
	UK2Node_EditablePinBase* FunctionEntryNode = FunctionEntryNodePtr.Get();
	UK2Node_CustomEvent* CustomEvent = Cast<UK2Node_CustomEvent>(FunctionEntryNode);
	if( CustomEvent )
	{
		if (NewCheckedState == ECheckBoxState::Checked)
		{
			if (UK2Node_FunctionEntry* TypedEntryNode = Cast<UK2Node_FunctionEntry>(FunctionEntryNode))
			{
				TypedEntryNode->AddExtraFlags(FUNC_NetReliable);
			}
			if (UK2Node_CustomEvent* CustomEventNode = Cast<UK2Node_CustomEvent>(FunctionEntryNode))
			{
				CustomEventNode->FunctionFlags |= FUNC_NetReliable;
			}
		}
		else
		{
			if (UK2Node_FunctionEntry* TypedEntryNode = Cast<UK2Node_FunctionEntry>(FunctionEntryNode))
			{
				TypedEntryNode->ClearExtraFlags(FUNC_NetReliable);
			}
			if (UK2Node_CustomEvent* CustomEventNode = Cast<UK2Node_CustomEvent>(FunctionEntryNode))
			{
				CustomEventNode->FunctionFlags &= ~FUNC_NetReliable;
			}
		}

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprintObj());
	}
}

ECheckBoxState FBlueprintGraphActionDetails::GetIsReliableReplicatedFunction() const
{
	const UK2Node_EditablePinBase* FunctionEntryNode = FunctionEntryNodePtr.Get();
	const UK2Node_CustomEvent* CustomEvent = Cast<const UK2Node_CustomEvent>(FunctionEntryNode);
	if(!CustomEvent)
	{
		return ECheckBoxState::Undetermined;
	}

	uint32 const NetReliableMask = (FUNC_Net | FUNC_NetReliable);
	if ((CustomEvent->GetNetFlags() & NetReliableMask) == NetReliableMask)
	{
		return ECheckBoxState::Checked;
	}
	
	return ECheckBoxState::Unchecked;
}

bool FBlueprintGraphActionDetails::IsPureFunctionVisible() const
{
	bool bSupportedType = false;
	bool bIsEditable = false;
	UK2Node_EditablePinBase* FunctionEntryNode = FunctionEntryNodePtr.Get();
	if(FunctionEntryNode)
	{
		UBlueprint* Blueprint = FunctionEntryNode->GetBlueprint();
		const bool bIsInterface = FBlueprintEditorUtils::IsInterfaceBlueprint(Blueprint);

		bSupportedType = !bIsInterface && FunctionEntryNode->IsA<UK2Node_FunctionEntry>();
		bIsEditable = FunctionEntryNode->IsEditable();
	}
	return bSupportedType && bIsEditable;
}

void FBlueprintGraphActionDetails::OnIsPureFunctionModified( const ECheckBoxState NewCheckedState )
{
	UK2Node_EditablePinBase* FunctionEntryNode = FunctionEntryNodePtr.Get();
	UFunction* Function = FindFunction();
	UK2Node_FunctionEntry* EntryNode = Cast<UK2Node_FunctionEntry>(FunctionEntryNode);
	if (EntryNode && Function)
	{
		const FScopedTransaction Transaction( LOCTEXT( "ChangePure", "Change Pure" ) );
		EntryNode->Modify();
		Function->Modify();

		//set flags on function entry node also
		Function->FunctionFlags ^= FUNC_BlueprintPure;
		EntryNode->SetExtraFlags(EntryNode->GetExtraFlags() ^ FUNC_BlueprintPure);
		OnParamsChanged(FunctionEntryNode);
	}
}

ECheckBoxState FBlueprintGraphActionDetails::GetIsPureFunction() const
{
	UK2Node_EditablePinBase* FunctionEntryNode = FunctionEntryNodePtr.Get();
	UK2Node_FunctionEntry* EntryNode = Cast<UK2Node_FunctionEntry>(FunctionEntryNode);
	if(!EntryNode)
	{
		return ECheckBoxState::Undetermined;
	}
	return (EntryNode->GetFunctionFlags() & FUNC_BlueprintPure) ? ECheckBoxState::Checked :  ECheckBoxState::Unchecked;
}

bool FBlueprintGraphActionDetails::IsConstFunctionVisible() const
{
	bool bVisible = false;
	UFunction* Function = FindFunction();
	UK2Node_FunctionEntry* EntryNode = Cast<UK2Node_FunctionEntry>(FunctionEntryNodePtr.Get());
	if(Function && EntryNode)
	{
		const bool bIsStatic = EntryNode->GetFunctionFlags() & FUNC_Static;
		const bool bIsEditable = EntryNode->IsEditable();
		bVisible = bIsEditable && !bIsStatic;
	}
	return bVisible;
}

void FBlueprintGraphActionDetails::OnIsConstFunctionModified( const ECheckBoxState NewCheckedState )
{
	UK2Node_EditablePinBase* FunctionEntryNode = FunctionEntryNodePtr.Get();
	UFunction* Function = FindFunction();
	UK2Node_FunctionEntry* EntryNode = Cast<UK2Node_FunctionEntry>(FunctionEntryNode);
	if(EntryNode && Function)
	{
		const FScopedTransaction Transaction( LOCTEXT( "ChangeConst", "Change Const" ) );
		EntryNode->Modify();
		Function->Modify();

		//set flags on function entry node also
		Function->FunctionFlags ^= FUNC_Const;
		EntryNode->SetExtraFlags(EntryNode->GetExtraFlags() ^ FUNC_Const);
		OnParamsChanged(FunctionEntryNode);
	}
}

ECheckBoxState FBlueprintGraphActionDetails::GetIsConstFunction() const
{
	UK2Node_EditablePinBase* FunctionEntryNode = FunctionEntryNodePtr.Get();
	UK2Node_FunctionEntry* EntryNode = Cast<UK2Node_FunctionEntry>(FunctionEntryNode);
	if(!EntryNode)
	{
		return ECheckBoxState::Undetermined;
	}
	return (EntryNode->GetFunctionFlags() & FUNC_Const) ? ECheckBoxState::Checked :  ECheckBoxState::Unchecked;
}

bool FBlueprintGraphActionDetails::IsExecFunctionVisible() const
{
	bool bSupportedType = false;
	bool bIsEditable = false;
	UK2Node_EditablePinBase* FunctionEntryNode = FunctionEntryNodePtr.Get();
	if (FunctionEntryNode)
	{
		bSupportedType = FunctionEntryNode->IsA<UK2Node_FunctionEntry>();
		bIsEditable = FunctionEntryNode->IsEditable();
	}
	return bSupportedType && bIsEditable;
}

void FBlueprintGraphActionDetails::OnIsExecFunctionModified(const ECheckBoxState NewCheckedState)
{
	UK2Node_EditablePinBase* FunctionEntryNode = FunctionEntryNodePtr.Get();
	UFunction* Function = FindFunction();
	UK2Node_FunctionEntry* EntryNode = Cast<UK2Node_FunctionEntry>(FunctionEntryNode);
	if (EntryNode && Function)
	{
		const FScopedTransaction Transaction(LOCTEXT("ChangeExec", "Change Exec"));
		EntryNode->Modify();
		Function->Modify();

		//set flags on function entry node also
		Function->FunctionFlags ^= FUNC_Exec;
		EntryNode->SetExtraFlags(EntryNode->GetExtraFlags() ^ FUNC_Exec);
		OnParamsChanged(FunctionEntryNode);
	}
}

ECheckBoxState FBlueprintGraphActionDetails::GetIsExecFunction() const
{
	UK2Node_EditablePinBase* FunctionEntryNode = FunctionEntryNodePtr.Get();
	UK2Node_FunctionEntry* EntryNode = Cast<UK2Node_FunctionEntry>(FunctionEntryNode);
	if (!EntryNode)
	{
		return ECheckBoxState::Undetermined;
	}
	return (EntryNode->GetFunctionFlags() & FUNC_Exec) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

bool FBlueprintGraphActionDetails::IsThreadSafeFunctionVisible() const
{
	bool bSupportedType = false;
	bool bIsEditable = false;
	
	UK2Node_EditablePinBase* FunctionEntryNode = FunctionEntryNodePtr.Get();
	if(FunctionEntryNode)
	{
		bSupportedType = FunctionEntryNode->IsA<UK2Node_FunctionEntry>();
		bIsEditable = FunctionEntryNode->IsEditable();
	}

	return bIsEditable && bSupportedType;
}

void FBlueprintGraphActionDetails::OnIsThreadSafeFunctionModified(const ECheckBoxState NewCheckedState)
{
	if( FunctionEntryNodePtr.IsValid() )
	{
		const bool bThreadSafe = NewCheckedState == ECheckBoxState::Checked;
		const FText TransactionType = bThreadSafe ? LOCTEXT( "DisableThreadSafe", "Disable Thread Safe" ) : LOCTEXT( "EnableThreadSafe", "Enable Thread Safe" );

		if( UK2Node_FunctionEntry* EntryPoint = Cast<UK2Node_FunctionEntry>(FunctionEntryNodePtr.Get()) )
		{
			const FScopedTransaction Transaction( TransactionType );
			EntryPoint->Modify();
			EntryPoint->MetaData.bThreadSafe = bThreadSafe;
			FBlueprintEditorUtils::MarkBlueprintAsModified( EntryPoint->GetBlueprint() );
		}
	}
}

ECheckBoxState FBlueprintGraphActionDetails::GetIsThreadSafeFunction() const
{
	UK2Node_EditablePinBase* FunctionEntryNode = FunctionEntryNodePtr.Get();
	UK2Node_FunctionEntry* EntryNode = Cast<UK2Node_FunctionEntry>(FunctionEntryNode);
	if (!EntryNode)
	{
		return ECheckBoxState::Undetermined;
	}
	return EntryNode->MetaData.bThreadSafe ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

bool FBlueprintGraphActionDetails::IsUnsafeDuringActorConstructionVisible() const
{
	bool bSupportedType = false;
	bool bIsEditable = false;

	UK2Node_EditablePinBase* FunctionEntryNode = FunctionEntryNodePtr.Get();
	if (FunctionEntryNode)
	{
		bSupportedType = FunctionEntryNode->IsA<UK2Node_FunctionEntry>();
		bIsEditable = FunctionEntryNode->IsEditable();
	}

	return bIsEditable && bSupportedType;
}

void FBlueprintGraphActionDetails::OnIsUnsafeDuringActorConstructionModified(const ECheckBoxState NewCheckedState)
{
	if (FunctionEntryNodePtr.IsValid())
	{
		const bool bIsUnsafeDuringActorConstruction = NewCheckedState == ECheckBoxState::Checked;
		const FText TransactionType = bIsUnsafeDuringActorConstruction ? 
			LOCTEXT("DisableIsUnsafeDuringActorConstruction", "Disable Unsafe During Actor Construction") : 
			LOCTEXT("EnableIsUnsafeDuringActorConstruction", "Enable Unsafe During Actor Construction");

		if (UK2Node_FunctionEntry* EntryPoint = Cast<UK2Node_FunctionEntry>(FunctionEntryNodePtr.Get()))
		{
			const FScopedTransaction Transaction(TransactionType);
			EntryPoint->Modify();
			EntryPoint->MetaData.bIsUnsafeDuringActorConstruction = bIsUnsafeDuringActorConstruction;
			FBlueprintEditorUtils::MarkBlueprintAsModified(EntryPoint->GetBlueprint());
		}
	}
}

ECheckBoxState FBlueprintGraphActionDetails::GetIsUnsafeDuringActorConstruction() const
{
	UK2Node_EditablePinBase* FunctionEntryNode = FunctionEntryNodePtr.Get();
	UK2Node_FunctionEntry* EntryNode = Cast<UK2Node_FunctionEntry>(FunctionEntryNode);
	if (!EntryNode)
	{
		return ECheckBoxState::Undetermined;
	}
	return EntryNode->MetaData.bIsUnsafeDuringActorConstruction ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

FReply FBaseBlueprintGraphActionDetails::OnAddNewInputClicked()
{
	UK2Node_EditablePinBase* FunctionEntryNode = FunctionEntryNodePtr.Get();

	if( FunctionEntryNode )
	{
		FScopedTransaction Transaction( LOCTEXT( "AddInParam", "Add In Parameter" ) );
		FunctionEntryNode->Modify();

		FEdGraphPinType PinType = MyBlueprint.Pin()->GetLastFunctionPinTypeUsed();

		// Make sure that if this is an exec node we are allowed one.
		if ((PinType.PinCategory == UEdGraphSchema_K2::PC_Exec) && (!FunctionEntryNode->CanModifyExecutionWires()))
		{
			MyBlueprint.Pin()->ResetLastPinType();
			PinType = MyBlueprint.Pin()->GetLastFunctionPinTypeUsed();
		}
		const FName NewPinName = TEXT("NewParam");
		if (FunctionEntryNode->CreateUserDefinedPin(NewPinName, PinType, EGPD_Output))
		{
			OnParamsChanged(FunctionEntryNode, true);
		}
		else
		{
			Transaction.Cancel();
		}
	}

	return FReply::Handled();
}

EVisibility FBlueprintGraphActionDetails::GetAddNewInputOutputVisibility() const
{
	UK2Node_EditablePinBase* FunctionEntryNode = FunctionEntryNodePtr.Get();
	if (FunctionEntryNodePtr.IsValid())
	{
		if(UEdGraph* Graph = FunctionEntryNode->GetGraph())
		{
			// Math expression graphs are read only, do not allow adding or removing of pins
			if(Cast<UK2Node_MathExpression>(Graph->GetOuter()))
			{
				return EVisibility::Collapsed;
			}
		}
	}
	return EVisibility::Visible;
}

bool FBlueprintGraphActionDetails::IsAddNewInputOutputEnabled() const
{
	if (DetailsLayoutPtr)
	{
		if (TSharedPtr<const IDetailsView> DetailsView = DetailsLayoutPtr->GetDetailsViewSharedPtr())
		{
			return DetailsView->IsPropertyEditingEnabled();
		}
	}

	return false;
}

EVisibility FBlueprintGraphActionDetails::OnGetSectionTextVisibility(TWeakPtr<SWidget> RowWidget) const
{
	bool ShowText = RowWidget.Pin()->IsHovered();

	// If the row is currently hovered, or a menu is being displayed for a button, keep the button expanded.
	if (ShowText)
	{
		return EVisibility::SelfHitTestInvisible;
	}
	else
	{
		return EVisibility::Collapsed;
	}
}

FReply FBlueprintGraphActionDetails::OnAddNewOutputClicked()
{
	FScopedTransaction Transaction( LOCTEXT( "AddOutParam", "Add Out Parameter" ) );
	
	GetBlueprintObj()->Modify();
	GetGraph()->Modify();
	UK2Node_EditablePinBase* EntryPin = FunctionEntryNodePtr.Get();	
	EntryPin->Modify();
	for (int32 iPin = 0; iPin < EntryPin->Pins.Num() ; iPin++)
	{
		EntryPin->Pins[iPin]->Modify();
	}
	
	UK2Node_EditablePinBase* PreviousResultNode = FunctionResultNodePtr.Get();

	AttemptToCreateResultNode();

	UK2Node_EditablePinBase* FunctionResultNode = FunctionResultNodePtr.Get();
	if( FunctionResultNode )
	{
		FEdGraphPinType PinType = MyBlueprint.Pin()->GetLastFunctionPinTypeUsed();
		PinType.bIsReference = false;
		// Make sure that if this is an exec node we are allowed one.
		if ((PinType.PinCategory == UEdGraphSchema_K2::PC_Exec) && (!FunctionResultNode->CanModifyExecutionWires()))
		{
			MyBlueprint.Pin()->ResetLastPinType();
			PinType = MyBlueprint.Pin()->GetLastFunctionPinTypeUsed();
		}

		const FName NewPinName = FunctionResultNode->CreateUniquePinName(TEXT("NewParam"));
		TArray<UK2Node_EditablePinBase*> TargetNodes = GatherAllResultNodes(FunctionResultNode);
		bool bAllChanged = TargetNodes.Num() > 0;
		for (UK2Node_EditablePinBase* Node : TargetNodes)
		{
			Node->Modify();
			UEdGraphPin* NewPin = Node->CreateUserDefinedPin(NewPinName, PinType, EGPD_Input, false);
			bAllChanged &= (nullptr != NewPin);

			if (bAllChanged)
			{
				OnParamsChanged(Node, true);
			}
			else
			{
				break;
			}
		}
		if (!bAllChanged)
		{
			Transaction.Cancel();
		}

		if (!PreviousResultNode)
		{
			DetailsLayoutPtr->ForceRefreshDetails();
		}
	}
	else
	{
		Transaction.Cancel();
	}

	return FReply::Handled();
}


FBlueprintGlobalOptionsManagedListDetails::FBlueprintGlobalOptionsManagedListDetails(TWeakPtr<class FBlueprintGlobalOptionsDetails> InGlobalOptionsDetailsPtr)
	: GlobalOptionsDetailsPtr(InGlobalOptionsDetailsPtr)
{
}

UBlueprint* FBlueprintGlobalOptionsManagedListDetails::GetBlueprintObjectChecked() const
{
	UBlueprint* BlueprintObject = nullptr;

	TSharedPtr<FBlueprintGlobalOptionsDetails> PinnedGlobalOptionsDetailsPtr = GlobalOptionsDetailsPtr.Pin();
	if (PinnedGlobalOptionsDetailsPtr.IsValid())
	{
		BlueprintObject = PinnedGlobalOptionsDetailsPtr->GetBlueprintObj();
	}

	check(BlueprintObject);
	return BlueprintObject;
}

TSharedPtr<FBlueprintEditor> FBlueprintGlobalOptionsManagedListDetails::GetPinnedBlueprintEditorPtr() const
{
	TSharedPtr<FBlueprintGlobalOptionsDetails> PinnedGlobalOptionsDetailsPtr = GlobalOptionsDetailsPtr.Pin();
	if (PinnedGlobalOptionsDetailsPtr.IsValid())
	{
		return PinnedGlobalOptionsDetailsPtr->GetBlueprintEditorPtr().Pin();
	}

	return TSharedPtr<FBlueprintEditor>();
}

void FBlueprintGlobalOptionsManagedListDetails::OnRefreshInDetailsView()
{
	TSharedPtr<FBlueprintEditor> BlueprintEditorPtr = GetPinnedBlueprintEditorPtr();
	if (BlueprintEditorPtr.IsValid())
	{
		TSharedPtr<SKismetInspector> Inspector = BlueprintEditorPtr->GetInspector();
		if (Inspector.IsValid())
		{
			// Show details for the Blueprint instance we're editing
			Inspector->ShowDetailsForSingleObject(GetBlueprintObjectChecked());
		}
	}
}


FBlueprintImportsLayout::FBlueprintImportsLayout(TWeakPtr<class FBlueprintGlobalOptionsDetails> InGlobalOptionsDetails, bool bInShowDefaultImports)
	: FBlueprintGlobalOptionsManagedListDetails(InGlobalOptionsDetails)
	, bShouldShowDefaultImports(bInShowDefaultImports)
{
	DisplayOptions.TitleText = bShouldShowDefaultImports ?
		LOCTEXT("BlueprintDefaultNamespaceTitle", "Default Namespaces") :
		LOCTEXT("BlueprintImportedNamespaceTitle", "Imported Namespaces");
	DisplayOptions.NoItemsLabelText = LOCTEXT("NoBlueprintImports", "No Imports");
}

TSharedPtr<SWidget> FBlueprintImportsLayout::MakeAddItemWidget()
{
	if (bShouldShowDefaultImports)
	{
		return nullptr;
	}

	return SNew(SBlueprintNamespaceEntry)
		.AllowTextEntry(false)
		.OnNamespaceSelected(this, &FBlueprintImportsLayout::OnNamespaceSelected)
		.OnGetNamespacesToExclude(this, &FBlueprintImportsLayout::OnGetNamespacesToExclude)
		.ExcludedNamespaceTooltipText(LOCTEXT("CannotSelectNamespaceForImport", "This namespace is already imported."))
		.ButtonContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BlueprintAddImportButton", "Add"))
		];
}

void FBlueprintImportsLayout::GetManagedListItems(TArray<FManagedListItem>& OutListItems) const
{
	auto AddNamespaceItemsToOutputList = [&OutListItems](const TSet<FString>& NamespaceItems, bool bIsRemovable)
	{
		for (const FString& NamespaceItem : NamespaceItems)
		{
			FManagedListItem ItemDesc;
			ItemDesc.ItemName = NamespaceItem;
			ItemDesc.DisplayName = FText::FromString(NamespaceItem);
			ItemDesc.bIsRemovable = bIsRemovable;

			OutListItems.Add(MoveTemp(ItemDesc));
		}
	};

	TSet<FString> NamespaceItems;
	const UBlueprint* Blueprint = GetBlueprintObjectChecked();

	// Default imports (non-removable). These include anything from the shared global set, as well as any namespaces assigned to the Blueprint hierarchy.
	FBlueprintNamespaceUtilities::GetSharedGlobalImports(NamespaceItems);
	FBlueprintNamespaceUtilities::GetDefaultImportsForObject(Blueprint, NamespaceItems);

	if(!bShouldShowDefaultImports)
	{
		// Blueprint imports (removable). A Blueprint may explicitly import a namespace that's also in the default set, but we exclude those here so they can't be removed.
		NamespaceItems = Blueprint->ImportedNamespaces.Difference(NamespaceItems);
	}

	AddNamespaceItemsToOutputList(NamespaceItems, !bShouldShowDefaultImports);
}

void FBlueprintImportsLayout::OnRemoveItem(const FManagedListItem& Item)
{
	TSharedPtr<FBlueprintEditor> BlueprintEditorPtr = GetPinnedBlueprintEditorPtr();
	if (BlueprintEditorPtr.IsValid())
	{
		BlueprintEditorPtr->RemoveNamespace(Item.ItemName);
	}
	
	RegenerateChildContent();

	OnRefreshInDetailsView();
}

void FBlueprintImportsLayout::OnNamespaceSelected(const FString& InNamespace)
{
	TSharedPtr<FBlueprintEditor> BlueprintEditorPtr = GetPinnedBlueprintEditorPtr();
	if (BlueprintEditorPtr.IsValid())
	{
		FBlueprintEditor::FImportNamespaceExParameters Params;
		Params.bIsAutoImport = false;
		Params.NamespacesToImport.Add(InNamespace);
		Params.OnPostImportCallback = FSimpleDelegate::CreateLambda([this]()
		{
			RegenerateChildContent();
		});

		// Add to edited Blueprint(s) and import into the current editor context.
		BlueprintEditorPtr->ImportNamespaceEx(Params);
	}

	OnRefreshInDetailsView();
}

void FBlueprintImportsLayout::OnGetNamespacesToExclude(TSet<FString>& OutNamespacesToExclude) const
{
	const UBlueprint* Blueprint = GetBlueprintObjectChecked();

	FBlueprintNamespaceUtilities::GetSharedGlobalImports(OutNamespacesToExclude);
	FBlueprintNamespaceUtilities::GetDefaultImportsForObject(Blueprint, OutNamespacesToExclude);

	OutNamespacesToExclude.Append(Blueprint->ImportedNamespaces);
}


FBlueprintInterfaceLayout::FBlueprintInterfaceLayout(TWeakPtr<class FBlueprintGlobalOptionsDetails> InGlobalOptionsDetails, TSharedPtr<IPropertyHandle> InInterfacesProperty)
	: FBlueprintGlobalOptionsManagedListDetails(InGlobalOptionsDetails)
	, InterfacesProperty(InInterfacesProperty)
{
	DisplayOptions.TitleText = InterfacesProperty ?
		LOCTEXT("BlueprintImplementedInterfaceTitle", "Implemented Interfaces") :
		LOCTEXT("BlueprintInheritedInterfaceTitle", "Inherited Interfaces");
	DisplayOptions.NoItemsLabelText = LOCTEXT("NoBlueprintInterface", "No Interfaces");
	DisplayOptions.BrowseButtonToolTipText = LOCTEXT("BlueprintInterfaceBrowseTooltip", "Opens this interface");
}

TSharedPtr<IPropertyHandle> FBlueprintInterfaceLayout::GetPropertyHandle() const
{
	return InterfacesProperty;
}

TSharedPtr<SWidget> FBlueprintInterfaceLayout::MakeAddItemWidget()
{
	if (InterfacesProperty)
	{
		return SAssignNew(AddInterfaceComboButton, SComboButton)
		.ButtonContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BlueprintAddInterfaceButton", "Add"))
		]
		.OnGetMenuContent(this, &FBlueprintInterfaceLayout::OnGetAddInterfaceMenuContent);
	}

	return nullptr;
}

void FBlueprintInterfaceLayout::GetManagedListItems(TArray<FManagedListItem>& OutListItems) const
{
	const UBlueprint* Blueprint = GetBlueprintObjectChecked();

	if (InterfacesProperty)
	{
		checkf(
			&Blueprint->ImplementedInterfaces == (TArray<FBPInterfaceDescription>*)InterfacesProperty->GetValueBaseAddress((uint8*)Blueprint),
			TEXT("Different Property provided than ImplementedInterfaces")
		);
		
		// Generate a list of interfaces already implemented
		for (int32 InterfaceIndex = 0; InterfaceIndex < Blueprint->ImplementedInterfaces.Num(); ++InterfaceIndex)
		{
			if (const TSubclassOf<UInterface> Interface = Blueprint->ImplementedInterfaces[InterfaceIndex].Interface)
			{
				FManagedListItem ItemDesc;
				ItemDesc.ItemName = Interface->GetPathName();
				ItemDesc.DisplayName = Interface->GetDisplayNameText();
				ItemDesc.bIsRemovable = true;
				const TSharedPtr<IPropertyHandle> Property = GetPropertyHandle()->GetChildHandle(InterfaceIndex);
				if (Property && Property->IsValidHandle())
				{
					ItemDesc.PropertyHandles.Add(Property);
				}

				// Allow browsing to Blueprint interface class assets.
				if (UBlueprintGeneratedClass* Class = Cast<UBlueprintGeneratedClass>(*Interface))
				{
					ItemDesc.AssetPtr = Class->ClassGeneratedBy;
				}

				OutListItems.Add(MoveTemp(ItemDesc));
			}
		}
	}
	else
	{
		// Generate a list of interfaces implemented by classes this blueprint inherited from
		UClass* BlueprintParent = Blueprint->ParentClass;
		while (BlueprintParent)
		{
			for (TArray<FImplementedInterface>::TIterator It(BlueprintParent->Interfaces); It; ++It)
			{
				FImplementedInterface& CurrentInterface = *It;
				if (CurrentInterface.Class)
				{
					FManagedListItem ItemDesc;
					ItemDesc.ItemName = CurrentInterface.Class->GetPathName();
					ItemDesc.DisplayName = CurrentInterface.Class->GetDisplayNameText();
					ItemDesc.bIsRemovable = false;

					OutListItems.Add(MoveTemp(ItemDesc));
				}
			}

			BlueprintParent = BlueprintParent->GetSuperClass();
		}
	}
}

void FBlueprintInterfaceLayout::OnRemoveItem(const FManagedListItem& Item)
{
	const EAppReturnType::Type DialogReturn = FMessageDialog::Open(EAppMsgType::YesNoCancel, NSLOCTEXT("UnrealEd", "TransferInterfaceFunctionsToBlueprint", "Would you like to transfer the interface functions to be part of your blueprint?"));

	if (DialogReturn == EAppReturnType::Cancel)
	{
		// We canceled!
		return;
	}
	const FTopLevelAssetPath InterfacePathName(Item.ItemName);

	UBlueprint* Blueprint = GetBlueprintObjectChecked();
	TSharedPtr<FBlueprintEditor> BlueprintEditorPtr = GetPinnedBlueprintEditorPtr();
	if (BlueprintEditorPtr.IsValid())
	{
		// Close all graphs that are about to be removed
		TArray<UEdGraph*> Graphs;
		FBlueprintEditorUtils::GetInterfaceGraphs(Blueprint, InterfacePathName, Graphs);
		for (TArray<UEdGraph*>::TIterator GraphIt(Graphs); GraphIt; ++GraphIt)
		{
			BlueprintEditorPtr->CloseDocumentTab(*GraphIt);
		}
	}

	// Do the work of actually removing the interface
	FBlueprintEditorUtils::RemoveInterface(Blueprint, InterfacePathName, DialogReturn == EAppReturnType::Yes);

	RegenerateChildContent();

	OnRefreshInDetailsView();
}

void FBlueprintInterfaceLayout::OnClassPicked(UClass* PickedClass)
{
	if (AddInterfaceComboButton.IsValid())
	{
		AddInterfaceComboButton->SetIsOpen(false);
	}

	if (PickedClass)
	{
		UBlueprint* Blueprint = GetBlueprintObjectChecked();

		FBlueprintEditorUtils::ImplementNewInterface(Blueprint, PickedClass->GetClassPathName());

		RegenerateChildContent();
	}

	OnRefreshInDetailsView();
}

TSharedRef<SWidget> FBlueprintInterfaceLayout::OnGetAddInterfaceMenuContent()
{
	UBlueprint* Blueprint = GetBlueprintObjectChecked();

	TArray<UBlueprint*> Blueprints;
	Blueprints.Add(Blueprint);
	TSharedRef<SWidget> ClassPicker = FBlueprintEditorUtils::ConstructBlueprintInterfaceClassPicker(Blueprints, FOnClassPicked::CreateSP(this, &FBlueprintInterfaceLayout::OnClassPicked));
	// Achieving fixed width by nesting items within a fixed width box.
	return SNew(SBox)
		.WidthOverride(350.0f)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.MaxHeight(400.0f)
			.AutoHeight()
			[
				ClassPicker
			]
		];
}

// Double the size of the default details view width for string values, which is otherwise too narrow since the customization adds a combo button.
float FBlueprintGlobalOptionsDetails::NamespacePropertyValueCustomization_MinDesiredWidth = 250.0f;

UBlueprint* FBlueprintGlobalOptionsDetails::GetBlueprintObj() const
{
	if(UBlueprint* BP = BlueprintObjOverride.Get())
	{
		return BP;
	}
	
	if(BlueprintEditorPtr.IsValid())
	{
		return BlueprintEditorPtr.Pin()->GetBlueprintObj();
	}

	return NULL;
}

FText FBlueprintGlobalOptionsDetails::GetParentClassName() const
{
	const UBlueprint* Blueprint = GetBlueprintObj();
	const UClass* ParentClass = Blueprint ? *Blueprint->ParentClass : NULL;
	return ParentClass ? ParentClass->GetDisplayNameText() : FText::FromName(NAME_None);
}

bool FBlueprintGlobalOptionsDetails::CanReparent() const
{
	return BlueprintEditorPtr.IsValid() && BlueprintEditorPtr.Pin()->ReparentBlueprint_IsVisible();
}

TSharedRef<SWidget> FBlueprintGlobalOptionsDetails::GetParentClassMenuContent()
{
	TArray<UBlueprint*> Blueprints;
	Blueprints.Add(GetBlueprintObj());
	TSharedRef<SWidget> ClassPicker = FBlueprintEditorUtils::ConstructBlueprintParentClassPicker(Blueprints, FOnClassPicked::CreateSP(this, &FBlueprintGlobalOptionsDetails::OnClassPicked));

	// Achieving fixed width by nesting items within a fixed width box.
	return SNew(SBox)
		.WidthOverride(350.0f)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.MaxHeight(400.0f)
			.AutoHeight()
			[
				ClassPicker
			]
		];
}

void FBlueprintGlobalOptionsDetails::OnClassPicked(UClass* PickedClass)
{
	ParentClassComboButton->SetIsOpen(false);
	if(BlueprintEditorPtr.IsValid())
	{
		BlueprintEditorPtr.Pin()->ReparentBlueprint_NewParentChosen(PickedClass);
	}

	check(BlueprintEditorPtr.IsValid());
	TSharedPtr<SKismetInspector> Inspector = BlueprintEditorPtr.Pin()->GetInspector();
	// Show details for the Blueprint instance we're editing
	Inspector->ShowDetailsForSingleObject(GetBlueprintObj());
}

bool FBlueprintGlobalOptionsDetails::CanDeprecateBlueprint() const
{
	if (UBlueprint* Blueprint = GetBlueprintObj())
	{
		// If the parent is deprecated, we cannot modify deprecation on this Blueprint
		if (Blueprint->ParentClass && Blueprint->ParentClass->HasAnyClassFlags(CLASS_Deprecated))
		{
			return false;
		}

		return true;
	}

	return false;
}

void FBlueprintGlobalOptionsDetails::OnDeprecateBlueprint(ECheckBoxState InCheckState)
{
	GetBlueprintObj()->bDeprecate = InCheckState == ECheckBoxState::Checked? true : false;
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprintObj());
}

ECheckBoxState FBlueprintGlobalOptionsDetails::IsDeprecatedBlueprint() const
{
	if (UBlueprint* Blueprint = GetBlueprintObj())
	{
		return Blueprint->bDeprecate ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Unchecked;
}

FText FBlueprintGlobalOptionsDetails::GetDeprecatedTooltip() const
{
	if(CanDeprecateBlueprint())
	{
		return LOCTEXT("DeprecateBlueprintTooltip", "Deprecate the Blueprint and all child Blueprints to make it no longer placeable in the World nor child classes created from it.");
	}
	
	return LOCTEXT("DisabledDeprecateBlueprintTooltip", "This Blueprint is deprecated because of a parent, it is not possible to remove deprecation from it!");
}

void FBlueprintGlobalOptionsDetails::OnNamespaceValueCommitted(const FString& InNamespace)
{
	if (UBlueprint* Blueprint = GetBlueprintObj())
	{
		// Skip if the value has not been changed.
		const FString OldNamespace = Blueprint->BlueprintNamespace;
		if (OldNamespace == InNamespace)
		{
			return;
		}

		// Update the current namespace value. This will handle pre/post-edit change notifications, etc.
		check(NamespacePropertyHandle.IsValid());
		NamespacePropertyHandle->SetValue(InNamespace);

		HandleNamespaceValueChange(OldNamespace, InNamespace);
	}
}

bool FBlueprintGlobalOptionsDetails::ShouldShowNamespaceResetToDefault() const
{
	check(NamespacePropertyHandle.IsValid());
	return NamespacePropertyHandle->CanResetToDefault();
}

void FBlueprintGlobalOptionsDetails::OnNamespaceResetToDefaultValue()
{
	check(NamespacePropertyHandle.IsValid());

	// Get the current value.
	FString OriginalValue;
	NamespacePropertyHandle->GetValue(OriginalValue);

	// Standard reset-to-default path.
	NamespacePropertyHandle->ResetToDefault();

	// Get the value after having been reset.
	FString DefaultNamespaceValue;
	NamespacePropertyHandle->GetValue(DefaultNamespaceValue);

	// Update the entry widget to reflect the new value.
	if (NamespaceValueWidget.IsValid())
	{
		NamespaceValueWidget->SetCurrentNamespace(DefaultNamespaceValue);
	}

	HandleNamespaceValueChange(OriginalValue, DefaultNamespaceValue);
}

void FBlueprintGlobalOptionsDetails::HandleNamespaceValueChange(const FString& InOldValue, const FString& InNewValue)
{
	// Refresh the namespace registry.
	FBlueprintNamespaceRegistry& BlueprintNamespaceRegistry = FBlueprintNamespaceRegistry::Get();
	if (!InOldValue.IsEmpty() && BlueprintNamespaceRegistry.IsRegisteredPath(InOldValue))
	{
		// @todo_namespaces - This may not scale for larger projects.
		// Using a slow task for now, but consider optimizing this path.
		FScopedSlowTask SlowTask(0.0f, LOCTEXT("RebuildingNamespaceRegistry", "Updating the namespace registry..."));

		// The old path is a non-global registered namespace. Revisit all assets and ensure that the registry is up-to-date.
		// If the old namespace is no longer in use by another Blueprint asset, this effectively removes it from the registry.
		BlueprintNamespaceRegistry.Rebuild();
	}

	if (!InNewValue.IsEmpty() && !BlueprintNamespaceRegistry.IsRegisteredPath(InNewValue))
	{
		// Add the new namespace into the registry (it has not been explicitly added yet).
		BlueprintNamespaceRegistry.RegisterNamespace(InNewValue);
	}
	
	// Refresh the Blueprint editor context.
	TSharedPtr<FBlueprintEditor> BlueprintEditor = GetBlueprintEditorPtr().Pin();
	if (BlueprintEditor.IsValid())
	{
		if (const UBlueprint* Blueprint = BlueprintEditor->GetBlueprintObj())
		{
			bool bRefreshDetailsView = false;
			if (Blueprint->ImportedNamespaces.Contains(InOldValue))
			{
				// Remove the import from the current editor context if it is no longer inclusive of any path.
				if (!BlueprintNamespaceRegistry.IsInclusivePath(InOldValue))
				{
					BlueprintEditor->RemoveNamespace(InOldValue);
				}

				// We need to refresh the details view if we unassigned and/or removed an imported namespace path.
				// If unassigned but still imported, it will return to a non-default namespace in the Imports table.
				bRefreshDetailsView = true;
			}
			
			if (Blueprint->ImportedNamespaces.Contains(InNewValue))
			{
				// We need to refresh the details view if we assigned an imported namespace.
				// In that case, it will switch to a default namespace in the Imports table.
				bRefreshDetailsView = true;
			}
			else
			{
				FBlueprintEditor::FImportNamespaceExParameters Params;
				Params.bIsAutoImport = false;
				Params.NamespacesToImport.Add(InNewValue);
				Params.OnPostImportCallback = FSimpleDelegate::CreateLambda([&bRefreshDetailsView]()
				{
					bRefreshDetailsView = true;
				});

				// Import the new namespace into the current editor context.
				BlueprintEditor->ImportNamespaceEx(Params);
			}

			// Refresh the details view if necessary.
			if (bRefreshDetailsView)
			{
				BlueprintEditor->RefreshInspector();
			}
		}
	}
}

void FBlueprintGlobalOptionsDetails::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	const UBlueprint* Blueprint = GetBlueprintObj();
	if(Blueprint != NULL)
	{
		// Hide any properties that aren't included in the "Option" category
		for (TFieldIterator<FProperty> PropertyIt(Blueprint->GetClass(), EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
		{
			FProperty* Property = *PropertyIt;
			const FString& Category = Property->GetMetaData(TEXT("Category"));

			if (Category != TEXT("BlueprintOptions") && Category != TEXT("ClassOptions"))
			{
				DetailLayout.HideProperty(DetailLayout.GetProperty(Property->GetFName()));
			}
		}

		// Display the parent class and set up the menu for reparenting
		IDetailCategoryBuilder& Category = DetailLayout.EditCategory("ClassOptions", LOCTEXT("ClassOptions", "Class Options"));

		// ParentClass is a hidden property so we have to add it to the property map manually to use it
		const TSharedPtr<IPropertyHandle> ParentClassProperty = DetailLayout.AddObjectPropertyData({const_cast<UBlueprint*>(Blueprint)}, TEXT("ParentClass"));

		Category.AddCustomRow( LOCTEXT("ClassOptions", "Class Options") )
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BlueprintDetails_ParentClass", "Parent Class"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SAssignNew(ParentClassComboButton, SComboButton)
				.IsEnabled(this, &FBlueprintGlobalOptionsDetails::CanReparent)
				.OnGetMenuContent(this, &FBlueprintGlobalOptionsDetails::GetParentClassMenuContent)
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text(this, &FBlueprintGlobalOptionsDetails::GetParentClassName)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			]
		]
		.PropertyHandleList({ParentClassProperty});

		const bool bIsInterfaceBP = FBlueprintEditorUtils::IsInterfaceBlueprint(Blueprint);
		const bool bIsMacroLibrary = Blueprint->BlueprintType == BPTYPE_MacroLibrary;
		const bool bIsLevelScriptBP = FBlueprintEditorUtils::IsLevelScriptBlueprint(Blueprint);
		const bool bIsFunctionLibrary = Blueprint->BlueprintType == BPTYPE_FunctionLibrary;
		
		// Interfaces/imports currently rely on the full Blueprint editor context to function properly (e.g. add/remove operations).
		TSharedPtr<FBlueprintEditor> PinnedBlueprintEditorPtr = BlueprintEditorPtr.Pin();
		if (PinnedBlueprintEditorPtr.IsValid() && PinnedBlueprintEditorPtr->GetCurrentMode() != FBlueprintEditorApplicationModes::BlueprintDefaultsMode)
		{
			const bool bSupportsInterfaces = !bIsInterfaceBP && !bIsMacroLibrary && !bIsFunctionLibrary;
			const bool bSupportsNamespaces = GetDefault<UBlueprintEditorSettings>()->bEnableNamespaceImportingFeatures;

			if (bSupportsNamespaces)
			{
				// Imported namespace details
				IDetailCategoryBuilder& ImportsCategory = DetailLayout.EditCategory("Imports", LOCTEXT("BlueprintImportDetailsCategory", "Imports"));

				TSharedRef<FBlueprintImportsLayout> DefaultImportsLayout = MakeShareable(new FBlueprintImportsLayout(SharedThis(this), /*bShowDefaultImports = */true));
				ImportsCategory.AddCustomBuilder(DefaultImportsLayout);

				TSharedRef<FBlueprintImportsLayout> LocalImportsLayout = MakeShareable(new FBlueprintImportsLayout(SharedThis(this), /*bShowDefaultImports = */false));
				ImportsCategory.AddCustomBuilder(LocalImportsLayout);
			}

			if (bSupportsInterfaces)
			{
				// Interface details customization
				IDetailCategoryBuilder& InterfacesCategory = DetailLayout.EditCategory("Interfaces", LOCTEXT("BlueprintInterfacesDetailsCategory", "Interfaces"));

				// ImplementedInterfaces is a hidden property so we have to add it to the property map manually to use it
				const TSharedPtr<IPropertyHandle> InterfacesProperty = DetailLayout.AddObjectPropertyData({ const_cast<UBlueprint*>(Blueprint) }, TEXT("ImplementedInterfaces"));

				TSharedRef<FBlueprintInterfaceLayout> InheritedInterfacesLayout = MakeShareable(new FBlueprintInterfaceLayout(
					SharedThis(this)
				));
				InterfacesCategory.AddCustomBuilder(InheritedInterfacesLayout);

				TSharedRef<FBlueprintInterfaceLayout> LocalInterfacesLayout = MakeShareable(new FBlueprintInterfaceLayout(
					SharedThis(this),
					InterfacesProperty.ToSharedRef()
				));
				InterfacesCategory.AddCustomBuilder(LocalInterfacesLayout);
			}
		}

		// Hide the bDeprecate, we override the functionality.
		static FName DeprecatePropName(TEXT("bDeprecate"));
		DetailLayout.HideProperty(DetailLayout.GetProperty(DeprecatePropName));

		// Hide the experimental CompileMode setting (if not enabled)
		const UBlueprintEditorSettings* EditorSettings = GetDefault<UBlueprintEditorSettings>();
		if (EditorSettings && !EditorSettings->bAllowExplicitImpureNodeDisabling)
		{
			static FName CompileModePropertyName(TEXT("CompileMode"));
			DetailLayout.HideProperty(DetailLayout.GetProperty(CompileModePropertyName));
		}

		// Hide 'run on drag' for LevelBP
		if (bIsLevelScriptBP)
		{
			static FName RunOnDragPropName(TEXT("bRunConstructionScriptOnDrag"));
			DetailLayout.HideProperty(DetailLayout.GetProperty(RunOnDragPropName));
		}
		else
		{
			// Only display the ability to deprecate a Blueprint on non-level Blueprints.
			Category.AddCustomRow( LOCTEXT("DeprecateLabel", "Deprecate"), true )
				.NameContent()
				[
					SNew(STextBlock)
					.Text( LOCTEXT("DeprecateLabel", "Deprecate") )
					.ToolTipText( this, &FBlueprintGlobalOptionsDetails::GetDeprecatedTooltip )
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
				.ValueContent()
				[
					SNew(SCheckBox)
					.IsEnabled( this, &FBlueprintGlobalOptionsDetails::CanDeprecateBlueprint )
					.IsChecked( this, &FBlueprintGlobalOptionsDetails::IsDeprecatedBlueprint )
					.OnCheckStateChanged( this, &FBlueprintGlobalOptionsDetails::OnDeprecateBlueprint )
					.ToolTipText( this, &FBlueprintGlobalOptionsDetails::GetDeprecatedTooltip )
				]
				.PropertyHandleList({DetailLayout.GetProperty(DeprecatePropName)});
		}

		static FName BlueprintNamespacePropertyName = GET_MEMBER_NAME_CHECKED(UBlueprint, BlueprintNamespace);
		NamespacePropertyHandle = DetailLayout.GetProperty(BlueprintNamespacePropertyName);
		DetailLayout.EditDefaultProperty(NamespacePropertyHandle)->CustomWidget()
			.NameContent()
			[
				NamespacePropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.MinDesiredWidth(NamespacePropertyValueCustomization_MinDesiredWidth)
			[
				SAssignNew(NamespaceValueWidget, SBlueprintNamespaceEntry)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.AllowTextEntry(true)
				.CurrentNamespace(Blueprint->BlueprintNamespace)
				.OnNamespaceSelected(this, &FBlueprintGlobalOptionsDetails::OnNamespaceValueCommitted)
			]
			.OverrideResetToDefault(FResetToDefaultOverride::Create(
				TAttribute<bool>(this, &FBlueprintGlobalOptionsDetails::ShouldShowNamespaceResetToDefault),
				FSimpleDelegate::CreateSP(this, &FBlueprintGlobalOptionsDetails::OnNamespaceResetToDefaultValue))
			);
	}
}

void FBlueprintComponentDetails::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	check( BlueprintEditorPtr.IsValid() );
	TSharedPtr<SSubobjectEditor> Editor = BlueprintEditorPtr.Pin()->GetSubobjectEditor();
	check( Editor.IsValid() );
	const UBlueprint* BlueprintObj = GetBlueprintObj();
	check(BlueprintObj != nullptr);

	TArray<FSubobjectEditorTreeNodePtrType> Nodes = Editor->GetSelectedNodes();

	if (!Nodes.Num())
	{
		CachedNodePtr = nullptr;
	}
	else if (Nodes.Num() == 1)
	{
		CachedNodePtr = Nodes[0];
	}

	if( CachedNodePtr.IsValid() )
	{
		IDetailCategoryBuilder& VariableCategory = DetailLayout.EditCategory("Variable", LOCTEXT("VariableDetailsCategory", "Variable"), ECategoryPriority::Variable);

		VariableNameEditableTextBox = SNew(SEditableTextBox)
			.Text(this, &FBlueprintComponentDetails::OnGetVariableText)
			.OnTextChanged(this, &FBlueprintComponentDetails::OnVariableTextChanged)
			.OnTextCommitted(this, &FBlueprintComponentDetails::OnVariableTextCommitted)
			.IsReadOnly(!CachedNodePtr->CanRename())
			.Font(IDetailLayoutBuilder::GetDetailFont());

		VariableCategory.AddCustomRow(LOCTEXT("BlueprintComponentDetails_VariableNameLabel", "Variable Name"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BlueprintComponentDetails_VariableNameLabel", "Variable Name"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			VariableNameEditableTextBox.ToSharedRef()
		];

		VariableCategory.AddCustomRow(LOCTEXT("BlueprintComponentDetails_VariableTooltipLabel", "Tooltip"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BlueprintComponentDetails_VariableTooltipLabel", "Tooltip"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SEditableTextBox)
			.Text(this, &FBlueprintComponentDetails::OnGetTooltipText)
			.OnTextCommitted(this, &FBlueprintComponentDetails::OnTooltipTextCommitted, CachedNodePtr->GetVariableName())
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];

		PopulateVariableCategories();
		const FText CategoryTooltip = LOCTEXT("EditCategoryName_Tooltip", "The category of the variable; editing this will place the variable into another category or create a new one.");

		VariableCategory.AddCustomRow( LOCTEXT("BlueprintComponentDetails_VariableCategoryLabel", "Category") )
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BlueprintComponentDetails_VariableCategoryLabel", "Category"))
			.ToolTipText(CategoryTooltip)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SAssignNew(VariableCategoryComboButton, SComboButton)
			.ContentPadding(FMargin(0,0,5,0))
			.IsEnabled(this, &FBlueprintComponentDetails::OnVariableCategoryChangeEnabled)
			.ButtonContent()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				.Padding(FMargin(0, 0, 5, 0))
				[
					SNew(SEditableTextBox)
					.Text(this, &FBlueprintComponentDetails::OnGetVariableCategoryText)
					.OnTextCommitted(this, &FBlueprintComponentDetails::OnVariableCategoryTextCommitted, CachedNodePtr->GetVariableName())
					.OnVerifyTextChanged_Lambda([&](const FText& InNewText, FText& OutErrorMessage) -> bool
					{
						UBlueprint* MyBlueprint = GetBlueprintObj();
						if (!MyBlueprint)
						{
							// Probably during shutdown, just ignore the change
							return false;
						}
						if (InNewText.IsEmpty())
						{
							OutErrorMessage = LOCTEXT("CategoryEmpty", "Cannot add a category with an empty string.");
							return false;
						}
						if (InNewText.EqualTo(FText::FromString(MyBlueprint->GetName())))
						{
							OutErrorMessage = LOCTEXT("CategoryEqualsBlueprintName", "Cannot add a category with the same name as the blueprint.");
							return false;
						}
						return true;
					})
					.ToolTipText(CategoryTooltip)
					.SelectAllTextWhenFocused(true)
					.RevertTextOnEscape(true)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			]
			.MenuContent()
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.MaxHeight(400.0f)
				[
					SAssignNew(VariableCategoryListView, SListView<TSharedPtr<FText>>)
					.ListItemsSource(&VariableCategorySource)
					.OnGenerateRow(this, &FBlueprintComponentDetails::MakeVariableCategoryViewWidget)
					.OnSelectionChanged(this, &FBlueprintComponentDetails::OnVariableCategorySelectionChanged)
				]
			]
		];

		// Keep an easy way to disable UI to specify overriden component class until there is confidence that it is robust
		if (GetAllowNativeComponentClassOverrides())
		{
			const UActorComponent* ComponentTemplate = (CachedNodePtr->IsNativeComponent() ? CachedNodePtr->GetComponentTemplate() : nullptr);
			if (ComponentTemplate)
			{
				UClass* BaseClass = ComponentTemplate->GetClass();

				if (const FBPComponentClassOverride* Override = BlueprintObj->ComponentClassOverrides.FindByKey(ComponentTemplate->GetFName()))
				{
					AActor* Owner = ComponentTemplate->GetOwner();
					AActor* OwnerArchetype = CastChecked<AActor>(Owner->GetArchetype());
					if (UActorComponent* ArchetypeComponent = Cast<UActorComponent>((UObject*)FindObjectWithOuter(OwnerArchetype, UActorComponent::StaticClass(), ComponentTemplate->GetFName())))
					{
						BaseClass = ArchetypeComponent->GetClass();
					}
				}

				const FText ComponentClassTooltip = LOCTEXT("BlueprintComponentDetails_ComponentClassOverrideTooltip", "The class to use when creating this component for this class. This can only be done for components defined in native at this time.");

				VariableCategory.AddCustomRow( LOCTEXT("BlueprintComponentDetails_ComponentClassOverride", "Component Class") )
				.NameContent()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("BlueprintComponentDetails_ComponentClassOverrideLabel", "Component Class"))
					.ToolTipText(ComponentClassTooltip)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
				.ValueContent()
				[
					SNew(SClassPropertyEntryBox)
					.MetaClass(BaseClass)
					.AllowNone(false)
					.SelectedClass(this, &FBlueprintComponentDetails::GetSelectedEntryClass)
					.OnSetClass(this, &FBlueprintComponentDetails::HandleNewEntryClassSelected)];
			}
		}

		IDetailCategoryBuilder& SocketsCategory = DetailLayout.EditCategory("Sockets", LOCTEXT("BlueprintComponentDetailsCategory", "Sockets"), ECategoryPriority::Important);

		SocketsCategory.AddCustomRow(LOCTEXT("BlueprintComponentDetails_Sockets", "Sockets"))
		.RowTag("ParentSocket")
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BlueprintComponentDetails_ParentSocket", "Parent Socket"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SEditableTextBox)
				.Text(this, &FBlueprintComponentDetails::GetSocketName)
				.IsReadOnly(true)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(2.0f, 1.0f)
			[
				PropertyCustomizationHelpers::MakeBrowseButton(
					FSimpleDelegate::CreateSP(this, &FBlueprintComponentDetails::OnBrowseSocket), 
					LOCTEXT( "SocketBrowseButtonToolTipText", "Select a different Parent Socket - cannot change socket on inherited components"), 
					TAttribute<bool>(this, &FBlueprintComponentDetails::CanChangeSocket)
				)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(2.0f, 1.0f)
			[
				PropertyCustomizationHelpers::MakeClearButton(
					FSimpleDelegate::CreateSP(this, &FBlueprintComponentDetails::OnClearSocket), 
					LOCTEXT("SocketClearButtonToolTipText", "Clear the Parent Socket - cannot change socket on inherited components"), 
					TAttribute<bool>(this, &FBlueprintComponentDetails::CanChangeSocket)
				)
			]
		];
	}

	// Handle event generation
	if ( FBlueprintEditorUtils::DoesSupportEventGraphs(BlueprintObj) && Nodes.Num() == 1 )
	{
		// Use the component template to support native components as well
		if (const UActorComponent* ComponentTemplate = CachedNodePtr->GetComponentTemplate())
		{
			AddEventsCategory(DetailLayout, CachedNodePtr->GetVariableName(), ComponentTemplate->GetClass());
		}
	}

	TSharedPtr<IPropertyHandle> PrimaryTickProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UActorComponent, PrimaryComponentTick));

	if (PrimaryTickProperty->IsValidHandle())
	{
		IDetailCategoryBuilder& TickCategory = DetailLayout.EditCategory("ComponentTick");

		TickCategory.AddProperty(PrimaryTickProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTickFunction, bStartWithTickEnabled)));
		TickCategory.AddProperty(PrimaryTickProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTickFunction, TickInterval)));
		TickCategory.AddProperty(PrimaryTickProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTickFunction, bTickEvenWhenPaused)), EPropertyLocation::Advanced);
		TickCategory.AddProperty(PrimaryTickProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTickFunction, bAllowTickOnDedicatedServer)), EPropertyLocation::Advanced);
		TickCategory.AddProperty(PrimaryTickProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTickFunction, TickGroup)), EPropertyLocation::Advanced);
	}

	PrimaryTickProperty->MarkHiddenByCustomization(); 
}

FText FBlueprintComponentDetails::OnGetVariableText() const
{
	check(CachedNodePtr.IsValid());

	return FText::FromName(CachedNodePtr->GetVariableName());
}

void FBlueprintComponentDetails::OnVariableTextChanged(const FText& InNewText)
{
	check(CachedNodePtr.IsValid());

	bIsVariableNameInvalid = true;

	const FString& NewTextStr = InNewText.ToString();

	if (USubobjectDataSubsystem* System = USubobjectDataSubsystem::Get())
	{
		FText ErrorMsg;
		if (!System->IsValidRename(CachedNodePtr->GetDataHandle(), InNewText, ErrorMsg))
		{
			VariableNameEditableTextBox->SetError(ErrorMsg);
			return;
		}
	}

	TSharedPtr<INameValidatorInterface> VariableNameValidator = MakeShareable(new FKismetNameValidator(GetBlueprintObj(), CachedNodePtr->GetVariableName()));

	EValidatorResult ValidatorResult = VariableNameValidator->IsValid(NewTextStr);
	if(ValidatorResult == EValidatorResult::AlreadyInUse)
	{
		VariableNameEditableTextBox->SetError(FText::Format(LOCTEXT("ComponentVariableRenameFailed_InUse", "{0} is in use by another variable or function!"), InNewText));
	}
	else if(ValidatorResult == EValidatorResult::EmptyName)
	{
		VariableNameEditableTextBox->SetError(LOCTEXT("RenameFailed_LeftBlank", "Names cannot be left blank!"));
	}
	else if(ValidatorResult == EValidatorResult::TooLong)
	{
		VariableNameEditableTextBox->SetError(FText::Format( LOCTEXT("RenameFailed_NameTooLong", "Names must have fewer than {0} characters!"), FText::AsNumber( FKismetNameValidator::GetMaximumNameLength())));
	}
	else
	{
		bIsVariableNameInvalid = false;
		VariableNameEditableTextBox->SetError(FText::GetEmpty());
	}
}

void FBlueprintComponentDetails::OnVariableTextCommitted(const FText& InNewName, ETextCommit::Type InTextCommit)
{
	if (!bIsVariableNameInvalid)
	{
		check(CachedNodePtr.IsValid());

		const FScopedTransaction Transaction(LOCTEXT("RenameComponentVariable", "Rename Component Variable"));
		USubobjectDataSubsystem::RenameSubobjectMemberVariable(GetBlueprintObj(), CachedNodePtr->GetDataHandle(), FName(*InNewName.ToString()));
	}

	bIsVariableNameInvalid = false;
	VariableNameEditableTextBox->SetError(FText::GetEmpty());
}

FText FBlueprintComponentDetails::OnGetTooltipText() const
{
	check(CachedNodePtr.IsValid());

	FName VarName = CachedNodePtr->GetVariableName();
	if (VarName != NAME_None)
	{
		FString Result;
		FBlueprintEditorUtils::GetBlueprintVariableMetaData(GetBlueprintObj(), VarName, NULL, TEXT("tooltip"), Result);
		return FText::FromString(Result);
	}

	return FText();
}

void FBlueprintComponentDetails::OnTooltipTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit, FName VarName)
{
	FBlueprintEditorUtils::SetBlueprintVariableMetaData(GetBlueprintObj(), VarName, NULL, TEXT("tooltip"), NewText.ToString() );
}

bool FBlueprintComponentDetails::OnVariableCategoryChangeEnabled() const
{
	check(CachedNodePtr.IsValid());
	const FSubobjectData* Data = CachedNodePtr->GetDataSource();
	return !Data->IsInheritedComponent();
}

FText FBlueprintComponentDetails::OnGetVariableCategoryText() const
{
	check(CachedNodePtr.IsValid());

	FName VarName = CachedNodePtr->GetVariableName();
	if (VarName != NAME_None)
	{
		FText Category = FBlueprintEditorUtils::GetBlueprintVariableCategory(GetBlueprintObj(), VarName, NULL);

		// Older blueprints will have their name as the default category
		if( Category.EqualTo(FText::FromString(GetBlueprintObj()->GetName())) )
		{
			return UEdGraphSchema_K2::VR_DefaultCategory;
		}
		else
		{
			return Category;
		}
	}

	return FText();
}

void FBlueprintComponentDetails::OnVariableCategoryTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit, FName VarName)
{
	check(CachedNodePtr.IsValid());

	if (InTextCommit == ETextCommit::OnEnter || InTextCommit == ETextCommit::OnUserMovedFocus)
	{
		FBlueprintEditorUtils::SetBlueprintVariableCategory(GetBlueprintObj(), CachedNodePtr->GetVariableName(), NULL, NewText);
		PopulateVariableCategories();
	}
}

void FBlueprintComponentDetails::OnVariableCategorySelectionChanged( TSharedPtr<FText> ProposedSelection, ESelectInfo::Type /*SelectInfo*/ )
{
	check(CachedNodePtr.IsValid());

	FName VarName = CachedNodePtr->GetVariableName();
	if (ProposedSelection.IsValid() && VarName != NAME_None)
	{
		FText NewCategory = *ProposedSelection.Get();
		FBlueprintEditorUtils::SetBlueprintVariableCategory(GetBlueprintObj(), VarName, NULL, NewCategory);

		check(VariableCategoryListView.IsValid());
		check(VariableCategoryComboButton.IsValid());

		VariableCategoryListView->ClearSelection();
		VariableCategoryComboButton->SetIsOpen(false);
	}
}

TSharedRef< ITableRow > FBlueprintComponentDetails::MakeVariableCategoryViewWidget( TSharedPtr<FText> Item, const TSharedRef< STableViewBase >& OwnerTable )
{
	return SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
	[
		SNew(STextBlock)
			.Text(*Item.Get())
	];
}

void FBlueprintComponentDetails::PopulateVariableCategories()
{
	auto IsNewCategorySource = [this](const FText& NewCategory)
	{
		return !VariableCategorySource.ContainsByPredicate([&NewCategory](const TSharedPtr<FText>& ExistingCategory)
		{
			return ExistingCategory->ToString().Equals(NewCategory.ToString(), ESearchCase::CaseSensitive);
		});
	};

	UBlueprint* BlueprintObj = GetBlueprintObj();

	check(BlueprintObj);
	check(BlueprintObj->SkeletonGeneratedClass);

	TSet<FName> VisibleVariables;
	for (TFieldIterator<FProperty> PropertyIt(BlueprintObj->SkeletonGeneratedClass, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
	{
		FProperty* Property = *PropertyIt;

		if ((!Property->HasAnyPropertyFlags(CPF_Parm) && Property->HasAllPropertyFlags(CPF_BlueprintVisible)))
		{
			VisibleVariables.Add(Property->GetFName());
		}
	}

	FBlueprintEditorUtils::GetSCSVariableNameList(BlueprintObj, VisibleVariables);

	VariableCategorySource.Reset();
	VariableCategorySource.Add(MakeShared<FText>(UEdGraphSchema_K2::VR_DefaultCategory));
	for (const FName& VariableName : VisibleVariables)
	{
		FText Category = FBlueprintEditorUtils::GetBlueprintVariableCategory(BlueprintObj, VariableName, nullptr);
		if (!Category.IsEmpty() && !Category.EqualTo(FText::FromString(BlueprintObj->GetName())))
		{
			if (IsNewCategorySource(Category))
			{
				VariableCategorySource.Add(MakeShared<FText>(Category));
			}
		}
	}

	// Sort categories, but keep the default category listed first
	VariableCategorySource.Sort([](const TSharedPtr <FText> &LHS, const TSharedPtr <FText> &RHS)
	{
		if (LHS.IsValid() && RHS.IsValid())
		{
			return (LHS->EqualTo(UEdGraphSchema_K2::VR_DefaultCategory) || LHS->CompareToCaseIgnored(*RHS) <= 0);
		}
		return false;
	});
}

const UClass* FBlueprintComponentDetails::GetSelectedEntryClass() const
{
	check(CachedNodePtr.IsValid());

	if (const UActorComponent* ComponentTemplate = CachedNodePtr->GetComponentTemplate())
	{
		UBlueprint* BlueprintObj = GetBlueprintObj();
		check(BlueprintObj);

		do
		{
			if (const FBPComponentClassOverride* Override = BlueprintObj->ComponentClassOverrides.FindByKey(ComponentTemplate->GetFName()))
			{
				return Override->ComponentClass;
			}

			BlueprintObj = UBlueprint::GetBlueprintFromClass(Cast<UBlueprintGeneratedClass>(BlueprintObj->ParentClass));

		} while (BlueprintObj);

		return ComponentTemplate->GetClass();
	}

	return nullptr;

}

void FBlueprintComponentDetails::HandleNewEntryClassSelected(const UClass* NewEntryClass) const
{
	const UClass* PreviousClass = GetSelectedEntryClass();

	if (PreviousClass != NewEntryClass)
	{
		check(CachedNodePtr.IsValid());

		if (const UActorComponent* ComponentTemplate = CachedNodePtr->GetComponentTemplate())
		{
			if (USubobjectDataSubsystem* System = USubobjectDataSubsystem::Get())
			{
				System->ChangeSubobjectClass(CachedNodePtr->GetDataHandle(), NewEntryClass);
			}
		}
	}
}


FText FBlueprintComponentDetails::GetSocketName() const
{
	check(CachedNodePtr.IsValid());
	if (FSubobjectData* Data = CachedNodePtr->GetDataSource())
	{
		return Data->GetSocketName();
	}
	return FText::GetEmpty();
}

bool FBlueprintComponentDetails::CanChangeSocket() const
{
	check(CachedNodePtr.IsValid());

	if (FSubobjectData* Data = CachedNodePtr->GetDataSource())
	{
		return !Data->IsInheritedComponent();
	}
	return true;
}

void FBlueprintComponentDetails::OnBrowseSocket()
{
	check(CachedNodePtr.IsValid());
	FSubobjectData* Data = CachedNodePtr->GetDataSource();
	if (Data && Data->HasValidSocket())
	{
		TSharedPtr<SSubobjectEditor> Editor = BlueprintEditorPtr.Pin()->GetSubobjectEditor();
		check(Editor.IsValid());

		FSubobjectEditorTreeNodePtrType ParentFNode = CachedNodePtr->GetParent();
		FSubobjectData* ParentData = ParentFNode.IsValid() ? ParentFNode->GetDataSource() : nullptr;
		if (ParentData)
		{
			// #TODO_BH Remove const cast
			if (USceneComponent* ParentSceneComponent = const_cast<USceneComponent*>(ParentData->GetObjectForBlueprint<USceneComponent>(Editor->GetBlueprint())))
			{
				if (ParentSceneComponent->HasAnySockets())
				{
					// Pop up a combo box to pick socket from mesh
					FSlateApplication::Get().PushMenu(
						BlueprintEditorPtr.Pin()->GetToolkitHost()->GetParentWidget(),
						FWidgetPath(),
						SNew(SSocketChooserPopup)
						.SceneComponent( ParentSceneComponent )
						.OnSocketChosen( this, &FBlueprintComponentDetails::OnSocketSelection ),
						FSlateApplication::Get().GetCursorPos(),
						FPopupTransitionEffect( FPopupTransitionEffect::TypeInPopup )
						);
				}
			}
		}
	}
}

void FBlueprintComponentDetails::OnClearSocket()
{
	check(CachedNodePtr.IsValid());
	FSubobjectData* Data = CachedNodePtr->GetDataSource();
	
	if (Data && Data->HasValidSocket())
	{
		Data->SetupAttachment(NAME_None);
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprintObj());
	}
}

void FBlueprintComponentDetails::OnSocketSelection(FName SocketName)
{
	check(CachedNodePtr.IsValid());

	FSubobjectData* Data = CachedNodePtr->GetDataSource();
	if (Data && Data->HasValidSocket())
	{
		// Record selection if there is an actual asset attached
		Data->SetSocketName(SocketName);
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprintObj());
	}
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FBlueprintGraphNodeDetails::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{
	TArray<TWeakObjectPtr<UObject>> SelectedObjects;
	DetailLayout.GetObjectsBeingCustomized(SelectedObjects);
	if( SelectedObjects.Num() == 1 )
	{
		if (SelectedObjects[0].IsValid() && SelectedObjects[0]->IsA<UEdGraphNode>())
		{
			GraphNodePtr = Cast<UEdGraphNode>(SelectedObjects[0].Get());
		}
	}

	if(!GraphNodePtr.IsValid() || !GraphNodePtr.Get()->GetCanRenameNode())
	{
		return;
	}

	IDetailCategoryBuilder& Category = DetailLayout.EditCategory("GraphNodeDetail", LOCTEXT("GraphNodeDetailsCategory", "Graph Node"), ECategoryPriority::Important);
	const FSlateFontInfo DetailFontInfo = IDetailLayoutBuilder::GetDetailFont();
	FText RowHeader;
	FText NameContent;

	if( GraphNodePtr->IsA( UEdGraphNode_Comment::StaticClass() ))
	{
		RowHeader = LOCTEXT("GraphNodeDetail_CommentRowTitle", "Comment");
		NameContent = LOCTEXT("GraphNodeDetail_CommentContentTitle", "Comment Text");
	}
	else
	{
		RowHeader = LOCTEXT("GraphNodeDetail_NodeRowTitle", "Node Title");
		NameContent = LOCTEXT("GraphNodeDetail_ContentTitle", "Name");
	}

	bool bNameAllowsMultiLine = false;
	if( GraphNodePtr.IsValid() && GraphNodePtr.Get()->IsA<UEdGraphNode_Comment>() )
	{
		bNameAllowsMultiLine = true;
	}

	TSharedPtr<SWidget> EditNameWidget;
	float WidgetMinDesiredWidth = BlueprintDocumentationDetailDefs::DetailsTitleMinWidth;
	float WidgetMaxDesiredWidth = BlueprintDocumentationDetailDefs::DetailsTitleMaxWidth;
	if( bNameAllowsMultiLine )
	{
		SAssignNew(MultiLineNameEditableTextBox, SMultiLineEditableTextBox)
		.Text(this, &FBlueprintGraphNodeDetails::OnGetName)
		.OnTextChanged(this, &FBlueprintGraphNodeDetails::OnNameChanged)
		.OnTextCommitted(this, &FBlueprintGraphNodeDetails::OnNameCommitted)
		.ClearKeyboardFocusOnCommit(true)
		.ModiferKeyForNewLine(EModifierKey::Shift)
		.RevertTextOnEscape(true)
		.SelectAllTextWhenFocused(true)
		.IsReadOnly(this, &FBlueprintGraphNodeDetails::IsNameReadOnly)
		.Font(DetailFontInfo)
		.WrapTextAt(WidgetMaxDesiredWidth - BlueprintDocumentationDetailDefs::DetailsTitleWrapPadding);

		EditNameWidget = MultiLineNameEditableTextBox;
	}
	else
	{
		SAssignNew(NameEditableTextBox, SEditableTextBox)
		.Text(this, &FBlueprintGraphNodeDetails::OnGetName)
		.OnTextChanged(this, &FBlueprintGraphNodeDetails::OnNameChanged)
		.OnTextCommitted(this, &FBlueprintGraphNodeDetails::OnNameCommitted)
		.Font(DetailFontInfo);

		EditNameWidget = NameEditableTextBox;
		WidgetMaxDesiredWidth = WidgetMinDesiredWidth;
	}

	Category.AddCustomRow( RowHeader )
	.NameContent()
	[
		SNew(STextBlock)
		.Text( NameContent )
		.Font(DetailFontInfo)
	]
	.ValueContent()
	.MinDesiredWidth(WidgetMinDesiredWidth)
	.MaxDesiredWidth(WidgetMaxDesiredWidth)
	[
		EditNameWidget.ToSharedRef()
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void FBlueprintGraphNodeDetails::SetNameError( const FText& Error )
{
	if( NameEditableTextBox.IsValid() )
	{
		NameEditableTextBox->SetError( Error );
	}
	if( MultiLineNameEditableTextBox.IsValid() )
	{
		MultiLineNameEditableTextBox->SetError( Error );
	}
}

bool FBlueprintGraphNodeDetails::IsNameReadOnly() const
{
	bool bReadOnly = true;
	if(GraphNodePtr.IsValid())
	{
		bReadOnly = !GraphNodePtr->bCanRenameNode;
	}
	return bReadOnly;
}

FText FBlueprintGraphNodeDetails::OnGetName() const
{
	FText Name;
	if(GraphNodePtr.IsValid())
	{
		Name = GraphNodePtr->GetNodeTitle( ENodeTitleType::EditableTitle );
	}
	return Name;
}

struct FGraphNodeNameValidatorHelper
{
	static EValidatorResult Validate(TWeakObjectPtr<UEdGraphNode> GraphNodePtr, TWeakPtr<FBlueprintEditor> BlueprintEditorPtr, const FString& NewName)
	{
		check(GraphNodePtr.IsValid() && BlueprintEditorPtr.IsValid());
		TSharedPtr<INameValidatorInterface> NameValidator = GraphNodePtr->MakeNameValidator();
		if (!NameValidator.IsValid())
		{
			const FName NodeName(*GraphNodePtr->GetNodeTitle(ENodeTitleType::EditableTitle).ToString());
			NameValidator = MakeShareable(new FKismetNameValidator(BlueprintEditorPtr.Pin()->GetBlueprintObj(), NodeName));
		}
		return NameValidator->IsValid(NewName);
	}
};

void FBlueprintGraphNodeDetails::OnNameChanged(const FText& InNewText)
{
	if( GraphNodePtr.IsValid() && BlueprintEditorPtr.IsValid() )
	{
		const EValidatorResult ValidatorResult = FGraphNodeNameValidatorHelper::Validate(GraphNodePtr, BlueprintEditorPtr, InNewText.ToString());
		if(ValidatorResult == EValidatorResult::AlreadyInUse)
		{
			SetNameError(FText::Format(LOCTEXT("RenameFailed_InUse", "{0} is in use by another variable or function!"), InNewText));
		}
		else if(ValidatorResult == EValidatorResult::EmptyName)
		{
			SetNameError(LOCTEXT("RenameFailed_LeftBlank", "Names cannot be left blank!"));
		}
		else if(ValidatorResult == EValidatorResult::TooLong)
		{
			SetNameError(FText::Format( LOCTEXT("RenameFailed_NameTooLong", "Names must have fewer than {0} characters!"), FText::AsNumber( FKismetNameValidator::GetMaximumNameLength())));
		}
		else
		{
			SetNameError(FText::GetEmpty());
		}
	}
}

void FBlueprintGraphNodeDetails::OnNameCommitted(const FText& InNewText, ETextCommit::Type InTextCommit)
{
	if (BlueprintEditorPtr.IsValid() && GraphNodePtr.IsValid())
	{
		if (FGraphNodeNameValidatorHelper::Validate(GraphNodePtr, BlueprintEditorPtr, InNewText.ToString()) == EValidatorResult::Ok)
		{
			BlueprintEditorPtr.Pin()->OnNodeTitleCommitted(InNewText, InTextCommit, GraphNodePtr.Get());
		}
	}
}

UBlueprint* FBlueprintGraphNodeDetails::GetBlueprintObj() const
{
	if(BlueprintEditorPtr.IsValid())
	{
		return BlueprintEditorPtr.Pin()->GetBlueprintObj();
	}

	return NULL;
}

TSharedRef<IDetailCustomization> FChildActorComponentDetails::MakeInstance(TWeakPtr<FBlueprintEditor> BlueprintEditorPtrIn)
{
	return MakeShareable(new FChildActorComponentDetails(BlueprintEditorPtrIn));
}

FChildActorComponentDetails::FChildActorComponentDetails(TWeakPtr<FBlueprintEditor> BlueprintEditorPtrIn)
	: BlueprintEditorPtr(BlueprintEditorPtrIn)
{
}

void FChildActorComponentDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TSharedPtr<IPropertyHandle> ActorClassProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UChildActorComponent, ChildActorClass));
	if (ActorClassProperty->IsValidHandle())
	{
		if (BlueprintEditorPtr.IsValid())
		{
			if (UBlueprint* Blueprint = BlueprintEditorPtr.Pin()->GetBlueprintObj())
			{
				static FText RestrictReason = LOCTEXT("NoSelfChildActors", "Cannot append a child-actor of this blueprint type (could cause infinite recursion).");
				TSharedPtr<FPropertyRestriction> ClassRestriction = MakeShareable(new FPropertyRestriction(RestrictReason));

				if (Blueprint->GeneratedClass)
				{
					ClassRestriction->AddDisabledValue(Blueprint->GeneratedClass->GetPathName());
				}

				ActorClassProperty->AddRestriction(ClassRestriction.ToSharedRef());
			}
		}

		TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
		DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);

		IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(TEXT("ChildActorComponent"));
				
		// Ensure ordering is what we want by adding class in first
		CategoryBuilder.AddProperty(GET_MEMBER_NAME_CHECKED(UChildActorComponent, ChildActorClass));

		// Hide the child actor template property from the details view in these situations:
		// a) Template is invalid (NULL)
		// b) Template is set to be shown/expanded as an actor subtree in the components tree view
		// c) Multiple CACs have been selected with one or more entries that meet the above criteria
		bool bHideChildActorTemplateProperty = false;
		for (const TWeakObjectPtr<UObject>& ObjectBeingCustomized : ObjectsBeingCustomized)
		{
			if (UChildActorComponent* CAC = Cast<UChildActorComponent>(ObjectBeingCustomized.Get()))
			{
				if (CAC->ChildActorTemplate)
				{
					if (FChildActorComponentEditorUtils::ShouldShowChildActorNodeInTreeView(CAC))
					{
						// Template is shown/expanded as a subtree with an explicit child actor node as the root
						bHideChildActorTemplateProperty = true;
					}
				}
				else
				{
					// Template is invalid or not set
					bHideChildActorTemplateProperty = true;
				}
			}

			// Exit the loop if we've met any of the above criteria for hiding the template property
			if (bHideChildActorTemplateProperty)
			{
				break;
			}
		}

		// Add a custom row to expose the child actor template for editing based on its visibility
		CategoryBuilder.AddProperty(GET_MEMBER_NAME_CHECKED(UChildActorComponent, ChildActorTemplate))
			.Visibility(bHideChildActorTemplateProperty ? EVisibility::Hidden : EVisibility::Visible);
	}
}

void FBlueprintDocumentationDetails::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	check( BlueprintEditorPtr.IsValid() );
	// find currently selected edgraph documentation node
	DocumentationNodePtr = EdGraphSelectionAsDocumentNode();

	if( DocumentationNodePtr.IsValid() )
	{
		// Cache Link
		DocumentationLink = DocumentationNodePtr->GetDocumentationLink();
		DocumentationExcerpt = DocumentationNodePtr->GetDocumentationExcerptName();

		IDetailCategoryBuilder& DocumentationCategory = DetailLayout.EditCategory("Documentation", LOCTEXT("DocumentationDetailsCategory", "Documentation"), ECategoryPriority::Default);

		DocumentationCategory.AddCustomRow( LOCTEXT( "DocumentationLinkLabel", "Documentation Link" ))
		.NameContent()
		.HAlign( HAlign_Fill )
		[
			SNew( STextBlock )
			.Text( LOCTEXT( "FBlueprintDocumentationDetails_Link", "Link" ) )
			.ToolTipText( LOCTEXT( "FBlueprintDocumentationDetails_LinkPathTooltip", "The documentation content path" ))
			.Font( IDetailLayoutBuilder::GetDetailFont() )
		]
		.ValueContent()
		.HAlign( HAlign_Left )
		.MinDesiredWidth( BlueprintDocumentationDetailDefs::DetailsTitleMinWidth )
		.MaxDesiredWidth( BlueprintDocumentationDetailDefs::DetailsTitleMaxWidth )
		[
			SNew( SEditableTextBox )
			.Padding( FMargin( 4.f, 2.f ))
			.Text( this, &FBlueprintDocumentationDetails::OnGetDocumentationLink )
			.ToolTipText( LOCTEXT( "FBlueprintDocumentationDetails_LinkTooltip", "The path of the documentation content relative to /Engine/Documentation/Source" ))
			.OnTextCommitted( this, &FBlueprintDocumentationDetails::OnDocumentationLinkCommitted )
			.Font( IDetailLayoutBuilder::GetDetailFont() )
		];

		DocumentationCategory.AddCustomRow( LOCTEXT( "DocumentationExcerptsLabel", "Documentation Excerpts" ))
		.NameContent()
		.HAlign( HAlign_Left )
		[
			SNew( STextBlock )
			.Text( LOCTEXT( "FBlueprintDocumentationDetails_Excerpt", "Excerpt" ) )
			.ToolTipText( LOCTEXT( "FBlueprintDocumentationDetails_ExcerptTooltip", "The current documentation excerpt" ))
			.Font( IDetailLayoutBuilder::GetDetailFont() )
		]
		.ValueContent()
		.HAlign( HAlign_Left )
		.MinDesiredWidth( BlueprintDocumentationDetailDefs::DetailsTitleMinWidth )
		.MaxDesiredWidth( BlueprintDocumentationDetailDefs::DetailsTitleMaxWidth )
		[
			SAssignNew( ExcerptComboButton, SComboButton )
			.ContentPadding( 2.f )
			.IsEnabled( this, &FBlueprintDocumentationDetails::OnExcerptChangeEnabled )
			.ButtonContent()
			[
				SNew(SBorder)
				.BorderImage( FAppStyle::GetBrush( "NoBorder" ))
				.Padding( FMargin( 0, 0, 5, 0 ))
				[
					SNew( STextBlock )
					.Text( this, &FBlueprintDocumentationDetails::OnGetDocumentationExcerpt )
					.ToolTipText( LOCTEXT( "FBlueprintDocumentationDetails_ExcerptComboTooltip", "Select Excerpt" ))
					.Font( IDetailLayoutBuilder::GetDetailFont() )
				]
			]
			.OnGetMenuContent( this, &FBlueprintDocumentationDetails::GenerateExcerptList )
		];
	}
}

TWeakObjectPtr<UEdGraphNode_Documentation> FBlueprintDocumentationDetails::EdGraphSelectionAsDocumentNode()
{
	DocumentationNodePtr.Reset();

	if( BlueprintEditorPtr.IsValid() )
	{
		/** Get the currently selected set of nodes */
		if( BlueprintEditorPtr.Pin()->GetNumberOfSelectedNodes() == 1 )
		{
			FGraphPanelSelectionSet Objects = BlueprintEditorPtr.Pin()->GetSelectedNodes();
			FGraphPanelSelectionSet::TIterator Iter( Objects );
			UObject* Object = *Iter;

			if( Object && Object->IsA<UEdGraphNode_Documentation>() )
			{
				DocumentationNodePtr = Cast<UEdGraphNode_Documentation>( Object );
			}
		}
	}
	return DocumentationNodePtr;
}

FText FBlueprintDocumentationDetails::OnGetDocumentationLink() const
{
	return FText::FromString( DocumentationLink );
}

FText FBlueprintDocumentationDetails::OnGetDocumentationExcerpt() const
{
	return FText::FromString( DocumentationExcerpt );
}

bool FBlueprintDocumentationDetails::OnExcerptChangeEnabled() const
{
	return IDocumentation::Get()->PageExists( DocumentationLink );
}

void FBlueprintDocumentationDetails::OnDocumentationLinkCommitted( const FText& InNewName, ETextCommit::Type InTextCommit )
{
	DocumentationLink = InNewName.ToString();
	DocumentationExcerpt = NSLOCTEXT( "FBlueprintDocumentationDetails", "ExcerptCombo_DefaultText", "Select Excerpt" ).ToString();
}

TSharedRef< ITableRow > FBlueprintDocumentationDetails::MakeExcerptViewWidget( TSharedPtr<FString> Item, const TSharedRef< STableViewBase >& OwnerTable )
{
	return 
		SNew( STableRow<TSharedPtr<FString>>, OwnerTable )
		[
			SNew( STextBlock )
			.Text( FText::FromString(*Item.Get()) )
		];
}

void FBlueprintDocumentationDetails::OnExcerptSelectionChanged( TSharedPtr<FString> ProposedSelection, ESelectInfo::Type /*SelectInfo*/ )
{
	if( ProposedSelection.IsValid() && DocumentationNodePtr.IsValid() )
	{
		DocumentationNodePtr->Link = DocumentationLink;
		DocumentationExcerpt = *ProposedSelection.Get();
		DocumentationNodePtr->Excerpt = DocumentationExcerpt;
		ExcerptComboButton->SetIsOpen( false );
	}
}

TSharedRef<SWidget> FBlueprintDocumentationDetails::GenerateExcerptList()
{
	ExcerptList.Empty();

	if( IDocumentation::Get()->PageExists( DocumentationLink ))
	{
		TSharedPtr<IDocumentationPage> DocumentationPage = IDocumentation::Get()->GetPage( DocumentationLink, NULL );
		TArray<FExcerpt> Excerpts;
		DocumentationPage->GetExcerpts( Excerpts );

		for (const FExcerpt& Excerpt : Excerpts)
		{
			ExcerptList.Add( MakeShareable( new FString( Excerpt.Name )));
		}
	}

	return
		SNew( SHorizontalBox )
		+SHorizontalBox::Slot()
		.Padding( 2.f )
		[
			SNew( SListView< TSharedPtr<FString>> )
			.ListItemsSource( &ExcerptList )
			.OnGenerateRow( this, &FBlueprintDocumentationDetails::MakeExcerptViewWidget )
			.OnSelectionChanged( this, &FBlueprintDocumentationDetails::OnExcerptSelectionChanged )
		];
}

#undef LOCTEXT_NAMESPACE
