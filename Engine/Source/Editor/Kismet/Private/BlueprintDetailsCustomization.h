// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintEditor.h"
#include "BlueprintManagedListDetails.h"
#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphPin.h"
#include "Engine/Blueprint.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IDetailCustomNodeBuilder.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "K2Node_EditablePinBase.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Optional.h"
#include "SMyBlueprint.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/Script.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakFieldPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Views/SListView.h"

class FDetailWidgetRow;
class FMulticastDelegateProperty;
class FProperty;
class FStructOnScope;
class FSubobjectEditorTreeNode;
class IDetailChildrenBuilder;
class IDetailLayoutBuilder;
class IPropertyHandle;
class ITableRow;
class SBlueprintNamespaceEntry;
class SComboButton;
class SEditableTextBox;
class SGraphPin;
class SMultiLineEditableTextBox;
class STableViewBase;
class STextComboBox;
class SWidget;
class UClass;
class UEdGraph;
class UEdGraphNode;
class UEdGraphNode_Documentation;
class UFunction;
class UK2Node_Variable;
class UObject;
class UStruct;
struct FGeometry;
struct FPointerEvent;
struct FPropertyChangedEvent;
struct FTopLevelAssetPath;

/**
 * Variable network replication options.
 * @see https://docs.unrealengine.com/InteractiveExperiences/Networking/Blueprints
 */
namespace EVariableReplication
{
	enum Type
	{
		/**
		 * Not replicated.
		 */
		None,

		/**
		 * Replicated from server to client.
		 * As values change on the server, client automatically receives new values, if Actor is set to replicate.
		 */
		Replicated,

		/**
		 * Replicated from server to client, with a notification function called on clients when a new value arrives.
		 * An event with the name "On Rep <VariableName>" is created.
		 */
		RepNotify,

		MAX
	};
}

class FBlueprintDetails : public IDetailCustomization
{
public:
	FBlueprintDetails(TWeakPtr<SMyBlueprint> InMyBlueprint)
		: Blueprint(InMyBlueprint.Pin()->GetBlueprintObj())
	{
	}

	FBlueprintDetails(TWeakPtr<FBlueprintEditor> InBlueprintEditorPtr)
		: Blueprint(InBlueprintEditorPtr.Pin()->GetBlueprintObj())
	{
	}

protected:

	UBlueprint* GetBlueprintObj() const { return Blueprint.Get(); }

	/**
	* Adds any FMulticastDelegateProperties on the given details builder with either
	* a green "+" symbol to create a bound node, or a "view" option if it already exists.
	* 
	* @param DetailBuilder		Details builder to add the events to
	* @param PropertyName		Name of the selected property
	* @param ComponentClass		The class to find any Multicast delegate properties on
	*/
	void AddEventsCategory(IDetailLayoutBuilder& DetailBuilder, FName PropertyName, UClass* ComponentClass);
	
	FReply HandleAddOrViewEventForVariable(const FName EventName, FName PropertyName, TWeakObjectPtr<UClass> PropertyClass);
	int32 HandleAddOrViewIndexForButton(const FName EventName, FName PropertyName) const;

private:
	/** Pointer back to my parent tab */
	TWeakObjectPtr<UBlueprint> Blueprint;
};

/** Details customization for variables selected in the MyBlueprint panel */
class FBlueprintVarActionDetails : public FBlueprintDetails
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IDetailCustomization> MakeInstance(TWeakPtr<SMyBlueprint> InMyBlueprint)
	{
		return MakeShareable(new FBlueprintVarActionDetails(InMyBlueprint));
	}

	FBlueprintVarActionDetails(TWeakPtr<SMyBlueprint> InMyBlueprint)
		: FBlueprintDetails(InMyBlueprint)
		, MyBlueprint(InMyBlueprint)
		, bIsVarNameInvalid(false)
	{
	}

	~FBlueprintVarActionDetails();
	
	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailLayout ) override;
	
	static void PopulateCategories(SMyBlueprint* MyBlueprint, TArray<TSharedPtr<FText>>& CategorySource);

private:
	/** Accessors passed to parent */
	UK2Node_Variable* EdGraphSelectionAsVar() const;
	FProperty* SelectionAsProperty() const;
	FName GetVariableName() const;

	/** Commonly queried attributes about the schema action */
	bool IsAUserVariable(FProperty* VariableProperty) const;
	bool IsASCSVariable(FProperty* VariableProperty) const;
	bool IsABlueprintVariable(FProperty* VariableProperty) const;
	bool IsALocalVariable(FProperty* VariableProperty) const;
	UStruct* GetLocalVariableScope(FProperty* VariableProperty) const;

	// Callbacks for uproperty details customization
	bool GetVariableNameChangeEnabled() const;
	FText OnGetVarName() const;
	void OnVarNameChanged(const FText& InNewText);
	void OnVarNameCommitted(const FText& InNewName, ETextCommit::Type InTextCommit);
	bool GetVariableTypeChangeEnabled() const;
	FEdGraphPinType OnGetVarType() const;
	void OnVarTypeChanged(const FEdGraphPinType& NewPinType);
	EVisibility IsTooltipEditVisible() const;

	void OnBrowseToVarType() const;
	bool CanBrowseToVarType() const;

	/**
	 * Callback when changing a variable property
	 *
	 * @param InPropertyChangedEvent	Information on the property changed
	 */
	void OnFinishedChangingVariable(const FPropertyChangedEvent& InPropertyChangedEvent);

	/**
	 * Callback when changing a local variable property
	 *
	 * @param InPropertyChangedEvent	Information on the property changed
	 * @param InStructData				The struct data where the value of the properties are stored
	 * @param InEntryNode				Entry node where the default values of local variables are stored
	 */
	void OnFinishedChangingLocalVariable(const FPropertyChangedEvent& InPropertyChangedEvent, TSharedPtr<FStructOnScope> InStructData, TWeakObjectPtr<UK2Node_EditablePinBase> InEntryNode);

	/**
	 * Auto-import any namespaces associated with a variable's value into the current editor context
	 *
	 * @param InProperty				A reference to the variable property
	 * @param InContainer				A pointer to the data container (e.g. struct or object) where the property's value is stored
	 */
	void ImportNamespacesForPropertyValue(const FProperty* InProperty, const void* InContainer);

	/** Callback to decide if the category drop down menu should be enabled */
	bool GetVariableCategoryChangeEnabled() const;

	FText OnGetTooltipText() const;
	void OnTooltipTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit, FName VarName);
	
	FText OnGetCategoryText() const;
	void OnCategoryTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit, FName VarName);
	TSharedRef< ITableRow > MakeCategoryViewWidget( TSharedPtr<FText> Item, const TSharedRef< STableViewBase >& OwnerTable );
	void OnCategorySelectionChanged( TSharedPtr<FText> ProposedSelection, ESelectInfo::Type /*SelectInfo*/ );
	
	EVisibility ShowEditableCheckboxVisibility() const;
	ECheckBoxState OnEditableCheckboxState() const;
	void OnEditableChanged(ECheckBoxState InNewState);

	EVisibility ShowReadOnlyCheckboxVisibility() const;
	ECheckBoxState OnReadyOnlyCheckboxState() const;
	void OnReadyOnlyChanged(ECheckBoxState InNewState);

	EVisibility GetVariableUnitsVisibility() const;
	TSharedPtr<FString> GetVariableUnits() const;
	void OnVariableUnitsChanged(TSharedPtr<FString> ItemSelected, ESelectInfo::Type SelectInfo);
	
	ECheckBoxState OnFieldNotifyCheckboxState() const;
	void OnFieldNotifyChanged(ECheckBoxState InNewState);
	EVisibility GetFieldNotifyCheckboxListVisibility() const;

	ECheckBoxState OnCreateWidgetCheckboxState() const;
	void OnCreateWidgetChanged(ECheckBoxState InNewState);
	EVisibility Show3DWidgetVisibility() const;
	bool Is3DWidgetEnabled();
	
	ECheckBoxState OnGetExposedToSpawnCheckboxState() const;
	void OnExposedToSpawnChanged(ECheckBoxState InNewState);
	EVisibility ExposeOnSpawnVisibility() const;

	ECheckBoxState OnGetPrivateCheckboxState() const;
	void OnPrivateChanged(ECheckBoxState InNewState);
	EVisibility ExposePrivateVisibility() const;
	
	ECheckBoxState OnGetExposedToCinematicsCheckboxState() const;
	void OnExposedToCinematicsChanged(ECheckBoxState InNewState);
	EVisibility ExposeToCinematicsVisibility() const;

	ECheckBoxState OnGetConfigVariableCheckboxState() const;
	void OnSetConfigVariableState(ECheckBoxState InNewState);
	EVisibility ExposeConfigVisibility() const;
	bool IsConfigCheckBoxEnabled() const;

	FText OnGetMetaKeyValue(FName Key) const;
	void OnMetaKeyValueChanged(const FText& NewMinValue, ETextCommit::Type CommitInfo, FName Key);
	EVisibility RangeVisibility() const;

	ECheckBoxState OnBitmaskCheckboxState() const;
	EVisibility BitmaskVisibility() const;
	void OnBitmaskChanged(ECheckBoxState InNewState);

	TSharedPtr<FTopLevelAssetPath> GetBitmaskEnumTypePath() const;
	void OnBitmaskEnumTypeChanged(TSharedPtr<FTopLevelAssetPath> ItemSelected, ESelectInfo::Type SelectInfo);
	TSharedRef<SWidget> GenerateBitmaskEnumTypeWidget(TSharedPtr<FTopLevelAssetPath> Item);
	FText GetBitmaskEnumTypeName() const;
	
	TSharedPtr<FString> GetVariableReplicationType() const;
	void OnChangeReplication(TSharedPtr<FString> ItemSelected, ESelectInfo::Type SelectInfo);
	void ReplicationOnRepFuncChanged(const FString& NewOnRepFunc) const;
	EVisibility ReplicationVisibility() const;

	/** Array of replication conditions for the combo text box */
	TArray<TSharedPtr<FString>> ReplicationConditionEnumTypeNames;
	TSharedPtr<FString> GetVariableReplicationCondition() const;
	void OnChangeReplicationCondition(TSharedPtr<FString> ItemSelected, ESelectInfo::Type SelectInfo);
	bool ReplicationConditionEnabled() const;
	bool ReplicationEnabled() const;
	FText ReplicationTooltip() const;

	EVisibility GetTransientVisibility() const;
	ECheckBoxState OnGetTransientCheckboxState() const;
	void OnTransientChanged(ECheckBoxState InNewState);

	EVisibility GetSaveGameVisibility() const;
	ECheckBoxState OnGetSaveGameCheckboxState() const;
	void OnSaveGameChanged(ECheckBoxState InNewState);

	EVisibility GetAdvancedDisplayVisibility() const;
	ECheckBoxState OnGetAdvancedDisplayCheckboxState() const;
	void OnAdvancedDisplayChanged(ECheckBoxState InNewState);

	EVisibility GetMultilineVisibility() const;
	ECheckBoxState OnGetMultilineCheckboxState() const;
	void OnMultilineChanged(ECheckBoxState InNewState);

	EVisibility GetDeprecatedVisibility() const;
	ECheckBoxState OnGetDeprecatedCheckboxState() const;
	void OnDeprecatedChanged(ECheckBoxState InNewState);

	EVisibility GetDropDownOptionsVisibility() const;
	void OnDropDownOptionSelectionChanged(TSharedPtr<FString> InString, ESelectInfo::Type);
	void OnDropDownOptionTextChanged(const FText& Text, ETextCommit::Type);
	TSharedRef<SWidget> GenerateDropDownOptionWidget(TSharedPtr<FString> InItem) const;
	void CollectDropDownOptions();
	FText GetDropDownOptionDisplayText() const;
	FString GetDropDownOptionsFunctionName() const;
	void SetDropDownOptionsFunctionName(const FString& InFunctionName);

	FText GetDeprecationMessageText() const;
	void OnDeprecationMessageTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit, FName VarName);

	/** Refresh the property flags list */
	void RefreshPropertyFlags();

	/** Generates the widget for the property flag list */
	TSharedRef<ITableRow> OnGenerateWidgetForPropertyList( TSharedPtr< FString > Item, const TSharedRef<STableViewBase>& OwnerTable );

	/** Delegate to build variable events droplist menu */
	TSharedRef<SWidget> BuildEventsMenuForVariable() const;
	
	/** Refreshes cached data that changes after a Blueprint recompile */
	void OnPostEditorRefresh();

	/** Returns the Property's Blueprint */
	UBlueprint* GetPropertyOwnerBlueprint() const { return PropertyOwnerBlueprint.Get(); }

	/** Returns TRUE if the Variable is in the current Blueprint */
	bool IsVariableInBlueprint() const { return GetPropertyOwnerBlueprint() == GetBlueprintObj(); }

	/** Returns TRUE if the Variable is inherited by the current Blueprint */
	bool IsVariableInheritedByBlueprint() const;

	/** Returns TRUE if the Variable is marked as deprecated */
	bool IsVariableDeprecated() const;
private:
	/** Pointer back to my parent tab */
	TWeakPtr<SMyBlueprint> MyBlueprint;

	/** Array of replication options for our combo text box */
	TArray<TSharedPtr<FString>> ReplicationOptions;

	/** Array of function names that can be used for GetOptions */
	TArray<TSharedPtr<FString>> DropDownFunctionOptions;

	/** Array of units options for our combo text box */
	TArray<TSharedPtr<FString>> UnitsOptions;

	/** Array of enum type names for integers used as bitmasks */
	TArray<TSharedPtr<FTopLevelAssetPath>> BitmaskEnumTypePaths;

	/** The widget used when in variable name editing mode */ 
	TSharedPtr<SEditableTextBox> VarNameEditableTextBox;

	/** Flag to indicate whether or not the variable name is invalid */
	bool bIsVarNameInvalid;
	
	/** A list of all category names to choose from */
	TArray<TSharedPtr<FText>> CategorySource;
	/** Widgets for the categories */
	TWeakPtr<SComboButton> CategoryComboButton;
	TWeakPtr<SListView<TSharedPtr<FText>>> CategoryListView;

	/** Array of names of property flags on the selected property */
	TArray< TSharedPtr< FString > > PropertyFlags;

	/** The listview widget for displaying property flags */
	TWeakPtr< SListView< TSharedPtr< FString > > > PropertyFlagWidget;

	/** Cached property for the variable we are affecting */
	TWeakFieldPtr<FProperty> CachedVariableProperty;

	/** Cached name for the variable we are affecting */
	FName CachedVariableName;

	/** Pointer back to the variable's Blueprint */
	TWeakObjectPtr<UBlueprint> PropertyOwnerBlueprint;

	/** External detail customizations */
	TArray<TSharedPtr<IDetailCustomization>> ExternalDetailCustomizations;
};

class FBaseBlueprintGraphActionDetails : public IDetailCustomization
{
public:
	virtual ~FBaseBlueprintGraphActionDetails();

	FBaseBlueprintGraphActionDetails(TWeakPtr<SMyBlueprint> InMyBlueprint)
		: MyBlueprint(InMyBlueprint)
	{
	}
	
	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailLayout ) override 
	{
		check(false);
	}

	/** Gets the graph that we are currently editing */
	virtual UEdGraph* GetGraph() const;

	/** Refreshes the graph and ensures the target node is up to date */
	void OnParamsChanged(UK2Node_EditablePinBase* TargetNode, bool bForceRefresh = false);

	/** Checks if the pin rename can occur */
	bool OnVerifyPinRename(UK2Node_EditablePinBase* InTargetNode, const FName InOldName, const FString& InNewName, FText& OutErrorMessage);

	/** Refreshes the graph and ensures the target node is up to date */
	bool OnPinRenamed(UK2Node_EditablePinBase* TargetNode, const FName OldName, const FString& NewName);
	
	/** Gets the blueprint we're editing */
	TWeakPtr<SMyBlueprint> GetMyBlueprint() const {return MyBlueprint.Pin();}

	/** Gets the node for the function entry point */
	TWeakObjectPtr<class UK2Node_EditablePinBase> GetFunctionEntryNode() const {return FunctionEntryNodePtr;}

	/** Sets the delegate to be called when refreshing our children */
	void SetRefreshDelegate(FSimpleDelegate RefreshDelegate, bool bForInputs);

	/** Accessors passed to parent */
	UBlueprint* GetBlueprintObj() const {return MyBlueprint.Pin()->GetBlueprintObj();}
	
	FReply OnAddNewInputClicked();

	/** Called when blueprint changes */
	void OnPostEditorRefresh();

protected:
	/** Tries to create the result node (if there are output args) */
	bool AttemptToCreateResultNode();

	/** Pointer to the parent */
	TWeakPtr<SMyBlueprint> MyBlueprint;
	
	/** The entry node in the graph */
	TWeakObjectPtr<class UK2Node_EditablePinBase> FunctionEntryNodePtr;

	/** The result node in the graph, if the function has any return or out params.  This can be the same as the entry point */
	TWeakObjectPtr<class UK2Node_EditablePinBase> FunctionResultNodePtr;

	/** Delegates to regenerate the lists of children */
	FSimpleDelegate RegenerateInputsChildrenDelegate;
	FSimpleDelegate RegenerateOutputsChildrenDelegate;

	/** Details layout builder we need to hold on to to refresh it at times */
	IDetailLayoutBuilder* DetailsLayoutPtr;

	/** Handle for graph refresh delegate */
	FDelegateHandle BlueprintEditorRefreshDelegateHandle;
	
	/** Array of nodes were were constructed to represent */
	TArray< TWeakObjectPtr<UObject> > ObjectsBeingEdited;
};

class FBlueprintDelegateActionDetails : public FBaseBlueprintGraphActionDetails
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IDetailCustomization> MakeInstance(TWeakPtr<SMyBlueprint> InMyBlueprint)
	{
		return MakeShareable(new FBlueprintDelegateActionDetails(InMyBlueprint));
	}

	FBlueprintDelegateActionDetails(TWeakPtr<SMyBlueprint> InMyBlueprint)
		: FBaseBlueprintGraphActionDetails(InMyBlueprint)
	{
	}
	
	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailLayout ) override;

	/** Gets the graph that we are currently editing */
	virtual UEdGraph* GetGraph() const override;

private:

	void SetEntryNode();

	FMulticastDelegateProperty* GetDelegateProperty() const;

	void CollectAvailibleSignatures();
	void OnFunctionSelected(TSharedPtr<FString> FunctionItemData, ESelectInfo::Type SelectInfo);
	bool IsBlueprintProperty() const;
	EVisibility OnGetSectionTextVisibility(TWeakPtr<SWidget> RowWidget) const;

private:
	TArray<TSharedPtr<FString>> FunctionsToCopySignatureFrom;
	TSharedPtr<STextComboBox> CopySignatureComboButton;
};

/** Custom struct for each group of arguments in the function editing details */
class FBlueprintGraphArgumentGroupLayout : public IDetailCustomNodeBuilder, public TSharedFromThis<FBlueprintGraphArgumentGroupLayout>
{
public:
	FBlueprintGraphArgumentGroupLayout(TWeakPtr<class FBaseBlueprintGraphActionDetails> InGraphActionDetails, UK2Node_EditablePinBase* InTargetNode)
		: GraphActionDetailsPtr(InGraphActionDetails)
		, TargetNode(InTargetNode) {}

private:
	/** IDetailCustomNodeBuilder Interface*/
	virtual void SetOnRebuildChildren( FSimpleDelegate InOnRegenerateChildren ) override;
	virtual void GenerateHeaderRowContent( FDetailWidgetRow& NodeRow ) override {}
	virtual void GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder ) override;
	virtual void Tick( float DeltaTime ) override {}
	virtual bool RequiresTick() const override { return false; }
	virtual FName GetName() const override { return NAME_None; }
	virtual bool InitiallyCollapsed() const override { return false; }
	
private:
	/** The parent graph action details customization */
	TWeakPtr<class FBaseBlueprintGraphActionDetails> GraphActionDetailsPtr;

	/** The target node that this argument is on */
	TWeakObjectPtr<UK2Node_EditablePinBase> TargetNode;
};

/** Custom struct for each argument in the function editing details */
class FBlueprintGraphArgumentLayout : public IDetailCustomNodeBuilder, public TSharedFromThis<FBlueprintGraphArgumentLayout>
{
public:
	FBlueprintGraphArgumentLayout(TWeakPtr<FUserPinInfo> PinInfo, UK2Node_EditablePinBase* InTargetNode, TWeakPtr<class FBaseBlueprintGraphActionDetails> InGraphActionDetails, FName InArgName, bool bInHasDefaultValue)
		: GraphActionDetailsPtr(InGraphActionDetails)
		, ParamItemPtr(PinInfo)
		, TargetNode(InTargetNode)
		, bHasDefaultValue(bInHasDefaultValue)
		, ArgumentName(InArgName) {}
	virtual ~FBlueprintGraphArgumentLayout();

private:
	/** IDetailCustomNodeBuilder Interface*/
	virtual void SetOnRebuildChildren( FSimpleDelegate InOnRegenerateChildren ) override {}
	virtual void GenerateHeaderRowContent( FDetailWidgetRow& NodeRow ) override;
	virtual void GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder ) override;
	virtual void Tick( float DeltaTime ) override {}
	virtual bool RequiresTick() const override { return false; }
	virtual FName GetName() const override { return ArgumentName; }
	virtual bool InitiallyCollapsed() const override { return true; }

private:
	/** Determines if this pin should not be editable */
	bool ShouldPinBeReadOnly(bool bIsEditingPinType = false) const;
	
	/** Determines if editing the pins on the node should be read only */
	bool IsPinEditingReadOnly(bool bIsEditingPinType = false) const;

	/** Callbacks for all the functionality for modifying arguments */
	void OnRemoveClicked();

	void OnArgumentGraphChanged(const FEdGraphEditAction& Action);

	FText OnGetArgNameText() const;
	FText OnGetArgToolTipText() const;
	void OnArgNameChange(const FText& InNewText);
	void OnArgNameTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit);
	
	FEdGraphPinType OnGetPinInfo() const;
	void PinInfoChanged(const FEdGraphPinType& PinType);
	void OnPrePinInfoChange(const FEdGraphPinType& PinType);

	/** Returns the graph pin representing this variable */
	UEdGraphPin* GetPin() const;

	/** Returns whether the "Pass-by-Reference" checkbox is checked or not */
	ECheckBoxState IsRefChecked() const;
	ECheckBoxState IsConstChecked() const;

	/** Handles toggling the "Pass-by-Reference" checkbox */
	void OnRefCheckStateChanged(ECheckBoxState InState);

	void OnConstCheckStateChanged(ECheckBoxState InState);
	bool ShouldBeForceConst() const;
	bool CanChangeConst() const;

private:
	/** The parent graph action details customization */
	TWeakPtr<class FBaseBlueprintGraphActionDetails> GraphActionDetailsPtr;

	/** The argument pin that this layout reflects */
	TWeakPtr<FUserPinInfo> ParamItemPtr;

	/** The target node that this argument is on */
	UK2Node_EditablePinBase* TargetNode;

	/** Whether or not this builder should have a default value edit control (input args only) */
	bool bHasDefaultValue;

	/** The name of this argument for remembering expansion state */
	FName ArgumentName;

	/** Holds a weak pointer to the argument name widget, used for error notifications */
	TWeakPtr<SEditableTextBox> ArgumentNameWidget;

	/** The SGraphPin widget created to show/edit default value */
	TSharedPtr<SGraphPin> DefaultValuePinWidget;

	/** Delegate handle for node changes */
	FDelegateHandle GraphChangedHandler;
};

// local variables in the details view only support read-only mode at the moment.
// the incomplete read/write functionality can be turned on by setting this to true but you'll have to write some more code to make it compile/work
#define UE_BP_LOCAL_VAR_LAYOUT_SETTERS_IMPLEMENTED false

/** Custom struct for each group of local variables in the function editing details */
class FBlueprintGraphLocalVariableGroupLayout : public IDetailCustomNodeBuilder, public TSharedFromThis<FBlueprintGraphLocalVariableGroupLayout>
{
public:
	FBlueprintGraphLocalVariableGroupLayout(TWeakPtr<class FBaseBlueprintGraphActionDetails> InGraphActionDetails, TWeakObjectPtr<UEdGraph> InTargetGraph,
		TWeakObjectPtr<UBlueprint> InBlueprintObj, TWeakObjectPtr<UFunction> InOwningFunction, TSharedPtr<IPropertyHandle> Property = nullptr)
		: GraphActionDetailsPtr(InGraphActionDetails)
		, TargetGraph(InTargetGraph)
		, BlueprintObj(InBlueprintObj)
		, PropertyHandle(Property)
		, OwningFunction(InOwningFunction) {}
	
	virtual TSharedPtr<IPropertyHandle> GetPropertyHandle() const override {return PropertyHandle;}

private:
	/** IDetailCustomNodeBuilder Interface*/
	virtual void GenerateHeaderRowContent( FDetailWidgetRow& NodeRow ) override {}
	virtual void GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder ) override;
	virtual void Tick( float DeltaTime ) override {}
	virtual bool RequiresTick() const override { return false; }
	virtual FName GetName() const override { return NAME_None; }
	virtual bool InitiallyCollapsed() const override { return false; }
	
private:
	/** The parent graph action details customization */
	TWeakPtr<class FBaseBlueprintGraphActionDetails> GraphActionDetailsPtr;

	/** The target graph that the local variable's are on */
	TWeakObjectPtr<UEdGraph> TargetGraph;

	/** The blueprint object that the target graph is in */
	TWeakObjectPtr<UBlueprint> BlueprintObj;

	/** handle to property for array of FBPVariableDescription's */
	TSharedPtr<IPropertyHandle> PropertyHandle;

	/** function that owns the local variable's being displayed */
	TWeakObjectPtr<UFunction> OwningFunction;
};

/** Custom struct for each local variable in the function editing details */
class FBlueprintGraphLocalVariableLayout : public IDetailCustomNodeBuilder, public TSharedFromThis<FBlueprintGraphLocalVariableLayout>
{
public:
	FBlueprintGraphLocalVariableLayout(TWeakObjectPtr<UFunction> InOwningFunction, const FBPVariableDescription& VariableDescription)
		: OwningFunction(InOwningFunction)
	{
		Data.Set<FBPVariableDescription>(VariableDescription);
	}
	
	FBlueprintGraphLocalVariableLayout(TWeakObjectPtr<UFunction> InOwningFunction, TSharedPtr<IPropertyHandle> Property = nullptr)
		: OwningFunction(InOwningFunction)
	{
		Data.Set<TSharedPtr<IPropertyHandle>>(Property);
	}
	
	virtual TSharedPtr<IPropertyHandle> GetPropertyHandle() const override;

	const FBPVariableDescription& GetVariable() const;
	void SetVariable(const FBPVariableDescription&);

private:
	/** IDetailCustomNodeBuilder Interface*/
	virtual void SetOnRebuildChildren( FSimpleDelegate InOnRegenerateChildren ) override {}
	virtual void GenerateHeaderRowContent( FDetailWidgetRow& NodeRow ) override;
	virtual void GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder ) override;
	virtual void Tick( float DeltaTime ) override {}
	virtual bool RequiresTick() const override { return false; }
	virtual FName GetName() const override { return GetVariable().VarName; }
	virtual bool InitiallyCollapsed() const override { return true; }
private:
	
	/** Determines if this pin should not be editable */
	bool ShouldVarBeReadOnly(bool bIsEditingPinType = false) const;
	
	/** Determines if editing the pins on the node should be read only */
	bool IsVariableEditingReadOnly(bool bIsEditingPinType = false) const;
	

	/** Callbacks for all the functionality for modifying local variables */
	FText OnGetVarNameText() const;
	FText OnGetVarToolTipText() const;
	FEdGraphPinType OnGetVariableType() const;

// currently this layout is for read only data. If you need to use it for read/write, you can implement the following methods.
#if UE_BP_LOCAL_VAR_LAYOUT_SETTERS_IMPLEMENTED
	void OnRemoveClicked();
	void OnVarNameChange(const FText& InNewText);
	void OnVarNameTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit);
	void VariableTypeChanged(const FEdGraphPinType& PinType);
	void OnPreVariableTypeChange(const FEdGraphPinType& PinType);
#endif

private:

	/** Holds a weak pointer to the argument name widget, used for error notifications */
	TWeakPtr<SEditableTextBox> VariableNameWidget;

	/** The SGraphPin widget created to show/edit default value */
	TSharedPtr<SGraphPin> DefaultValueWidget;

	/** either the read only copy of FBPVariableDescription or the property handle to the original variable description */
	TVariant<TSharedPtr<IPropertyHandle>, FBPVariableDescription> Data;

	/** function that owns this local variable */
	TWeakObjectPtr<UFunction> OwningFunction;
};

/** Details customization for functions and graphs selected in the MyBlueprint panel */
class FBlueprintGraphActionDetails : public FBaseBlueprintGraphActionDetails
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IDetailCustomization> MakeInstance(TWeakPtr<SMyBlueprint> InMyBlueprint, bool bInShowLocalVariables = false)
	{
		return MakeShareable(new FBlueprintGraphActionDetails(InMyBlueprint, bInShowLocalVariables));
	}

	FBlueprintGraphActionDetails(TWeakPtr<SMyBlueprint> InMyBlueprint, bool bInShowLocalVariables = false)
		: FBaseBlueprintGraphActionDetails(InMyBlueprint)
		, bShowLocalVariables(bInShowLocalVariables)
	{
	}
	
	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailLayout ) override;

private:
	struct FAccessSpecifierLabel
	{
		FText LocalizedName;
		EFunctionFlags SpecifierFlag;

		FAccessSpecifierLabel( FText InLocalizedName, EFunctionFlags InSpecifierFlag) :
			LocalizedName( InLocalizedName ), SpecifierFlag( InSpecifierFlag )
		{}
	};

private:

	/** Setup for the nodes this details customizer needs to access */
	void SetEntryAndResultNodes();

	/** Gets the node we are currently editing if available */
	UK2Node_EditablePinBase* GetEditableNode() const;
	
	/* Get function associated with the selected graph*/
	UFunction* FindFunction() const;
	
	/** Utility for editing metadata on the function */
	FKismetUserDeclaredFunctionMetadata* GetMetadataBlock() const;

	// Callbacks for uproperty details customization
	FText OnGetTooltipText() const;
	void OnTooltipTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit);
	
	FText OnGetCategoryText() const;
	void OnCategoryTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit);

	FText OnGetKeywordsText() const;
	void OnKeywordsTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit);

	FText OnGetCompactNodeTitleText() const;
	void OnCompactNodeTitleTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit);
	
	FText AccessSpecifierProperName( uint32 AccessSpecifierFlag ) const;
	bool IsAccessSpecifierVisible() const;
	TSharedRef<ITableRow> HandleGenerateRowAccessSpecifier( TSharedPtr<FAccessSpecifierLabel> SpecifierName, const TSharedRef<STableViewBase>& OwnerTable );
	FText GetCurrentAccessSpecifierName() const;
	void OnAccessSpecifierSelected( TSharedPtr<FAccessSpecifierLabel> SpecifierName, ESelectInfo::Type SelectInfo );

	bool GetInstanceColorVisibility() const;
	FLinearColor GetNodeTitleColor() const;
	FReply ColorBlock_OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	
	bool IsCustomEvent() const;
	void OnIsReliableReplicationFunctionModified(const ECheckBoxState NewCheckedState);
	ECheckBoxState GetIsReliableReplicatedFunction() const;

	struct FReplicationSpecifierLabel
	{
		FText LocalizedName;
		FText LocalizedToolTip;
		uint32 SpecifierFlag;

		FReplicationSpecifierLabel( FText InLocalizedName, uint32 InSpecifierFlag, FText InLocalizedToolTip ) :
			LocalizedName( InLocalizedName ), LocalizedToolTip( InLocalizedToolTip ), SpecifierFlag( InSpecifierFlag )
		{}
	};

	FText GetCurrentReplicatedEventString() const;
	FText ReplicationSpecifierProperName( uint32 ReplicationSpecifierFlag ) const;
	TSharedRef<ITableRow> OnGenerateReplicationComboWidget( TSharedPtr<FReplicationSpecifierLabel> InNetFlag, const TSharedRef<STableViewBase>& OwnerTable );
	
	bool IsPureFunctionVisible() const;
	void OnIsPureFunctionModified(const ECheckBoxState NewCheckedState);
	ECheckBoxState GetIsPureFunction() const;

	bool IsConstFunctionVisible() const;
	void OnIsConstFunctionModified(const ECheckBoxState NewCheckedState);
	ECheckBoxState GetIsConstFunction() const;

	bool IsExecFunctionVisible() const;
	void OnIsExecFunctionModified(const ECheckBoxState NewCheckedState);
	ECheckBoxState GetIsExecFunction() const;

	bool IsThreadSafeFunctionVisible() const;
	void OnIsThreadSafeFunctionModified(const ECheckBoxState NewCheckedState);
	ECheckBoxState GetIsThreadSafeFunction() const;
	
	bool IsUnsafeDuringActorConstructionVisible() const;
	void OnIsUnsafeDuringActorConstructionModified(const ECheckBoxState NewCheckedState);
	ECheckBoxState GetIsUnsafeDuringActorConstruction() const;

	bool IsFieldNotifyCheckVisible() const;
	bool GetIsFieldNotfyEnabled() const;
	ECheckBoxState OnFieldNotifyCheckboxState() const;
	void OnFieldNotifyChanged(ECheckBoxState InNewState);

	/** Determines if the selected event is identified as editor callable */
	ECheckBoxState GetIsEditorCallableEvent() const;

	/** Enables/Disables selected event as editor callable  */
	void OnEditorCallableEventModified( const ECheckBoxState NewCheckedState ) const;

	bool IsFunctionDeprecated() const;
	ECheckBoxState OnGetDeprecatedCheckboxState() const;
	void OnDeprecatedChanged(ECheckBoxState InNewState);

	FText GetDeprecationMessageText() const;
	void OnDeprecationMessageTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit);
	
	FReply OnAddNewOutputClicked();

	/** Callback to determine if the "New" button for adding input/output pins is visible */
	EVisibility GetAddNewInputOutputVisibility() const;

	/** Callback to determine if the "new" button for adding input/output pins is enabled */
	bool IsAddNewInputOutputEnabled() const;

	EVisibility OnGetSectionTextVisibility(TWeakPtr<SWidget> RowWidget) const;

	/** Called to set the replication type from the details view combo */
	static void SetNetFlags( TWeakObjectPtr<UK2Node_EditablePinBase> FunctionEntryNode, uint32 NetFlags);

	/** Callback when a graph category is changed */
	void OnCategorySelectionChanged( TSharedPtr<FText> ProposedSelection, ESelectInfo::Type /*SelectInfo*/ );

	/** Callback to make category widgets */
	TSharedRef< ITableRow > MakeCategoryViewWidget( TSharedPtr<FText> Item, const TSharedRef< STableViewBase >& OwnerTable );

private:

	/** List of available localized access specifiers names */
	TArray<TSharedPtr<FAccessSpecifierLabel> > AccessSpecifierLabels;

	/** ComboButton with access specifiers */
	TSharedPtr< class SComboButton > AccessSpecifierComboButton;

	/** Color block for parenting the color picker */
	TSharedPtr<class SColorBlock> ColorBlock;

	/** A list of all category names to choose from */
	TArray<TSharedPtr<FText>> CategorySource;

	/** Widgets for the categories */
	TWeakPtr<SComboButton> CategoryComboButton;
	TWeakPtr<SListView<TSharedPtr<FText>>> CategoryListView;

	/** External detail customizations */
	TArray<TSharedPtr<IDetailCustomization>> ExternalDetailCustomizations;

	bool bShowLocalVariables = false;
};

/** Blueprint global options - managed list details customization base */
class FBlueprintGlobalOptionsManagedListDetails : public FBlueprintManagedListDetails
{
public:
	/** Constructor method */
	FBlueprintGlobalOptionsManagedListDetails(TWeakPtr<class FBlueprintGlobalOptionsDetails> InGlobalOptionsDetailsPtr);

protected:
	/** Access to external resources */
	UBlueprint* GetBlueprintObjectChecked() const;
	TSharedPtr<FBlueprintEditor> GetPinnedBlueprintEditorPtr() const;

	/** Helper function to set the Blueprint back into the KismetInspector's details view */
	void OnRefreshInDetailsView();

private:
	/** The parent graph action details customization */
	TWeakPtr<class FBlueprintGlobalOptionsDetails> GlobalOptionsDetailsPtr;
};

/** Blueprint Imports List Details */
class FBlueprintImportsLayout : public FBlueprintGlobalOptionsManagedListDetails, public TSharedFromThis<FBlueprintImportsLayout>
{
public:
	FBlueprintImportsLayout(TWeakPtr<class FBlueprintGlobalOptionsDetails> InGlobalOptionsDetails, bool bInShowDefaultImports);

protected:
	/** FBlueprintManagedListDetails interface*/
	virtual TSharedPtr<SWidget> MakeAddItemWidget() override;
	virtual void GetManagedListItems(TArray<FManagedListItem>& OutListItems) const override;
	virtual void OnRemoveItem(const FManagedListItem& Item) override;
	/** END FBlueprintManagedListDetails interface */

	void OnNamespaceSelected(const FString& InNamespace);
	void OnGetNamespacesToExclude(TSet<FString>& OutNamespacesToExclude) const;

private:
	/** Whether we should show default (i.e. global) imports versus custom (i.e. local) imports */
	bool bShouldShowDefaultImports;
};

/** Blueprint Interface List Details */
class FBlueprintInterfaceLayout : public FBlueprintGlobalOptionsManagedListDetails, public TSharedFromThis<FBlueprintInterfaceLayout>
{
public:
	FBlueprintInterfaceLayout(TWeakPtr<class FBlueprintGlobalOptionsDetails> InGlobalOptionsDetails, TSharedPtr<IPropertyHandle> InInterfacesProperty = nullptr);

	/** IDetailCustomNodeBuilder interface */
	virtual TSharedPtr<IPropertyHandle> GetPropertyHandle() const override;
	/** END IDetailCustomNodeBuilder interface */
protected:
	/** FBlueprintManagedListDetails interface */
	virtual TSharedPtr<SWidget> MakeAddItemWidget() override;
	virtual void GetManagedListItems(TArray<FManagedListItem>& OutListItems) const override;
	virtual void OnRemoveItem(const FManagedListItem& Item) override;
	/** END FBlueprintManagedListDetails interface */
	
private:
	/** Callbacks for details UI */
	TSharedRef<SWidget> OnGetAddInterfaceMenuContent();

	/** Callback function when an interface class is picked */
	void OnClassPicked(UClass* PickedClass);

private:
	/** The add interface combo button */
	TSharedPtr<SComboButton> AddInterfaceComboButton;

	/** Property that stores interfaces */
	TSharedPtr<IPropertyHandle> InterfacesProperty;
};

/** Details customization for Blueprint settings */
class FBlueprintGlobalOptionsDetails : public IDetailCustomization
{
public:
	/** Constructor */
	FBlueprintGlobalOptionsDetails(TWeakPtr<FBlueprintEditor> InBlueprintEditorPtr)
		:BlueprintEditorPtr(InBlueprintEditorPtr)
		,BlueprintObjOverride(nullptr)
	{

	}
	
	FBlueprintGlobalOptionsDetails(TWeakObjectPtr<UBlueprint> InBlueprintPtr)
		:BlueprintObjOverride(InBlueprintPtr)
	{

	}

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IDetailCustomization> MakeInstance(TWeakPtr<FBlueprintEditor> InBlueprintEditorPtr)
	{
		return MakeShareable(new FBlueprintGlobalOptionsDetails(InBlueprintEditorPtr));
	}

	/** Diff functionality doesn't need access to the FBlueprintEditor so this allows creation without editor access
	 *  but with limited functionality */
	static TSharedRef<IDetailCustomization> MakeInstanceForDiff(TWeakObjectPtr<UBlueprint> InBlueprintPtr)
	{
		return MakeShareable(new FBlueprintGlobalOptionsDetails(InBlueprintPtr));
	}

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailLayout ) override;
	
	/** Gets the Blueprint being edited */
	UBlueprint* GetBlueprintObj() const;

	/** Gets the Blueprint editor */
	TWeakPtr<FBlueprintEditor> GetBlueprintEditorPtr() const {return BlueprintEditorPtr;}

protected:
	/** Gets the Blueprint parent class name text */
	FText GetParentClassName() const;

	// Determine whether or not we should be allowed to reparent (but still display the parent class regardless)
	bool CanReparent() const;

	/** Gets the menu content that's displayed when the parent class combo box is clicked */
	TSharedRef<SWidget>	GetParentClassMenuContent();

	/** Delegate called when a class is selected from the class picker */
	void OnClassPicked(UClass* SelectedClass);

	/** Returns TRUE if the Blueprint can be deprecated */
	bool CanDeprecateBlueprint() const;

	/** Callback when toggling the Deprecate checkbox, handles marking a Blueprint as deprecated */
	void OnDeprecateBlueprint(ECheckBoxState InCheckState);

	/** Callback for Deprecate checkbox, returns checked if the Blueprint is deprecated */
	ECheckBoxState IsDeprecatedBlueprint() const;

	/** Returns the tooltip explaining deprecation */
	FText GetDeprecatedTooltip() const;

	/** Callback for when a new Blueprint namespace value is entered */
	void OnNamespaceValueCommitted(const FString& InNamespace);

	/** Determine Blueprint namespace "reset to default" button visibility */
	bool ShouldShowNamespaceResetToDefault() const;

	/** Invoked when the Blueprint namespace "reset to default" button is clicked */
	void OnNamespaceResetToDefaultValue();

	/** Called when the assigned Blueprint namespace value is changed */
	void HandleNamespaceValueChange(const FString& InOldValue, const FString& InNewValue);

private:
	/** Weak reference to the Blueprint editor */
	TWeakPtr<FBlueprintEditor> BlueprintEditorPtr;
	
	/** Weak reference to a Blueprint override - used to customize a blueprint that is displayed without an editor, e.g. for diff */
	TWeakObjectPtr<UBlueprint> BlueprintObjOverride;

	/** Combo button used to choose a parent class */
	TSharedPtr<SComboButton> ParentClassComboButton;

	/** Handle to the customized namespace property */
	TSharedPtr<IPropertyHandle> NamespacePropertyHandle;

	/** Widget used for namespace value customization */
	TSharedPtr<SBlueprintNamespaceEntry> NamespaceValueWidget;

	/** Desired width for namespace value customization */
	static float NamespacePropertyValueCustomization_MinDesiredWidth;
};



/** Details customization for Blueprint Component settings */
class FBlueprintComponentDetails : public FBlueprintDetails
{
public:
	/** Constructor */
	FBlueprintComponentDetails(TWeakPtr<FBlueprintEditor> InBlueprintEditorPtr)
		: FBlueprintDetails(InBlueprintEditorPtr)
		, BlueprintEditorPtr(InBlueprintEditorPtr)
		, bIsVariableNameInvalid(false)
	{

	}

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IDetailCustomization> MakeInstance(TWeakPtr<FBlueprintEditor> InBlueprintEditorPtr)
	{
		return MakeShareable(new FBlueprintComponentDetails(InBlueprintEditorPtr));
	}

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailLayout ) override;

protected:
	/** Callbacks for widgets */
	FText OnGetVariableText() const;
	void OnVariableTextChanged(const FText& InNewText);
	void OnVariableTextCommitted(const FText& InNewName, ETextCommit::Type InTextCommit);
	FText OnGetTooltipText() const;
	void OnTooltipTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit, FName VarName);
	bool OnVariableCategoryChangeEnabled() const;
	FText OnGetVariableCategoryText() const;
	void OnVariableCategoryTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit, FName VarName);
	void OnVariableCategorySelectionChanged(TSharedPtr<FText> ProposedSelection, ESelectInfo::Type /*SelectInfo*/);
	TSharedRef<ITableRow> MakeVariableCategoryViewWidget(TSharedPtr<FText> Item, const TSharedRef< STableViewBase >& OwnerTable);
	const UClass* GetSelectedEntryClass() const;
	void HandleNewEntryClassSelected(const UClass* NewEntryClass) const;

	FText GetSocketName() const;
	bool CanChangeSocket() const;
	void OnBrowseSocket();
	void OnClearSocket();
	
	void OnSocketSelection( FName SocketName );

	void PopulateVariableCategories();

private:
	/** Weak reference to the Blueprint editor */
	TWeakPtr<FBlueprintEditor> BlueprintEditorPtr;

	/** The cached tree Node we're editing */
	TSharedPtr<FSubobjectEditorTreeNode> CachedNodePtr;

	/** The widget used when in variable name editing mode */ 
	TSharedPtr<SEditableTextBox> VariableNameEditableTextBox;

	/** Flag to indicate whether or not the variable name is invalid */
	bool bIsVariableNameInvalid;

	/** A list of all category names to choose from */
	TArray<TSharedPtr<FText>> VariableCategorySource;

	/** Widgets for the categories */
	TSharedPtr<SComboButton> VariableCategoryComboButton;
	TSharedPtr<SListView<TSharedPtr<FText>>> VariableCategoryListView;
};

/** Details customization for All Graph Nodes */
class FBlueprintGraphNodeDetails : public IDetailCustomization
{
public:
	/** Constructor */
	FBlueprintGraphNodeDetails(TWeakPtr<FBlueprintEditor> InBlueprintEditorPtr)
		: GraphNodePtr(NULL)
		, BlueprintEditorPtr(InBlueprintEditorPtr)
	{

	}

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IDetailCustomization> MakeInstance(TWeakPtr<FBlueprintEditor> InBlueprintEditorPtr)
	{
		return MakeShareable(new FBlueprintGraphNodeDetails(InBlueprintEditorPtr));
	}

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailLayout ) override;

protected:

	/** Returns the currently edited blueprint */
	UBlueprint* GetBlueprintObj() const;

private:

	/** Set error to name textbox */
	void SetNameError( const FText& Error );

	// Callbacks for uproperty details customization
	FText OnGetName() const;
	bool IsNameReadOnly() const;
	void OnNameChanged(const FText& InNewText);
	void OnNameCommitted(const FText& InNewName, ETextCommit::Type InTextCommit);

	/** The widget used when editing a singleline name */ 
	TSharedPtr<SEditableTextBox> NameEditableTextBox;
	/** The widget used when editing a multiline name */ 
	TSharedPtr<SMultiLineEditableTextBox> MultiLineNameEditableTextBox;
	/** The target GraphNode */
	TWeakObjectPtr<UEdGraphNode> GraphNodePtr;
	/** Weak reference to the Blueprint editor */
	TWeakPtr<FBlueprintEditor> BlueprintEditorPtr;
};

/** Details customization for ChildActorComponents */
class FChildActorComponentDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance(TWeakPtr<FBlueprintEditor> BlueprintEditorPtr);

	/** Constructor */
	FChildActorComponentDetails(TWeakPtr<FBlueprintEditor> BlueprintEditorPtr);

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	/** Weak reference to the Blueprint editor */
	TWeakPtr<FBlueprintEditor> BlueprintEditorPtr;
};

/** Details customization for Blueprint Documentation */
class FBlueprintDocumentationDetails : public IDetailCustomization
{
public:
	/** Constructor */
	FBlueprintDocumentationDetails(TWeakPtr<FBlueprintEditor> InBlueprintEditorPtr)
		: BlueprintEditorPtr(InBlueprintEditorPtr)
	{
	}

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IDetailCustomization> MakeInstance(TWeakPtr<FBlueprintEditor> InBlueprintEditorPtr)
	{
		return MakeShareable(new FBlueprintDocumentationDetails(InBlueprintEditorPtr));
	}

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailLayout ) override;

protected:

	/** Accessors passed to parent */
	UBlueprint* GetBlueprintObj() const { return BlueprintEditorPtr.Pin()->GetBlueprintObj(); }

	/** Get the currently selected node from the edgraph */
	TWeakObjectPtr<UEdGraphNode_Documentation> EdGraphSelectionAsDocumentNode();

	/** Accessor for the current nodes documentation link */
	FText OnGetDocumentationLink() const;

	/** Accessor for the nodes current documentation excerpt  */
	FText OnGetDocumentationExcerpt() const;
	
	/** Accessor to evaluate if the current excerpt can be modified */
	bool OnExcerptChangeEnabled() const;

	/** Handler for the documentation link being committed */
	void OnDocumentationLinkCommitted( const FText& InNewName, ETextCommit::Type InTextCommit );
	
	/** Generate table row for excerpt combo */
	TSharedRef< ITableRow > MakeExcerptViewWidget( TSharedPtr<FString> Item, const TSharedRef< STableViewBase >& OwnerTable );
	
	/** Apply selection changes from the excerpt combo */
	void OnExcerptSelectionChanged( TSharedPtr<FString> ProposedSelection, ESelectInfo::Type SelectInfo );

	/** Generate excerpt list widget from documentation page */
	TSharedRef<SWidget> GenerateExcerptList();


private:

	/** Documentation Link */
	FString DocumentationLink;
	/** Current Excerpt */
	FString DocumentationExcerpt;
	/** Weak reference to the Blueprint editor */
	TWeakPtr<FBlueprintEditor> BlueprintEditorPtr;
	/** The editor node we're editing */
	TWeakObjectPtr<UEdGraphNode_Documentation> DocumentationNodePtr;
	/** Excerpt combo widget */
	TSharedPtr<SComboButton> ExcerptComboButton;
	/** Excerpt List */
	TArray<TSharedPtr<FString>> ExcerptList;

};
