// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintDragDropMenuItem.h"

#include "BPDelegateDragDropAction.h"
#include "BPVariableDragDropAction.h"
#include "BlueprintActionMenuItem.h"
#include "BlueprintDelegateNodeSpawner.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintVariableNodeSpawner.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/Casts.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintDragDropMenuItem)

class FDragDropOperation;
class UEdGraphPin;
struct FBlueprintActionContext;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "BlueprintDragDropMenuItem"
DEFINE_LOG_CATEGORY_STATIC(LogBlueprintDragDropMenuItem, Log, All);

//------------------------------------------------------------------------------
FBlueprintDragDropMenuItem::FBlueprintDragDropMenuItem(FBlueprintActionContext const& Context, UBlueprintNodeSpawner const* SampleAction, int32 MenuGrouping, FText InNodeCategory, FText InMenuDesc, FText InToolTip)
: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), MenuGrouping)
{
	AppendAction(SampleAction);
	check(SampleAction != nullptr);
}

//------------------------------------------------------------------------------
UEdGraphNode* FBlueprintDragDropMenuItem::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	// we shouldn't get here (this should just be used for drag/drop ops), but just in case we do...
	FBlueprintActionMenuItem BlueprintActionItem(GetSampleAction());
	return BlueprintActionItem.PerformAction(ParentGraph, FromPin, Location, bSelectNewNode);
}

//------------------------------------------------------------------------------
UEdGraphNode* FBlueprintDragDropMenuItem::PerformAction(UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	UEdGraphPin* FromPin = nullptr;
	if (FromPins.Num() > 0)
	{
		FromPin = FromPins[0];
	}
	
	UEdGraphNode* SpawnedNode = PerformAction(ParentGraph, FromPin, Location, bSelectNewNode);
	// try auto-wiring the rest of the pins (if there are any)
	for (int32 PinIndex = 1; PinIndex < FromPins.Num(); ++PinIndex)
	{
		SpawnedNode->AutowireNewNode(FromPins[PinIndex]);
	}
	
	return SpawnedNode;
}

//------------------------------------------------------------------------------
void FBlueprintDragDropMenuItem::AddReferencedObjects(FReferenceCollector& Collector)
{
	FEdGraphSchemaAction::AddReferencedObjects(Collector);
	
	// these don't get saved to disk, but we want to make sure the objects don't
	// get GC'd while the action array is around
	Collector.AddReferencedObjects(ActionSet);
}

//------------------------------------------------------------------------------
void FBlueprintDragDropMenuItem::AppendAction(UBlueprintNodeSpawner const* Action)
{
	ActionSet.Add(Action);
}

//------------------------------------------------------------------------------
FSlateBrush const* FBlueprintDragDropMenuItem::GetMenuIcon(FSlateColor& ColorOut)
{
	FSlateBrush const* IconBrush = nullptr;
	ColorOut = FSlateColor(FLinearColor::White);

	UBlueprintNodeSpawner const* SampleAction = GetSampleAction();
	if (UBlueprintDelegateNodeSpawner const* DelegateSpawner = Cast<UBlueprintDelegateNodeSpawner const>(SampleAction))
	{
		IconBrush = FAppStyle::GetBrush(TEXT("GraphEditor.Delegate_16x"));
	}
	else if (UBlueprintVariableNodeSpawner const* VariableSpawner = Cast<UBlueprintVariableNodeSpawner const>(SampleAction))
	{
		if (FProperty const* Property = VariableSpawner->GetVarProperty())
		{
			UStruct* const PropertyOwner = Property->GetOwnerChecked<UStruct>();
			// @note(DanO): not sure wht relies on this logic, but discarding extra icon info util
			// I can confirm that containers are a possility here:
			FSlateBrush const* Discarded = nullptr;
			FSlateColor DiscardedColor;
			IconBrush = FBlueprintEditor::GetVarIconAndColor(PropertyOwner, Property->GetFName(), ColorOut, Discarded, DiscardedColor);
		}
	}
	return IconBrush;
}

//------------------------------------------------------------------------------
UBlueprintNodeSpawner const* FBlueprintDragDropMenuItem::GetSampleAction() const
{
	check(ActionSet.Num() > 0);
	return *ActionSet.CreateConstIterator();
}

//------------------------------------------------------------------------------
TSet<UBlueprintNodeSpawner const*> const& FBlueprintDragDropMenuItem::GetActionSet() const
{
	return ObjectPtrDecay(ActionSet);
}

//------------------------------------------------------------------------------
TSharedPtr<FDragDropOperation> FBlueprintDragDropMenuItem::OnDragged(FNodeCreationAnalytic AnalyticsDelegate) const
{
	TSharedPtr<FDragDropOperation> DragDropAction = nullptr;

	UBlueprintNodeSpawner const* SampleAction = GetSampleAction();
	if (UBlueprintDelegateNodeSpawner const* DelegateSpawner = Cast<UBlueprintDelegateNodeSpawner const>(SampleAction))
	{
		if (FMulticastDelegateProperty const* Property = DelegateSpawner->GetDelegateProperty())
		{
			UStruct* const PropertyOwner = Property->GetOwnerChecked<UStruct>();
			TSharedRef<FDragDropOperation> DragDropOpRef = FKismetDelegateDragDropAction::New(nullptr, Property->GetFName(), PropertyOwner, AnalyticsDelegate);
			DragDropAction = TSharedPtr<FDragDropOperation>(DragDropOpRef);
		}
	}
	else if (UBlueprintVariableNodeSpawner const* VariableSpawner = Cast<UBlueprintVariableNodeSpawner const>(SampleAction))
	{
		if (FProperty const* Property = VariableSpawner->GetVarProperty())
		{
			UStruct* const PropertyOwner = Property->GetOwnerChecked<UStruct>();
			TSharedRef<FDragDropOperation> DragDropOpRef = FKismetVariableDragDropAction::New(nullptr, Property->GetFName(), PropertyOwner, AnalyticsDelegate);
			DragDropAction = TSharedPtr<FDragDropOperation>(DragDropOpRef);
		}
		// @TODO: handle local variables as well
	}
	else
	{
		UE_LOGF(LogBlueprintDragDropMenuItem, Warning, "Unhandled spawner type: '%ls'", *SampleAction->GetClass()->GetName());
	}
	return DragDropAction;
}

#undef LOCTEXT_NAMESPACE
