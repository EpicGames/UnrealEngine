// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintEditor.h"
#include "Widgets/Text/STextBlock.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/Engine.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/SimpleConstructionScript.h"
#include "Engine/UserDefinedEnum.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/PackageName.h"
#include "AssetRegistry/AssetData.h"
#include "Editor/EditorEngine.h"
#include "Widgets/Layout/SBorder.h"
#include "HAL/FileManager.h"
#include "Misc/FeedbackContext.h"
#include "UObject/MetaData.h"
#include "EdGraph/EdGraph.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Dialog/SCustomDialog.h"
#include "SCheckBoxList.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphNode_Comment.h"
#include "Editor/UnrealEdEngine.h"
#include "Editor/Transactor.h"
#include "Settings/EditorExperimentalSettings.h"
#include "Settings/BlueprintEditorProjectSettings.h"
#include "GeneralProjectSettings.h"
#include "Kismet/GameplayStatics.h"
#include "Components/TimelineComponent.h"
#include "Engine/LevelStreamingDynamic.h"
#include "Dialogs/Dialogs.h"
#include "UnrealEdGlobals.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "WorkflowOrientedApp/WorkflowUObjectDocuments.h"
#include "EdGraphSchema_K2.h"
#include "K2Node.h"
#include "EdGraphSchema_K2_Actions.h"
#include "K2Node_Event.h"
#include "K2Node_ActorBoundEvent.h"
#include "K2Node_CallFunction.h"
#include "K2Node_Variable.h"
#include "K2Node_CallFunctionOnMember.h"
#include "K2Node_CallParentFunction.h"
#include "K2Node_Tunnel.h"
#include "K2Node_Composite.h"
#include "K2Node_CustomEvent.h"
#include "K2Node_ExecutionSequence.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_FunctionResult.h"
#include "K2Node_Literal.h"
#include "K2Node_MacroInstance.h"
#include "K2Node_Select.h"
#include "K2Node_Switch.h"
#include "K2Node_SwitchInteger.h"
#include "K2Node_SwitchName.h"
#include "K2Node_Timeline.h"
#include "K2Node_VariableGet.h"
#include "K2Node_VariableSet.h"
#include "K2Node_SetFieldsInStruct.h"
#include "K2Node_Knot.h"
#include "Engine/LevelScriptBlueprint.h"
#include "Kismet2/Breakpoint.h"
#include "ScopedTransaction.h"
#include "Kismet2/KismetDebugUtilities.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "KismetCompilerModule.h"
#include "EngineUtils.h"
#include "EdGraphToken.h"
#include "Kismet2/CompilerResultsLog.h"
#include "EdGraphUtilities.h"
#include "IMessageLogListing.h"
#include "Logging/MessageLog.h"
#include "MessageLogModule.h"
#include "Misc/UObjectToken.h"
#include "BlueprintEditorCommands.h"
#include "GraphEditorActions.h"
#include "SNodePanel.h"
#include "Widgets/Docking/SDockTab.h"
#include "EditorClassUtils.h"
#include "IDocumentation.h"
#include "BlueprintNodeBinder.h"
#include "BlueprintFunctionNodeSpawner.h"
#include "SBlueprintEditorToolbar.h"
#include "FindInBlueprints.h"
#include "ImaginaryBlueprintData.h"
#include "SGraphTitleBar.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Kismet2/DebuggerCommands.h"
#include "Editor.h"
#include "IDetailsView.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Stats/StatsHierarchical.h"
#include "BlueprintEditorLibrary.h"
#include "BlueprintNamespaceHelper.h"
#include "BlueprintNamespaceUtilities.h"

#include "BlueprintEditorTabs.h"

#include "ToolMenus.h"
#include "BlueprintEditorContext.h"

#include "Interfaces/IProjectManager.h"
#include "BlueprintDebugger.h"

// Core kismet tabs
#include "SGraphNode.h"
#include "SSubobjectBlueprintEditor.h"
#include "SubobjectDataSubsystem.h"
#include "SSCSEditorViewport.h"
#include "AssetEditorModeManager.h"
#include "SKismetInspector.h"
#include "SBlueprintPalette.h"
#include "SBlueprintBookmarks.h"
#include "SBlueprintActionMenu.h"
#include "SMyBlueprint.h"
#include "SReplaceNodeReferences.h"
// End of core kismet tabs

// Debugging
#include "Debugging/SKismetDebuggingView.h"
// End of debugging

// Misc diagnostics
#include "ObjectTools.h"
// End of misc diagnostics

#include "AssetRegistry/AssetRegistryModule.h"
#include "BlueprintEditorTabFactories.h"
#include "ClassViewerFilter.h"
#include "SPinTypeSelector.h"
#include "Animation/AnimBlueprint.h"
#include "AnimStateConduitNode.h"
#include "AnimationGraphSchema.h"
#include "AnimationGraph.h"
#include "AnimationStateGraph.h"
#include "AnimationStateMachineSchema.h"
#include "AnimationTransitionGraph.h"
#include "BlueprintEditorModes.h"
#include "BlueprintEditorSettings.h"
#include "Settings/EditorProjectSettings.h"
#include "K2Node_SwitchString.h"
#include "AnimGraphNode_StateMachineBase.h"
#include "AnimationStateMachineGraph.h"

#include "EngineAnalytics.h"
#include "AnalyticsEventAttribute.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "SourceCodeNavigation.h"
#include "IHotReload.h"

#include "AudioDevice.h"

#include "SFixupSelfContextDlg.h"

// Blueprint merging
#include "Widgets/Input/SHyperlink.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

// Focusing related nodes feature
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Subsystems/AssetEditorSubsystem.h"

#include "BlendSpaceGraph.h"
#include "AnimationBlendSpaceSampleGraph.h"
#include "SSubobjectEditor.h"
#include "BlueprintActionDatabase.h"
#include "BlueprintEditorSCSEditorUICustomization.h"
#include "EditorModeManager.h"
#include "Algo/MinElement.h"
#include "AssetEditorModeManager.h"
#include "BlueprintEditorEdMode.h"
#include "Editor/EditorEngine.h"
#include "EditorViewportSelectabilityBridge.h"
#include "Elements/Component/ComponentElementSubobjectEditorCommonActionsCustomization.h"
#include "IBlueprintSelectionNotifier.h"
#include "Tools/EdModeInteractiveToolsContext.h"

DEFINE_LOG_CATEGORY_STATIC(LogBlueprintEditor, Log, All);

#define LOCTEXT_NAMESPACE "BlueprintEditor"

static int32 EnableAutomaticLibraryAssetLoading = 1;
static FAutoConsoleVariableRef CVarEnableAutomaticLibraryAssetLoading(
	TEXT("bp.EnableAutomaticLibraryAssetLoading"),
	EnableAutomaticLibraryAssetLoading,
	TEXT("Should opening the BP editor load all macro and function library assets or not?\n0: Disable, 1: Enable (defaults to enabled)\nNodes defined in unloaded libraries will not show up in the context menu!"),
	ECVF_Default);

/////////////////////////////////////////////////////
// FSelectionDetailsSummoner

FSelectionDetailsSummoner::FSelectionDetailsSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp)
	: FWorkflowTabFactory(FBlueprintEditorTabs::DetailsID, InHostingApp)
{
	TabLabel = LOCTEXT("DetailsView_TabTitle", "Details");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("DetailsView_MenuTitle", "Details");
	ViewMenuTooltip = LOCTEXT("DetailsView_ToolTip", "Shows the details view");
}

TSharedRef<SWidget> FSelectionDetailsSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FBlueprintEditor> BlueprintEditorPtr = StaticCastSharedPtr<FBlueprintEditor>(HostingApp.Pin());

	return BlueprintEditorPtr->GetInspector();
}

TSharedRef<SDockTab> FSelectionDetailsSummoner::SpawnTab(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedRef<SDockTab> Tab = FWorkflowTabFactory::SpawnTab(Info);

	TSharedPtr<FBlueprintEditor> BlueprintEditorPtr = StaticCastSharedPtr<FBlueprintEditor>(HostingApp.Pin());
	BlueprintEditorPtr->GetInspector()->SetOwnerTab(Tab);

	BlueprintEditorPtr->GetInspector()->GetPropertyView()->SetHostTabManager(Info.TabManager);

	return Tab;
}


/////////////////////////////////////////////////////
// FBlueprintEditor

namespace BlueprintEditorImpl
{
	static const float InstructionFadeDuration = 0.5f;

	/** Class viewer filter proxy for imported namespace type selectors, controlled by a custom filter option */
	class FImportedClassViewerFilterProxy : public IClassViewerFilter, public TSharedFromThis<FImportedClassViewerFilterProxy>
	{
	public:
		FImportedClassViewerFilterProxy(TSharedPtr<IClassViewerFilter> InClassViewerFilter)
			: ClassViewerFilter(InClassViewerFilter)
			, bIsFilterEnabled(true)
		{
		}

		// IClassViewerFilter interface
		virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
		{
			if (!GetDefault<UBlueprintEditorSettings>()->bEnableNamespaceFilteringFeatures)
			{
				return true;
			}

			if (bIsFilterEnabled && ClassViewerFilter.IsValid())
			{
				return ClassViewerFilter->IsClassAllowed(InInitOptions, InClass, InFilterFuncs);
			}

			return true;
		}

		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InBlueprint, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
		{
			if (!GetDefault<UBlueprintEditorSettings>()->bEnableNamespaceFilteringFeatures)
			{
				return true;
			}

			if (bIsFilterEnabled && ClassViewerFilter.IsValid())
			{
				return ClassViewerFilter->IsUnloadedClassAllowed(InInitOptions, InBlueprint, InFilterFuncs);
			}

			return true;
		}

		virtual void GetFilterOptions(TArray<TSharedRef<FClassViewerFilterOption>>& OutFilterOptions)
		{
			if (!GetDefault<UBlueprintEditorSettings>()->bEnableNamespaceFilteringFeatures)
			{
				return;
			}

			if (!ToggleFilterOption.IsValid())
			{
				ToggleFilterOption = MakeShared<FClassViewerFilterOption>();
				ToggleFilterOption->bEnabled = bIsFilterEnabled;
				ToggleFilterOption->LabelText = LOCTEXT("ClassViewerNamespaceFilterMenuOptionLabel", "Show Only Imported Types");
				ToggleFilterOption->ToolTipText = LOCTEXT("ClassViewerNamespaceFilterMenuOptionToolTip", "Don't include non-imported class types.");
				ToggleFilterOption->OnOptionChanged = FOnClassViewerFilterOptionChanged::CreateSP(this, &FImportedClassViewerFilterProxy::OnFilterOptionChanged);
			}

			OutFilterOptions.Add(ToggleFilterOption.ToSharedRef());
		}

	protected:
		void OnFilterOptionChanged(bool bIsEnabled)
		{
			bIsFilterEnabled = bIsEnabled;
		}

	private:
		/** Imported namespace class viewer filter. */
		TSharedPtr<IClassViewerFilter> ClassViewerFilter;

		/** Filter option for the class viewer settings menu. */
		TSharedPtr<FClassViewerFilterOption> ToggleFilterOption;

		/** Whether or not the filter is enabled. */
		bool bIsFilterEnabled;
	};

	/** Pin type filter proxy for imported namespace type selectors, controlled by a custom filter option */
	class FImportedPinTypeSelectorFilterProxy : public IPinTypeSelectorFilter, public TSharedFromThis<FImportedPinTypeSelectorFilterProxy>
	{
		DECLARE_MULTICAST_DELEGATE(FOnFilterChanged);
	
	public:
		FImportedPinTypeSelectorFilterProxy(TSharedPtr<IPinTypeSelectorFilter> InPinTypeSelectorFilter)
			: PinTypeSelectorFilter(InPinTypeSelectorFilter)
			, bIsFilterEnabled(true)
		{
		}

		// IPinTypeSelectorFilter interface
		virtual bool ShouldShowPinTypeTreeItem(FPinTypeTreeItem InItem) const override
		{
			if (!GetDefault<UBlueprintEditorSettings>()->bEnableNamespaceFilteringFeatures)
			{
				return true;
			}

			if (bIsFilterEnabled && PinTypeSelectorFilter.IsValid())
			{
				return PinTypeSelectorFilter->ShouldShowPinTypeTreeItem(InItem);
			}

			return true;
		}

		virtual FDelegateHandle RegisterOnFilterChanged(FSimpleDelegate InOnFilterChanged) override
		{
			return OnFilterChanged.Add(InOnFilterChanged);
		}

		virtual void UnregisterOnFilterChanged(FDelegateHandle InDelegateHandle) override
		{
			OnFilterChanged.Remove(InDelegateHandle);
		}

		virtual TSharedPtr<SWidget> GetFilterOptionsWidget() override
		{
			if (!GetDefault<UBlueprintEditorSettings>()->bEnableNamespaceFilteringFeatures)
			{
				return nullptr;
			}

			if (!FilterOptionsWidget.IsValid())
			{
				SAssignNew(FilterOptionsWidget, SCheckBox)
					.IsChecked(this, &FImportedPinTypeSelectorFilterProxy::IsFilterToggleChecked)
					.OnCheckStateChanged(this, &FImportedPinTypeSelectorFilterProxy::OnToggleFilter)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("PinTypeNamespaceFilterToggleOptionLabel", "Hide Non-Imported Types"))
					];
			}

			return FilterOptionsWidget;
		}

	protected:
		ECheckBoxState IsFilterToggleChecked() const
		{
			return bIsFilterEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}

		void OnToggleFilter(ECheckBoxState NewState)
		{
			bIsFilterEnabled = (NewState == ECheckBoxState::Checked);

			// Notify any listeners that the filter has been changed.
			OnFilterChanged.Broadcast();
		}

	private:
		/** Imported namespace pin type selector filter. */
		TSharedPtr<IPinTypeSelectorFilter> PinTypeSelectorFilter;

		/** Cached filter options widget. */
		TSharedPtr<SWidget> FilterOptionsWidget;

		/** Delegate that's called whenever filter options are changed. */
		FOnFilterChanged OnFilterChanged;

		/** Whether or not the filter is enabled. */
		bool bIsFilterEnabled;
	};

	/** Pin type filter for asset permissions */
	class FPermissionsPinTypeSelectorFilter : public IPinTypeSelectorFilter, public TSharedFromThis<FPermissionsPinTypeSelectorFilter>
	{
	public:
		FPermissionsPinTypeSelectorFilter(TConstArrayView<UBlueprint*> InBlueprints)
		{
			FAssetReferenceFilterContext Context;
			for (UBlueprint* Blueprint : InBlueprints)
			{
				Context.AddReferencingAsset(FAssetData(Blueprint));
			}

			AssetReferenceFilter = GEditor->MakeAssetReferenceFilter(Context);
		}

		// IPinTypeSelectorFilter interface
		virtual bool ShouldShowPinTypeTreeItem(FPinTypeTreeItem InItem) const override
		{
			FTopLevelAssetPath TopLevelAssetPath;
			const FAssetData& AssetData = InItem->GetCachedAssetData();
			if (AssetData.IsValid())
			{
				TopLevelAssetPath = FTopLevelAssetPath(AssetData.PackageName, AssetData.AssetName);
			}

			// First check pin type permissions
			if (!FBlueprintActionDatabase::IsPinTypeAllowed(InItem->GetPinTypeNoResolve(), TopLevelAssetPath))
			{
				return false;
			}

			// Then asset permissions
			if(AssetReferenceFilter.IsValid() && AssetData.IsValid())
			{
				if (!AssetReferenceFilter->PassesFilter(AssetData))
				{
					return false;
				}
			}

			return true;
		}

	private:
		/** Filter for asset references */
		TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter;
	};

	/**
	 * Utility function that will check to see if the specified graph has any 
	 * nodes other than those that come default, pre-placed, in the graph.
	 *
	 * @param  InGraph  The graph to check.
	 * @return True if the graph has any nodes added by the user, otherwise false.
	 */
	static bool GraphHasUserPlacedNodes(UEdGraph const* InGraph);

	/**
	 * Utility function that will check to see if the specified graph has any
	 * nodes that were default, pre-placed, in the graph.
	 *
	 * @param  InGraph  The graph to check.
	 * @return True if the graph has any pre-placed nodes, otherwise false.
	 */
	static bool GraphHasDefaultNode(UEdGraph const* InGraph);

	/**
	 * Utility function that will set the global save-on-compile setting to the
	 * specified value.
	 * 
	 * @param  NewSetting	The new save-on-compile setting that you want applied.
	 */
	static void SetSaveOnCompileSetting(ESaveOnCompile NewSetting);

	/**
	 * Utility function used to determine what save-on-compile setting should be 
	 * presented to the user.
	 * 
	 * @param  Editor	The editor currently querying for the setting value.
	 * @param  Option	The setting to check for.
	 * @return False if the option isn't set, or if the save-on-compile is disabled for the blueprint being edited (otherwise true). 
	 */
	static bool IsSaveOnCompileOptionSet(TWeakPtr<FBlueprintEditor> Editor, ESaveOnCompile Option);

	/**  Flips the value of the editor's "JumpToNodeErrors" setting. */
	static void ToggleJumpToErrorNodeSetting();

	/**
	 * Utility function that will check to see if the "Jump to Error Nodes" 
	 * setting is enabled.
	 * 
	 * @return True if UBlueprintEditorSettings::bJumpToNodeErrors is set, otherwise false.
	 */
	static bool IsJumpToErrorNodeOptionSet();

	/**
	 * Searches through a blueprint, looking for the most severe error'ing node.
	 * 
	 * @param  Blueprint	The blueprint to search through.
	 * @param  Severity		Defines the severity of the error/warning to search for.
	 * @return The first node found with the specified error.
	 */
	static UEdGraphNode* FindNodeWithError(UBlueprint* Blueprint, EMessageSeverity::Type Severity = EMessageSeverity::Error);

	/**
	 * Searches through an error log, looking for the most severe error'ing node.
	 * 
	 * @param  ErrorLog		The error log you want to search through.
	 * @param  Severity		Defines the severity of the error/warning to search for.
	 * @return The first node found with the specified error.
	 */
	static UEdGraphNode* FindNodeWithError(FCompilerResultsLog const& ErrorLog, EMessageSeverity::Type Severity = EMessageSeverity::Error);
}

static bool BlueprintEditorImpl::GraphHasUserPlacedNodes(UEdGraph const* InGraph)
{
	bool bHasUserPlacedNodes = false;

	for (UEdGraphNode const* Node : InGraph->Nodes)
	{
		if (Node == nullptr)
		{
			continue;
		}

		if (!Node->GetPackage()->GetMetaData().HasValue(Node, FNodeMetadata::DefaultGraphNode))
		{
			bHasUserPlacedNodes = true;
			break;
		}
	}

	return bHasUserPlacedNodes;
}

static bool BlueprintEditorImpl::GraphHasDefaultNode(UEdGraph const* InGraph)
{
	bool bHasDefaultNodes = false;

	for (UEdGraphNode const* Node : InGraph->Nodes)
	{
		if (Node == nullptr)
		{
			continue;
		}

		if (Node->GetPackage()->GetMetaData().HasValue(Node, FNodeMetadata::DefaultGraphNode) && Node->IsNodeEnabled())
		{
			bHasDefaultNodes = true;
			break;
		}
	}

	return bHasDefaultNodes;
}

static void BlueprintEditorImpl::SetSaveOnCompileSetting(ESaveOnCompile NewSetting)
{
	UBlueprintEditorSettings* Settings = GetMutableDefault<UBlueprintEditorSettings>();
	Settings->SaveOnCompile = NewSetting;
	Settings->SaveConfig();
}

static bool BlueprintEditorImpl::IsSaveOnCompileOptionSet(TWeakPtr<FBlueprintEditor> Editor, ESaveOnCompile Option)
{
	const UBlueprintEditorSettings* Settings = GetDefault<UBlueprintEditorSettings>();

	ESaveOnCompile CurrentSetting = Settings->SaveOnCompile;
	if (!Editor.IsValid() || !Editor.Pin()->IsSaveOnCompileEnabled())
	{
		// if save-on-compile is disabled for the blueprint, then we want to 
		// show "Never" as being selected
		// 
		// @TODO: a tooltip explaining why would be nice too
		CurrentSetting = SoC_Never;
	}

	return (CurrentSetting == Option);
}

static void BlueprintEditorImpl::ToggleJumpToErrorNodeSetting()
{
	UBlueprintEditorSettings* Settings = GetMutableDefault<UBlueprintEditorSettings>();
	Settings->bJumpToNodeErrors = !Settings->bJumpToNodeErrors;
	Settings->SaveConfig();
}

static bool BlueprintEditorImpl::IsJumpToErrorNodeOptionSet()
{
	const UBlueprintEditorSettings* Settings = GetDefault<UBlueprintEditorSettings>();
	return Settings->bJumpToNodeErrors;
}

static UEdGraphNode* BlueprintEditorImpl::FindNodeWithError(UBlueprint* Blueprint, EMessageSeverity::Type Severity/* = EMessageSeverity::Error*/)
{
	TArray<UEdGraph*> Graphs;
	Blueprint->GetAllGraphs(Graphs);

	UEdGraphNode* ChoiceNode = nullptr;
	for (UEdGraph* Graph : Graphs)
	{
		for (UEdGraphNode* Node : Graph->Nodes)
		{
			if (Node && Node->bHasCompilerMessage && !Node->ErrorMsg.IsEmpty() && (Node->ErrorType <= Severity))
			{
				if ((ChoiceNode == nullptr) || (ChoiceNode->ErrorType > Node->ErrorType))
				{
					ChoiceNode = Node;
					if (ChoiceNode->ErrorType == 0)
					{
						break;
					}
				}
			}
		}
	}
	return ChoiceNode;
}

static UEdGraphNode* BlueprintEditorImpl::FindNodeWithError(FCompilerResultsLog const& ErrorLog, EMessageSeverity::Type Severity/* = EMessageSeverity::Error*/)
{
	UEdGraphNode* ChoiceNode = nullptr;
	for (TWeakObjectPtr<UEdGraphNode> NodePtr : ErrorLog.AnnotatedNodes)
	{
		UEdGraphNode* Node = NodePtr.Get();
		if ((Node != nullptr) && (Node->ErrorType <= Severity))
		{
			if ((ChoiceNode == nullptr) || (Node->ErrorType < ChoiceNode->ErrorType))
			{
				ChoiceNode = Node;
				if (ChoiceNode->ErrorType == 0)
				{
					break;
				}
			}
		}
	}

	return ChoiceNode;
}


FName FBlueprintEditor::SelectionState_MyBlueprint(TEXT("MyBlueprint"));
FName FBlueprintEditor::SelectionState_Components(TEXT("Components"));
FName FBlueprintEditor::SelectionState_Graph(TEXT("Graph"));
FName FBlueprintEditor::SelectionState_ClassSettings(TEXT("ClassSettings"));
FName FBlueprintEditor::SelectionState_ClassDefaults(TEXT("ClassDefaults"));


bool FBlueprintEditor::IsASubGraph( const UEdGraph* GraphPtr )
{
	if( GraphPtr && GraphPtr->GetOuter() )
	{
		//Check whether the outer is a composite node
		if( GraphPtr->GetOuter()->IsA( UK2Node_Composite::StaticClass() ) )
		{
			return true;
		}
	}
	return false;
}

/** Util for finding a glyph for a graph */
const FSlateBrush* FBlueprintEditor::GetGlyphForGraph(const UEdGraph* Graph, bool bInLargeIcon)
{
	const FSlateBrush* ReturnValue = FAppStyle::GetBrush( bInLargeIcon ? TEXT("GraphEditor.Function_24x") : TEXT("GraphEditor.Function_16x") );

	check(Graph != nullptr);
	const UEdGraphSchema* Schema = Graph->GetSchema();
	if (Schema != nullptr)
	{
		const EGraphType GraphType = Schema->GetGraphType(Graph);
		switch (GraphType)
		{
		default:
		case GT_StateMachine:
			{
				ReturnValue = FAppStyle::GetBrush( bInLargeIcon ? TEXT("GraphEditor.StateMachine_24x") : TEXT("GraphEditor.StateMachine_16x") );
			}
			break;
		case GT_Function:
			{
				if ( Graph->IsA(UAnimationTransitionGraph::StaticClass()) )
				{
					UObject* GraphOuter = Graph->GetOuter();
					if ( GraphOuter != nullptr && GraphOuter->IsA(UAnimStateConduitNode::StaticClass()) )
					{
						ReturnValue = FAppStyle::GetBrush( bInLargeIcon ? TEXT("GraphEditor.Conduit_24x") : TEXT("GraphEditor.Conduit_16x") );
					}
					else
					{
						ReturnValue = FAppStyle::GetBrush( bInLargeIcon ? TEXT("GraphEditor.Rule_24x") : TEXT("GraphEditor.Rule_16x") );
					}
				}
				else
				{
					//Check for subgraph
					if( IsASubGraph( Graph ) )
					{
						ReturnValue = FAppStyle::GetBrush( bInLargeIcon ? TEXT("GraphEditor.SubGraph_24x") : TEXT("GraphEditor.SubGraph_16x") );
					}
					else
					{
						ReturnValue = FAppStyle::GetBrush( bInLargeIcon ? TEXT("GraphEditor.Function_24x") : TEXT("GraphEditor.Function_16x") );
					}
				}
			}
			break;
		case GT_Macro:
			{
				ReturnValue = FAppStyle::GetBrush( bInLargeIcon ? TEXT("GraphEditor.Macro_24x") : TEXT("GraphEditor.Macro_16x") );
			}
			break;
		case GT_Ubergraph:
			{
				ReturnValue = FAppStyle::GetBrush( bInLargeIcon ? TEXT("GraphEditor.EventGraph_24x") : TEXT("GraphEditor.EventGraph_16x") );
			}
			break;
		case GT_Animation:
			{
				if ( Graph->IsA(UAnimationStateGraph::StaticClass()) )
				{
					ReturnValue = FAppStyle::GetBrush( bInLargeIcon ? TEXT("GraphEditor.State_24x") : TEXT("GraphEditor.State_16x") );
				}
				else if ( Graph->IsA(UBlendSpaceGraph::StaticClass()) )
				{
					ReturnValue = FAppStyle::GetBrush(TEXT("BlendSpace.Graph") );
				}
				else if ( Graph->IsA(UAnimationBlendSpaceSampleGraph::StaticClass()) )
				{
					ReturnValue = FAppStyle::GetBrush(TEXT("BlendSpace.SampleGraph") );
				}
				else
				{
					// If it has overriden an interface, show it as a function
					if ( Graph->InterfaceGuid.IsValid() )
					{ 
						ReturnValue = FAppStyle::GetBrush(bInLargeIcon ? TEXT("GraphEditor.Function_24x") : TEXT("GraphEditor.Function_16x"));
					}
					else
					{
						ReturnValue = FAppStyle::GetBrush(bInLargeIcon ? TEXT("GraphEditor.Animation_24x") : TEXT("GraphEditor.Animation_16x"));	
					}
				}
			}
		}
	}

	return ReturnValue;
}

FSlateBrush const* FBlueprintEditor::GetVarIconAndColor(const UStruct* VarScope, FName VarName, FSlateColor& IconColorOut, FSlateBrush const*& SecondaryBrushOut, FSlateColor& SecondaryColorOut)
{
	SecondaryBrushOut = nullptr;
	if (VarScope != nullptr)
	{
		FProperty* Property = FindFProperty<FProperty>(VarScope, VarName);
		return GetVarIconAndColorFromProperty(Property, IconColorOut, SecondaryBrushOut, SecondaryColorOut);
	}
	return FAppStyle::GetBrush(TEXT("Kismet.AllClasses.VariableIcon"));
}

FSlateBrush const* FBlueprintEditor::GetVarIconAndColorFromProperty(const FProperty* Property, FSlateColor& IconColorOut, FSlateBrush const*& SecondaryBrushOut, FSlateColor& SecondaryColorOut)
{
	SecondaryBrushOut = nullptr;
	if (Property != nullptr)
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

		FEdGraphPinType PinType;
		if (K2Schema->ConvertPropertyToPinType(Property, PinType)) // use schema to get the color
		{
			return GetVarIconAndColorFromPinType(PinType, IconColorOut, SecondaryBrushOut, SecondaryColorOut);
		}
	}
	return FAppStyle::GetBrush(TEXT("Kismet.AllClasses.VariableIcon"));
}

FSlateBrush const* FBlueprintEditor::GetVarIconAndColorFromPinType(const FEdGraphPinType& PinType,
	FSlateColor& IconColorOut, FSlateBrush const*& SecondaryBrushOut, FSlateColor& SecondaryColorOut)
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	IconColorOut = K2Schema->GetPinTypeColor(PinType);
	SecondaryBrushOut = FBlueprintEditorUtils::GetSecondaryIconFromPin(PinType);
	SecondaryColorOut = K2Schema->GetSecondaryPinTypeColor(PinType);
	return FBlueprintEditorUtils::GetIconFromPin(PinType);
}

bool FBlueprintEditor::IsInAScriptingMode() const
{
	return IsModeCurrent(FBlueprintEditorApplicationModes::StandardBlueprintEditorMode) || IsModeCurrent(FBlueprintEditorApplicationModes::BlueprintMacroMode);
}

void FBlueprintEditor::CreateEditorModeManager()
{
	EditorModeManager = MakeShared<FAssetEditorModeManager>();
}

bool FBlueprintEditor::OnRequestClose(EAssetEditorCloseReason InCloseReason)
{
	return FWorkflowCentricApplication::OnRequestClose(InCloseReason);
}

void FBlueprintEditor::OnClose()
{
	UTypedElementSelectionSet* SelectionSet = GetEditorModeManager().GetEditorSelectionSet();
	const bool bHasSelectionSet = SelectionSet != nullptr;
	if (bHasSelectionSet)
	{
		SelectionSet->ClearSelection(FTypedElementSelectionOptions());
	}

	// Also close the Find Results tab if we're not in full edit mode.
	TSharedPtr<SDockTab> FindResultsTab = TabManager->FindExistingLiveTab(FBlueprintEditorTabs::FindResultsID);
	if (FindResultsTab.IsValid() && !IsInAScriptingMode())
	{
		FindResultsTab->RequestCloseTab();
	}

	// Close the Replace References Tab so it doesn't open the next time we do
	TSharedPtr<SDockTab> ReplaceRefsTab = TabManager->FindExistingLiveTab(FBlueprintEditorTabs::ReplaceNodeReferencesID);
	if (ReplaceRefsTab.IsValid())
	{
		ReplaceRefsTab->RequestCloseTab();
	}

	bEditorMarkedAsClosed = true;

	FWorkflowCentricApplication::OnClose();
}

void FBlueprintEditor::AddViewportOverlayWidget(TSharedRef<SWidget> InViewportOverlayWidget, int32 ZOrder)
{
	if (SubobjectViewport.IsValid())
	{
		SubobjectViewport->AddOverlayWidget(InViewportOverlayWidget, ZOrder);
	}
}

void FBlueprintEditor::RemoveViewportOverlayWidget(TSharedRef<SWidget> InViewportOverlayWidget)
{
	if (SubobjectViewport.IsValid())
	{
		SubobjectViewport->RemoveOverlayWidget(InViewportOverlayWidget);
	}
}

bool FBlueprintEditor::InEditingMode() const
{
	UBlueprint* Blueprint = GetBlueprintObj();
	if (!FSlateApplication::Get().InKismetDebuggingMode())
	{
		if (!IsPlayInEditorActive())
		{
			return true;
		}
		else
		{
			if (Blueprint)
			{
				if (Blueprint->CanAlwaysRecompileWhilePlayingInEditor())
				{
					return true;
				}
				else
				{
					if (ModifyDuringPIEStatus == ESafeToModifyDuringPIEStatus::Unknown)
					{
						// Check for project settings or editor preferences that allow this anyways
						ModifyDuringPIEStatus = ESafeToModifyDuringPIEStatus::Safe;

						auto CheckClassList = [](UClass* TestClass, const TArray<TSoftClassPtr<UObject>>& ClassList)
						{
							for (const TSoftClassPtr<UObject>& SoftBaseClass : ClassList)
							{
								// Safe to call Get() instead of doing a load, as we can't possibly be derived from an unloaded class
								if (UClass* BaseClass = SoftBaseClass.Get())
								{
									if (TestClass->IsChildOf(BaseClass))
									{
										return true;
									}
								}
							}
							return false;
						};

						UClass* TestClass = Blueprint->GeneratedClass;
						if(!TestClass)
						{
							// fall back to user parent class if there's no generated class,
							// this could be the case for newly created blueprints
							TestClass = Blueprint->ParentClass;
						}
						if (TestClass)
						{
							if (CheckClassList(TestClass, GetDefault<UBlueprintEditorSettings>()->BaseClassesToDisallowRecompilingDuringPlayInEditor) ||
								CheckClassList(TestClass, GetDefault<UBlueprintEditorProjectSettings>()->BaseClassesToDisallowRecompilingDuringPlayInEditor))
							{
								ModifyDuringPIEStatus = ESafeToModifyDuringPIEStatus::NotSafe;
							}
						}
					}

					return ModifyDuringPIEStatus == ESafeToModifyDuringPIEStatus::Safe;
				}
			}
		}
	}

	return false;
}

bool FBlueprintEditor::IsCompilingEnabled() const
{
	UBlueprint* Blueprint = GetBlueprintObj();
	return Blueprint && Blueprint->BlueprintType != BPTYPE_MacroLibrary && InEditingMode();
}

bool FBlueprintEditor::IsPlayInEditorActive() const
{
	return GEditor->PlayWorld != nullptr;
}

EVisibility FBlueprintEditor::IsDebuggerVisible() const
{
	return IsPlayInEditorActive() ? EVisibility::Visible : EVisibility::Collapsed;
}

int32 FBlueprintEditor::GetNumberOfSelectedNodes() const
{
	return GetSelectedNodes().Num();
}

FGraphPanelSelectionSet FBlueprintEditor::GetSelectedNodes() const
{
	FGraphPanelSelectionSet CurrentSelection;
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		CurrentSelection = FocusedGraphEd->GetSelectedNodes();
	}
	return CurrentSelection;
}

UEdGraphNode* FBlueprintEditor::GetSingleSelectedNode() const
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	return (SelectedNodes.Num() == 1) ? Cast<UEdGraphNode>(*SelectedNodes.CreateConstIterator()) : nullptr;
}

void FBlueprintEditor::AnalyticsTrackNodeEvent(UBlueprint* Blueprint, UEdGraphNode *GraphNode, bool bNodeDelete) const
{
	if(Blueprint && GraphNode && FEngineAnalytics::IsAvailable())
	{
		// we'd like to see if this was happening in normal blueprint editor or persona 
		const FString EditorName = Cast<UAnimBlueprint>(Blueprint) != nullptr ? TEXT("Persona") : TEXT("BlueprintEditor");

		// Build Node Details
		const UGeneralProjectSettings& ProjectSettings = *GetDefault<UGeneralProjectSettings>();
		FString ProjectID = ProjectSettings.ProjectID.ToString();
		TArray<FAnalyticsEventAttribute> NodeAttributes;
		NodeAttributes.Add(FAnalyticsEventAttribute(TEXT("ProjectId"), ProjectID));
		NodeAttributes.Add(FAnalyticsEventAttribute(TEXT("BlueprintId"), Blueprint->GetBlueprintGuid().ToString()));
		TArray<TKeyValuePair<FString, FString>> Attributes;

		if (UK2Node* K2Node = Cast<UK2Node>(GraphNode))
		{
			K2Node->GetNodeAttributes(Attributes);
		}
		else if (UEdGraphNode_Comment* CommentNode = Cast<UEdGraphNode_Comment>(GraphNode))
		{
			Attributes.Add(TKeyValuePair<FString, FString>(TEXT("Type"), TEXT("Comment")));
			Attributes.Add(TKeyValuePair<FString, FString>(TEXT("Class"), CommentNode->GetClass()->GetName()));
			Attributes.Add(TKeyValuePair<FString, FString>(TEXT("Name"), CommentNode->GetName()));
		}
		if (Attributes.Num() > 0)
		{
			// Build Node Attributes
			for (const TKeyValuePair<FString, FString>& Attribute : Attributes)
			{
				NodeAttributes.Add(FAnalyticsEventAttribute(Attribute.Key, Attribute.Value));
			}
			// Send Analytics event 
			FString EventType = bNodeDelete ?	FString::Printf(TEXT("Editor.Usage.%s.NodeDeleted"), *EditorName) :
												FString::Printf(TEXT("Editor.Usage.%s.NodeCreated"), *EditorName);
			FEngineAnalytics::GetProvider().RecordEvent(EventType, NodeAttributes);
		}
	}
}

void FBlueprintEditor::AnalyticsTrackCompileEvent(UBlueprint* Blueprint, int32 NumErrors, int32 NumWarnings) const
{
	if (Blueprint && FEngineAnalytics::IsAvailable())
	{
		// we'd like to see if this was happening in normal blueprint editor or persona 
		const FString EditorName = Cast<UAnimBlueprint>(Blueprint) != nullptr ? TEXT("Persona") : TEXT("BlueprintEditor");

		// Build Node Details
		const UGeneralProjectSettings& ProjectSettings = *GetDefault<UGeneralProjectSettings>();
		FString ProjectID = ProjectSettings.ProjectID.ToString();

		const bool bSuccess = NumErrors == 0;
		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Add(FAnalyticsEventAttribute(TEXT("ProjectId"), ProjectID));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("BlueprintId"), Blueprint->GetBlueprintGuid().ToString()));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Success"), bSuccess? TEXT("True") : TEXT("False")));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("NumErrors"), FString::FromInt(NumErrors)));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("NumWarnings"), FString::FromInt(NumWarnings)));

		// Send Analytics event 
		FEngineAnalytics::GetProvider().RecordEvent(FString::Printf(TEXT("Editor.Usage.%s.Compile"), *EditorName), Attributes);
	}
}

void FBlueprintEditor::GetPinTypeSelectorFilters(TArray<TSharedPtr<IPinTypeSelectorFilter>>& OutFilters) const
{ 
	OutFilters.Add(ImportedPinTypeSelectorFilter);
	OutFilters.Add(PermissionsPinTypeSelectorFilter);
}

void FBlueprintEditor::RefreshEditors(ERefreshBlueprintEditorReason::Type Reason)
{
	bool bForceFocusOnSelectedNodes = false;

	if (CurrentUISelection == SelectionState_MyBlueprint)
	{
		// Handled below, here to avoid tripping the ensure
	}
	else if (CurrentUISelection == SelectionState_Components)
	{
		if(SubobjectEditor.IsValid())
		{
			SubobjectEditor->RefreshSelectionDetails();
		}
	}
	else if (CurrentUISelection == SelectionState_Graph)
	{
		bForceFocusOnSelectedNodes = true;
	}
	else if (CurrentUISelection == SelectionState_ClassSettings)
	{
		// No need for a refresh, the Blueprint object didn't change
	}
	else if (CurrentUISelection == SelectionState_ClassDefaults)
	{
		StartEditingDefaults(/*bAutoFocus=*/ false, true);
	}

	// Remove any tabs are that are pending kill or otherwise invalid UObject pointers.
	DocumentManager->CleanInvalidTabs();

	//@TODO: Should determine when we need to do the invalid/refresh business and if the graph node selection change
	// under non-compiles is necessary (except when the selection mode is appropriate, as already detected above)
	if (Reason != ERefreshBlueprintEditorReason::BlueprintCompiled)
	{
		DocumentManager->RefreshAllTabs();

		bForceFocusOnSelectedNodes = true;
	}

	if (bForceFocusOnSelectedNodes)
	{
		FocusInspectorOnGraphSelection(GetSelectedNodes(), /*bForceRefresh=*/ true);
	}

	if (ReplaceReferencesWidget.IsValid())
	{
		ReplaceReferencesWidget->Refresh();
	}

	if (MyBlueprintWidget.IsValid())
	{
		MyBlueprintWidget->Refresh();
	}

	if(SubobjectEditor.IsValid())
	{
		SubobjectEditor->RefreshComponentTypesList();
		SubobjectEditor->UpdateTree();

		// Note: Don't pass 'true' here because we don't want the preview actor to be reconstructed until after Blueprint modification is complete.
		UpdateSubobjectPreview();
	}

	if (BookmarksWidget.IsValid())
	{
		BookmarksWidget->RefreshBookmarksTree();
	}

	// Note: There is an optimization inside of ShowDetailsForSingleObject() that skips the refresh if the object being selected is the same as the previous object.
	// The SKismetInspector class is shared between both Defaults mode and Components mode, but in Defaults mode the object selected is always going to be the CDO. Given
	// that the selection does not really change, we force it to refresh and skip the optimization. Otherwise, some things may not work correctly in Defaults mode. For
	// example, transform details are customized and the rotation value is cached at customization time; if we don't force refresh here, then after an undo of a previous
	// rotation edit, transform details won't be re-customized and thus the cached rotation value will be stale, resulting in an invalid rotation value on the next edit.
	//@TODO: Probably not always necessary
	RefreshStandAloneDefaultsEditor();

	// Update associated controls like the function editor
	BroadcastRefresh();
}

void FBlueprintEditor::RefreshMyBlueprint()
{
	if (MyBlueprintWidget.IsValid())
	{
		MyBlueprintWidget->Refresh();
	}
}

void FBlueprintEditor::RefreshInspector()
{
	if (Inspector.IsValid())
	{
		Inspector->GetPropertyView()->ForceRefresh();
	}
}

void FBlueprintEditor::SetUISelectionState(FName SelectionOwner)
{
	if ( SelectionOwner != CurrentUISelection )
	{
		ClearSelectionStateFor(CurrentUISelection);

		CurrentUISelection = SelectionOwner;
	}
}

void FBlueprintEditor::AddToSelection(UEdGraphNode* InNode)
{
	FocusedGraphEdPtr.Pin()->SetNodeSelection(InNode, true);
}

void FBlueprintEditor::ClearSelectionStateFor(FName SelectionOwner)
{
	if ( SelectionOwner == SelectionState_Graph )
	{
		TArray< TSharedPtr<SDockTab> > GraphEditorTabs;
		DocumentManager->FindAllTabsForFactory(GraphEditorTabFactoryPtr, /*out*/ GraphEditorTabs);

		for (TSharedPtr<SDockTab>& GraphEditorTab : GraphEditorTabs)
		{
			TSharedRef<SGraphEditor> Editor = StaticCastSharedRef<SGraphEditor>((GraphEditorTab)->GetContent());

			Editor->ClearSelectionSet();
		}
	}
	else if ( SelectionOwner == SelectionState_Components )
	{	
		if ( SubobjectEditor.IsValid() )
		{
			UTypedElementSelectionSet* SelectionSet = GetEditorModeManager().GetEditorSelectionSet();
			const bool bHasSelectionSet = SelectionSet != nullptr;
			if (bHasSelectionSet)
			{
				SelectionSet->ClearSelection(FTypedElementSelectionOptions());
			}

			SubobjectEditor->ClearSelection();
		}
	}
	else if ( SelectionOwner == SelectionState_MyBlueprint )
	{
		if ( MyBlueprintWidget.IsValid() )
		{
			MyBlueprintWidget->ClearGraphActionMenuSelection();
		}
	}
}

void FBlueprintEditor::SummonSearchUI(bool bSetFindWithinBlueprint, FString NewSearchTerms, bool bSelectFirstResult)
{
	TSharedPtr<SFindInBlueprints> FindResultsToUse;

	if (bSetFindWithinBlueprint)
	{
		FindResultsToUse = FindResults;
		TabManager->TryInvokeTab(FBlueprintEditorTabs::FindResultsID);
	}
	else
	{
		FindResultsToUse = FFindInBlueprintSearchManager::Get().GetGlobalFindResults();
	}

	if (FindResultsToUse.IsValid())
	{
		FindResultsToUse->FocusForUse(bSetFindWithinBlueprint, NewSearchTerms, bSelectFirstResult);
	}
}

void FBlueprintEditor::SummonFindAndReplaceUI()
{
	TabManager->TryInvokeTab(FBlueprintEditorTabs::ReplaceNodeReferencesID);
}

void FBlueprintEditor::EnableSubobjectPreview(bool bEnable)
{
	if(SubobjectViewport.IsValid())
	{
		SubobjectViewport->EnablePreview(bEnable);
	}
}

void FBlueprintEditor::UpdateSubobjectPreview(bool bUpdateNow)
{
	// refresh widget
	if(SubobjectViewport.IsValid())
	{
		TSharedPtr<SDockTab> OwnerTab = Inspector->GetOwnerTab();
		if ( OwnerTab.IsValid() )
		{
			bUpdateNow &= OwnerTab->IsForeground();
		}

		// Only request a refresh immediately if the viewport tab is in the foreground.
		SubobjectViewport->RequestRefresh(false, bUpdateNow);
	}
}

UObject* FBlueprintEditor::GetSubobjectEditorObjectContext() const
{
	// Return the current CDO that was last generated for the class
	UBlueprint* Blueprint = GetBlueprintObj();
	if(Blueprint != nullptr && Blueprint->GeneratedClass != nullptr)
	{
		return Blueprint->GeneratedClass->GetDefaultObject();
	}
	
	return nullptr;
}

void FBlueprintEditor::OnSelectionUpdated(const TArray<TSharedPtr<FSubobjectEditorTreeNode>>& SelectedNodes)
{
	// Check whether the active component visualizer is relevant for the selected components ...
	if (const TSharedPtr<FComponentVisualizer> ComponentVisualizer = GUnrealEd->ComponentVisManager.GetActiveComponentVis())
	{
		bool bClearActiveVisualizer = true;
		for (const FSubobjectEditorTreeNodePtrType& SelectedNode : SelectedNodes)
		{
			const FSubobjectData* const Data = SelectedNode->GetDataSource();
			const UActorComponent* const Component = Data ? Data->FindComponentInstanceInActor(GetPreviewActor()) : nullptr;
			if (Component != nullptr && Component->IsRegistered() && Component == ComponentVisualizer->GetEditedComponent())
			{
				bClearActiveVisualizer = false;
				break;
			}
		}

		// If the relevant component for the active visualizer is no longer selected, clear the active visualizer.
		if (bClearActiveVisualizer)
		{
			GUnrealEd->ComponentVisManager.ClearActiveComponentVis();
		}
	}

	if (SubobjectViewport.IsValid())
	{
		SubobjectViewport->OnComponentSelectionChanged();
	}

	UBlueprint* Blueprint = GetBlueprintObj();
	check(Blueprint != nullptr && Blueprint->SimpleConstructionScript != nullptr);

	// Update the selection visualization
	AActor* EditorActorInstance = Blueprint->SimpleConstructionScript->GetComponentEditorActorInstance();
	if (EditorActorInstance != nullptr)
	{
		for (UActorComponent* Component : EditorActorInstance->GetComponents())
		{
			if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Component))
			{
				PrimitiveComponent->PushSelectionToProxy();
			}
		}
	}

	if (Inspector.IsValid())
	{
		// Clear the my blueprints selection
		if (SelectedNodes.Num() > 0)
		{
			SetUISelectionState(FBlueprintEditor::SelectionState_Components);
		}

		// Convert the selection set to an array of UObject* pointers
		FText InspectorTitle = FText::GetEmpty();
		TArray<UObject*> InspectorObjects;
		bool bShowComponents = true;
		InspectorObjects.Empty(SelectedNodes.Num());
		for (FSubobjectEditorTreeNodePtrType NodePtr : SelectedNodes)
		{
			const FSubobjectData* NodeData = NodePtr ? NodePtr->GetDataSource() : nullptr;
			if (NodeData)
			{
				if (const AActor* Actor = NodeData->GetObject<AActor>())
				{
					if (const AActor* DefaultActor = NodeData->GetObjectForBlueprint<AActor>(GetBlueprintObj()))
					{
						InspectorObjects.Add(const_cast<AActor*>(DefaultActor));
					
						FString Title;
						DefaultActor->GetName(Title);
						InspectorTitle = FText::FromString(Title);
						bShowComponents = false;
					
						TryInvokingDetailsTab();
					}
				}
				else
				{
					const UActorComponent* EditableComponent = NodeData->GetObjectForBlueprint<UActorComponent>(GetBlueprintObj());
					if (EditableComponent)
					{
						InspectorTitle = FText::FromString(NodePtr->GetDisplayString());
						InspectorObjects.Add(const_cast<UActorComponent*>(EditableComponent));
					}
				
					if (SubobjectViewport.IsValid())
					{
						TSharedPtr<SDockTab> OwnerTab = SubobjectViewport->GetOwnerTab();
						if (OwnerTab.IsValid())
						{
							OwnerTab->FlashTab();
						}
					}
				}
			}
		}

		const bool bHasSceneComponent = InspectorObjects.ContainsByPredicate([](const UObject* Obj) { return Obj && Obj->IsA<USceneComponent>(); });
		GetEditorModeManager().SetSelectionHasSceneComponent(bHasSceneComponent);

		// Update the details panel
		SKismetInspector::FShowDetailsOptions Options(InspectorTitle, true);
		Options.bShowComponents = bShowComponents;
		Inspector->ShowDetailsForObjects(InspectorObjects, Options);
	}

	// Get all registered Blueprint Selection Notifiers
	TArray<IBlueprintSelectionNotifier*> Notifiers = IModularFeatures::Get().GetModularFeatureImplementations<IBlueprintSelectionNotifier>(
		IBlueprintSelectionNotifier::GetModularFeatureName());

	UTypedElementSelectionSet* SelectionSet = GetEditorModeManager().GetEditorSelectionSet();
	const bool bHasSelectionSet = SelectionSet != nullptr;
	if (bHasSelectionSet)
	{
		SelectionSet->ClearSelection(FTypedElementSelectionOptions());
	}

	// Notify each registered implementation (including ModeTools SelectionSet)
	TArray<FTypedElementHandle> SelectedElements;
	for (const FSubobjectEditorTreeNodePtrType& SelectedNode : SelectedNodes)
	{
		if (const FSubobjectData* const Data = SelectedNode->GetDataSource())
		{
			if (const UActorComponent* Component = Data->FindComponentInstanceInActor(GetPreviewActor()))
			{
				// Convert component to TypedElementHandle
				FTypedElementHandle ElementHandle = UEngineElementsLibrary::AcquireEditorComponentElementHandle(Component);
				if (ElementHandle.IsSet())
				{
					SelectedElements.Add(ElementHandle);

					if (bHasSelectionSet)
					{
						SelectionSet->SelectElement(ElementHandle, FTypedElementSelectionOptions());
					}
				}
			}
		}
	}

	for (IBlueprintSelectionNotifier* Notifier : Notifiers)
	{
		if (Notifier)
		{
			Notifier->OnBlueprintSelectionUpdated(this, Blueprint, SelectedElements);
		}
	}
}

void FBlueprintEditor::OnComponentDoubleClicked(TSharedPtr<FSubobjectEditorTreeNode> Node)
{
	TSharedPtr<SDockTab> OwnerTab = Inspector->GetOwnerTab();
	if ( OwnerTab.IsValid() )
	{
		GetTabManager()->TryInvokeTab(FBlueprintEditorTabs::SCSViewportID);
	}
}

void FBlueprintEditor::OnComponentAddedToBlueprint(const FSubobjectData& NewSubobjectData)
{
	// Determine if we've added to a Blueprint instance that we're editing within this context.
	bool bIsEditingEventTarget = false;
	if (const UBlueprint* TargetBlueprint = NewSubobjectData.GetBlueprint())
	{
		for (UObject* EditorObject : GetEditingObjects())
		{
			if (EditorObject == TargetBlueprint)
			{
				bIsEditingEventTarget = true;
				break;
			}
		}
	}

	// Only handle add events if the editor context includes its targeted Blueprint object.
	if (!bIsEditingEventTarget)
	{
		return;
	}

	const UObject* NewSubobject = NewSubobjectData.GetObject();
	check(NewSubobject);

	// Get the default namespace set associated with the new subobject's class. Because we
	// might receive multiple events within a single frame (e.g. dragging multiple component
	// Blueprint class assets into the components tree will result in multiple notifications),
	// we'll add these into the deferred list for now and auto-import them all on the next tick.
	FBlueprintNamespaceUtilities::GetDefaultImportsForObject(NewSubobject->GetClass(), DeferredNamespaceImports);
}

TSharedRef<SWidget> FBlueprintEditor::CreateGraphTitleBarWidget(TSharedRef<FTabInfo> InTabInfo, UEdGraph* InGraph)
{
	// Create the title bar widget
	return SNew(SGraphTitleBar)
		.EdGraphObj(InGraph)
		.Kismet2(SharedThis(this))
		.OnDifferentGraphCrumbClicked(this, &FBlueprintEditor::OnChangeBreadCrumbGraph)
		.HistoryNavigationWidget(InTabInfo->CreateHistoryNavigationWidget());
}

/** Create new tab for the supplied graph - don't call this directly.*/
TSharedRef<SGraphEditor> FBlueprintEditor::CreateGraphEditorWidget(TSharedRef<FTabInfo> InTabInfo, UEdGraph* InGraph)
{
	check((InGraph != nullptr) && IsEditingSingleBlueprint());

	// No need to regenerate the commands.
	if(!GraphEditorCommands.IsValid())
	{
		GraphEditorCommands = MakeShareable( new FUICommandList );
		{
			GraphEditorCommands->MapAction(FGraphEditorCommands::Get().PromoteToVariable,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnPromoteToVariable, true ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanPromoteToVariable, true )
				);

			GraphEditorCommands->MapAction(FGraphEditorCommands::Get().PromoteToLocalVariable,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnPromoteToVariable, false ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanPromoteToVariable, false )
				);

			GraphEditorCommands->MapAction(FGraphEditorCommands::Get().SplitStructPin,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnSplitStructPin ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanSplitStructPin )
				);

			GraphEditorCommands->MapAction(FGraphEditorCommands::Get().RecombineStructPin,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnRecombineStructPin ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanRecombineStructPin )
				);

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().AddExecutionPin,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnAddExecutionPin ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanAddExecutionPin )
				);

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().InsertExecutionPinBefore,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnInsertExecutionPinBefore ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanInsertExecutionPin )
				);

			GraphEditorCommands->MapAction(FGraphEditorCommands::Get().InsertExecutionPinAfter,
				FExecuteAction::CreateSP(this, &FBlueprintEditor::OnInsertExecutionPinAfter),
				FCanExecuteAction::CreateSP(this, &FBlueprintEditor::CanInsertExecutionPin)
			);

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().RemoveExecutionPin,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnRemoveExecutionPin ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanRemoveExecutionPin )
				);

			GraphEditorCommands->MapAction(FGraphEditorCommands::Get().RemoveThisStructVarPin,
				FExecuteAction::CreateSP(this, &FBlueprintEditor::OnRemoveThisStructVarPin),
				FCanExecuteAction::CreateSP(this, &FBlueprintEditor::CanRemoveThisStructVarPin)
				);

			GraphEditorCommands->MapAction(FGraphEditorCommands::Get().RemoveOtherStructVarPins,
				FExecuteAction::CreateSP(this, &FBlueprintEditor::OnRemoveOtherStructVarPins),
				FCanExecuteAction::CreateSP(this, &FBlueprintEditor::CanRemoveOtherStructVarPins)
				);

			GraphEditorCommands->MapAction(FGraphEditorCommands::Get().RestoreAllStructVarPins,
				FExecuteAction::CreateSP(this, &FBlueprintEditor::OnRestoreAllStructVarPins),
				FCanExecuteAction::CreateSP(this, &FBlueprintEditor::CanRestoreAllStructVarPins)
				);

			GraphEditorCommands->MapAction(FGraphEditorCommands::Get().ResetPinToDefaultValue,
				FExecuteAction::CreateSP(this, &FBlueprintEditor::OnResetPinToDefaultValue),
				FCanExecuteAction::CreateSP(this, &FBlueprintEditor::CanResetPinToDefaultValue)
			);

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().AddOptionPin,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnAddOptionPin ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanAddOptionPin )
				);

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().RemoveOptionPin,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnRemoveOptionPin ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanRemoveOptionPin )
				);

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().ChangePinType,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnChangePinType ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanChangePinType )
				);

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().AddParentNode,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnAddParentNode ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanAddParentNode )
				);
			
			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().CreateMatchingFunction,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnCreateMatchingFunction ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanCreateMatchingFunction )
				);

			// Debug actions
			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().AddBreakpoint,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnAddBreakpoint ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanAddBreakpoint ),
				FIsActionChecked(),
				FIsActionButtonVisible::CreateSP( this, &FBlueprintEditor::CanAddBreakpoint )
				);

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().RemoveBreakpoint,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnRemoveBreakpoint ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanRemoveBreakpoint ),
				FIsActionChecked(),
				FIsActionButtonVisible::CreateSP( this, &FBlueprintEditor::CanRemoveBreakpoint )
				);

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().EnableBreakpoint,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnEnableBreakpoint ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanEnableBreakpoint ),
				FIsActionChecked(),
				FIsActionButtonVisible::CreateSP( this, &FBlueprintEditor::CanEnableBreakpoint )
				);

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().DisableBreakpoint,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnDisableBreakpoint ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanDisableBreakpoint ),
				FIsActionChecked(),
				FIsActionButtonVisible::CreateSP( this, &FBlueprintEditor::CanDisableBreakpoint )
				);

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().ToggleBreakpoint,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnToggleBreakpoint ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanToggleBreakpoint ),
				FIsActionChecked(),
				FIsActionButtonVisible::CreateSP( this, &FBlueprintEditor::CanToggleBreakpoint )
				);

			// Encapsulation commands
			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().CollapseNodes,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnCollapseNodes ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanCollapseNodes )
				);

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().CollapseSelectionToFunction,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnCollapseSelectionToFunction ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanCollapseSelectionToFunction ),
				FIsActionChecked(),
				FIsActionButtonVisible::CreateSP( this, &FBlueprintEditor::NewDocument_IsVisibleForType, CGT_NewFunctionGraph )
				);

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().CollapseSelectionToMacro,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnCollapseSelectionToMacro ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanCollapseSelectionToMacro ),
				FIsActionChecked(),
				FIsActionButtonVisible::CreateSP( this, &FBlueprintEditor::NewDocument_IsVisibleForType, CGT_NewMacroGraph )
				);

			GraphEditorCommands->MapAction(FGraphEditorCommands::Get().ConvertFunctionToEvent,
				FExecuteAction::CreateSP(this, &FBlueprintEditor::OnConvertFunctionToEvent),
				FCanExecuteAction::CreateSP(this, &FBlueprintEditor::CanConvertFunctionToEvent),
				FIsActionChecked()
			);

			GraphEditorCommands->MapAction(FGraphEditorCommands::Get().ConvertEventToFunction,
				FExecuteAction::CreateSP(this, &FBlueprintEditor::OnConvertEventToFunction),
				FCanExecuteAction::CreateSP(this, &FBlueprintEditor::CanConvertEventToFunction),
				FIsActionChecked()
			);

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().PromoteSelectionToFunction,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnPromoteSelectionToFunction ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanPromoteSelectionToFunction ),
				FIsActionChecked(),
				FIsActionButtonVisible::CreateSP( this, &FBlueprintEditor::NewDocument_IsVisibleForType, CGT_NewFunctionGraph )
				);

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().PromoteSelectionToMacro,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnPromoteSelectionToMacro ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanPromoteSelectionToMacro ),
				FIsActionChecked(),
				FIsActionButtonVisible::CreateSP( this, &FBlueprintEditor::NewDocument_IsVisibleForType, CGT_NewMacroGraph )
				);
			
			// Alignment Commands
			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().AlignNodesTop,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnAlignTop )
				);

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().AlignNodesMiddle,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnAlignMiddle )
				);

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().AlignNodesBottom,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnAlignBottom )
				);

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().AlignNodesLeft,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnAlignLeft )
				);

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().AlignNodesCenter,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnAlignCenter )
				);

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().AlignNodesRight,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnAlignRight )
				);

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().StraightenConnections,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnStraightenConnections )
				);

			// Distribution Commands
			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().DistributeNodesHorizontally,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnDistributeNodesH )
				);

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().DistributeNodesVertically,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnDistributeNodesV )
				);

			GraphEditorCommands->MapAction(FGraphEditorCommands::Get().StackNodesHorizontally,
				FExecuteAction::CreateSP(this, &FBlueprintEditor::OnStackNodesH)
				);

			GraphEditorCommands->MapAction(FGraphEditorCommands::Get().StackNodesVertically,
				FExecuteAction::CreateSP(this, &FBlueprintEditor::OnStackNodesV)
				);

			GraphEditorCommands->MapAction( FGenericCommands::Get().Rename,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnRenameNode ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanRenameNodes )
				);

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().ExpandNodes,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnExpandNodes ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanExpandNodes ),
				FIsActionChecked(),
				FIsActionButtonVisible::CreateSP( this, &FBlueprintEditor::CanExpandNodes )
				);

			// Editing commands
			GraphEditorCommands->MapAction( FGenericCommands::Get().SelectAll,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::SelectAllNodes ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanSelectAllNodes )
				);

			GraphEditorCommands->MapAction( FGenericCommands::Get().Delete,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::DeleteSelectedNodes ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanDeleteNodes )
				);

			GraphEditorCommands->MapAction(FGraphEditorCommands::Get().DeleteAndReconnectNodes,
				FExecuteAction::CreateSP(this, &FBlueprintEditor::DeleteSelectedNodes),
				FCanExecuteAction::CreateSP(this, &FBlueprintEditor::CanDeleteNodes)
			);

			GraphEditorCommands->MapAction( FGenericCommands::Get().Copy,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::CopySelectedNodes ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanCopyNodes )
				);

			GraphEditorCommands->MapAction( FGenericCommands::Get().Cut,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::CutSelectedNodes ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanCutNodes )
				);

			GraphEditorCommands->MapAction( FGenericCommands::Get().Paste,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::PasteGeneric ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanPasteGeneric )
				);

			GraphEditorCommands->MapAction( FGenericCommands::Get().Duplicate,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::DuplicateNodes ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanDuplicateNodes )
				);

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().SelectReferenceInLevel,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnSelectReferenceInLevel ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanSelectReferenceInLevel ),
				FIsActionChecked(),
				FIsActionButtonVisible::CreateSP( this, &FBlueprintEditor::CanSelectReferenceInLevel )
				);

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().AssignReferencedActor,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnAssignReferencedActor ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanAssignReferencedActor ) );

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().StartWatchingPin,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnStartWatchingPin ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanStartWatchingPin )
				);

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().StopWatchingPin,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnStopWatchingPin ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanStopWatchingPin )
				);

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().CreateComment,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnCreateComment )
				);
				
			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().CreateCustomEvent,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnCreateCustomEvent )
				);

			GraphEditorCommands->MapAction(FGraphEditorCommands::Get().ShowAllPins,
				FExecuteAction::CreateSP(this, &FBlueprintEditor::SetPinVisibility, SGraphEditor::Pin_Show)
				);

			GraphEditorCommands->MapAction(FGraphEditorCommands::Get().HideNoConnectionPins,
				FExecuteAction::CreateSP(this, &FBlueprintEditor::SetPinVisibility, SGraphEditor::Pin_HideNoConnection)
				);

			GraphEditorCommands->MapAction(FGraphEditorCommands::Get().HideNoConnectionNoDefaultPins,
				FExecuteAction::CreateSP(this, &FBlueprintEditor::SetPinVisibility, SGraphEditor::Pin_HideNoConnectionNoDefault)
				);

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().FindReferences,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnFindReferences, /*bSearchAllBlueprints=*/false, EGetFindReferenceSearchStringFlags::Legacy),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanFindReferences )
				);
			
			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().FindReferencesByNameLocal,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnFindReferences, /*bSearchAllBlueprints=*/false, EGetFindReferenceSearchStringFlags::None ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanFindReferences )
				);
			
			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().FindReferencesByNameGlobal,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnFindReferences, /*bSearchAllBlueprints=*/true, EGetFindReferenceSearchStringFlags::None ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanFindReferences )
				);
			
			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().FindReferencesByClassMemberLocal,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnFindReferences, /*bSearchAllBlueprints=*/false, EGetFindReferenceSearchStringFlags::UseSearchSyntax ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanFindReferences )
				);
			
			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().FindReferencesByClassMemberGlobal,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnFindReferences, /*bSearchAllBlueprints=*/true, EGetFindReferenceSearchStringFlags::UseSearchSyntax ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanFindReferences )
				);

			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().GoToDefinition,
				FExecuteAction::CreateSP( this, &FBlueprintEditor::OnGoToDefinition ),
				FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanGoToDefinition )
				);

			GraphEditorCommands->MapAction(FGraphEditorCommands::Get().GoToDocumentation,
				FExecuteAction::CreateSP(this, &FBlueprintEditor::OnGoToDocumentation),
				FCanExecuteAction::CreateSP(this, &FBlueprintEditor::CanGoToDocumentation)
				);

			GraphEditorCommands->MapAction(FGraphEditorCommands::Get().OpenInNewTab,
				FExecuteAction::CreateSP(this, &FBlueprintEditor::OnOpenSelectedNodesInNewTab),
				FCanExecuteAction::CreateSP(this, &FBlueprintEditor::CanOpenSelectedNodesInNewTab),
				FIsActionButtonVisible::CreateSP(this, &FBlueprintEditor::CanOpenSelectedNodesInNewTab)
			);

			GraphEditorCommands->MapAction(FGraphEditorCommands::Get().EnableNodes,
				FExecuteAction(),
				FCanExecuteAction(),
				FGetActionCheckState::CreateSP(this, &FBlueprintEditor::GetEnabledCheckBoxStateForSelectedNodes)
				);

			GraphEditorCommands->MapAction(FGraphEditorCommands::Get().DisableNodes,
				FExecuteAction::CreateSP(this, &FBlueprintEditor::OnSetEnabledStateForSelectedNodes, ENodeEnabledState::Disabled),
				FCanExecuteAction(),
				FGetActionCheckState::CreateSP(this, &FBlueprintEditor::CheckEnabledStateForSelectedNodes, ENodeEnabledState::Disabled)
				);

			GraphEditorCommands->MapAction(FGraphEditorCommands::Get().EnableNodes_Always,
				FExecuteAction::CreateSP(this, &FBlueprintEditor::OnSetEnabledStateForSelectedNodes, ENodeEnabledState::Enabled),
				FCanExecuteAction(),
				FGetActionCheckState::CreateSP(this, &FBlueprintEditor::CheckEnabledStateForSelectedNodes, ENodeEnabledState::Enabled)
				);

			GraphEditorCommands->MapAction(FGraphEditorCommands::Get().EnableNodes_DevelopmentOnly,
				FExecuteAction::CreateSP(this, &FBlueprintEditor::OnSetEnabledStateForSelectedNodes, ENodeEnabledState::DevelopmentOnly),
				FCanExecuteAction(),
				FGetActionCheckState::CreateSP(this, &FBlueprintEditor::CheckEnabledStateForSelectedNodes, ENodeEnabledState::DevelopmentOnly)
				);

			OnCreateGraphEditorCommands(GraphEditorCommands);
		}
	}

	// Create the title bar widget
	TSharedPtr<SWidget> TitleBarWidget = CreateGraphTitleBarWidget(InTabInfo, InGraph);

	SGraphEditor::FGraphEditorEvents InEvents;
	SetupGraphEditorEvents(InGraph, InEvents);

	// Append play world commands
	GraphEditorCommands->Append( FPlayWorldCommands::GlobalPlayWorldActions.ToSharedRef() );

	TSharedRef<SGraphEditor> Editor = SNew(SGraphEditor)
		.AdditionalCommands(GraphEditorCommands)
		.IsEditable(this, &FBlueprintEditor::IsEditable, InGraph)
		.DisplayAsReadOnly(this, &FBlueprintEditor::IsGraphReadOnly, InGraph)
		.TitleBar(TitleBarWidget)
		.Appearance(this, &FBlueprintEditor::GetGraphAppearance, InGraph)
		.GraphToEdit(InGraph)
		.GraphEvents(InEvents)
		.OnNavigateHistoryBack(FSimpleDelegate::CreateSP(this, &FBlueprintEditor::NavigateTab, FDocumentTracker::NavigateBackwards))
		.OnNavigateHistoryForward(FSimpleDelegate::CreateSP(this, &FBlueprintEditor::NavigateTab, FDocumentTracker::NavigateForwards))
		.AssetEditorToolkit(this->AsShared());
		//@TODO: Crashes in command list code during the callback .OnGraphModuleReloaded(FEdGraphEvent::CreateSP(this, &FBlueprintEditor::ChangeOpenGraphInDocumentEditorWidget, WeakParent))
		;

	OnSetPinVisibility.AddSP(&Editor.Get(), &SGraphEditor::SetPinVisibility);

	FVector2f ViewOffset = FVector2f::ZeroVector;
	float ZoomAmount = INDEX_NONE;

	TSharedPtr<SDockTab> ActiveTab = DocumentManager->GetActiveTab();
	if(ActiveTab.IsValid())
	{
		// Check if the graph is already opened in the current tab, if it is we want to start at the same position to stop the graph from jumping around oddly
		TSharedPtr<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(ActiveTab->GetContent());

		if(GraphEditor.IsValid() && GraphEditor->GetCurrentGraph() == InGraph)
		{
			GraphEditor->GetViewLocation(ViewOffset, ZoomAmount);
		}
	}

	Editor->SetViewLocation(ViewOffset, ZoomAmount);

	return Editor;
}

void FBlueprintEditor::SetupGraphEditorEvents(UEdGraph* InGraph, SGraphEditor::FGraphEditorEvents& InEvents)
{
	InEvents.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateSP( this, &FBlueprintEditor::OnSelectedNodesChanged );
	InEvents.OnDropActors = SGraphEditor::FOnDropActors::CreateSP( this, &FBlueprintEditor::OnGraphEditorDropActor );
	InEvents.OnDropStreamingLevels = SGraphEditor::FOnDropStreamingLevels::CreateSP( this, &FBlueprintEditor::OnGraphEditorDropStreamingLevel );
	InEvents.OnNodeDoubleClicked = FSingleNodeEvent::CreateSP(this, &FBlueprintEditor::OnNodeDoubleClicked);
	InEvents.OnVerifyTextCommit = FOnNodeVerifyTextCommit::CreateSP(this, &FBlueprintEditor::OnNodeVerifyTitleCommit);
	InEvents.OnTextCommitted = FOnNodeTextCommitted::CreateSP(this, &FBlueprintEditor::OnNodeTitleCommitted);
	InEvents.OnSpawnNodeByShortcutAtLocation = SGraphEditor::FOnSpawnNodeByShortcutAtLocation::CreateSP(this, &FBlueprintEditor::OnSpawnGraphNodeByShortcut, InGraph);
	InEvents.OnNodeSpawnedByKeymap = SGraphEditor::FOnNodeSpawnedByKeymap::CreateSP(this, &FBlueprintEditor::OnNodeSpawnedByKeymap );
	InEvents.OnDisallowedPinConnection = SGraphEditor::FOnDisallowedPinConnection::CreateSP(this, &FBlueprintEditor::OnDisallowedPinConnection);
	InEvents.OnDoubleClicked = SGraphEditor::FOnDoubleClicked::CreateSP(this, &FBlueprintEditor::NavigateToParentGraphByDoubleClick);

	// Custom menu for K2 schemas
	if(InGraph->Schema != nullptr && InGraph->Schema->IsChildOf(UEdGraphSchema_K2::StaticClass()))
	{
		InEvents.OnCreateActionMenuAtLocation = SGraphEditor::FOnCreateActionMenuAtLocation::CreateSP(this, &FBlueprintEditor::OnCreateGraphActionMenu);
	}
}

FGraphAppearanceInfo FBlueprintEditor::GetCurrentGraphAppearance() const
{
	return GetGraphAppearance(GetFocusedGraph());
}

FGraphAppearanceInfo FBlueprintEditor::GetGraphAppearance(UEdGraph* InGraph) const
{
	// Create the appearance info
	FGraphAppearanceInfo AppearanceInfo;

	UBlueprint* Blueprint = (InGraph != nullptr) ? FBlueprintEditorUtils::FindBlueprintForGraph(InGraph) : GetBlueprintObj();
	if (Blueprint != nullptr)
	{
		if (FBlueprintEditorUtils::IsEditorUtilityBlueprint(Blueprint))
		{
			AppearanceInfo.CornerText = LOCTEXT("EditorUtilityAppearanceCornerText", "EDITOR UTILITY");
		}
		else
		{
			switch (Blueprint->BlueprintType)
			{
			case BPTYPE_LevelScript:
				AppearanceInfo.CornerText = LOCTEXT("AppearanceCornerText_LevelScript", "LEVEL BLUEPRINT");
				break;
			case BPTYPE_MacroLibrary:
				AppearanceInfo.CornerText = LOCTEXT("AppearanceCornerText_Macro", "MACRO");
				break;
			case BPTYPE_Interface:
				AppearanceInfo.CornerText = LOCTEXT("AppearanceCornerText_Interface", "INTERFACE");
				break;
			default:
				AppearanceInfo.CornerText = LOCTEXT("AppearanceCornerText_Blueprint", "BLUEPRINT");
				break;
			}
		}
	}

	UEdGraph const* EditingGraph = GetFocusedGraph();
	if (InGraph && BlueprintEditorImpl::GraphHasDefaultNode(InGraph))
	{
		AppearanceInfo.InstructionText = LOCTEXT("AppearanceInstructionText_DefaultGraph", "Drag Off Pins to Create/Connect New Nodes.");
	}
	else // if the graph is empty...
	{
		AppearanceInfo.InstructionText = LOCTEXT("AppearanceInstructionText_EmptyGraph", "Right-Click to Create New Nodes.");
	}
	auto InstructionOpacityDelegate = TAttribute<float>::FGetter::CreateSP(this, &FBlueprintEditor::GetInstructionTextOpacity, InGraph);
	AppearanceInfo.InstructionFade.Bind(InstructionOpacityDelegate);

	AppearanceInfo.PIENotifyText = GetPIEStatus();

	return AppearanceInfo;
}

// Open the editor for a given graph
void FBlueprintEditor::OnChangeBreadCrumbGraph(UEdGraph* InGraph)
{
	if (InGraph)
	{
		JumpToHyperlink(InGraph, false);
	}
}

FBlueprintEditor::FBlueprintEditor()
	: bSaveIntermediateBuildProducts(false)
	, bIsReparentingBlueprint(false)
	, bPendingDeferredClose(false)
	, CachedNumWarnings(0)
	, CachedNumErrors(0)
	, bRequestedSavingOpenDocumentState(false)
	, bBlueprintModifiedOnOpen (false)
	, PinVisibility(SGraphEditor::Pin_Show)
	, bIsActionMenuContextSensitive(true)
	, CurrentUISelection(NAME_None)
	, bEditorMarkedAsClosed(false)
	, bHideUnrelatedNodes(false)
	, bLockNodeFadeState(false)
	, bSelectRegularNode(false)
	, HasOpenActionMenu(nullptr)
	, InstructionsFadeCountdown(0.f)
{
	AnalyticsStats.GraphActionMenusNonCtxtSensitiveExecCount = 0;
	AnalyticsStats.GraphActionMenusCtxtSensitiveExecCount = 0;
	AnalyticsStats.GraphActionMenusCancelledCount = 0;
	AnalyticsStats.MyBlueprintNodeDragPlacementCount = 0;
	AnalyticsStats.PaletteNodeDragPlacementCount = 0;
	AnalyticsStats.NodeGraphContextCreateCount = 0;
	AnalyticsStats.NodePinContextCreateCount = 0;
	AnalyticsStats.NodeKeymapCreateCount = 0;
	AnalyticsStats.NodePasteCreateCount = 0;

	UEditorEngine* Editor = (UEditorEngine*)GEngine;
	if (Editor != nullptr)
	{
		Editor->RegisterForUndo(this);
	}

	DocumentManager = MakeShareable(new FDocumentTracker);
}

void FBlueprintEditor::EnsureBlueprintIsUpToDate(UBlueprint* BlueprintObj)
{
	// Purge any nullptr graphs
	FBlueprintEditorUtils::PurgeNullGraphs(BlueprintObj);

	// Make sure the blueprint is cosmetically up to date
	FKismetEditorUtilities::UpgradeCosmeticallyStaleBlueprint(BlueprintObj);

	if (FBlueprintEditorUtils::SupportsConstructionScript(BlueprintObj))
	{
		// If we don't have an SCS yet, make it
		if(BlueprintObj->SimpleConstructionScript == nullptr)
		{
			check(nullptr != BlueprintObj->GeneratedClass);
			BlueprintObj->SimpleConstructionScript = NewObject<USimpleConstructionScript>(BlueprintObj->GeneratedClass);
			BlueprintObj->SimpleConstructionScript->SetFlags(RF_Transactional);

			// Recreate (or create) any widgets that depend on the SCS
			CreateSubobjectEditors();
		}

		// If we should have a UCS but don't yet, make it
		if (UEdGraph* ExistingUCS = FBlueprintEditorUtils::FindUserConstructionScript(BlueprintObj))
		{
			ExistingUCS->bAllowDeletion = false;
		}
		else if(GetDefault<UBlueprintEditorSettings>()->IsFunctionAllowedForAsset(BlueprintObj, UEdGraphSchema_K2::FN_UserConstructionScript))
		{
			UEdGraph* UCSGraph = FBlueprintEditorUtils::CreateNewGraph(BlueprintObj, UEdGraphSchema_K2::FN_UserConstructionScript, UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());
			FBlueprintEditorUtils::AddFunctionGraph(BlueprintObj, UCSGraph, /*bIsUserCreated=*/ false, AActor::StaticClass());
			UCSGraph->bAllowDeletion = false;
		}

		// Check to see if we have gained a component from our parent (that would require us removing our scene root)
		// (or lost one, which requires adding one)
		if (BlueprintObj->SimpleConstructionScript != nullptr)
		{
			BlueprintObj->SimpleConstructionScript->ValidateSceneRootNodes();
		}
	}
	else
	{
		// If we have an SCS but don't support it, then we remove it
		if (BlueprintObj->SimpleConstructionScript)
		{
			// Remove any SCS variable nodes
			for (USCS_Node* SCS_Node : BlueprintObj->SimpleConstructionScript->GetAllNodes())
			{
				if (SCS_Node)
				{
					FBlueprintEditorUtils::RemoveVariableNodes(BlueprintObj, SCS_Node->GetVariableName());
				}
			}
		
			// Remove the SCS object reference
			BlueprintObj->SimpleConstructionScript = nullptr;

			// Mark the Blueprint as having been structurally modified
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(BlueprintObj);
		}

		// Allow deleting the UCS if we've somehow changed away from being an actor (e.g., because of C++ reparenting the parent of our native parent)
		if (UEdGraph* ExistingUCS = FBlueprintEditorUtils::FindUserConstructionScript(BlueprintObj))
		{
			ExistingUCS->bAllowDeletion = true;
		}
	}

	// Make sure that this blueprint is up-to-date with regards to its parent functions
	FBlueprintEditorUtils::ConformCallsToParentFunctions(BlueprintObj);

	// Make sure that this blueprint is up-to-date with regards to its implemented events
	FBlueprintEditorUtils::ConformImplementedEvents(BlueprintObj);

	// Make sure that this blueprint is up-to-date with regards to its implemented interfaces
	FBlueprintEditorUtils::ConformImplementedInterfaces(BlueprintObj);

	// Update old composite nodes(can't do this in PostLoad)
	FBlueprintEditorUtils::UpdateOutOfDateCompositeNodes(BlueprintObj);

	// Update any nodes which might have dropped their RF_Transactional flag due to copy-n-paste issues
	FBlueprintEditorUtils::UpdateTransactionalFlags(BlueprintObj);
}

struct FLoadObjectsFromAssetRegistryHelper
{
	template<class TObjectType>
	static void Load(TSet<TWeakObjectPtr<TObjectType>>& Collection)
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

		const double CompileStartTime = FPlatformTime::Seconds();

		TArray<FAssetData> AssetDatas;
		AssetRegistryModule.Get().GetAssetsByClass(TObjectType::StaticClass()->GetClassPathName(), AssetDatas);

		for (FAssetData& AssetData : AssetDatas)
		{
			if (AssetData.IsValid())
			{
				FString AssetPath = AssetData.GetObjectPathString();
				// Workaround for UE-178174: AssetRegistry returning unmounted AssetPaths. Test the path for mountedness before loading.
				FString PackagePathString = FPackageName::ObjectPathToPackageName(AssetPath);
				FPackagePath UnusedPackagePath;
				if (FPackagePath::TryFromMountedName(PackagePathString, UnusedPackagePath))
				{
					TObjectType* Object = LoadObject<TObjectType>(nullptr, *AssetPath, nullptr, 0, nullptr);
					if (Object)
					{
						Collection.Add(MakeWeakObjectPtr(Object));
					}
				}
			}
		}

		const double FinishTime = FPlatformTime::Seconds();

		UE_LOGF(LogBlueprint, Log, "Loading all assets of type: %ls took %.2f seconds", *TObjectType::StaticClass()->GetName(), static_cast<float>(FinishTime - CompileStartTime));
	}
};

void FBlueprintEditor::CommonInitialization(const TArray<UBlueprint*>& InitBlueprints, bool bShouldOpenInDefaultsMode)
{
	TSharedPtr<FBlueprintEditor> ThisPtr(SharedThis(this));

	// @todo TabManagement
	DocumentManager->Initialize(ThisPtr);

	// Register the document factories
	{
		DocumentManager->RegisterDocumentFactory(MakeShareable(new FTimelineEditorSummoner(ThisPtr)));

		TSharedRef<FDocumentTabFactory> GraphEditorFactory = MakeShareable(new FGraphEditorSummoner(ThisPtr,
			FGraphEditorSummoner::FOnCreateGraphEditorWidget::CreateSP(this, &FBlueprintEditor::CreateGraphEditorWidget)
			));

		// Also store off a reference to the grapheditor factory so we can find all the tabs spawned by it later.
		GraphEditorTabFactoryPtr = GraphEditorFactory;
		DocumentManager->RegisterDocumentFactory(GraphEditorFactory);
	}

	// Create a namespace helper to keep track of imports for all BPs being edited.
	ImportedNamespaceHelper = MakeShared<FBlueprintNamespaceHelper>();

	// Add each Blueprint instance to be edited into the namespace helper's context.
	for (const UBlueprint* BP : InitBlueprints)
	{
		ImportedNamespaceHelper->AddBlueprint(BP);
	}

	// Create imported namespace type filters for value editing.
	ImportedClassViewerFilter = MakeShared<BlueprintEditorImpl::FImportedClassViewerFilterProxy>(ImportedNamespaceHelper->GetClassViewerFilter());
	ImportedPinTypeSelectorFilter = MakeShared<BlueprintEditorImpl::FImportedPinTypeSelectorFilterProxy>(ImportedNamespaceHelper->GetPinTypeSelectorFilter());
	PermissionsPinTypeSelectorFilter = MakeShared<BlueprintEditorImpl::FPermissionsPinTypeSelectorFilter>(InitBlueprints);

	// Make sure we know when tabs become active to update details tab
	OnActiveTabChangedDelegateHandle = FGlobalTabmanager::Get()->OnActiveTabChanged_Subscribe( FOnActiveTabChanged::FDelegate::CreateRaw(this, &FBlueprintEditor::OnActiveTabChanged) );

	if (InitBlueprints.Num() == 1)
	{
		if (!bShouldOpenInDefaultsMode)
		{
			// Load blueprint libraries
			if (ShouldLoadBPLibrariesFromAssetRegistry())
			{
				LoadLibrariesFromAssetRegistry();
			}

			// Init the action DB for the context menu/palette if not already constructed
			FBlueprintActionDatabase::Get();
		}

		FLoadObjectsFromAssetRegistryHelper::Load<UUserDefinedEnum>(UserDefinedEnumerators);

		UBlueprint* InitBlueprint = InitBlueprints[0];

		// Update the blueprint if required
		EBlueprintStatus OldStatus = InitBlueprint->Status;
		EnsureBlueprintIsUpToDate(InitBlueprint);
		UPackage* BpPackage = InitBlueprint->GetOutermost();
		bBlueprintModifiedOnOpen = (InitBlueprint->Status != OldStatus) && !BpPackage->HasAnyPackageFlags(PKG_NewlyCreated);

		// Flag the blueprint as having been opened
		InitBlueprint->bIsNewlyCreated = false;

		// When the blueprint that we are observing changes, it will notify this wrapper widget.
		InitBlueprint->OnChanged().AddSP(this, &FBlueprintEditor::OnBlueprintChanged);
		InitBlueprint->OnCompiled().AddSP(this, &FBlueprintEditor::OnBlueprintCompiled);
		InitBlueprint->OnSetObjectBeingDebugged().AddSP(this, &FBlueprintEditor::HandleSetObjectBeingDebugged);
	}

	bWasOpenedInDefaultsMode = bShouldOpenInDefaultsMode;

	CreateDefaultTabContents(InitBlueprints);

	// Only register our component common actions customization in Standalone mode.
	if (!IsWorldCentricAssetEditor())
	{
		const TSharedPtr<IToolkitHost> PinnedToolkitHost = ToolkitHost.Pin();
		if (ensure(PinnedToolkitHost.IsValid()))
		{
			TUniquePtr<FComponentElementSubobjectEditorCommonActionsCustomization> ComponentCustomization = MakeUnique<FComponentElementSubobjectEditorCommonActionsCustomization>();
			ComponentCustomization->SetSubobjectEditor(GetSubobjectEditor());

			PinnedToolkitHost->GetCommonActions()->RegisterInterfaceCustomizationByTypeName(NAME_Components, MoveTemp(ComponentCustomization));
		}
	}

	FCoreUObjectDelegates::OnPreObjectPropertyChanged.AddSP(this, &FBlueprintEditor::OnPreObjectPropertyChanged);
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this, &FBlueprintEditor::OnPostObjectPropertyChanged);

	FKismetEditorUtilities::OnBlueprintUnloaded.AddSP(this, &FBlueprintEditor::OnBlueprintUnloaded);
}

void FBlueprintEditor::LoadLibrariesFromAssetRegistry()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FBlueprintEditor::LoadLibrariesFromAssetRegistry);

	if (EnableAutomaticLibraryAssetLoading == 0)
	{
		return;
	}

	if (UBlueprint* BP = GetBlueprintObj())
	{
		const FString UserDeveloperPath = FPackageName::FilenameToLongPackageName( FPaths::GameUserDeveloperDir());
		const FString DeveloperPath = FPackageName::FilenameToLongPackageName( FPaths::GameDevelopersDir() );

		// Interface blueprints don't show a node context menu anywhere so we can skip library loading
		if (BP->BlueprintType != BPTYPE_Interface)
		{
			// Load the asset registry module
			FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

			// Collect a full list of assets with the specified class
			TArray<FAssetData> AssetData;
			AssetRegistryModule.Get().GetAssetsByClass(UBlueprint::StaticClass()->GetClassPathName(), AssetData);

			const FName BPTypeName(GET_MEMBER_NAME_STRING_CHECKED(UBlueprint, BlueprintType));
			TArray<const FAssetData*> RelevantAssets;
			const TCHAR* BPMacroTypeStr = TEXT("BPTYPE_MacroLibrary");
			const TCHAR* BPFunctionTypeStr = TEXT("BPTYPE_FunctionLibrary");
			for (const FAssetData& AssetEntry : AssetData)
			{
				const FString AssetBPType = AssetEntry.GetTagValueRef<FString>(BPTypeName);

				// Only check for Blueprint Macros & Functions in the asset data for loading
				if ((AssetBPType == BPMacroTypeStr) || (AssetBPType == BPFunctionTypeStr))
				{
					RelevantAssets.Add(&AssetEntry);
				}
			}

			FScopedSlowTask LoadingMacrosAndFunctions(RelevantAssets.Num(), LOCTEXT("LoadingBlueprintAssetData", "Loading Blueprint Asset Data"));
			LoadingMacrosAndFunctions.Visibility = ESlowTaskVisibility::Important; // this function can be very slow, users will benefit from our messages
			const FName BPNamespaceName(GET_MEMBER_NAME_STRING_CHECKED(UBlueprint, BlueprintNamespace));
			
			struct FExpensiveObjectRecord
			{
				FExpensiveObjectRecord() : Seconds(0.0) {}
				FExpensiveObjectRecord(double InSeconds, const FName InPath) : Seconds(InSeconds), Path(InPath) {}
				double Seconds;
				FName Path;
			};

			TArray<FExpensiveObjectRecord> ExpensiveObjects;
			const double MinSecondsToReportExpensiveObject = 0.2;
			const int32 MaxExpensiveObjectsToList = 100;
			int32 NumLibariesLoaded = 0;

			const double StartTimeAll = FPlatformTime::Seconds();
			for (const FAssetData* AssetEntryPtr : RelevantAssets)
			{
				const FAssetData& AssetEntry = *AssetEntryPtr;
				const FString BlueprintPath = AssetEntry.GetSoftObjectPath().ToString();

				// See if this passes the namespace check
				bool bAllowLoadBP = !ImportedNamespaceHelper.IsValid() || ImportedNamespaceHelper->IsImportedAsset(AssetEntry);
					
				// For blueprints inside developers folder, only allow the ones inside current user's developers folder.
				if (bAllowLoadBP)
				{
					if (BlueprintPath.StartsWith(DeveloperPath) && 
						!BlueprintPath.StartsWith(UserDeveloperPath))
					{
						bAllowLoadBP = false;
					}
				}

				if (bAllowLoadBP)
				{
					LoadingMacrosAndFunctions.EnterProgressFrame(1.f, 
						FText::Format(LOCTEXT("LoadingFuncMacroLib", "Loading Function or Macro library: {0}"), FText::FromName(AssetEntry.AssetName)));

					++NumLibariesLoaded;
					const double StartTime = FPlatformTime::Seconds();

					// Load the blueprint
					UBlueprint* BlueprintLibPtr = LoadObject<UBlueprint>(nullptr, *BlueprintPath, nullptr, 0, nullptr);
					if (BlueprintLibPtr)
					{
						StandardLibraries.AddUnique(BlueprintLibPtr);
					}

					const double ElapsedTime = FPlatformTime::Seconds() - StartTime;
					if (ElapsedTime > MinSecondsToReportExpensiveObject)
					{
						ExpensiveObjects.Add(FExpensiveObjectRecord(ElapsedTime, AssetEntry.PackageName));
					}
				}
			}

			if (ExpensiveObjects.Num() > 0)
			{
				const double TotalSeconds = FPlatformTime::Seconds() - StartTimeAll;
				UE_LOGF(LogBlueprintEditor, Log, "Perf: %.1f total seconds to load all %d blueprint libraries in project. Avoid references to content in blueprint libraries to shorten this time.", TotalSeconds, NumLibariesLoaded);

				// Log the most expensive objects to load
				Algo::Sort(ExpensiveObjects, [](FExpensiveObjectRecord& A, FExpensiveObjectRecord& B) { return A.Seconds > B.Seconds; });
				for (int32 i=0; i < ExpensiveObjects.Num() && i < MaxExpensiveObjectsToList; ++i)
				{
					FExpensiveObjectRecord& ExpensiveObjectRecord = ExpensiveObjects[i];
					UE_LOGF(LogBlueprintEditor, Log, "Perf: %.1f seconds loading: %ls", ExpensiveObjectRecord.Seconds, *ExpensiveObjectRecord.Path.ToString());
				}
			}
		}
	}
}

void FBlueprintEditor::ImportNamespace(const FString& InNamespace)
{
	FImportNamespaceExParameters Params;
	Params.NamespacesToImport.Add(InNamespace);
	ImportNamespaceEx(Params);
}

void FBlueprintEditor::ImportNamespaceEx(const FImportNamespaceExParameters& InParams)
{
	// No auto-import actions if features are disabled.
	const bool bIsAutoImportEnabled = GetDefault<UBlueprintEditorSettings>()->bEnableNamespaceImportingFeatures;
	if (InParams.bIsAutoImport && !bIsAutoImportEnabled)
	{
		return;
	}

	// Exit now if no input was given.
	if (InParams.NamespacesToImport.IsEmpty())
	{
		return;
	}

	auto AddNamespaceToImportList = [](UBlueprint* InBlueprint, const FString& InNamespace) -> bool
	{
		// No need to include the global namespace (empty string) in the import list.
		if (!InNamespace.IsEmpty() && !InBlueprint->ImportedNamespaces.Contains(InNamespace))
		{
			InBlueprint->Modify();
			InBlueprint->ImportedNamespaces.Add(InNamespace);

			return true;
		}

		return false;
	};

	// Update the imported set for all edited objects.
	bool bWasAddedAsImport = false;
	const auto& EditingObjs = GetEditingObjects();
	for (UObject* EditingObj : EditingObjs)
	{
		if (UBlueprint* BlueprintObj = Cast<UBlueprint>(EditingObj))
		{
			// Add each namespace into the Blueprint's user-facing import set.
			for (const FString& NamespaceToImport : InParams.NamespacesToImport)
			{
				bWasAddedAsImport |= AddNamespaceToImportList(BlueprintObj, NamespaceToImport);
			}
		}
	}

	auto AddNamespaceToEditorContext = [](TSharedPtr<FBlueprintNamespaceHelper> ImportsHelper, const FString& InNamespace) -> bool
	{
		// Note: The global namespace (empty string) is implicitly included.
		if (!InNamespace.IsEmpty() && !ImportsHelper->IsIncludedInNamespaceList(InNamespace))
		{
			ImportsHelper->AddNamespace(InNamespace);
			return true;
		}

		return false;
	};

	// Add to the current scope of the Blueprint's editor context. Note that in certain cases, imports may already be associated
	// with the Blueprint, but not yet associated with the editor context (e.g. - auto-import after setting a Blueprint's namespace;
	// we won't add it to the Blueprint's import list, but we still want to add to the editor context and do any post-import actions).
	bool bWasAddedToEditorContext = false;
	if (ImportedNamespaceHelper.IsValid())
	{
		for (const FString& NamespaceToImport : InParams.NamespacesToImport)
		{
			bWasAddedToEditorContext |= AddNamespaceToEditorContext(ImportedNamespaceHelper, NamespaceToImport);
		}
	}

	if (bWasAddedAsImport || bWasAddedToEditorContext)
	{
		// Load additional libraries that may now be in scope.
		// @todo_namespaces - Make this more targeted - i.e. get/load only those assets tagged w/ the given namespace
		LoadLibrariesFromAssetRegistry();

		// Refresh class details on an auto-import if visible, since the list of imports has implicitly changed.
		if (InParams.bIsAutoImport && IsDetailsPanelEditingGlobalOptions())
		{
			RefreshInspector();
		}

		// If bound, execute the post-import callback.
		InParams.OnPostImportCallback.ExecuteIfBound();

		// Display a notification for auto-import events only if we've added it to the editor context.
		if (InParams.bIsAutoImport && bWasAddedToEditorContext)
		{
			const int32 ImportCount = InParams.NamespacesToImport.Num();

			FText NotificationText;
			if (ImportCount > 1)
			{
				FFormatNamedArguments FormatArgs;
				FormatArgs.Add(TEXT("ImportCount"), ImportCount);
				NotificationText = FText::Format(LOCTEXT("AutoImportNotification_Multiple", "Imported {ImportCount} namespaces"), FormatArgs);
			}
			else
			{
				NotificationText = FText::Format(LOCTEXT("AutoImportNotification_Single", "Imported namespace \"{0}\""), FText::FromString(InParams.NamespacesToImport.Array()[0]));
			}

			FNotificationInfo Notification(NotificationText);
			Notification.ExpireDuration = 3.0f;
			FSlateNotificationManager::Get().AddNotification(Notification);
		}
	}
}

void FBlueprintEditor::RemoveNamespace(const FString& InNamespace)
{
	// Cannot remove the global namespace.
	if (InNamespace.IsEmpty())
	{
		return;
	}

	// Update the imported set for all edited objects.
	const auto& EditingObjs = GetEditingObjects();
	for (UObject* EditingObj : EditingObjs)
	{
		if (UBlueprint* BlueprintObj = Cast<UBlueprint>(EditingObj))
		{
			if (BlueprintObj->ImportedNamespaces.Contains(InNamespace))
			{
				BlueprintObj->Modify();
				BlueprintObj->ImportedNamespaces.Remove(InNamespace);
			}
		}
	}

	// Remove it from the current scope of the Blueprint's editor context.
	if (ImportedNamespaceHelper.IsValid())
	{
		if (ImportedNamespaceHelper->IsIncludedInNamespaceList(InNamespace))
		{
			ImportedNamespaceHelper->RemoveNamespace(InNamespace);
		}
	}
}

void FBlueprintEditor::SelectAndDuplicateNode(UEdGraphNode* InNode)
{
	check(InNode != nullptr);

	ClearSelectionStateFor(CurrentUISelection);
	AddToSelection(InNode);
	DuplicateNodes();
}

void FBlueprintEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	//@TODO: Can't we do this sooner?
	DocumentManager->SetTabManager(InTabManager);

	FWorkflowCentricApplication::RegisterTabSpawners(InTabManager);
}

void FBlueprintEditor::SetCurrentMode(FName NewMode)
{
	// Clear the selection state when the mode changes.
	SetUISelectionState(NAME_None);

	OnModeSetData.Broadcast( NewMode );
	FWorkflowCentricApplication::SetCurrentMode(NewMode);
}

void FBlueprintEditor::InitBlueprintEditor(
	const EToolkitMode::Type Mode,
	const TSharedPtr< IToolkitHost >& InitToolkitHost,
	const TArray<UBlueprint*>& InBlueprints,
	bool bShouldOpenInDefaultsMode, 
	TSharedPtr<ISCSEditorUICustomization> InSCSEditorUICustomization)
{
	check(InBlueprints.Num() == 1 || bShouldOpenInDefaultsMode);

	SCSEditorUICustomization = InSCSEditorUICustomization;

	// TRUE if a single Blueprint is being opened and is marked as newly created
	bool bNewlyCreated = InBlueprints.Num() == 1 && InBlueprints[0]->bIsNewlyCreated;

	// Load editor settings from disk.
	LoadEditorSettings();

	TArray< UObject* > Objects;
	for (UBlueprint* Blueprint : InBlueprints)
	{
		// Flag the blueprint as having been opened
		Blueprint->bIsNewlyCreated = false;

		Objects.Add( Blueprint );
	}
	
	if (!Toolbar.IsValid())
	{
		Toolbar = MakeShareable(new FBlueprintEditorToolbar(SharedThis(this)));
	}

	GetToolkitCommands()->Append(FPlayWorldCommands::GlobalPlayWorldActions.ToSharedRef());

	CreateDefaultCommands();

	RegisterMenus();

	// Initialize the asset editor and spawn nothing (dummy layout)
	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	const FName BlueprintEditorAppName = FName(TEXT("BlueprintEditorApp"));
	InitAssetEditor(Mode, InitToolkitHost, BlueprintEditorAppName, FTabManager::FLayout::NullLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, Objects);
	GetEditorModeManager().SetDefaultMode(UBlueprintEditorEdMode::Id);

	if (UModeManagerInteractiveToolsContext* ToolsContext = GetEditorModeManager().GetInteractiveToolsContext())
	{
		ToolsContext->SetViewportInteractionsEnabled(true);
	}

	CommonInitialization(InBlueprints, bShouldOpenInDefaultsMode);

	InitalizeExtenders();

	RegenerateMenusAndToolbars();

	RegisterApplicationModes(InBlueprints, bShouldOpenInDefaultsMode, bNewlyCreated);

	// Post-layout initialization
	PostLayoutBlueprintEditorInitialization();

	// Find and set any instances of this blueprint type if any exists and we are not already editing one
	FBlueprintEditorUtils::FindAndSetDebuggableBlueprintInstances();

	if ( bNewlyCreated )
	{
		if ( UBlueprint* Blueprint = GetBlueprintObj() )
		{
			if ( Blueprint->BlueprintType == BPTYPE_MacroLibrary )
			{
				NewDocument_OnClick(CGT_NewMacroGraph);
			}
			else if ( Blueprint->BlueprintType == BPTYPE_Interface )
			{
				NewDocument_OnClick(CGT_NewFunctionGraph);
			}
			else if ( Blueprint->BlueprintType == BPTYPE_FunctionLibrary )
			{
				NewDocument_OnClick(CGT_NewFunctionGraph);
			}
		}
	}

	if ( UBlueprint* Blueprint = GetBlueprintObj() )
	{
		if ( Blueprint->GetClass() == UBlueprint::StaticClass() && Blueprint->BlueprintType == BPTYPE_Normal )
		{
			if ( !bShouldOpenInDefaultsMode )
			{
				GetToolkitCommands()->ExecuteAction(FFullBlueprintEditorCommands::Get().EditClassDefaults.ToSharedRef());
			}
		}

		// There are upgrade notes, open the log and dump the messages to it
		if (Blueprint->UpgradeNotesLog.IsValid())
		{
			DumpMessagesToCompilerLog(Blueprint->UpgradeNotesLog->Messages, true);
		}
	}

	// Register for notifications when settings change
	BlueprintEditorSettingsChangedHandle = GetMutableDefault<UBlueprintEditorSettings>()->OnSettingChanged()
		.AddRaw(this, &FBlueprintEditor::OnBlueprintEditorPreferencesChanged);
	BlueprintProjectSettingsChangedHandle = GetMutableDefault<UBlueprintEditorProjectSettings>()->OnSettingChanged()
		.AddRaw(this, &FBlueprintEditor::OnBlueprintProjectSettingsChanged);

	if (const TSharedPtr<SSCSEditorViewport> Viewport = GetSubobjectViewport())
	{
		ViewportSelectabilityBridge = MakeUnique<FEditorViewportSelectabilityBridge>(Viewport->GetViewportClient());

		if (const TSharedPtr<FEditorViewportClient>& ViewportClient = Viewport->GetViewportClient())
		{
			if (FEditorModeTools* const ModeTools = ViewportClient->GetModeTools())
			{
				// Opt int to new ITF gizmos and viewport inputs
				ModeTools->SetSupportsViewportITF(true);
			}
		}
	}
}

void FBlueprintEditor::InitToolMenuContext(FToolMenuContext& MenuContext)
{
	FAssetEditorToolkit::InitToolMenuContext(MenuContext);

	UBlueprintEditorToolMenuContext* Context = NewObject<UBlueprintEditorToolMenuContext>();
	Context->BlueprintEditor = SharedThis(this);
	MenuContext.AddObject(Context);
}

void FBlueprintEditor::InitalizeExtenders()
{
	FBlueprintEditorModule* BlueprintEditorModule = &FModuleManager::LoadModuleChecked<FBlueprintEditorModule>("Kismet");
	TSharedPtr<FExtender> CustomExtenders = BlueprintEditorModule->GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects());
	BlueprintEditorModule->OnGatherBlueprintMenuExtensions().Broadcast(CustomExtenders, GetBlueprintObj());

	AddMenuExtender(CustomExtenders);
	AddToolbarExtender(CustomExtenders);
}

void FBlueprintEditor::RegisterMenus()
{
	const FName MainMenuName = GetToolMenuName();
	if (!UToolMenus::Get()->IsMenuRegistered(MainMenuName))
	{
		FKismet2Menu::SetupBlueprintEditorMenu(MainMenuName);
	}
}

void FBlueprintEditor::RegisterApplicationModes(const TArray<UBlueprint*>& InBlueprints, bool bShouldOpenInDefaultsMode, bool bNewlyCreated/* = false*/)
{
	// Newly-created Blueprints will open in Components mode rather than Standard mode
	bool bShouldOpenInComponentsMode = !bShouldOpenInDefaultsMode && bNewlyCreated;

	// Create the modes and activate one (which will populate with a real layout)
	if ( UBlueprint* SingleBP = GetBlueprintObj() )
	{
		if ( !bShouldOpenInDefaultsMode && FBlueprintEditorUtils::IsInterfaceBlueprint(SingleBP) )
		{
			// Interfaces are only valid in the Interface mode
			AddApplicationMode(
				FBlueprintEditorApplicationModes::BlueprintInterfaceMode,
				MakeShareable(new FBlueprintInterfaceApplicationMode(SharedThis(this))));
			SetCurrentMode(FBlueprintEditorApplicationModes::BlueprintInterfaceMode);
		}
		else if ( SingleBP->BlueprintType == BPTYPE_MacroLibrary )
		{
			// Macro libraries are only valid in the Macro mode
			AddApplicationMode(
				FBlueprintEditorApplicationModes::BlueprintMacroMode,
				MakeShareable(new FBlueprintMacroApplicationMode(SharedThis(this))));
			SetCurrentMode(FBlueprintEditorApplicationModes::BlueprintMacroMode);
		}
		else if ( SingleBP->BlueprintType == BPTYPE_FunctionLibrary )
		{
			AddApplicationMode(
				FBlueprintEditorApplicationModes::StandardBlueprintEditorMode,
				MakeShareable(new FBlueprintEditorUnifiedMode(SharedThis(this), FBlueprintEditorApplicationModes::StandardBlueprintEditorMode, FBlueprintEditorApplicationModes::GetLocalizedMode, CanAccessComponentsMode())));
			SetCurrentMode(FBlueprintEditorApplicationModes::StandardBlueprintEditorMode);
		}
		else
		{
			if ( bShouldOpenInDefaultsMode )
			{
				// We either have no blueprints or many, open in the defaults mode for multi-editing
				AddApplicationMode(
					FBlueprintEditorApplicationModes::BlueprintDefaultsMode,
					MakeShareable(new FBlueprintDefaultsApplicationMode(SharedThis(this))));
				SetCurrentMode(FBlueprintEditorApplicationModes::BlueprintDefaultsMode);
			}
			else
			{
				AddApplicationMode(
					FBlueprintEditorApplicationModes::StandardBlueprintEditorMode,
					MakeShareable(new FBlueprintEditorUnifiedMode(SharedThis(this), FBlueprintEditorApplicationModes::StandardBlueprintEditorMode, FBlueprintEditorApplicationModes::GetLocalizedMode, CanAccessComponentsMode())));
				SetCurrentMode(FBlueprintEditorApplicationModes::StandardBlueprintEditorMode);

				if ( bShouldOpenInComponentsMode && CanAccessComponentsMode() )
				{
					TabManager->TryInvokeTab(FBlueprintEditorTabs::SCSViewportID);
				}
			}
		}
	}
	else
	{
		// We either have no blueprints or many, open in the defaults mode for multi-editing
		AddApplicationMode(
			FBlueprintEditorApplicationModes::BlueprintDefaultsMode,
			MakeShareable(new FBlueprintDefaultsApplicationMode(SharedThis(this))));
		SetCurrentMode(FBlueprintEditorApplicationModes::BlueprintDefaultsMode);
	}
}

void FBlueprintEditor::PostRegenerateMenusAndToolbars()
{
	UBlueprint* BluePrint = GetBlueprintObj();

	if ( BluePrint && !FBlueprintEditorUtils::IsLevelScriptBlueprint( BluePrint ) )
	{
		const FText NativeClassNameText = GetParentClassNativeNameText();

		// build and attach the menu overlay
		TSharedRef<SHorizontalBox> MenuOverlayBox = SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.ColorAndOpacity( FSlateColor::UseSubduedForeground() )
				.ShadowOffset( FVector2D::UnitVector )
				.Text(LOCTEXT("BlueprintEditor_ParentClass", "Parent class: "))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SSpacer)
				.Size(FVector2D(2.0f,1.0f))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.ShadowOffset(FVector2D::UnitVector)
				.Text(this, &FBlueprintEditor::GetParentClassNameText)
				.TextStyle(FAppStyle::Get(), "Common.InheritedFromBlueprintTextStyle")
				.ToolTipText(LOCTEXT("ParentClassToolTip", "The class that the current Blueprint is based on. The parent provides the base definition, which the current Blueprint extends."))
				.Visibility(this, &FBlueprintEditor::GetParentClassNameVisibility)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.VAlign(VAlign_Center)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.OnClicked(this, &FBlueprintEditor::OnFindParentClassInContentBrowserClicked)
				.IsEnabled(this, &FBlueprintEditor::IsParentClassABlueprint)
				.Visibility(this, &FBlueprintEditor::GetFindParentClassVisibility)
				.ToolTipText(LOCTEXT("FindParentInCBToolTip", "Find parent in Content Browser"))
				.ContentPadding(4.0f)
				.ForegroundColor(FSlateColor::UseForeground())
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Search"))
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.VAlign(VAlign_Center)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.OnClicked(this, &FBlueprintEditor::OnEditParentClassClicked)
				.IsEnabled(this, &FBlueprintEditor::IsParentClassAnEditableBlueprint)
				.Visibility(this, &FBlueprintEditor::GetEditParentClassVisibility)
				.ToolTipText(LOCTEXT("EditParentClassToolTip", "Open parent in editor"))
				.ContentPadding(4.0f)
				.ForegroundColor(FSlateColor::UseForeground())
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Edit"))
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SHyperlink)
				.Style(FAppStyle::Get(), "Common.GotoNativeCodeHyperlink")
				.IsEnabled(this, &FBlueprintEditor::IsNativeParentClassCodeLinkEnabled)
				.Visibility(this, &FBlueprintEditor::GetNativeParentClassButtonsVisibility)
				.OnNavigate(this, &FBlueprintEditor::OnEditParentClassNativeCodeClicked)
				.Text(this, &FBlueprintEditor::GetTextForNativeParentClassHeaderLink)
				.ToolTipText(FText::Format(LOCTEXT("GoToCode_ToolTip", "Click to open the source file for the underlying class ({0}) in {1}"), 
					NativeClassNameText, FSourceCodeNavigation::GetSelectedSourceCodeIDE()))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SSpacer)
				.Size(FVector2D(8.0f, 1.0f))
			]
			;
		SetMenuOverlay( MenuOverlayBox );
	}
}

FText FBlueprintEditor::GetParentClassNameText() const
{
	UClass* ParentClass = (GetBlueprintObj() != nullptr) ? *GetBlueprintObj()->ParentClass : nullptr;
	return (ParentClass != nullptr) ? ParentClass->GetDisplayNameText() : LOCTEXT("BlueprintEditor_NoParentClass", "None");
}

FText FBlueprintEditor::GetParentClassNativeNameText() const
{
	UClass* ParentClass = (GetBlueprintObj() != nullptr) ? *GetBlueprintObj()->ParentClass : nullptr;
	return (ParentClass != nullptr) ? FText::FromName(ParentClass->GetFName()) : LOCTEXT("BlueprintEditor_NoParentClass", "None");
}

bool FBlueprintEditor::IsParentClassABlueprint() const
{
	return FBlueprintEditorUtils::IsParentClassABlueprint(GetBlueprintObj());
}

bool FBlueprintEditor::IsParentClassAnEditableBlueprint() const
{
	return FBlueprintEditorUtils::IsParentClassAnEditableBlueprint(GetBlueprintObj());
}

EVisibility FBlueprintEditor::GetFindParentClassVisibility() const
{
	return IsParentClassABlueprint() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FBlueprintEditor::GetEditParentClassVisibility() const
{
	return IsParentClassAnEditableBlueprint() ? EVisibility::Visible : EVisibility::Collapsed;
}

bool FBlueprintEditor::IsParentClassNative() const
{
	const UBlueprint* Blueprint = GetBlueprintObj();
	if (Blueprint != nullptr)
	{
		UClass* ParentClass = Blueprint->ParentClass;
		if (ParentClass != nullptr)
		{
			if (ParentClass->HasAllClassFlags(CLASS_Native))
			{
				return true;
			}
		}
	}

	return false;
}

bool FBlueprintEditor::IsNativeParentClassCodeLinkEnabled() const
{
	const UBlueprint* Blueprint = GetBlueprintObj();
	return Blueprint && FSourceCodeNavigation::CanNavigateToClass(Blueprint->ParentClass);
}

EVisibility FBlueprintEditor::GetNativeParentClassButtonsVisibility() const
{
	return IsNativeParentClassCodeLinkEnabled() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FBlueprintEditor::GetParentClassNameVisibility() const
{
	return !IsNativeParentClassCodeLinkEnabled() ? EVisibility::Visible : EVisibility::Collapsed;
}

void FBlueprintEditor::OnEditParentClassNativeCodeClicked()
{
	const UBlueprint* Blueprint = GetBlueprintObj();
	if (Blueprint)
	{
		FSourceCodeNavigation::NavigateToClass(Blueprint->ParentClass);
	}
}

FText FBlueprintEditor::GetTextForNativeParentClassHeaderLink() const
{
	// it could be done using FSourceCodeNavigation, but it could be slow
	return GetParentClassNameText();
}

FReply FBlueprintEditor::OnFindParentClassInContentBrowserClicked()
{
	UBlueprint* Blueprint = GetBlueprintObj();
	if (Blueprint)
	{
		UObject* ParentClass = Blueprint->ParentClass;
		if (ParentClass)
		{
			UBlueprintGeneratedClass* ParentBlueprintGeneratedClass = Cast<UBlueprintGeneratedClass>( ParentClass );
			if (ParentBlueprintGeneratedClass)
			{
				TArray<UObject*> ParentObjectList;
				if (ParentBlueprintGeneratedClass->ClassGeneratedBy)
				{
					ParentObjectList.Add(ParentBlueprintGeneratedClass->ClassGeneratedBy);
				}
				else
				{
					ParentObjectList.Add(ParentBlueprintGeneratedClass);
				}
				GEditor->SyncBrowserToObjects(ParentObjectList);
			}
		}
	}

	return FReply::Handled();
}

FReply FBlueprintEditor::OnEditParentClassClicked()
{
	UBlueprint* Blueprint = GetBlueprintObj();
	if (Blueprint)
	{
		UObject* ParentClass = Blueprint->ParentClass;
		if (ParentClass)
		{
			UBlueprintGeneratedClass* ParentBlueprintGeneratedClass = Cast<UBlueprintGeneratedClass>( ParentClass );
			if (ParentBlueprintGeneratedClass)
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ParentBlueprintGeneratedClass->ClassGeneratedBy);
				
				if (UObject* DebugObject = Blueprint->GetObjectBeingDebugged())
				{
					if (UBlueprint* ParentBlueprint = Cast<UBlueprint>(ParentBlueprintGeneratedClass->ClassGeneratedBy))
					{
						ParentBlueprint->SetObjectBeingDebugged(DebugObject);
					}
				}
			}
		}
	}

	return FReply::Handled();
}

void FBlueprintEditor::PostLayoutBlueprintEditorInitialization()
{
	if (UBlueprint* Blueprint = GetBlueprintObj())
	{
		// Refresh the graphs
		RefreshEditors();
		
		// EnsureBlueprintIsUpToDate may have updated the blueprint so show notifications to user.
		if (bBlueprintModifiedOnOpen)
		{
			bBlueprintModifiedOnOpen = false;

			if (FocusedGraphEdPtr.IsValid())
			{
				FNotificationInfo Info( NSLOCTEXT("Kismet", "Blueprint Modified", "Blueprint requires updating. Please resave.") );
				Info.Image = FAppStyle::GetBrush(TEXT("Icons.Info"));
				Info.bFireAndForget = true;
				Info.bUseSuccessFailIcons = false;
				Info.ExpireDuration = 5.0f;

				FocusedGraphEdPtr.Pin()->AddNotification(Info, true);
			}

			// Fire log message
			FString BlueprintName;
			Blueprint->GetName(BlueprintName);

			FFormatNamedArguments Args;
			Args.Add( TEXT("BlueprintName"), FText::FromString( BlueprintName ) );
			LogSimpleMessage( FText::Format( LOCTEXT("Blueprint Modified Long", "Blueprint \"{BlueprintName}\" was updated to fix issues detected on load. Please resave."), Args ) );
		}

		// Determine if the current "mode" supports invoking the Compiler Results tab.
		const bool bCanInvokeCompilerResultsTab = TabManager->HasTabSpawner(FBlueprintEditorTabs::CompilerResultsID);

		// If we have a warning/error, open output log if the current mode allows us to invoke it.
		const bool bIsBlueprintInWarningOrErrorState = !Blueprint->IsUpToDate() || (Blueprint->Status == BS_UpToDateWithWarnings);
		if (bIsBlueprintInWarningOrErrorState && bCanInvokeCompilerResultsTab)
		{
			TabManager->TryInvokeTab(FBlueprintEditorTabs::CompilerResultsID);
		}
		else
		{
			// Toolkit modes that don't include this tab may have been incorrectly saved with layout information for restoring it
			// as an "unrecognized" tab, due to having previously invoked it above without checking to see if the layout can open
			// it first. To correct this, we check if the tab was restored from a saved layout here, and close it if not supported.
			TSharedPtr<SDockTab> TabPtr = TabManager->FindExistingLiveTab(FBlueprintEditorTabs::CompilerResultsID);
			if (TabPtr.IsValid() && !bCanInvokeCompilerResultsTab)
			{
				TabPtr->RequestCloseTab();
			}
		}
	}
}

void FBlueprintEditor::SetupViewForBlueprintEditingMode()
{
	// Make sure the defaults tab is pointing to the defaults
	StartEditingDefaults(/*bAutoFocus=*/ true);

	// Make sure the inspector is always on top
	//@TODO: This is necessary right now because of a bug in restoring layouts not remembering which tab is on top (to get it right initially), but do we want this behavior always?
	TryInvokingDetailsTab();

	UBlueprint* Blueprint = GetBlueprintObj();
	if ((Blueprint != nullptr) && (Blueprint->Status == EBlueprintStatus::BS_Error))
	{
		UBlueprintEditorSettings const* BpEditorSettings = GetDefault<UBlueprintEditorSettings>();
		if (BpEditorSettings->bJumpToNodeErrors)
		{
			if (UEdGraphNode* NodeWithError = BlueprintEditorImpl::FindNodeWithError(Blueprint))
			{
				JumpToNode(NodeWithError, /*bRequestRename =*/false);
			}
		}
	}
}

FBlueprintEditor::~FBlueprintEditor()
{
	// Stop watching the settings
	GetMutableDefault<UBlueprintEditorSettings>()->OnSettingChanged().Remove(BlueprintEditorSettingsChangedHandle);
	GetMutableDefault<UBlueprintEditorProjectSettings>()->OnSettingChanged().Remove(BlueprintProjectSettingsChangedHandle);

	// Clean up the preview
	DestroyPreview();

	// NOTE: Any tabs that we still have hanging out when destroyed will be cleaned up by FBaseToolkit's destructor
	UEditorEngine* Editor = (UEditorEngine*)GEngine;
	if (Editor)
	{
		Editor->UnregisterForUndo(this);
	}

	CloseMergeTool();

	if (GetBlueprintObj())
	{
		GetBlueprintObj()->OnChanged().RemoveAll(this);
		GetBlueprintObj()->OnCompiled().RemoveAll(this);
		GetBlueprintObj()->OnSetObjectBeingDebugged().RemoveAll(this);
	}

	if (USubobjectDataSubsystem* SubobjectDataSubsystem = USubobjectDataSubsystem::Get())
	{
		SubobjectDataSubsystem->OnNewSubobjectAdded().RemoveAll(this);
	}

	FCoreUObjectDelegates::OnPreObjectPropertyChanged.RemoveAll(this);
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);

	FKismetEditorUtilities::OnBlueprintUnloaded.RemoveAll(this);

	FGlobalTabmanager::Get()->OnActiveTabChanged_Unsubscribe(OnActiveTabChangedDelegateHandle);

	if (FEngineAnalytics::IsAvailable())
	{
		const UGeneralProjectSettings& ProjectSettings = *GetDefault<UGeneralProjectSettings>();
		FString ProjectID = ProjectSettings.ProjectID.ToString();

		TArray<FAnalyticsEventAttribute> BPEditorAttribs;
		BPEditorAttribs.Add(FAnalyticsEventAttribute(TEXT("GraphActionMenusExecuted.NonContextSensitive"), AnalyticsStats.GraphActionMenusNonCtxtSensitiveExecCount));
		BPEditorAttribs.Add(FAnalyticsEventAttribute(TEXT("GraphActionMenusExecuted.ContextSensitive"), AnalyticsStats.GraphActionMenusCtxtSensitiveExecCount));
		BPEditorAttribs.Add(FAnalyticsEventAttribute(TEXT("GraphActionMenusClosed"), AnalyticsStats.GraphActionMenusCancelledCount));

		BPEditorAttribs.Add(FAnalyticsEventAttribute(TEXT("MyBlueprintDragPlacedNodesCreated"), AnalyticsStats.MyBlueprintNodeDragPlacementCount));
		BPEditorAttribs.Add(FAnalyticsEventAttribute(TEXT("BlueprintPaletteDragPlacedNodesCreated"), AnalyticsStats.PaletteNodeDragPlacementCount));
		BPEditorAttribs.Add(FAnalyticsEventAttribute(TEXT("GraphContextNodesCreated" ), AnalyticsStats.NodeGraphContextCreateCount));
		BPEditorAttribs.Add(FAnalyticsEventAttribute(TEXT("GraphPinContextNodesCreated" ), AnalyticsStats.NodePinContextCreateCount));
		BPEditorAttribs.Add(FAnalyticsEventAttribute(TEXT("KeymapNodesCreated"), AnalyticsStats.NodeKeymapCreateCount));
		BPEditorAttribs.Add(FAnalyticsEventAttribute(TEXT("PastedNodesCreated"), AnalyticsStats.NodePasteCreateCount));

		BPEditorAttribs.Add(FAnalyticsEventAttribute(TEXT("ProjectId"), ProjectID));
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.BlueprintEditorSummary"), BPEditorAttribs);

		for (auto Iter = AnalyticsStats.GraphDisallowedPinConnections.CreateConstIterator(); Iter; ++Iter)
		{
			TArray<FAnalyticsEventAttribute> BPEditorPinConnectAttribs;
			BPEditorPinConnectAttribs.Add(FAnalyticsEventAttribute(TEXT("FromPin.Category"), Iter->PinTypeCategoryA));
			BPEditorPinConnectAttribs.Add(FAnalyticsEventAttribute(TEXT("FromPin.IsArray"), Iter->bPinIsArrayA));
			BPEditorPinConnectAttribs.Add(FAnalyticsEventAttribute(TEXT("FromPin.IsReference"), Iter->bPinIsReferenceA));
			BPEditorPinConnectAttribs.Add(FAnalyticsEventAttribute(TEXT("FromPin.IsWeakPointer"), Iter->bPinIsWeakPointerA));
			BPEditorPinConnectAttribs.Add(FAnalyticsEventAttribute(TEXT("ToPin.Category"), Iter->PinTypeCategoryB));
			BPEditorPinConnectAttribs.Add(FAnalyticsEventAttribute(TEXT("ToPin.IsArray"), Iter->bPinIsArrayB));
			BPEditorPinConnectAttribs.Add(FAnalyticsEventAttribute(TEXT("ToPin.IsReference"), Iter->bPinIsReferenceB));
			BPEditorPinConnectAttribs.Add(FAnalyticsEventAttribute(TEXT("ToPin.IsWeakPointer"), Iter->bPinIsWeakPointerB));
			BPEditorPinConnectAttribs.Add(FAnalyticsEventAttribute(TEXT("ProjectId"), ProjectID));

			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.BPDisallowedPinConnection"), BPEditorPinConnectAttribs);
		}
	}

	SaveEditorSettings();
}

void FBlueprintEditor::FocusInspectorOnGraphSelection(const FGraphPanelSelectionSet& NewSelection, bool bForceRefresh)
{
	// If this graph has selected nodes update the details panel to match.
	if ( NewSelection.Num() > 0 || CurrentUISelection == FBlueprintEditor::SelectionState_Graph )
	{
		SetUISelectionState(FBlueprintEditor::SelectionState_Graph);

		SKismetInspector::FShowDetailsOptions ShowDetailsOptions;
		ShowDetailsOptions.bForceRefresh = bForceRefresh;

		Inspector->ShowDetailsForObjects(NewSelection.Array(), ShowDetailsOptions);
	}
}

void FBlueprintEditor::CreateDefaultTabContents(const TArray<UBlueprint*>& InBlueprints)
{
	UBlueprint* InBlueprint = InBlueprints.Num() == 1 ? InBlueprints[0] : nullptr;

	// Cache off whether or not this is an interface, since it is used to govern multiple widget's behavior
	const bool bIsInterface = (InBlueprint && InBlueprint->BlueprintType == BPTYPE_Interface);
	const bool bIsMacro = (InBlueprint && InBlueprint->BlueprintType == BPTYPE_MacroLibrary);

	if (InBlueprint)
	{
		this->BookmarksWidget =
			SNew(SBlueprintBookmarks)
				.EditorContext(SharedThis(this));
	}

	if (IsEditingSingleBlueprint())
	{
		this->MyBlueprintWidget = SNew(SMyBlueprint, SharedThis(this));
		this->ReplaceReferencesWidget = SNew(SReplaceNodeReferences, SharedThis(this));
	}
	
	CompilerResultsListing = FCompilerResultsLog::GetBlueprintMessageLog(InBlueprint);
	CompilerResultsListing->OnMessageTokenClicked().AddSP(this, &FBlueprintEditor::OnLogTokenClicked);

	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	CompilerResults = MessageLogModule.CreateLogListingWidget( CompilerResultsListing.ToSharedRef() );
	FindResults = SNew(SFindInBlueprints, SharedThis(this));
	
	this->Inspector = 
		SNew(SKismetInspector)
		. HideNameArea(true)
		. ViewIdentifier(FName("BlueprintInspector"))
		. Kismet2(SharedThis(this))
		. OnFinishedChangingProperties( FOnFinishedChangingProperties::FDelegate::CreateSP(this, &FBlueprintEditor::OnFinishedChangingProperties) );

	if ( InBlueprints.Num() > 0 )
	{
		// Don't show the object name in defaults mode.
		const bool bHideNameArea = bWasOpenedInDefaultsMode;

		this->DefaultEditor = 
			SNew(SKismetInspector)
			. Kismet2(SharedThis(this))
			. ViewIdentifier(FName("BlueprintDefaults"))
			. IsEnabled(!bIsInterface)
			. ShowPublicViewControl(this, &FBlueprintEditor::ShouldShowPublicViewControl)
			. ShowTitleArea(false)
			. HideNameArea(bHideNameArea)
			. OnFinishedChangingProperties( FOnFinishedChangingProperties::FDelegate::CreateSP( this, &FBlueprintEditor::OnFinishedChangingProperties ) );
	}

	if (InBlueprint && 
		InBlueprint->ParentClass &&
		InBlueprint->ParentClass->IsChildOf(AActor::StaticClass()) && 
		InBlueprint->SimpleConstructionScript )
	{
		CreateSubobjectEditors();
	}
}

void FBlueprintEditor::CreateSubobjectEditors()
{
	TArray<TSharedRef<IClassViewerFilter>> ClassFilters;
	if (ImportedClassViewerFilter.IsValid())
	{
		ClassFilters.Add(ImportedClassViewerFilter.ToSharedRef());
	}

	SubobjectEditor = SAssignNew(SubobjectEditor, SSubobjectBlueprintEditor)
		.ObjectContext(this, &FBlueprintEditor::GetSubobjectEditorObjectContext)
		.PreviewActor(this, &FBlueprintEditor::GetPreviewActor)
		.AllowEditing(this, &FBlueprintEditor::InEditingMode)
		.OnSelectionUpdated(this, &FBlueprintEditor::OnSelectionUpdated)
		.OnItemDoubleClicked(this, &FBlueprintEditor::OnComponentDoubleClicked)
		.SubobjectClassListFilters(ClassFilters)
		.SCSEditorUICustomization(SCSEditorUICustomization);

	LLM_SCOPE_BYNAME(TEXT("BPCreateSubobjectEditorViewportClient"));

	SubobjectViewport = SAssignNew(SubobjectViewport, SSCSEditorViewport)
		.BlueprintEditor(SharedThis(this))
		.ModeTools(EditorModeManager);

	if (USubobjectDataSubsystem* SubobjectDataSubsystem = USubobjectDataSubsystem::Get())
	{
		SubobjectDataSubsystem->OnNewSubobjectAdded().AddSP(this, &FBlueprintEditor::OnComponentAddedToBlueprint);
	}
}

void FBlueprintEditor::OnLogTokenClicked(const TSharedRef<IMessageToken>& Token)
{
	if (Token->GetType() == EMessageToken::Object)
	{
		const TSharedRef<FUObjectToken> UObjectToken = StaticCastSharedRef<FUObjectToken>(Token);
		if(UObjectToken->GetObject().IsValid())
		{
			JumpToHyperlink(UObjectToken->GetObject().Get());
		}
	}
	else if (Token->GetType() == EMessageToken::EdGraph)
	{
		const TSharedRef<FEdGraphToken> EdGraphToken = StaticCastSharedRef<FEdGraphToken>(Token);
		const UEdGraphPin* PinBeingReferenced = EdGraphToken->GetPin();
		const UObject* ObjectBeingReferenced = EdGraphToken->GetGraphObject();
		if (PinBeingReferenced)
		{
			JumpToPin(PinBeingReferenced);
		}
		else if(ObjectBeingReferenced)
		{
			JumpToHyperlink(ObjectBeingReferenced);
		}
	}
}

/** Create Default Commands **/
void FBlueprintEditor::CreateDefaultCommands()
{
	// Tell Kismet2 how to handle all the UI actions that it can handle
	// @todo: remove this once GraphEditorActions automatically register themselves.

	FGraphEditorCommands::Register();
	FBlueprintEditorCommands::Register();
	FFullBlueprintEditorCommands::Register();
	FMyBlueprintCommands::Register();
	FBlueprintSpawnNodeCommands::Register();

	static const FName BpEditorModuleName("Kismet");
	FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>(BpEditorModuleName);
	ToolkitCommands->Append(BlueprintEditorModule.GetsSharedBlueprintEditorCommands());

	ToolkitCommands->MapAction(
		FFullBlueprintEditorCommands::Get().Compile,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::Compile),
		FCanExecuteAction::CreateSP(this, &FBlueprintEditor::IsCompilingEnabled));

	TWeakPtr<FBlueprintEditor> WeakThisPtr = SharedThis(this);
	ToolkitCommands->MapAction(
		FFullBlueprintEditorCommands::Get().SaveOnCompile_Never,
		FExecuteAction::CreateStatic(&BlueprintEditorImpl::SetSaveOnCompileSetting, (ESaveOnCompile)SoC_Never),
		FCanExecuteAction::CreateSP(this, &FBlueprintEditor::IsSaveOnCompileEnabled),
		FIsActionChecked::CreateStatic(&BlueprintEditorImpl::IsSaveOnCompileOptionSet, WeakThisPtr, (ESaveOnCompile)SoC_Never)
	);
	ToolkitCommands->MapAction(
		FFullBlueprintEditorCommands::Get().SaveOnCompile_SuccessOnly,
		FExecuteAction::CreateStatic(&BlueprintEditorImpl::SetSaveOnCompileSetting, (ESaveOnCompile)SoC_SuccessOnly),
		FCanExecuteAction::CreateSP(this, &FBlueprintEditor::IsSaveOnCompileEnabled),
		FIsActionChecked::CreateStatic(&BlueprintEditorImpl::IsSaveOnCompileOptionSet, WeakThisPtr, (ESaveOnCompile)SoC_SuccessOnly)
	);
	ToolkitCommands->MapAction(
		FFullBlueprintEditorCommands::Get().SaveOnCompile_Always,
		FExecuteAction::CreateStatic(&BlueprintEditorImpl::SetSaveOnCompileSetting, (ESaveOnCompile)SoC_Always),
		FCanExecuteAction::CreateSP(this, &FBlueprintEditor::IsSaveOnCompileEnabled),
		FIsActionChecked::CreateStatic(&BlueprintEditorImpl::IsSaveOnCompileOptionSet, WeakThisPtr, (ESaveOnCompile)SoC_Always)
	);

	ToolkitCommands->MapAction(
		FFullBlueprintEditorCommands::Get().JumpToErrorNode,
		FExecuteAction::CreateStatic(&BlueprintEditorImpl::ToggleJumpToErrorNodeSetting),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic(&BlueprintEditorImpl::IsJumpToErrorNodeOptionSet)
	);
	
	ToolkitCommands->MapAction(
		FFullBlueprintEditorCommands::Get().SwitchToScriptingMode,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::SetCurrentMode, FBlueprintEditorApplicationModes::StandardBlueprintEditorMode),
		FCanExecuteAction::CreateSP(this, &FBlueprintEditor::IsEditingSingleBlueprint),
		FIsActionChecked::CreateSP(this, &FBlueprintEditor::IsModeCurrent, FBlueprintEditorApplicationModes::StandardBlueprintEditorMode));

	ToolkitCommands->MapAction(
		FFullBlueprintEditorCommands::Get().SwitchToBlueprintDefaultsMode,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::SetCurrentMode, FBlueprintEditorApplicationModes::BlueprintDefaultsMode),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FBlueprintEditor::IsModeCurrent, FBlueprintEditorApplicationModes::BlueprintDefaultsMode));
	
	ToolkitCommands->MapAction(
		FFullBlueprintEditorCommands::Get().SwitchToComponentsMode,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::SetCurrentMode, FBlueprintEditorApplicationModes::BlueprintComponentsMode),
		FCanExecuteAction::CreateSP(this, &FBlueprintEditor::CanAccessComponentsMode),
		FIsActionChecked::CreateSP(this, &FBlueprintEditor::IsModeCurrent, FBlueprintEditorApplicationModes::BlueprintComponentsMode));

	ToolkitCommands->MapAction(
		FFullBlueprintEditorCommands::Get().EditGlobalOptions,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::EditGlobalOptions_Clicked),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FBlueprintEditor::IsDetailsPanelEditingGlobalOptions));

	ToolkitCommands->MapAction(
		FFullBlueprintEditorCommands::Get().EditClassDefaults,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::EditClassDefaults_Clicked),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FBlueprintEditor::IsDetailsPanelEditingClassDefaults));

	// Edit menu actions
	ToolkitCommands->MapAction( FBlueprintEditorCommands::Get().FindInBlueprint,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::FindInBlueprint_Clicked),
		FCanExecuteAction::CreateSP( this, &FBlueprintEditor::IsInAScriptingMode )
		);

	ToolkitCommands->MapAction( FBlueprintEditorCommands::Get().ReparentBlueprint,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::ReparentBlueprint_Clicked),
		FCanExecuteAction(),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &FBlueprintEditor::ReparentBlueprint_IsVisible)
		);

	ToolkitCommands->MapAction( FGenericCommands::Get().Undo,
		FExecuteAction::CreateSP( this, &FBlueprintEditor::UndoGraphAction ),
		FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanUndoGraphAction )
		);

	ToolkitCommands->MapAction( FGenericCommands::Get().Redo,
		FExecuteAction::CreateSP( this, &FBlueprintEditor::RedoGraphAction ),
		FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanRedoGraphAction )
		);


	// View commands
	ToolkitCommands->MapAction( FBlueprintEditorCommands::Get().ZoomToWindow,
		FExecuteAction::CreateSP( this, &FBlueprintEditor::ZoomToWindow_Clicked ),
		FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanZoomToWindow )
		);

	ToolkitCommands->MapAction( FBlueprintEditorCommands::Get().ZoomToSelection,
		FExecuteAction::CreateSP( this, &FBlueprintEditor::ZoomToSelection_Clicked ),
		FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanZoomToSelection )
		);

	ToolkitCommands->MapAction( FBlueprintEditorCommands::Get().NavigateToParent,
		FExecuteAction::CreateSP( this, &FBlueprintEditor::NavigateToParentGraph ),
		FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanNavigateToParentGraph )
		);

	ToolkitCommands->MapAction( FBlueprintEditorCommands::Get().NavigateToParentBackspace,
		FExecuteAction::CreateSP( this, &FBlueprintEditor::NavigateToParentGraph ),
		FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanNavigateToParentGraph )
		);

	ToolkitCommands->MapAction( FBlueprintEditorCommands::Get().NavigateToChild,
		FExecuteAction::CreateSP( this, &FBlueprintEditor::NavigateToChildGraph ),
		FCanExecuteAction::CreateSP( this, &FBlueprintEditor::CanNavigateToChildGraph )
		);

	ToolkitCommands->MapAction(FGraphEditorCommands::Get().ShowAllPins,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::SetPinVisibility, SGraphEditor::Pin_Show),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FBlueprintEditor::GetPinVisibility, SGraphEditor::Pin_Show));

	ToolkitCommands->MapAction(FGraphEditorCommands::Get().HideNoConnectionPins,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::SetPinVisibility, SGraphEditor::Pin_HideNoConnection),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FBlueprintEditor::GetPinVisibility, SGraphEditor::Pin_HideNoConnection));

	ToolkitCommands->MapAction(FGraphEditorCommands::Get().HideNoConnectionNoDefaultPins,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::SetPinVisibility, SGraphEditor::Pin_HideNoConnectionNoDefault),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FBlueprintEditor::GetPinVisibility, SGraphEditor::Pin_HideNoConnectionNoDefault));

	// Compile
	ToolkitCommands->MapAction( FBlueprintEditorCommands::Get().CompileBlueprint,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::Compile),
		FCanExecuteAction::CreateSP(this, &FBlueprintEditor::IsCompilingEnabled)
		);

	ToolkitCommands->MapAction( FBlueprintEditorCommands::Get().RefreshAllNodes,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::RefreshAllNodes_OnClicked),
		FCanExecuteAction::CreateSP( this, &FBlueprintEditor::IsInAScriptingMode )
		);

	ToolkitCommands->MapAction( FBlueprintEditorCommands::Get().DeleteUnusedVariables,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::DeleteUnusedVariables_OnClicked),
		FCanExecuteAction::CreateSP( this, &FBlueprintEditor::IsInAScriptingMode )
		);
	
	ToolkitCommands->MapAction( FBlueprintEditorCommands::Get().FindInBlueprints,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::FindInBlueprints_OnClicked)
		);

	// Debug actions
	ToolkitCommands->MapAction( FBlueprintEditorCommands::Get().ClearAllBreakpoints,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::ClearAllBreakpoints),
		FCanExecuteAction::CreateSP(this, &FBlueprintEditor::HasAnyBreakpoints)
		);

	ToolkitCommands->MapAction( FBlueprintEditorCommands::Get().DisableAllBreakpoints,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::DisableAllBreakpoints),
		FCanExecuteAction::CreateSP(this, &FBlueprintEditor::HasAnyEnabledBreakpoints)
		);

	ToolkitCommands->MapAction( FBlueprintEditorCommands::Get().EnableAllBreakpoints,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::EnableAllBreakpoints),
		FCanExecuteAction::CreateSP(this, &FBlueprintEditor::HasAnyDisabledBreakpoints)
		);

	ToolkitCommands->MapAction( FBlueprintEditorCommands::Get().ClearAllWatches,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::ClearAllWatches),
		FCanExecuteAction::CreateSP(this, &FBlueprintEditor::HasAnyWatches)
		);

	ToolkitCommands->MapAction( FBlueprintEditorCommands::Get().OpenBlueprintDebugger,
	    FExecuteAction::CreateSP(this, &FBlueprintEditor::OpenBlueprintDebugger),
	    FCanExecuteAction::CreateSP(this, &FBlueprintEditor::CanOpenBlueprintDebugger)
    );

	// New document actions
	ToolkitCommands->MapAction( FBlueprintEditorCommands::Get().AddNewVariable,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::OnAddNewVariable),
		FCanExecuteAction::CreateSP(this, &FBlueprintEditor::InEditingMode),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &FBlueprintEditor::NewDocument_IsVisibleForType, CGT_NewVariable));
	
	ToolkitCommands->MapAction( FBlueprintEditorCommands::Get().AddNewLocalVariable,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::OnAddNewLocalVariable),
		FCanExecuteAction::CreateSP(this, &FBlueprintEditor::CanAddNewLocalVariable),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &FBlueprintEditor::NewDocument_IsVisibleForType, CGT_NewLocalVariable));

	ToolkitCommands->MapAction( FBlueprintEditorCommands::Get().AddNewFunction,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::NewDocument_OnClicked, CGT_NewFunctionGraph),
		FCanExecuteAction::CreateSP(this, &FBlueprintEditor::InEditingMode),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &FBlueprintEditor::NewDocument_IsVisibleForType, CGT_NewFunctionGraph));

	ToolkitCommands->MapAction( FBlueprintEditorCommands::Get().AddNewEventGraph,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::NewDocument_OnClicked, CGT_NewEventGraph),
		FCanExecuteAction::CreateSP(this, &FBlueprintEditor::InEditingMode),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &FBlueprintEditor::NewDocument_IsVisibleForType, CGT_NewEventGraph));

	ToolkitCommands->MapAction( FBlueprintEditorCommands::Get().AddNewMacroDeclaration,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::NewDocument_OnClicked, CGT_NewMacroGraph),
		FCanExecuteAction::CreateSP(this, &FBlueprintEditor::InEditingMode),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &FBlueprintEditor::NewDocument_IsVisibleForType, CGT_NewMacroGraph));

	ToolkitCommands->MapAction( FBlueprintEditorCommands::Get().AddNewDelegate,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::OnAddNewDelegate),
		FCanExecuteAction::CreateSP(this, &FBlueprintEditor::InEditingMode),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &FBlueprintEditor::AddNewDelegateIsVisible));

	ToolkitCommands->MapAction( FBlueprintEditorCommands::Get().FindReferencesFromClass,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::OnListObjectsReferencedByClass),
		FCanExecuteAction());

	ToolkitCommands->MapAction( FBlueprintEditorCommands::Get().FindReferencesFromBlueprint,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::OnListObjectsReferencedByBlueprint),
		FCanExecuteAction());

	ToolkitCommands->MapAction( FBlueprintEditorCommands::Get().RepairCorruptedBlueprint,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::OnRepairCorruptedBlueprint),
		FCanExecuteAction());

	ToolkitCommands->MapAction( FBlueprintEditorCommands::Get().AddNewAnimationLayer,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::NewDocument_OnClicked, CGT_NewAnimationLayer),
		FCanExecuteAction(),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &FBlueprintEditor::NewDocument_IsVisibleForType, CGT_NewAnimationLayer));
	
	ToolkitCommands->MapAction(FBlueprintEditorCommands::Get().SaveIntermediateBuildProducts,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::ToggleSaveIntermediateBuildProducts),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FBlueprintEditor::GetSaveIntermediateBuildProducts));

	ToolkitCommands->MapAction(FBlueprintEditorCommands::Get().BeginBlueprintMerge,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::CreateMergeToolTab),
		FCanExecuteAction());

	ToolkitCommands->MapAction(FBlueprintEditorCommands::Get().GenerateSearchIndex,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::OnGenerateSearchIndexForDebugging),
		FCanExecuteAction());

	ToolkitCommands->MapAction(FBlueprintEditorCommands::Get().DumpCachedIndexData,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::OnDumpCachedIndexDataForBlueprint),
		FCanExecuteAction());

	ToolkitCommands->MapAction(FBlueprintEditorCommands::Get().ShowActionMenuItemSignatures,
		FExecuteAction::CreateLambda([]()
			{ 
				UBlueprintEditorSettings* Settings = GetMutableDefault<UBlueprintEditorSettings>();
				Settings->bShowActionMenuItemSignatures = !Settings->bShowActionMenuItemSignatures;
				Settings->SaveConfig();
			}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([]()->bool{ return GetDefault<UBlueprintEditorSettings>()->bShowActionMenuItemSignatures; }));

	for (int32 QuickJumpIndex = 0; QuickJumpIndex < FGraphEditorCommands::Get().QuickJumpCommands.Num(); ++QuickJumpIndex)
	{
		ToolkitCommands->MapAction(
			FGraphEditorCommands::Get().QuickJumpCommands[QuickJumpIndex].QuickJump,
			FExecuteAction::CreateSP(this, &FBlueprintEditor::OnGraphEditorQuickJump, QuickJumpIndex)
		);

		ToolkitCommands->MapAction(
			FGraphEditorCommands::Get().QuickJumpCommands[QuickJumpIndex].SetQuickJump,
			FExecuteAction::CreateSP(this, &FBlueprintEditor::SetGraphEditorQuickJump, QuickJumpIndex)
		);

		ToolkitCommands->MapAction(
			FGraphEditorCommands::Get().QuickJumpCommands[QuickJumpIndex].ClearQuickJump,
			FExecuteAction::CreateSP(this, &FBlueprintEditor::ClearGraphEditorQuickJump, QuickJumpIndex)
		);
	}

	ToolkitCommands->MapAction(
		FGraphEditorCommands::Get().ClearAllQuickJumps,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::ClearAllGraphEditorQuickJumps)
	);

	ToolkitCommands->MapAction(
		FBlueprintEditorCommands::Get().ToggleHideUnrelatedNodes,
		FExecuteAction::CreateSP(this, &FBlueprintEditor::ToggleHideUnrelatedNodes),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FBlueprintEditor::IsToggleHideUnrelatedNodesChecked),
		FIsActionButtonVisible()
	);
}

void FBlueprintEditor::OnGenerateSearchIndexForDebugging()
{
	UBlueprint* Blueprint = GetBlueprintObj();
	if (Blueprint)
	{
		FString FileLocation = FPaths::ConvertRelativePathToFull(FPaths::ProjectIntermediateDir() + TEXT("BlueprintSearchTools"));
		FString FullPath = FString::Printf(TEXT("%s/%s.index.xml"), *FileLocation, *Blueprint->GetName());

		FArchive* DumpFile = IFileManager::Get().CreateFileWriter(*FullPath);
		if (DumpFile)
		{
			FString JsonOutput = FFindInBlueprintSearchManager::Get().GenerateSearchIndexForDebugging(Blueprint);
			DumpFile->Serialize(TCHAR_TO_ANSI(*JsonOutput), JsonOutput.Len());

			DumpFile->Close();
			delete DumpFile;

			UE_LOGF(LogFindInBlueprint, Log, "Wrote search index to %ls", *FullPath);
		}
	}
}

void FBlueprintEditor::OnDumpCachedIndexDataForBlueprint()
{
	UBlueprint* Blueprint = GetBlueprintObj();
	if (Blueprint)
	{
		FString FileLocation = FPaths::ConvertRelativePathToFull(FPaths::ProjectIntermediateDir() + TEXT("BlueprintSearchTools"));
		FString FullPath = FString::Printf(TEXT("%s/%s.cache.csv"), *FileLocation, *Blueprint->GetName());

		FArchive* DumpFile = IFileManager::Get().CreateFileWriter(*FullPath);
		if (DumpFile)
		{
			const FSoftObjectPath AssetPath(Blueprint);
			FSearchData SearchData = FFindInBlueprintSearchManager::Get().GetSearchDataForAssetPath(AssetPath);
			if (SearchData.IsValid() && SearchData.ImaginaryBlueprint.IsValid())
			{
				SearchData.ImaginaryBlueprint->DumpParsedObject(*DumpFile);
			}

			DumpFile->Close();
			delete DumpFile;

			UE_LOGF(LogFindInBlueprint, Log, "Wrote cached index data to %ls", *FullPath);
		}
	}
}

void FBlueprintEditor::FindInBlueprint_Clicked()
{
	SummonSearchUI(true);
}

void FBlueprintEditor::ReparentBlueprint_Clicked()
{
	if (!ReparentBlueprint_IsVisible())
	{
		return;
	}

	TArray<UBlueprint*> Blueprints;
	for (int32 i = 0; i < GetEditingObjects().Num(); ++i)
	{
		UBlueprint* Blueprint = Cast<UBlueprint>(GetEditingObjects()[i]);
		if (Blueprint) 
		{
			Blueprints.Add(Blueprint);
		}
	}
	FBlueprintEditorUtils::OpenReparentBlueprintMenu(Blueprints, GetToolkitHost()->GetParentWidget(), FOnClassPicked::CreateSP(this, &FBlueprintEditor::ReparentBlueprint_NewParentChosen));
}

void FBlueprintEditor::ReparentBlueprint_NewParentChosen(UClass* ChosenClass)
{
	UBlueprint* BlueprintObj = GetBlueprintObj();

	if ((BlueprintObj != nullptr) && (ChosenClass != nullptr) && (ChosenClass != BlueprintObj->ParentClass))
	{
		// Notify user, about common interfaces
		bool bReparent = true;
		{
			FString CommonInterfacesNames;
			for (const FBPInterfaceDescription& InterdaceDesc : BlueprintObj->ImplementedInterfaces)
			{
				if (ChosenClass->ImplementsInterface(*InterdaceDesc.Interface))
				{
					CommonInterfacesNames += InterdaceDesc.Interface->GetName();
					CommonInterfacesNames += TCHAR('\n');
				}
			}
			if (!CommonInterfacesNames.IsEmpty())
			{
				const FText Title = LOCTEXT("CommonInterfacesTitle", "Common interfaces");
				const FText Message = FText::Format(
					LOCTEXT("ReparentWarning_InterfacesImplemented", "Following interfaces are already implemented. Continue reparenting? \n {0}"),
					FText::FromString(CommonInterfacesNames));

				FSuppressableWarningDialog::FSetupInfo Info(Message, Title, "Warning_CommonInterfacesWhileReparenting");
				Info.ConfirmText = LOCTEXT("ReparentYesButton", "Reparent");
				Info.CancelText = LOCTEXT("ReparentNoButton", "Cancel");

				FSuppressableWarningDialog ReparentBlueprintDlg(Info);
				if (ReparentBlueprintDlg.ShowModal() == FSuppressableWarningDialog::Cancel)
				{
					bReparent = false;
				}
			}
		}

		// If the chosen class differs hierarchically from the current class, warn that there may be data loss
		if (bReparent && (!BlueprintObj->ParentClass || !ChosenClass->GetDefaultObject()->IsA(BlueprintObj->ParentClass)))
		{
			const FText Title = LOCTEXT("ReparentTitle", "Reparent Blueprint"); 
			const FText Message = LOCTEXT("ReparentWarning", "Reparenting this blueprint may cause data loss.  Continue reparenting?"); 

			// Warn the user that this may result in data loss
			FSuppressableWarningDialog::FSetupInfo Info( Message, Title, "Warning_ReparentTitle" );
			Info.ConfirmText = LOCTEXT("ReparentYesButton", "Reparent");
			Info.CancelText = LOCTEXT("ReparentNoButton", "Cancel");
			Info.CheckBoxText = FText::GetEmpty();	// not suppressible

			FSuppressableWarningDialog ReparentBlueprintDlg( Info );
			if( ReparentBlueprintDlg.ShowModal() == FSuppressableWarningDialog::Cancel )
			{
				bReparent = false;
			}
		}

		if ( bReparent )
		{
			// Notify that we are currently reparenting this blueprint so that we get the proper compilation flags
			TGuardValue<bool> GuardValue(bIsReparentingBlueprint, true);

			const FScopedTransaction Transaction( LOCTEXT("ReparentBlueprint", "Reparent Blueprint") );
			UE_LOGF(LogBlueprint, Warning, "Reparenting blueprint %ls from %ls to %ls...", *BlueprintObj->GetFullName(), BlueprintObj->ParentClass ? *BlueprintObj->ParentClass->GetName() : TEXT("[None]"), *ChosenClass->GetName());
			
			BlueprintObj->Modify();
			if(USimpleConstructionScript* SCS = BlueprintObj->SimpleConstructionScript)
			{
				SCS->Modify();

				const TArray<USCS_Node*>& AllNodes = SCS->GetAllNodes();
				for(USCS_Node* Node : AllNodes )
				{
					Node->Modify();
				}
			}

			// Gather the set of default imports with the old parent class set.
			TSet<FString> OldDefaultImports;
			FBlueprintNamespaceUtilities::GetDefaultImportsForObject(BlueprintObj, OldDefaultImports);

			UClass* OldParentClass = BlueprintObj->ParentClass;
			BlueprintObj->ParentClass = ChosenClass;

			// Gather the set of default imports with the new parent class set.
			TSet<FString> NewDefaultImports;
			FBlueprintNamespaceUtilities::GetDefaultImportsForObject(BlueprintObj, NewDefaultImports);

			// Move namespace imports that no longer appear in the default set to the explicit set.
			FImportNamespaceExParameters Params;
			Params.bIsAutoImport = false;
			Params.NamespacesToImport = OldDefaultImports.Difference(NewDefaultImports);
			ImportNamespaceEx(Params);

			// Ensure that the Blueprint is up-to-date (valid SCS etc.) before compiling
			EnsureBlueprintIsUpToDate(BlueprintObj);
			FBlueprintEditorUtils::RefreshAllNodes(GetBlueprintObj());
			FBlueprintEditorUtils::MarkBlueprintAsModified(BlueprintObj);

			// Changing the parent may change the sparse data used, so mark any current 
			// sparse data as requiring a conform post-compile against the new archetype
			if (UBlueprintGeneratedClass* Class = Cast<UBlueprintGeneratedClass>(BlueprintObj->GeneratedClass))
			{
				Class->PrepareToConformSparseClassData(ChosenClass->GetSparseClassDataStruct());
			}

			Compile();

			// Ensure that the Blueprint is up-to-date (valid SCS etc.) after compiling (new parent class)
			EnsureBlueprintIsUpToDate(BlueprintObj);

			if(SubobjectEditor.IsValid())
			{
				SubobjectEditor->UpdateTree();
			}
		}
	}

	//@TODO: This is probably insufficient as you could reparent the parent instead and we wouldn't get notified
	ModifyDuringPIEStatus = ESafeToModifyDuringPIEStatus::Unknown;

	FSlateApplication::Get().DismissAllMenus();
}

bool FBlueprintEditor::ReparentBlueprint_IsVisible() const
{
	UBlueprint* Blueprint = GetBlueprintObj();
	if (Blueprint != nullptr)
	{
		// Don't show the reparent option if it's an Interface or we're not in editing mode
		return !FBlueprintEditorUtils::IsInterfaceBlueprint(Blueprint) && InEditingMode() && (BPTYPE_FunctionLibrary != Blueprint->BlueprintType);
	}
	else
	{
		return false;
	}
}

bool FBlueprintEditor::IsDetailsPanelEditingGlobalOptions() const
{
	return CurrentUISelection == FBlueprintEditor::SelectionState_ClassSettings;
}

void FBlueprintEditor::EditGlobalOptions_Clicked()
{
	SetUISelectionState(FBlueprintEditor::SelectionState_ClassSettings);

	if (bWasOpenedInDefaultsMode)
	{
		RefreshStandAloneDefaultsEditor();
	}
	else
	{
		UBlueprint* Blueprint = GetBlueprintObj();
		if (Blueprint != nullptr)
		{
			// Show details for the Blueprint instance we're editing
			Inspector->ShowDetailsForSingleObject(Blueprint);

			TryInvokingDetailsTab();
		}
	}
}

bool FBlueprintEditor::IsDetailsPanelEditingClassDefaults() const
{
	if (bWasOpenedInDefaultsMode)
	{
		return !IsDetailsPanelEditingGlobalOptions();
	}

	UBlueprint* Blueprint = GetBlueprintObj();
	if ( Blueprint != nullptr )
	{
		if ( Blueprint->GeneratedClass != nullptr )
		{
			UObject* DefaultObject = GetBlueprintObj()->GeneratedClass->GetDefaultObject();
			return Inspector->IsSelected(DefaultObject);
		}
	}

	return false;
}

void FBlueprintEditor::EditClassDefaults_Clicked()
{
	StartEditingDefaults(true, true);
}

// Zooming to fit the entire graph
void FBlueprintEditor::ZoomToWindow_Clicked()
{
	if (SGraphEditor* GraphEd = FocusedGraphEdPtr.Pin().Get())
	{
		GraphEd->ZoomToFit(/*bOnlySelection=*/ false);
	}
}

bool FBlueprintEditor::CanZoomToWindow() const
{
	return FocusedGraphEdPtr.IsValid();
}

// Zooming to fit the current selection
void FBlueprintEditor::ZoomToSelection_Clicked()
{
	if (SGraphEditor* GraphEd = FocusedGraphEdPtr.Pin().Get())
	{
		GraphEd->ZoomToFit(/*bOnlySelection=*/ true);
	}
}

bool FBlueprintEditor::CanZoomToSelection() const
{
	return FocusedGraphEdPtr.IsValid();
}

// Navigating into/out of graphs
void FBlueprintEditor::NavigateToParentGraphByDoubleClick()
{
	UBlueprintEditorSettings const* Settings = GetDefault<UBlueprintEditorSettings>();
	if (Settings->bDoubleClickNavigatesToParent)
	{
		NavigateToParentGraph();
	}
}

void FBlueprintEditor::NavigateToParentGraph()
{
	if (FocusedGraphEdPtr.IsValid())
	{
		if (UEdGraph* ParentGraph = UEdGraph::GetOuterGraph(FocusedGraphEdPtr.Pin()->GetCurrentGraph()))
		{
			JumpToHyperlink(ParentGraph, false);
		}
	}
}

bool FBlueprintEditor::CanNavigateToParentGraph() const
{
	return FocusedGraphEdPtr.IsValid() && UEdGraph::GetOuterGraph(FocusedGraphEdPtr.Pin()->GetCurrentGraph());
}

void FBlueprintEditor::NavigateToChildGraph()
{
	if (FocusedGraphEdPtr.IsValid())
	{
		UEdGraph* CurrentGraph = FocusedGraphEdPtr.Pin()->GetCurrentGraph();

		if (CurrentGraph->Nodes.Num() > 0)
		{
			// Display a child jump list
			FMenuBuilder MenuBuilder(true, nullptr);
			MenuBuilder.BeginSection("NavigateToGraph", LOCTEXT("ChildGraphPickerDesc", "Navigate to graph"));

			TArray<TObjectPtr<UEdGraphNode>> SortedGraphNodes = CurrentGraph->Nodes;
			SortedGraphNodes.Sort([](UEdGraphNode& A, UEdGraphNode& B) {
				FText AName = A.GetNodeTitle(ENodeTitleType::ListView);
				FText BName = B.GetNodeTitle(ENodeTitleType::ListView);
				return AName.CompareToCaseIgnored(BName) < 0; });

			TObjectPtr<UEdGraphNode> SingleNode;
			int32 NumEntries = 0;

			for (const TObjectPtr<UEdGraphNode>& Node : SortedGraphNodes)
			{
				// Just calling CanJumpToDefinition isn't enough as it returns true for functions (resulting in a jump
				// to code, which isn't desired).
				UObject* TargetObject = Node->GetJumpTargetForDoubleClick();
				if (TargetObject && Node->CanJumpToDefinition())
				{
					++NumEntries;
					SingleNode = Node;
					MenuBuilder.AddMenuEntry(
						Node->GetNodeTitle(ENodeTitleType::ListView),
						LOCTEXT("ChildGraphPickerTooltip", "Pick the graph to enter"),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateLambda(
								[this, Node]() 
								{
									if (Node->CanJumpToDefinition())
									{
										Node->JumpToDefinition();
									}
									// Note that setting the keyboard focus here doesn't work when navigating into a
									// blendspace (i.e. a UEdGraph). Neither does trying to set the focus to 
									// Node->DEPRECATED_NodeWidget
									SetKeyboardFocus();
								}),
							FCanExecuteAction()));
				}
			}
			MenuBuilder.EndSection();
			
			if (NumEntries > 0)
			{
				// If there is only one entry we could just jump straight to it. However, sometimes that can be a little
				// disorientating if it was not obvious to the user exactly what the targets might be.
				FSlateApplication::Get().PushMenu( 
					GetToolkitHost()->GetParentWidget(),
					FWidgetPath(),
					MenuBuilder.MakeWidget(),
					FSlateApplication::Get().GetCursorPos(), // summon location
					FPopupTransitionEffect( FPopupTransitionEffect::TypeInPopup )
				);
			}
		}
	}
}

bool FBlueprintEditor::CanNavigateToChildGraph() const
{
	if (FocusedGraphEdPtr.IsValid())
	{
		UEdGraph* CurrentGraph = FocusedGraphEdPtr.Pin()->GetCurrentGraph();
		for (const TObjectPtr<UEdGraphNode>& Node : CurrentGraph->Nodes)
		{
			UObject* TargetObject = Node->GetJumpTargetForDoubleClick();
			if (TargetObject && Node->CanJumpToDefinition())
			{
				return true;
			}
		}
	}
	return false;
}

void FBlueprintEditor::SetKeyboardFocus()
{
	FSlateApplication::Get().SetKeyboardFocus(GetMyBlueprintWidget());
}

TSharedRef<SBlueprintPalette> FBlueprintEditor::GetPalette()
{
	// Note: construction is deferred until first access; in large-scale projects this can be an expensive widget to construct during editor
	// initialization logic. It's an unnecessary cost if the tab it's hosted in is closed and/or unavailable (e.g. as in the default layout).
	if (!Palette.IsValid())
	{
		SAssignNew(Palette, SBlueprintPalette, SharedThis(this))
			.IsEnabled(this, &FBlueprintEditor::IsFocusedGraphEditable);
	}

	return Palette.ToSharedRef();
}

bool FBlueprintEditor::TransactionObjectAffectsBlueprint(UObject* InTransactedObject)
{
	check(InTransactedObject);

	UBlueprint* BlueprintObj = GetBlueprintObj();
	check(BlueprintObj);

	return InTransactedObject->GetOutermost() == BlueprintObj->GetOutermost();
}

void FBlueprintEditor::HandleUndoTransaction(const FTransaction* Transaction)
{
	UBlueprint* BlueprintObj = GetBlueprintObj();
	if (BlueprintObj && Transaction)
	{
		bool bAffectsBlueprint = false;
		const UPackage* BlueprintOutermost = BlueprintObj->GetOutermost();

		// Look at the transaction this function is responding to, see if any object in it has an outermost of the Blueprint
		TArray<UObject*> TransactionObjects;
		Transaction->GetTransactionObjects(TransactionObjects);

		for (UObject* Object : TransactionObjects)
		{
			if(TransactionObjectAffectsBlueprint(Object))
			{
				bAffectsBlueprint = true;
				break;
			}
		}

		// Transaction affects the Blueprint this editor handles, so react as necessary
		if (bAffectsBlueprint)
		{
			// Do not clear the selection on undo for the component tree so that the details panel
			// does not get cleared unnecessarily
			if(CurrentUISelection != SelectionState_Components)
			{
				SetUISelectionState(NAME_None);
			}

			RefreshEditors();

			FSlateApplication::Get().DismissAllMenus();
		}
	}
}

void FBlueprintEditor::PostUndo(bool bSuccess)
{	
	if (bSuccess)
	{
		const FTransaction* Transaction = GEditor->Trans->GetTransaction(GEditor->Trans->GetQueueLength() - GEditor->Trans->GetUndoCount());
		HandleUndoTransaction(Transaction);
	}
}

void FBlueprintEditor::PostRedo(bool bSuccess)
{
	if (bSuccess)
	{
		const FTransaction* Transaction = GEditor->Trans->GetTransaction(GEditor->Trans->GetQueueLength() - GEditor->Trans->GetUndoCount() - 1);
		HandleUndoTransaction(Transaction);
	}
}

void FBlueprintEditor::UndoGraphAction()
{
	GEditor->UndoTransaction();
}

bool FBlueprintEditor::CanUndoGraphAction() const
{
	//@TODO: Should probably allow this for BPs that can be edited during PIE (basically returning InEditingMode instead)
	return !IsPlayInEditorActive();
}

void FBlueprintEditor::RedoGraphAction()
{
	GEditor->RedoTransaction();
}

bool FBlueprintEditor::CanRedoGraphAction() const
{
	//@TODO: Should probably allow this for BPs that can be edited during PIE (basically returning InEditingMode instead)
	return !IsPlayInEditorActive();
}

void FBlueprintEditor::OnActiveTabChanged(TSharedPtr<SDockTab> PreviouslyActive, TSharedPtr<SDockTab> NewlyActivated)
{
}

void FBlueprintEditor::OnGraphEditorFocused(const TSharedRef<SGraphEditor>& InGraphEditor)
{
	// Update the graph editor that is currently focused
	FocusedGraphEdPtr = InGraphEditor;
	InGraphEditor->SetPinVisibility(PinVisibility);

	// Update the inspector as well, to show selection from the focused graph editor
	FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	FocusInspectorOnGraphSelection(SelectedNodes, /*bForceRefresh=*/ true);

	// During undo, garbage graphs can be temporarily brought into focus, ensure that before a refresh of the MyBlueprint window that the graph is owned by a Blueprint
	if ( FocusedGraphEdPtr.IsValid() && MyBlueprintWidget.IsValid() )
	{
		// The focused graph can be garbage as well
		TWeakObjectPtr< UEdGraph > FocusedGraphPtr = FocusedGraphEdPtr.Pin()->GetCurrentGraph();
		UEdGraph* FocusedGraph = FocusedGraphPtr.Get();

		if ( FocusedGraph != nullptr )
		{
			if ( FBlueprintEditorUtils::FindBlueprintForGraph(FocusedGraph) )
			{
				MyBlueprintWidget->Refresh();
			}
		}
	}

	if (bHideUnrelatedNodes && SelectedNodes.Num() <= 0)
	{
		ResetAllNodesUnrelatedStates();
	}

	// If the bookmarks view is active, check whether or not we're restricting the view to the current graph. If we are, update the tree to reflect the focused graph context.
	if (BookmarksWidget.IsValid()
		&& GetDefault<UBlueprintEditorSettings>()->bShowBookmarksForCurrentDocumentOnlyInTab)
	{
		BookmarksWidget->RefreshBookmarksTree();
	}
}

void FBlueprintEditor::OnGraphEditorBackgrounded(const TSharedRef<SGraphEditor>& InGraphEditor)
{
	// If the newly active document tab isn't a graph we want to make sure we clear the focused graph pointer.
	// Several other UI reads that, like the MyBlueprints view uses it to determine if it should show the "Local Variable" section.
	FocusedGraphEdPtr = nullptr;

	if ( MyBlueprintWidget.IsValid() == true )
	{
		MyBlueprintWidget->Refresh();
	}
}

void FBlueprintEditor::OnGraphEditorDropActor(const TArray< TWeakObjectPtr<AActor> >& Actors, UEdGraph* Graph, const FVector2f& DropLocation)
{
	// We need to check that the dropped actor is in the right sublevel for the reference
	ULevel* BlueprintLevel = FBlueprintEditorUtils::GetLevelFromBlueprint(GetBlueprintObj());

	if (BlueprintLevel && FBlueprintEditorUtils::IsLevelScriptBlueprint(GetBlueprintObj()))
	{
		FDeprecateSlateVector2D NodeLocation = DropLocation;
		for (int32 i = 0; i < Actors.Num(); i++)
		{
			AActor* DroppedActor = Actors[i].Get();
			if (DroppedActor&& (DroppedActor->GetLevel() == BlueprintLevel) && !DroppedActor->IsChildActor())
			{
				UK2Node_Literal* ActorRefNode = FEdGraphSchemaAction_K2NewNode::SpawnNode<UK2Node_Literal>(
					Graph,
					NodeLocation,
					EK2NewNodeFlags::SelectNewNode,
					[DroppedActor](UK2Node_Literal* NewInstance)
					{
						NewInstance->SetObjectRef(DroppedActor);
					}
				);
				NodeLocation.Y += UEdGraphSchema_K2::EstimateNodeHeight(ActorRefNode);
			}
		}
	}
}

void FBlueprintEditor::OnGraphEditorDropStreamingLevel(const TArray< TWeakObjectPtr<ULevelStreaming> >& Levels, UEdGraph* Graph, const FVector2f& DropLocation)
{
	UFunction* TargetFunc = UGameplayStatics::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UGameplayStatics, GetStreamingLevel));
	check(TargetFunc);

	for (int32 i = 0; i < Levels.Num(); i++)
	{
		ULevelStreaming* DroppedLevel = Levels[i].Get();
		if (DroppedLevel && DroppedLevel->IsA<ULevelStreamingDynamic>())
		{
			UK2Node_CallFunction* Node = FEdGraphSchemaAction_K2NewNode::SpawnNode<UK2Node_CallFunction>(
				Graph,
				FDeprecateSlateVector2D(DropLocation + (i * FVector2f(0.0f, 80.0f))),
				EK2NewNodeFlags::SelectNewNode,
				[TargetFunc](UK2Node_CallFunction* NewInstance)
				{
					NewInstance->SetFromFunction(TargetFunc);
				}
			);
						
			// Set dropped level package name
			UEdGraphPin* PackageNameInputPin = Node->FindPinChecked(TEXT("PackageName"));
			PackageNameInputPin->DefaultValue = DroppedLevel->GetWorldAssetPackageName();
		}
	}
}

FActionMenuContent FBlueprintEditor::OnCreateGraphActionMenu(UEdGraph* InGraph, const FVector2D& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed)
{
	return OnCreateGraphActionMenu(InGraph, UE::Slate::CastToVector2f(InNodePosition), InDraggedPins, bAutoExpand, InOnMenuClosed);
}

FActionMenuContent FBlueprintEditor::OnCreateGraphActionMenu(UEdGraph* InGraph, const FVector2f& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed)
{
	HasOpenActionMenu = InGraph;
	if (!BlueprintEditorImpl::GraphHasUserPlacedNodes(InGraph))
	{
		InstructionsFadeCountdown = BlueprintEditorImpl::InstructionFadeDuration;
	}

	TSharedRef<SBlueprintActionMenu> ActionMenu = 
		SNew(SBlueprintActionMenu, SharedThis(this))
		.GraphObj(InGraph)
		.NewNodePosition(FDeprecateSlateVector2D(InNodePosition))
		.DraggedFromPins(InDraggedPins)
		.AutoExpandActionMenu(bAutoExpand)
		.OnClosedCallback(InOnMenuClosed)
		.OnCloseReason(this, &FBlueprintEditor::OnGraphActionMenuClosed);

	return FActionMenuContent( ActionMenu, ActionMenu->GetFilterTextBox() );
}

void FBlueprintEditor::OnGraphActionMenuClosed(bool bActionExecuted, bool bContextSensitiveChecked, bool bGraphPinContext)
{
	if (bActionExecuted)
	{
		bContextSensitiveChecked ? AnalyticsStats.GraphActionMenusCtxtSensitiveExecCount++ : AnalyticsStats.GraphActionMenusNonCtxtSensitiveExecCount++;
		UpdateNodeCreationStats( bGraphPinContext ? ENodeCreateAction::PinContext : ENodeCreateAction::GraphContext );
	}
	else
	{
		AnalyticsStats.GraphActionMenusCancelledCount++;
	}

	if (UEdGraph* EditingGraph = GetFocusedGraph())
	{
		// if the user didn't place any nodes...
		if (!BlueprintEditorImpl::GraphHasUserPlacedNodes(EditingGraph))
		{
			InstructionsFadeCountdown = 0.0f;
		}
	}
	HasOpenActionMenu = nullptr;
}

void FBlueprintEditor::OnSelectedNodesChangedImpl(const FGraphPanelSelectionSet& NewSelection)
{
	if ( NewSelection.Num() > 0 )
	{
		SetUISelectionState(FBlueprintEditor::SelectionState_Graph);
	}

	SKismetInspector::FShowDetailsOptions DetailsOptions;
	DetailsOptions.bForceRefresh = true;
	Inspector->ShowDetailsForObjects(NewSelection.Array(), DetailsOptions);

	bSelectRegularNode = false;
	for (FGraphPanelSelectionSet::TConstIterator It(NewSelection); It; ++It)
	{
		UEdGraphNode_Comment* SeqNode = Cast<UEdGraphNode_Comment>(*It);
		if (!SeqNode)
		{
			bSelectRegularNode = true;
			break;
		}
	}

	if (bHideUnrelatedNodes && !bLockNodeFadeState)
	{
		ResetAllNodesUnrelatedStates();

		if ( bSelectRegularNode )
		{
			HideUnrelatedNodes();
		}
	}
}

void FBlueprintEditor::OnBlueprintChangedImpl(UBlueprint* InBlueprint, bool bIsJustBeingCompiled )
{
	if (InBlueprint)
	{
		// Notify that the blueprint has been changed (update Content browser, etc)
		InBlueprint->PostEditChange();

		// Call PostEditChange() on any Actors that are based on this Blueprint 
		FBlueprintEditorUtils::PostEditChangeBlueprintActors(InBlueprint);

		// Refresh the graphs
		ERefreshBlueprintEditorReason::Type Reason = bIsJustBeingCompiled ? ERefreshBlueprintEditorReason::BlueprintCompiled : ERefreshBlueprintEditorReason::UnknownReason;
		RefreshEditors(Reason);

		// In case objects were deleted, which should close the tab
		if (GetCurrentMode() == FBlueprintEditorApplicationModes::StandardBlueprintEditorMode)
		{
			SaveEditedObjectState();
		}
	}
}

void FBlueprintEditor::OnBlueprintCompiled(UBlueprint* InBlueprint)
{	
	if( InBlueprint )
	{
		UUnrealEdEngine* EditorEngine = GUnrealEd;
		// GUnrealEd can be nullptr after a hot-reload... this seems like a bigger 
		// problem worth investigating (that could affect other systems), but 
		// as I cannot repro it a second time (to see if it gets reset soon after), 
		// we'll just gaurd here for now and see if we can tie this ensure to any 
		// future crash reports
		if (ensure(EditorEngine != nullptr))
		{
			// Compiling will invalidate any cached components in the component visualizer, so clear out active components here
			EditorEngine->ComponentVisManager.ClearActiveComponentVis();
		}

		// This could be made more efficient by tracking which nodes change
		// their bHasCompilerMessage flag, or immediately updating the error info
		// when we assign the flag:
		TArray<UEdGraph*> Graphs;
		InBlueprint->GetAllGraphs(Graphs);
		for (const UEdGraph* Graph : Graphs)
		{
			for (const UEdGraphNode* Node : Graph->Nodes)
			{
				if (Node)
				{
					TSharedPtr<SGraphNode> Widget = Node->DEPRECATED_NodeWidget.Pin();
					if (Widget.IsValid())
					{
						Widget->RefreshErrorInfo();
					}
				}
			}
		}
	}

	OnBlueprintChangedImpl( InBlueprint, true );
}

void FBlueprintEditor::OnBlueprintUnloaded(UBlueprint* InBlueprint)
{
	for (UObject* EditingObj : GetEditingObjects())
	{
		if (Cast<UBlueprint>(EditingObj) == InBlueprint)
		{
			// give the editor a chance to open a replacement
			bPendingDeferredClose = true;
			break;
		}
	}
}

void FBlueprintEditor::Compile()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	UBlueprint* BlueprintObj = GetBlueprintObj();
	if (BlueprintObj)
	{
		FMessageLog BlueprintLog("BlueprintLog");

		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("BlueprintName"), FText::FromString(BlueprintObj->GetName()));
		BlueprintLog.NewPage(FText::Format(LOCTEXT("CompilationPageLabel", "Compile {BlueprintName}"), Arguments));

		FCompilerResultsLog LogResults;
		LogResults.SetSourcePath(BlueprintObj->GetPathName());
		LogResults.BeginEvent(TEXT("Compile"));
		EBlueprintCompileOptions CompileOptions = EBlueprintCompileOptions::None;
		if( bSaveIntermediateBuildProducts )
		{
			CompileOptions |= EBlueprintCompileOptions::SaveIntermediateProducts;
		}

		if (bIsReparentingBlueprint)
		{
			CompileOptions |= (EBlueprintCompileOptions::UseDeltaSerializationDuringReinstancing | EBlueprintCompileOptions::SkipNewVariableDefaultsDetection);
		}

		// If compilation is enabled during PIE/simulation, references to the CDO might be held by a script variable.
		// Thus, we set the flag to direct the compiler to allow those references to be replaced during reinstancing.
		if (IsPlayInEditorActive())
		{
			CompileOptions |= EBlueprintCompileOptions::IncludeCDOInReferenceReplacement;
		}

		FKismetEditorUtilities::CompileBlueprint(BlueprintObj, CompileOptions, &LogResults);

		LogResults.EndEvent();

		CachedNumWarnings = LogResults.NumWarnings;
		CachedNumErrors = LogResults.NumErrors;

		const bool bForceMessageDisplay = (LogResults.NumWarnings > 0 || LogResults.NumErrors > 0) && !BlueprintObj->bIsRegeneratingOnLoad;
		DumpMessagesToCompilerLog(LogResults.Messages, bForceMessageDisplay);

		UBlueprintEditorSettings const* BpEditorSettings = GetDefault<UBlueprintEditorSettings>();
		if ((LogResults.NumErrors > 0) && BpEditorSettings->bJumpToNodeErrors)
		{
			if (UEdGraphNode* NodeWithError = BlueprintEditorImpl::FindNodeWithError(LogResults))
			{
				JumpToNode(NodeWithError, /*bRequestRename =*/false);
			}
		}

		if (BlueprintObj->UpgradeNotesLog.IsValid())
		{
			CompilerResultsListing->AddMessages(BlueprintObj->UpgradeNotesLog->Messages);
		}

		// send record when player clicks compile and send the result
		// this will make sure how the users activity is
		AnalyticsTrackCompileEvent(BlueprintObj, LogResults.NumErrors, LogResults.NumWarnings);

		RefreshInspector();
	}
}

bool FBlueprintEditor::IsSaveOnCompileEnabled() const
{
	UBlueprint* Blueprint = GetBlueprintObj();
	bool const bIsLevelScript = (Cast<ULevelScriptBlueprint>(Blueprint) != nullptr);

	return !bIsLevelScript;
}

FReply FBlueprintEditor::Compile_OnClickWithReply()
{
	Compile();
	return FReply::Handled();
}

void FBlueprintEditor::RefreshAllNodes_OnClicked()
{
	FBlueprintEditorUtils::RefreshAllNodes(GetBlueprintObj());
	RefreshEditors();
	Compile();
}

void FBlueprintEditor::DeleteUnusedVariables_OnClicked()
{
	UBlueprint* BlueprintObj = GetBlueprintObj();
	
	// Gather FProperties from this BP and see if we can remove any
	TArray<FProperty*> VariableProperties;
	bool bHasAtLeastOneVariableToCheck = UBlueprintEditorLibrary::GatherUnusedVariables(BlueprintObj, VariableProperties);

	if (VariableProperties.Num() > 0)
	{
		TSharedRef<SCheckBoxList> CheckBoxList = SNew(SCheckBoxList)
			.ItemHeaderLabel(LOCTEXT("DeleteUnusedVariablesDialog_VariableLabel", "Variable"));
		for (FProperty* Variable : VariableProperties)
		{
			CheckBoxList->AddItem(FText::FromString(UEditorEngine::GetFriendlyName(Variable)), true);
		}

		TSharedRef<SWidget> DialogContain = SNew(SVerticalBox)
			+ SVerticalBox::Slot().Padding(10)
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("VariableDialog_Message", "These variables are not used in the graph or in other blueprints' graphs.\nThey may be used in other places.\nYou may use 'Find in Blueprint' or the 'Asset Search' to find out if they are referenced elsewhere."))
				.AutoWrapText(true)
			]
			+ SVerticalBox::Slot().FillHeight(0.8)
			[
				CheckBoxList
			];

		TSharedRef<SCustomDialog> CustomDialog = SNew(SCustomDialog)
			.Title(LOCTEXT("DeleteUnusedVariablesDialog_Title", "Delete Unused Variables"))
			.Icon(FAppStyle::Get().GetBrush("NotificationList.DefaultMessage"))
			.Content()
			[
				DialogContain
			]
			.Buttons(
			{
				SCustomDialog::FButton(LOCTEXT("DeleteUnusedVariablesDialog_ButtonDelete", "Delete")),
				SCustomDialog::FButton(LOCTEXT("DeleteUnusedVariablesDialog_ButtonCancel", "Cancel"))
			});

		int32 Result = CustomDialog->ShowModal();
		if (Result == 0)
		{
			TArray<FName> VariableNames;
			FString PropertyList;
			VariableNames.Reserve(VariableProperties.Num());
			for (int32 Index = 0; Index < VariableProperties.Num(); ++Index)
			{
				if (CheckBoxList->IsItemChecked(Index))
				{
					VariableNames.Add(VariableProperties[Index]->GetFName());
					if (PropertyList.IsEmpty())
					{
						PropertyList = UEditorEngine::GetFriendlyName(VariableProperties[Index]);
					}
					else
					{
						PropertyList += FString::Printf(TEXT(", %s"), *UEditorEngine::GetFriendlyName(VariableProperties[Index]));
					}
				}
			}

			if (VariableNames.Num() > 0)
			{
				VariableProperties.Empty(); // Emptying this array because these properties will be deleted and we don't want to keep raw pointers to deleted objects
				FBlueprintEditorUtils::BulkRemoveMemberVariables(BlueprintObj, VariableNames);
				LogSimpleMessage(FText::Format(LOCTEXT("UnusedVariablesDeletedMessage", "The following variable(s) were deleted successfully: {0}."), FText::FromString(PropertyList)));
			}
			else
			{
				LogSimpleMessage(LOCTEXT("NoVariablesSelectedMessage", "No variables were selected for deletion."));
			}
		}
	}
	else if (bHasAtLeastOneVariableToCheck)
	{
		LogSimpleMessage(LOCTEXT("AllVariablesInUseMessage", "All variables are currently in use."));
	}
	else
	{
		LogSimpleMessage(LOCTEXT("NoVariablesToSeeMessage", "No variables to check for."));
	}
}

void FBlueprintEditor::FindInBlueprints_OnClicked()
{
	SetCurrentMode(FBlueprintEditorApplicationModes::StandardBlueprintEditorMode);
	
	SummonSearchUI(false);
}

void FBlueprintEditor::ClearAllBreakpoints()
{
	FKismetDebugUtilities::ClearBreakpoints(GetBlueprintObj());
}

void FBlueprintEditor::DisableAllBreakpoints()
{
	FKismetDebugUtilities::ForeachBreakpoint(GetBlueprintObj(),
		[](FBlueprintBreakpoint& Breakpoint)
		{
			FKismetDebugUtilities::SetBreakpointEnabled(Breakpoint, false);
		}
	);
}

void FBlueprintEditor::EnableAllBreakpoints()
{
	FKismetDebugUtilities::ForeachBreakpoint(GetBlueprintObj(),
		[](FBlueprintBreakpoint& Breakpoint)
		{
			FKismetDebugUtilities::SetBreakpointEnabled(Breakpoint, true);
		}
	);
}

void FBlueprintEditor::ClearAllWatches()
{
	FKismetDebugUtilities::ClearPinWatches(GetBlueprintObj());
}

void FBlueprintEditor::OpenBlueprintDebugger()
{
	FGlobalTabmanager::Get()->TryInvokeTab(FBlueprintEditorTabs::BlueprintDebuggerID);
	FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::Get().LoadModuleChecked<FBlueprintEditorModule>(TEXT("Kismet"));
	BlueprintEditorModule.GetBlueprintDebugger()->SetDebuggedBlueprint(GetBlueprintObj());
}

bool FBlueprintEditor::CanOpenBlueprintDebugger() const
{
	// The BP debugger can always be spawned because it will get updated on PIE
	return true;
}

bool FBlueprintEditor::HasAnyBreakpoints() const
{
	const UBlueprint* Blueprint = GetBlueprintObj();
	if(!Blueprint)
	{
		return false;
	}
	return FKismetDebugUtilities::BlueprintHasBreakpoints(Blueprint);
}

bool FBlueprintEditor::HasAnyEnabledBreakpoints() const
{
	if (!IsEditingSingleBlueprint()) {return false;}

	return FKismetDebugUtilities::FindBreakpointByPredicate(
		GetBlueprintObj(),
		[](const FBlueprintBreakpoint& Breakpoint)
		{
			return Breakpoint.IsEnabledByUser();
		}
	) != nullptr;
}

bool FBlueprintEditor::HasAnyDisabledBreakpoints() const
{
	if (!IsEditingSingleBlueprint()) {return false;}

	return FKismetDebugUtilities::FindBreakpointByPredicate(
		GetBlueprintObj(),
		[](const FBlueprintBreakpoint& Breakpoint)
		{
			return !Breakpoint.IsEnabledByUser();
		}
	) != nullptr;
}

bool FBlueprintEditor::HasAnyWatches() const
{
	const UBlueprint* Blueprint = GetBlueprintObj();;
	return Blueprint && FKismetDebugUtilities::BlueprintHasPinWatches(Blueprint);
}

// Jumps to a hyperlinked node, pin, or graph, if it belongs to this blueprint
void FBlueprintEditor::JumpToHyperlink(const UObject* ObjectReference, bool bRequestRename)
{
	SetCurrentMode(FBlueprintEditorApplicationModes::StandardBlueprintEditorMode);

	if (const UEdGraphNode* Node = Cast<const UEdGraphNode>(ObjectReference))
	{
		if (bRequestRename)
		{
			IsNodeTitleVisible(Node, bRequestRename);
		}
		else
		{
			JumpToNode(Node, false);
		}
	}
	else if (const UEdGraph* Graph = Cast<const UEdGraph>(ObjectReference))
	{
		// Navigating into things should re-use the current tab when it makes sense
		FDocumentTracker::EOpenDocumentCause OpenMode = FDocumentTracker::OpenNewDocument;
		if ((Graph->GetSchema()->GetGraphType(Graph) == GT_Ubergraph) || Cast<UK2Node>(Graph->GetOuter()) || Cast<UEdGraph>(Graph->GetOuter()))
		{
			// Ubergraphs directly reuse the current graph
			OpenMode = FDocumentTracker::NavigatingCurrentDocument;
		}
		else
		{
			// Walk up the outer chain to see if any tabs have a parent of this document open for edit, and if so
			// we should reuse that one and drill in deeper instead
			for (UObject* WalkPtr = const_cast<UEdGraph*>(Graph); WalkPtr != nullptr; WalkPtr = WalkPtr->GetOuter())
			{
				TArray< TSharedPtr<SDockTab> > TabResults;
				if (FindOpenTabsContainingDocument(WalkPtr, /*out*/ TabResults))
				{
					// See if the parent was active
					bool bIsActive = false;
					for (TSharedPtr<SDockTab> Tab : TabResults)
					{
						if (Tab->IsActive())
						{
							bIsActive = true;
							break;
						}
					}

					if (bIsActive)
					{
						OpenMode = FDocumentTracker::NavigatingCurrentDocument;
						break;
					}
				}
			}
		}

		// Force it to open in a new document if shift is pressed
		const bool bIsShiftPressed = FSlateApplication::Get().GetModifierKeys().IsShiftDown();
		if (bIsShiftPressed)
		{
			OpenMode = FDocumentTracker::ForceOpenNewDocument;
		}

		// Open the document
		OpenDocument(Graph, OpenMode);
	}
	else if (const AActor* ReferencedActor = Cast<const AActor>(ObjectReference))
	{
		// Check if the world is active in the editor. It's possible to open level BPs without formally opening
		// the levels through Find-in-Blueprints
		bool bInOpenWorld = false;
		const TIndirectArray<FWorldContext>& WorldContextList = GEditor->GetWorldContexts();
		const UWorld* ReferencedActorOwningWorld = ReferencedActor->GetWorld();
		for (const FWorldContext& WorldContext : WorldContextList)
		{
			if (WorldContext.World() == ReferencedActorOwningWorld)
			{
				bInOpenWorld = true;
				break;
			}
		}

		// Clear the selection even if we couldn't find it, so the existing selection doesn't get mistaken for the desired to be selected actor
		GEditor->SelectNone(false, false);

		if (bInOpenWorld)
		{
			// Select the in-level actor
			GEditor->SelectActor(const_cast<AActor*>(ReferencedActor), true, true, true);

			// Point the camera at it
			GUnrealEd->Exec(ReferencedActor->GetWorld(), TEXT("CAMERA ALIGN ACTIVEVIEWPORTONLY"));
		}
	}
	else if(const UFunction* Function = Cast<const UFunction>(ObjectReference))
	{
		UBlueprint* BP = GetBlueprintObj();
		if(BP)
		{
			if (UEdGraph* FunctionGraph = FBlueprintEditorUtils::FindScopeGraph(BP, Function))
			{
				OpenDocument(FunctionGraph, FDocumentTracker::OpenNewDocument);
			}
		}
	}
	else if(const UBlueprintGeneratedClass* Class = Cast<const UBlueprintGeneratedClass>(ObjectReference))
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Class->ClassGeneratedBy);
	}
	else if (const UTimelineTemplate* Timeline = Cast<const UTimelineTemplate>(ObjectReference))
	{
		OpenDocument(Timeline, FDocumentTracker::OpenNewDocument);
	}
	else if ((ObjectReference != nullptr) && ObjectReference->IsAsset())
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(const_cast<UObject*>(ObjectReference));
	}
	else
	{
		UE_LOGF(LogBlueprint, Warning, "Unknown type of hyperlinked object (%ls), cannot focus it", *GetNameSafe(ObjectReference));
	}

	//@TODO: Hacky way to ensure a message is seen when hitting an exception and doing intraframe debugging
	const FText ExceptionMessage = FKismetDebugUtilities::GetAndClearLastExceptionMessage();
	if (!ExceptionMessage.IsEmpty())
	{
		LogSimpleMessage( ExceptionMessage );
	}
}

void FBlueprintEditor::JumpToPin(const UEdGraphPin* Pin)
{
	if (!Pin->IsPendingKill())
	{
		// Open a graph editor and jump to the pin
		TSharedPtr<SGraphEditor> GraphEditor = OpenGraphAndBringToFront(Pin->GetOwningNode()->GetGraph());
		if (GraphEditor.IsValid())
		{
			GraphEditor->JumpToPin(Pin);
		}
	}
}

void FBlueprintEditor::AddReferencedObjects( FReferenceCollector& Collector )
{
	if (GetObjectsCurrentlyBeingEdited()->Num() > 0)
	{
		Collector.AddReferencedObjects(GetEditingObjectPtrs());
	}

	Collector.AddReferencedObjects(StandardLibraries);

	UserDefinedEnumerators.Remove(TWeakObjectPtr<UUserDefinedEnum>()); // Remove NULLs
	for (TWeakObjectPtr<UUserDefinedEnum>& ObjectPtr : UserDefinedEnumerators)
	{
			Collector.AddReferencedObject(ObjectPtr);
	}

	UserDefinedStructures.Remove(TWeakObjectPtr<UUserDefinedStruct>()); // Remove NULLs
	for (TWeakObjectPtr<UUserDefinedStruct>& ObjectPtr : UserDefinedStructures)
	{
			Collector.AddReferencedObject(ObjectPtr);
	}
}

FString FBlueprintEditor::GetReferencerName() const
{
	return TEXT("FBlueprintEditor");
}

bool FBlueprintEditor::IsNodeTitleVisible(const UEdGraphNode* Node, bool bRequestRename)
{
	TSharedPtr<SGraphEditor> GraphEditor;
	if(bRequestRename)
	{
		// If we are renaming, the graph will be open already, just grab the tab and it's content and jump to the node.
		TSharedPtr<SDockTab> ActiveTab = DocumentManager->GetActiveTab();
		check(ActiveTab.IsValid());
		GraphEditor = StaticCastSharedRef<SGraphEditor>(ActiveTab->GetContent());
	}
	else
	{
		// Open a graph editor and jump to the node
		GraphEditor = OpenGraphAndBringToFront(Node->GetGraph());
	}

	bool bVisible = false;
	if (GraphEditor.IsValid())
	{
		bVisible = GraphEditor->IsNodeTitleVisible(Node, bRequestRename);
	}
	return bVisible;
}

void FBlueprintEditor::JumpToNode(const UEdGraphNode* Node, bool bRequestRename)
{
	TSharedPtr<SGraphEditor> GraphEditor;
	if(bRequestRename)
	{
		// If we are renaming, the graph will be open already, just grab the tab and it's content and jump to the node.
		TSharedPtr<SDockTab> ActiveTab = DocumentManager->GetActiveTab();
		check(ActiveTab.IsValid());
		GraphEditor = StaticCastSharedRef<SGraphEditor>(ActiveTab->GetContent());
	}
	else
	{
		// Ensure that the node is owned by a Blueprint before attempting to open its outer graph, which requires this.
		if (FBlueprintEditorUtils::FindBlueprintForNode(Node) != nullptr)
		{
			// Open a graph editor and jump to the node
			GraphEditor = OpenGraphAndBringToFront(Node->GetGraph());
		}
	}

	if (GraphEditor.IsValid())
	{
		GraphEditor->JumpToNode(Node, bRequestRename);
	}
}

UBlueprint* FBlueprintEditor::GetBlueprintObj() const
{
	return GetEditingObjects().Num() == 1 ? Cast<UBlueprint>(GetEditingObjects()[0]) : nullptr;
}

bool FBlueprintEditor::IsEditingSingleBlueprint() const
{
	return GetBlueprintObj() != nullptr;
}

FString FBlueprintEditor::GetDocumentationLink() const
{
	UBlueprint* Blueprint = GetBlueprintObj();
	if(Blueprint)
	{
		// Jump to more relevant docs if editing macro library or interface
		if(Blueprint->BlueprintType == BPTYPE_MacroLibrary)
		{
			return TEXT("Engine/Blueprints/UserGuide/Types/MacroLibrary");
		}
		else if (Blueprint->BlueprintType == BPTYPE_Interface)
		{
			return TEXT("Engine/Blueprints/UserGuide/Types/Interface");
		}
	}

	return FString(TEXT("Engine/Blueprints"));
}


bool FBlueprintEditor::CanAccessComponentsMode() const
{
	bool bCanAccess = false;

	// Ensure that we're editing a Blueprint
	if(IsEditingSingleBlueprint())
	{
		UBlueprint* Blueprint = GetBlueprintObj();
		bCanAccess = FBlueprintEditorUtils::DoesSupportComponents(Blueprint);
	}
	
	return bCanAccess;
}

bool FBlueprintEditor::IsEditorClosing() const
{
	return bEditorMarkedAsClosed;
}

void FBlueprintEditor::RegisterToolbarTab(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);
}

void FBlueprintEditor::LogSimpleMessage(const FText& MessageText)
{
	FNotificationInfo Info( MessageText );
	Info.ExpireDuration = 3.0f;
	Info.bUseLargeFont = false;
	TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
	if ( Notification.IsValid() )
	{
		Notification->SetCompletionState( SNotificationItem::CS_Fail );
	}
}

void FBlueprintEditor::DumpMessagesToCompilerLog(const TArray<TSharedRef<FTokenizedMessage>>& Messages, bool bForceMessageDisplay)
{
	CompilerResultsListing->ClearMessages();

	// Note we dont mirror to the output log here as the compiler already does that
	CompilerResultsListing->AddMessages(Messages, false);
	
	if (!bEditorMarkedAsClosed && bForceMessageDisplay && GetCurrentMode() == FBlueprintEditorApplicationModes::StandardBlueprintEditorMode)
	{
		TabManager->TryInvokeTab(FBlueprintEditorTabs::CompilerResultsID);
	}
}

void FBlueprintEditor::DoPromoteToVariable(UBlueprint* InBlueprint, UEdGraphPin* InTargetPin, bool bInToMemberVariable, const FVector2D* InOptionalLocation /* = nullptr */)
{
	TOptional<FVector2f> Location;
	if (InOptionalLocation)
	{
		Location = UE::Slate::CastToVector2f(*InOptionalLocation);
	}
	DoPromoteToVariable2f(InBlueprint, InTargetPin, bInToMemberVariable, Location);
}

void FBlueprintEditor::DoPromoteToVariable2f(UBlueprint* InBlueprint, UEdGraphPin* InTargetPin, bool bInToMemberVariable, TOptional<FVector2f> InOptionalLocation)
{
	FName PinName = InTargetPin->PinName;
	UEdGraphNode* PinNode = InTargetPin->GetOwningNode();
	check(PinNode);
	UEdGraph* GraphObj = PinNode->GetGraph();
	check(GraphObj);

	// Used for promoting to local variable
	UEdGraph* FunctionGraph = nullptr;

	const FScopedTransaction Transaction( bInToMemberVariable? LOCTEXT("PromoteToVariable", "Promote To Variable") : LOCTEXT("PromoteToLocalVariable", "Promote to Local Variable") );
	InBlueprint->Modify();
	GraphObj->Modify();

	FName VarName;
	bool bWasSuccessful = false;
	FEdGraphPinType NewPinType = InTargetPin->PinType;
	NewPinType.bIsConst = false;
	NewPinType.bIsReference = false;
	NewPinType.bIsWeakPointer = false;
	if (bInToMemberVariable)
	{
#if WITH_EDITORONLY_DATA
		static const FName NAME_UIMin(TEXT("UIMin"));
		static const FName NAME_UIMax(TEXT("UIMax"));
		static const FName NAME_ClampMin(TEXT("ClampMin"));
		static const FName NAME_ClampMax(TEXT("ClampMax"));

		FString Meta_UIMin;
		FString Meta_UIMax;
		FString Meta_ClampMin;
		FString Meta_ClampMax;

		if (const UEdGraphSchema* Schema = InTargetPin->GetSchema())
		{
			// Name the variable to match its target blueprint pin name
			FString IdealVarName = FText::TrimPrecedingAndTrailing(Schema->GetPinDisplayName(InTargetPin)).ToString();

			// Ignore unnamed return values
			if (IdealVarName == TEXT("Return Value"))
			{
				IdealVarName.Empty();
			}

			// Ignore names from compact nodes that don't usually display the pin names
			if (const UK2Node* K2Node = Cast<UK2Node>(InTargetPin->GetOwningNode()))
			{
				if (K2Node->ShouldDrawCompact())
				{
					IdealVarName.Empty();
				}
			}

			// Set the variable name to its ideal name (with an optional numeric suffix if there is a conflict)
			if (!IdealVarName.IsEmpty())
			{
				TSharedPtr<FKismetNameValidator> NameValidator = MakeShareable(new FKismetNameValidator(GetBlueprintObj(), NAME_None));
				if (NameValidator->IsValid(IdealVarName) == EValidatorResult::Ok)
				{
					VarName = FName(*IdealVarName);
				}
				else
				{
					VarName = FBlueprintEditorUtils::FindUniqueKismetName(GetBlueprintObj(), IdealVarName);
				}

				if (UEdGraphNode* Node = InTargetPin->GetOwningNode())
				{
					// Extract the target pin's numeric limits so that we can copy them to the new variable
					Meta_UIMin = Node->GetPinMetaData(InTargetPin->PinName, NAME_UIMin);
					Meta_UIMax = Node->GetPinMetaData(InTargetPin->PinName, NAME_UIMax);
					Meta_ClampMin = Node->GetPinMetaData(InTargetPin->PinName, NAME_ClampMin);
					Meta_ClampMax = Node->GetPinMetaData(InTargetPin->PinName, NAME_ClampMax);
				}
			}
		}
#endif // WITH_EDITORONLY_DATA

		if (VarName == NAME_None)
		{
			VarName = FBlueprintEditorUtils::FindUniqueKismetName(GetBlueprintObj(), TEXT("NewVar"));
		}
		bWasSuccessful = FBlueprintEditorUtils::AddMemberVariable( GetBlueprintObj(), VarName, NewPinType, InTargetPin->GetDefaultAsString() );

#if WITH_EDITORONLY_DATA
		if (bWasSuccessful)
		{
			if (!Meta_UIMin.IsEmpty())
			{
				FBlueprintEditorUtils::SetBlueprintVariableMetaData(GetBlueprintObj(), VarName, nullptr, NAME_UIMin, Meta_UIMin);
			}
			if (!Meta_UIMax.IsEmpty())
			{
				FBlueprintEditorUtils::SetBlueprintVariableMetaData(GetBlueprintObj(), VarName, nullptr, NAME_UIMax, Meta_UIMax);
			}
			if (!Meta_ClampMin.IsEmpty())
			{
				FBlueprintEditorUtils::SetBlueprintVariableMetaData(GetBlueprintObj(), VarName, nullptr, NAME_ClampMin, Meta_ClampMin);
			}
			if (!Meta_ClampMax.IsEmpty())
			{
				FBlueprintEditorUtils::SetBlueprintVariableMetaData(GetBlueprintObj(), VarName, nullptr, NAME_ClampMax, Meta_ClampMax);
			}
		}
#endif // WITH_EDITORONLY_DATA
	}
	else
	{
		ensure(FBlueprintEditorUtils::DoesSupportLocalVariables(GraphObj));
		VarName = FBlueprintEditorUtils::FindUniqueKismetName(GetBlueprintObj(), TEXT("NewLocalVar"));
		FunctionGraph = FBlueprintEditorUtils::GetTopLevelGraph(GraphObj);
		bWasSuccessful = FBlueprintEditorUtils::AddLocalVariable( GetBlueprintObj(), FunctionGraph, VarName, NewPinType, InTargetPin->GetDefaultAsString() );
	}

	if (bWasSuccessful)
	{
		// The owning node may have been reconstructed as a result of adding a new variable above, so ensure the pin reference is up-to-date.
		InTargetPin = PinNode->FindPinChecked(PinName);

		// Create the new setter node
		FEdGraphSchemaAction_K2NewNode NodeInfo;

		// Create get or set node, depending on whether we clicked on an input or output pin
		UK2Node_Variable* TemplateNode = nullptr;
		if (InTargetPin->Direction == EGPD_Input)
		{
			TemplateNode = NewObject<UK2Node_VariableGet>();
		}
		else
		{
			TemplateNode = NewObject<UK2Node_VariableSet>();
		}

		if (bInToMemberVariable)
		{
			TemplateNode->VariableReference.SetSelfMember(VarName);
		}
		else
		{
			TemplateNode->VariableReference.SetLocalMember(VarName, FunctionGraph->GetName(), FBlueprintEditorUtils::FindLocalVariableGuidByName(InBlueprint, FunctionGraph, VarName));
		}
		NodeInfo.NodeTemplate = TemplateNode;

		// Set position of new node to be close to node we clicked on
		FVector2f NewNodePos;

		if (InOptionalLocation.IsSet())
		{
			NewNodePos = InOptionalLocation.GetValue();
		}
		else
		{
			NewNodePos.X = (InTargetPin->Direction == EGPD_Input) ? PinNode->NodePosX - 200 : PinNode->NodePosX + 400;
			NewNodePos.Y = PinNode->NodePosY;
		}

		NodeInfo.PerformAction(GraphObj, InTargetPin, NewNodePos, false);

		RenameNewlyAddedAction(VarName);
	}
}

void FBlueprintEditor::OnPromoteToVariable(bool bInToMemberVariable)
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		UEdGraphPin* TargetPin = FocusedGraphEd->GetGraphPinForMenu();

		check(IsEditingSingleBlueprint());
		check(GetBlueprintObj()->SkeletonGeneratedClass);
		check(TargetPin);

		DoPromoteToVariable2f( GetBlueprintObj(), TargetPin, bInToMemberVariable );
	}
}

bool FBlueprintEditor::CanPromoteToVariable(bool bInToMemberVariable) const
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	bool bCanPromote = false;
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		if (UEdGraphPin* Pin = FocusedGraphEd->GetGraphPinForMenu())
		{
			if (!Pin->bOrphanedPin && (bInToMemberVariable || FBlueprintEditorUtils::DoesSupportLocalVariables(FocusedGraphEd->GetCurrentGraph())))
			{
				bCanPromote = K2Schema->CanPromotePinToVariable(*Pin, bInToMemberVariable);
			}
		}
	}
	
	return bCanPromote;
}

void FBlueprintEditor::OnSplitStructPin()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		UEdGraphPin* TargetPin = FocusedGraphEd->GetGraphPinForMenu();

		check(IsEditingSingleBlueprint());
		check(GetBlueprintObj()->SkeletonGeneratedClass);
		check(TargetPin);

		const FScopedTransaction Transaction( LOCTEXT("SplitStructPin", "Split Struct Pin") );

		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		K2Schema->SplitPin(TargetPin);
	}
}

bool FBlueprintEditor::CanSplitStructPin() const
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	bool bCanSplit = false;
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		if (UEdGraphPin* Pin = FocusedGraphEd->GetGraphPinForMenu())
		{
			bCanSplit = K2Schema->CanSplitStructPin(*Pin);
		}
	}

	return bCanSplit;
}

void FBlueprintEditor::OnRecombineStructPin()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		UEdGraphPin* TargetPin = FocusedGraphEd->GetGraphPinForMenu();

		check(IsEditingSingleBlueprint());
		check(GetBlueprintObj()->SkeletonGeneratedClass);
		check(TargetPin);

		const FScopedTransaction Transaction( LOCTEXT("RecombineStructPin", "Recombine Struct Pin") );

		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		K2Schema->RecombinePin(TargetPin);
	}
}

bool FBlueprintEditor::CanRecombineStructPin() const
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	bool bCanRecombine = false;
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		if (UEdGraphPin* Pin = FocusedGraphEd->GetGraphPinForMenu())
		{
			bCanRecombine = K2Schema->CanRecombineStructPin(*Pin);
		}
	}

	return bCanRecombine;
}

void FBlueprintEditor::OnAddExecutionPin()
{
	const FGraphPanelSelectionSet& SelectedNodes = GetSelectedNodes();
	
	// Iterate over all nodes, and add the pin
	for (FGraphPanelSelectionSet::TConstIterator It(SelectedNodes); It; ++It)
	{
		UK2Node_ExecutionSequence* SeqNode = Cast<UK2Node_ExecutionSequence>(*It);
		if (SeqNode != nullptr)
		{
			const FScopedTransaction Transaction( LOCTEXT("AddExecutionPin", "Add Execution Pin") );
			SeqNode->Modify();

			SeqNode->AddInputPin();

			const UEdGraphSchema* Schema = SeqNode->GetSchema();
			Schema->ReconstructNode(*SeqNode);
		}
		else if (UK2Node_Switch* SwitchNode = Cast<UK2Node_Switch>(*It))
		{
			const FScopedTransaction Transaction( LOCTEXT("AddExecutionPin", "Add Execution Pin") );
			SwitchNode->Modify();

			SwitchNode->AddPinToSwitchNode();

			const UEdGraphSchema* Schema = SwitchNode->GetSchema();
			Schema->ReconstructNode(*SwitchNode);
		}
	}

	// Refresh the current graph, so the pins can be updated
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		FocusedGraphEd->NotifyGraphChanged();
	}
}

bool FBlueprintEditor::CanAddExecutionPin() const
{
	const FGraphPanelSelectionSet& SelectedNodes = GetSelectedNodes();

	// Iterate over all nodes, and see if all can have a pin added
	for (FGraphPanelSelectionSet::TConstIterator It(SelectedNodes); It; ++It)
	{
		if (UK2Node_ExecutionSequence* AddPinNode = Cast<UK2Node_ExecutionSequence>(*It))
		{
			if (!AddPinNode->CanAddPin())
			{
				return false;
			}
		}
	}

	return true;
}

void FBlueprintEditor::OnInsertExecutionPinBefore()
{
	OnInsertExecutionPin(EPinInsertPosition::Before);
}

void FBlueprintEditor::OnInsertExecutionPinAfter()
{
	OnInsertExecutionPin(EPinInsertPosition::After);
}

void FBlueprintEditor::OnInsertExecutionPin(EPinInsertPosition Position)
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		const FScopedTransaction Transaction(LOCTEXT("InsertExecutionPinBefore", "Insert Execution Pin Before"));

		UEdGraphPin* SelectedPin = FocusedGraphEd->GetGraphPinForMenu();
		if (SelectedPin)
		{
			UEdGraphNode* OwningNode = SelectedPin->GetOwningNode();

			if (OwningNode)
			{
				if (UK2Node_ExecutionSequence* SeqNode = Cast<UK2Node_ExecutionSequence>(OwningNode))
				{
					SeqNode->InsertPinIntoExecutionNode(SelectedPin, Position);
					FocusedGraphEd->RefreshNode(*OwningNode);

					if (UBlueprint* BP = SeqNode->GetBlueprint())
					{
						FBlueprintEditorUtils::MarkBlueprintAsModified(BP);
					}
				}
			}
		}
	}
}

bool FBlueprintEditor::CanInsertExecutionPin() const
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		UEdGraphPin* SelectedPin = FocusedGraphEd->GetGraphPinForMenu();
		if (SelectedPin)
		{
			if (UK2Node_ExecutionSequence* ExecutionSequence = Cast<UK2Node_ExecutionSequence>(SelectedPin->GetOwningNode()))
			{
				return ExecutionSequence->CanAddPin();
			}
		}
	}

	return false;
}

void  FBlueprintEditor::OnRemoveExecutionPin()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		const FScopedTransaction Transaction( LOCTEXT("RemoveExecutionPin", "Remove Execution Pin") );

		UEdGraphPin* SelectedPin = FocusedGraphEd->GetGraphPinForMenu();
		UEdGraphNode* OwningNode = SelectedPin->GetOwningNode();

		OwningNode->Modify();
		SelectedPin->Modify();

		if (UK2Node_ExecutionSequence* SeqNode = Cast<UK2Node_ExecutionSequence>(OwningNode))
		{
			SeqNode->RemovePinFromExecutionNode(SelectedPin);
		}
		else if (UK2Node_Switch* SwitchNode = Cast<UK2Node_Switch>(OwningNode))
		{
			SwitchNode->RemovePinFromSwitchNode(SelectedPin);
		}

		// Update the graph so that the node will be refreshed
		FocusedGraphEd->NotifyGraphChanged();
	
		UEdGraph* CurrentGraph = FocusedGraphEd->GetCurrentGraph();
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(CurrentGraph);
		FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
	}
}

bool FBlueprintEditor::CanRemoveExecutionPin() const
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		if (UEdGraphPin* SelectedPin = FocusedGraphEd->GetGraphPinForMenu())
		{
			UEdGraphNode* OwningNode = SelectedPin->GetOwningNode();

			if (UK2Node_ExecutionSequence* SeqNode = Cast<UK2Node_ExecutionSequence>(OwningNode))
			{
				return SeqNode->CanRemoveExecutionPin();
			}
			else if (UK2Node_Switch* SwitchNode = Cast<UK2Node_Switch>(OwningNode))
			{
				return SwitchNode->CanRemoveExecutionPin(SelectedPin);
			}
		}
	}
	return false;
}

void  FBlueprintEditor::OnRemoveThisStructVarPin()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	UEdGraphPin* SelectedPin = FocusedGraphEd.IsValid() ? FocusedGraphEd->GetGraphPinForMenu() : nullptr;
	UEdGraphNode* OwningNode = SelectedPin ? SelectedPin->GetOwningNodeUnchecked() : nullptr;
	if (UK2Node_SetFieldsInStruct* SetFilestInStructNode = Cast<UK2Node_SetFieldsInStruct>(OwningNode))
	{
		const FScopedTransaction Transaction(LOCTEXT("RemoveThisStructVarPin", "Remove Struct Var Pin"));
		SetFilestInStructNode->Modify();
		SelectedPin->Modify();
		SetFilestInStructNode->RemoveFieldPins(SelectedPin, UK2Node_SetFieldsInStruct::EPinsToRemove::GivenPin);

		// Update the graph so that the node will be refreshed
		FocusedGraphEd->NotifyGraphChanged();

		UEdGraph* CurrentGraph = FocusedGraphEd->GetCurrentGraph();
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(CurrentGraph);
		FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
	}
}

bool FBlueprintEditor::CanRemoveThisStructVarPin() const
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	const UEdGraphPin*  SelectedPin = FocusedGraphEd.IsValid() ? FocusedGraphEd->GetGraphPinForMenu() : nullptr;
	return UK2Node_SetFieldsInStruct::ShowCustomPinActions(SelectedPin, false);
}

void  FBlueprintEditor::OnRemoveOtherStructVarPins()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	UEdGraphPin* SelectedPin = FocusedGraphEd.IsValid() ? FocusedGraphEd->GetGraphPinForMenu() : nullptr;
	UEdGraphNode* OwningNode = SelectedPin ? SelectedPin->GetOwningNodeUnchecked() : nullptr;
	if (UK2Node_SetFieldsInStruct* SetFilestInStructNode = Cast<UK2Node_SetFieldsInStruct>(OwningNode))
	{
		const FScopedTransaction Transaction(LOCTEXT("RemoveOtherStructVarPins", "Remove Other Struct Var Pins"));
		SetFilestInStructNode->Modify();
		SelectedPin->Modify();
		SetFilestInStructNode->RemoveFieldPins(SelectedPin, UK2Node_SetFieldsInStruct::EPinsToRemove::AllOtherPins);

		// Update the graph so that the node will be refreshed
		FocusedGraphEd->NotifyGraphChanged();

		UEdGraph* CurrentGraph = FocusedGraphEd->GetCurrentGraph();
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(CurrentGraph);
		FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
	}
}

bool FBlueprintEditor::CanRemoveOtherStructVarPins() const
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	const UEdGraphPin* SelectedPin = FocusedGraphEd.IsValid() ? FocusedGraphEd->GetGraphPinForMenu() : nullptr;
	return UK2Node_SetFieldsInStruct::ShowCustomPinActions(SelectedPin, false);
}

void FBlueprintEditor::OnRestoreAllStructVarPins()
{
	const FGraphPanelSelectionSet& SelectedNodes = GetSelectedNodes();
	FGraphPanelSelectionSet::TConstIterator It(SelectedNodes);
	UK2Node_SetFieldsInStruct* Node = (!!It) ? Cast<UK2Node_SetFieldsInStruct>(*It) : nullptr;
	if (Node && !Node->AllPinsAreShown())
	{
		const FScopedTransaction Transaction(LOCTEXT("RestoreAllStructVarPins", "Restore all struct var pins"));
		Node->Modify();
		Node->RestoreAllPins();

		// Refresh the current graph, so the pins can be updated
		TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
		if (FocusedGraphEd.IsValid())
		{
			UEdGraph* CurrentGraph = FocusedGraphEd->GetCurrentGraph();
			UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(CurrentGraph);
			FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);

			FocusedGraphEd->NotifyGraphChanged();
		}
	}

	
}

bool FBlueprintEditor::CanRestoreAllStructVarPins() const
{
	const FGraphPanelSelectionSet& SelectedNodes = GetSelectedNodes();
	FGraphPanelSelectionSet::TConstIterator It(SelectedNodes);
	UK2Node_SetFieldsInStruct* Node = (!!It) ? Cast<UK2Node_SetFieldsInStruct>(*It) : nullptr;
	return Node && !Node->AllPinsAreShown();
}

void FBlueprintEditor::OnResetPinToDefaultValue()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		UEdGraphPin* TargetPin = FocusedGraphEd->GetGraphPinForMenu();

		check(TargetPin);

		const FScopedTransaction Transaction(LOCTEXT("ResetPinToDefaultValue", "Reset Pin To Default Value"));

		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		TargetPin->Modify();
		K2Schema->ResetPinToAutogeneratedDefaultValue(TargetPin);
	}
}

bool FBlueprintEditor::CanResetPinToDefaultValue() const
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	bool bCanRecombine = false;
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		if (UEdGraphPin* Pin = FocusedGraphEd->GetGraphPinForMenu())
		{
			return !Pin->DoesDefaultValueMatchAutogenerated();
		}
	}

	return false;
}

void FBlueprintEditor::OnAddOptionPin()
{
	const FGraphPanelSelectionSet& SelectedNodes = GetSelectedNodes();

	// Iterate over all nodes, and add the pin
	for (FGraphPanelSelectionSet::TConstIterator It(SelectedNodes); It; ++It)
	{
		UK2Node_Select* SeqNode = Cast<UK2Node_Select>(*It);
		if (SeqNode != nullptr)
		{
			const FScopedTransaction Transaction( LOCTEXT("AddOptionPin", "Add Option Pin") );
			SeqNode->Modify();

			SeqNode->AddInputPin();

			const UEdGraphSchema* Schema = SeqNode->GetSchema();
			Schema->ReconstructNode(*SeqNode);
		}
	}

	// Refresh the current graph, so the pins can be updated
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		FocusedGraphEd->NotifyGraphChanged();
	}
}

bool FBlueprintEditor::CanAddOptionPin() const
{
	const FGraphPanelSelectionSet& SelectedNodes = GetSelectedNodes();

	// Iterate over all nodes, and see if all can have a pin removed
	for (FGraphPanelSelectionSet::TConstIterator It(SelectedNodes); It; ++It)
	{
		UK2Node_Select* SeqNode = Cast<UK2Node_Select>(*It);
		// There's a bad node so return false
		if (SeqNode == nullptr || !SeqNode->CanAddPin())
		{
			return false;
		}
	}

	return true;
}

void FBlueprintEditor::OnRemoveOptionPin()
{
	const FGraphPanelSelectionSet& SelectedNodes = GetSelectedNodes();

	// Iterate over all nodes, and add the pin
	for (FGraphPanelSelectionSet::TConstIterator It(SelectedNodes); It; ++It)
	{
		UK2Node_Select* SeqNode = Cast<UK2Node_Select>(*It);
		if (SeqNode != nullptr)
		{
			const FScopedTransaction Transaction( LOCTEXT("RemoveOptionPin", "Remove Option Pin") );
			SeqNode->Modify();

			SeqNode->RemoveOptionPinToNode();

			const UEdGraphSchema* Schema = SeqNode->GetSchema();
			Schema->ReconstructNode(*SeqNode);
		}
	}

	// Refresh the current graph, so the pins can be updated
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		FocusedGraphEd->NotifyGraphChanged();
	}
}

bool FBlueprintEditor::CanRemoveOptionPin() const
{
	const FGraphPanelSelectionSet& SelectedNodes = GetSelectedNodes();

	// Iterate over all nodes, and see if all can have a pin removed
	for (FGraphPanelSelectionSet::TConstIterator It(SelectedNodes); It; ++It)
	{
		UK2Node_Select* SeqNode = Cast<UK2Node_Select>(*It);
		// There's a bad node so return false
		if (SeqNode == nullptr || !SeqNode->CanRemoveOptionPinToNode())
		{
			return false;
		}
		// If this node doesn't have at least 3 options return false (need at least 2)
		else
		{
			TArray<UEdGraphPin*> OptionPins;
			SeqNode->GetOptionPins(OptionPins);
			if (OptionPins.Num() <= 2)
			{
				return false;
			}
		}
	}

	return true;
}

void FBlueprintEditor::OnChangePinType()
{
	if (UEdGraphPin* SelectedPin = GetCurrentlySelectedPin())
	{
		// Grab the root pin, that is what we want to edit
		UEdGraphPin* RootPin = SelectedPin;
		while(RootPin->ParentPin)
			{
			RootPin = RootPin->ParentPin;
		}

				const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

				// If this is the index node of the select node, we need to use the index list of types
				UK2Node_Select* SelectNode = Cast<UK2Node_Select>(SelectedPin->GetOwningNode());
				if (SelectNode && SelectNode->GetIndexPin() == SelectedPin)
				{
					TSharedRef<SCompoundWidget> PinChange = SNew(SPinTypeSelector, FGetPinTypeTree::CreateUObject(Schema, &UEdGraphSchema_K2::GetVariableTypeTree))
				.TargetPinType(this, &FBlueprintEditor::OnGetPinType, RootPin)
						.OnPinTypeChanged(this, &FBlueprintEditor::OnChangePinTypeFinished, SelectedPin)
						.Schema(Schema)
						.TypeTreeFilter(ETypeTreeFilter::IndexTypesOnly)
						.IsEnabled(true)
						.bAllowArrays(false);

					PinTypeChangeMenu = FSlateApplication::Get().PushMenu(
						GetToolkitHost()->GetParentWidget(), // Parent widget should be k2 not the menu thats open or it will be closed when the menu is dismissed
						FWidgetPath(),
						PinChange,
						FSlateApplication::Get().GetCursorPos(), // summon location
						FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup)
						);
				}
				else
				{
					TSharedRef<SCompoundWidget> PinChange = SNew(SPinTypeSelector, FGetPinTypeTree::CreateUObject(Schema, &UEdGraphSchema_K2::GetVariableTypeTree))
				.TargetPinType(this, &FBlueprintEditor::OnGetPinType, RootPin)
						.OnPinTypeChanged(this, &FBlueprintEditor::OnChangePinTypeFinished, SelectedPin)
						.Schema(Schema)
						.TypeTreeFilter(ETypeTreeFilter::None)
						.IsEnabled(true)
						.bAllowArrays(false);

					PinTypeChangeMenu = FSlateApplication::Get().PushMenu(
						GetToolkitHost()->GetParentWidget(), // Parent widget should be k2 not the menu thats open or it will be closed when the menu is dismissed
						FWidgetPath(),
						PinChange,
						FSlateApplication::Get().GetCursorPos(), // summon location
						FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup)
						);
				}
			}
}

FEdGraphPinType FBlueprintEditor::OnGetPinType(UEdGraphPin* SelectedPin) const
{
	return SelectedPin->PinType;
}

void FBlueprintEditor::OnChangePinTypeFinished(const FEdGraphPinType& PinType, UEdGraphPin* InSelectedPin)
{
	if (FBlueprintEditorUtils::IsPinTypeValid(PinType))
	{
		UEdGraphNode* OwningNode = InSelectedPin->GetOwningNode();
		OwningNode->Modify();
		InSelectedPin->PinType = PinType;
		if (UK2Node_Select* SelectNode = Cast<UK2Node_Select>(InSelectedPin->GetOwningNode()))
		{
			SelectNode->ChangePinType(InSelectedPin);
		}
	}

	if (PinTypeChangeMenu.IsValid())
	{
		PinTypeChangeMenu.Pin()->Dismiss();
	}
}

bool FBlueprintEditor::CanChangePinType() const
{
	if (UEdGraphPin* Pin = GetCurrentlySelectedPin())
	{
		if (UK2Node_Select* SelectNode = Cast<UK2Node_Select>(Pin->GetOwningNode()))
		{
			return SelectNode->CanChangePinType(Pin);
		}
	}
	return false;
}

void FBlueprintEditor::OnAddParentNode()
{
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	if (UEdGraphNode* SelectedObj = GetSingleSelectedNode())
	{
		// Get the function that the event node or function entry represents
		FFunctionFromNodeHelper FunctionFromNode(SelectedObj);
		if (FunctionFromNode.Function && FunctionFromNode.Node)
		{
			UFunction* ValidParent = Schema->GetCallableParentFunction(FunctionFromNode.Function);
			UEdGraph* TargetGraph = FunctionFromNode.Node->GetGraph();
			if (ValidParent && TargetGraph)
			{
				const FScopedTransaction Transaction(LOCTEXT("AddParentNode", "Add Parent Node"));
				TargetGraph->Modify();

				FGraphNodeCreator<UK2Node_CallParentFunction> FunctionNodeCreator(*TargetGraph);
				UK2Node_CallParentFunction* ParentFunctionNode = FunctionNodeCreator.CreateNode();
				ParentFunctionNode->SetFromFunction(ValidParent);
				ParentFunctionNode->AllocateDefaultPins();

				int32 NodeSizeY = 15;
				if( UK2Node* Node = Cast<UK2Node>(SelectedObj))
				{
					NodeSizeY += Node->DEPRECATED_NodeWidget.IsValid() ? static_cast<int32>(Node->DEPRECATED_NodeWidget.Pin()->GetDesiredSize().Y) : 0;
				}
				ParentFunctionNode->NodePosX = FunctionFromNode.Node->NodePosX;
				ParentFunctionNode->NodePosY = FunctionFromNode.Node->NodePosY + NodeSizeY;
				FunctionNodeCreator.Finalize();
			}
		}
	}
}

bool FBlueprintEditor::CanAddParentNode() const
{
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	if (UEdGraphNode* SelectedObj = GetSingleSelectedNode())
	{
		// Get the function that the event node or function entry represents
		FFunctionFromNodeHelper FunctionFromNode(SelectedObj);
		if (FunctionFromNode.Function)
		{
			return (Schema->GetCallableParentFunction(FunctionFromNode.Function) != nullptr);
		}
	}

	return false;
}

void FBlueprintEditor::OnCreateMatchingFunction()
{
	if (UK2Node_CallFunction* SelectedNode = Cast<UK2Node_CallFunction>(GetSingleSelectedNode()))
	{
		FBlueprintEditorUtils::CreateMatchingFunction(SelectedNode, GetDefaultSchemaClass());
	}
}

bool FBlueprintEditor::CanCreateMatchingFunction() const
{
	if (NewDocument_IsVisibleForType(CGT_NewFunctionGraph))
	{
		if (const UBlueprint* Blueprint = GetBlueprintObj())
		{
			if (const UK2Node_CallFunction* SelectedNode = Cast<UK2Node_CallFunction>(GetSingleSelectedNode()))
			{
				return FKismetNameValidator(Blueprint).IsValid(SelectedNode->GetFunctionName()) == EValidatorResult::Ok;
			}
		}
	}

	return false;
}

void FBlueprintEditor::OnToggleBreakpoint()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UK2Node* SelectedNode = Cast<UK2Node>(*NodeIt);
		if ((SelectedNode != nullptr) && SelectedNode->CanPlaceBreakpoints())
		{
			FBlueprintBreakpoint* ExistingBreakpoint = FKismetDebugUtilities::FindBreakpointForNode(SelectedNode, GetBlueprintObj());
			if (ExistingBreakpoint == nullptr)
			{
				// Add a breakpoint on this node if there isn't one there already
				FKismetDebugUtilities::CreateBreakpoint(GetBlueprintObj(), SelectedNode, /* bIsEnabled = */ true);
			}
			else
			{
				// Remove the breakpoint if it was present
				FKismetDebugUtilities::RemoveBreakpointFromNode(SelectedNode, GetBlueprintObj());
			}
		}
	}
}

bool FBlueprintEditor::CanToggleBreakpoint() const
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UK2Node* SelectedNode = Cast<UK2Node>(*NodeIt);
		if ((SelectedNode != nullptr) && SelectedNode->CanPlaceBreakpoints())
		{
			return true;
		}
	}

	return false;
}

void FBlueprintEditor::OnAddBreakpoint()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UK2Node* SelectedNode = Cast<UK2Node>(*NodeIt);
		if ((SelectedNode != nullptr) && SelectedNode->CanPlaceBreakpoints())
		{
			// Add a breakpoint on this node if there isn't one there already
			const FBlueprintBreakpoint* ExistingBreakpoint = FKismetDebugUtilities::FindBreakpointForNode(SelectedNode, GetBlueprintObj());
			if (ExistingBreakpoint == nullptr)
			{
				FKismetDebugUtilities::CreateBreakpoint(GetBlueprintObj(), SelectedNode, /* bIsEnabled = */ true);
			}
		}
	}
}

bool FBlueprintEditor::CanAddBreakpoint() const
{
	// See if any of the selected nodes are impure, and thus could have a breakpoint set on them
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UK2Node* SelectedNode = Cast<UK2Node>(*NodeIt);
		if ((SelectedNode != nullptr) && SelectedNode->CanPlaceBreakpoints())
		{
			FBlueprintBreakpoint* ExistingBreakpoint = FKismetDebugUtilities::FindBreakpointForNode(SelectedNode, GetBlueprintObj());
			if (ExistingBreakpoint == nullptr)
			{
				return true;
			}
		}
	}

	return false;
}

void FBlueprintEditor::OnRemoveBreakpoint()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UEdGraphNode* SelectedNode = CastChecked<UEdGraphNode>(*NodeIt);

		// Remove the breakpoint
		FKismetDebugUtilities::RemoveBreakpointFromNode(SelectedNode, GetBlueprintObj());
	}
}

bool FBlueprintEditor::CanRemoveBreakpoint() const
{
	// See if any of the selected nodes have a breakpoint set
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UEdGraphNode* SelectedNode = CastChecked<UEdGraphNode>(*NodeIt);
		if (FKismetDebugUtilities::FindBreakpointForNode(SelectedNode, GetBlueprintObj()))
		{
			return true;
		}
	}

	return false;
}

void FBlueprintEditor::OnDisableBreakpoint()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UEdGraphNode* SelectedNode = CastChecked<UEdGraphNode>(*NodeIt);

		FKismetDebugUtilities::SetBreakpointEnabled(SelectedNode, GetBlueprintObj(), false);
	}
}

bool FBlueprintEditor::CanDisableBreakpoint() const
{
	// See if any of the selected nodes have a breakpoint set
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UEdGraphNode* SelectedNode = CastChecked<UEdGraphNode>(*NodeIt);
		if (FBlueprintBreakpoint* ExistingBreakpoint = FKismetDebugUtilities::FindBreakpointForNode(SelectedNode, GetBlueprintObj()))
		{
			if (ExistingBreakpoint->IsEnabledByUser())
			{
				return true;
			}
		}
	}

	return false;
}

void FBlueprintEditor::OnEnableBreakpoint()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UEdGraphNode* SelectedNode = CastChecked<UEdGraphNode>(*NodeIt);

		FKismetDebugUtilities::SetBreakpointEnabled(SelectedNode, GetBlueprintObj(), true);
	}
}

bool FBlueprintEditor::CanEnableBreakpoint() const
{
	// See if any of the selected nodes have a breakpoint set
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UEdGraphNode* SelectedNode = CastChecked<UEdGraphNode>(*NodeIt);
		if (FBlueprintBreakpoint* ExistingBreakpoint = FKismetDebugUtilities::FindBreakpointForNode(SelectedNode, GetBlueprintObj()))
		{
			if (!ExistingBreakpoint->IsEnabledByUser())
			{
				return true;
			}
		}
	}

	return false;
}

namespace CollapseGraphUtils
{
	/** Helper function to gather any moveable nodes out of a collapsed graph */
	static void GatherMoveableNodes(const UEdGraph* const SourceGraph, TSet<UEdGraphNode*>& OutNodes)
	{
		for (UEdGraphNode* Node : SourceGraph->Nodes)
		{
			// Ignore tunnel nodes and break the links because new ones will be created during the collapse of this node
			if (!Node->IsA<UK2Node_Tunnel>())
			{
				OutNodes.Add(Node);
			}
		}
	}
}

void FBlueprintEditor::OnCollapseNodes()
{
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	// Does the selection set contain anything that is legal to collapse?
	TSet<UEdGraphNode*> CollapsableNodes;
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		if (UEdGraphNode* SelectedNode = Cast<UEdGraphNode>(*NodeIt))
		{
			if (Schema->CanEncapuslateNode(*SelectedNode))
			{
				CollapsableNodes.Add(SelectedNode);
			}
		}
	}

	// Collapse them
	if (CollapsableNodes.Num())
	{
		UBlueprint* BlueprintObj = GetBlueprintObj();
		const FScopedTransaction Transaction( FGraphEditorCommands::Get().CollapseNodes->GetDescription() );
		BlueprintObj->Modify();

		CollapseNodes(CollapsableNodes);

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified( BlueprintObj );
	}
}

bool FBlueprintEditor::CanCollapseNodes() const
{
	//@TODO: ANIM: Determine what collapsing nodes means in an animation graph, and add any necessary compiler support for it
	if (IsEditingAnimGraph())
	{
		return false;
	}

	// Does the selection set contain anything that is legal to collapse?
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		if (UEdGraphNode* Node = Cast<UEdGraphNode>(*NodeIt))
		{
			if (Schema->CanEncapuslateNode(*Node))
			{
				return true;
			}
		}
	}

	return false;
}

void FBlueprintEditor::OnCollapseSelectionToFunction()
{
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	// Does the selection set contain anything that is legal to collapse?
	TSet<UEdGraphNode*> CollapsableNodes;
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		if (UEdGraphNode* SelectedNode = Cast<UEdGraphNode>(*NodeIt))
		{
			if (Schema->CanEncapuslateNode(*SelectedNode))
			{
				CollapsableNodes.Add(SelectedNode);
			}
		}
	}

	// Collapse them
	if (CollapsableNodes.Num() && CanCollapseSelectionToFunction(CollapsableNodes))
	{
		UBlueprint* BlueprintObj = GetBlueprintObj();
		const FScopedTransaction Transaction( FGraphEditorCommands::Get().CollapseNodes->GetDescription() );
		BlueprintObj->Modify();

		UEdGraphNode* FunctionNode = nullptr;
		UEdGraph* FunctionGraph = CollapseSelectionToFunction(FocusedGraphEdPtr.Pin(), CollapsableNodes, FunctionNode);

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified( BlueprintObj );

		RenameNewlyAddedAction(FunctionGraph->GetFName());
	}
}

bool FBlueprintEditor::CanCollapseSelectionToFunction(TSet<class UEdGraphNode*>& InSelection) const
{
	bool bBadConnection = false;
	UEdGraphPin* OutputConnection = nullptr;
	UEdGraphPin* InputConnection = nullptr;

	// Create a function graph
	UEdGraph* FunctionGraph = FBlueprintEditorUtils::CreateNewGraph(GetBlueprintObj(), FBlueprintEditorUtils::FindUniqueKismetName(GetBlueprintObj(), TEXT("TempGraph")), UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());
	FBlueprintEditorUtils::AddFunctionGraph<UClass>(GetBlueprintObj(), FunctionGraph, /*bIsUserCreated=*/ true, nullptr);

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	FCompilerResultsLog LogResults;
	LogResults.bAnnotateMentionedNodes = false;

	UEdGraphNode* InterfaceTemplateNode = nullptr;

	TArray<UEdGraphPin*> EntryGatewayPins;

	// Runs through every node and fully validates errors with placing selection in a function graph, reporting all errors.
	for (TSet<UEdGraphNode*>::TConstIterator NodeIt(InSelection); NodeIt; ++NodeIt)
	{
		UEdGraphNode* Node = *NodeIt;

		if(!Node->CanPasteHere(FunctionGraph))
		{
			if (UK2Node_CustomEvent* const CustomEvent = Cast<UK2Node_CustomEvent>(Node))
			{
				UEdGraphPin* EventExecPin = K2Schema->FindExecutionPin(*CustomEvent, EGPD_Output);
				check(EventExecPin);

				if(InterfaceTemplateNode)
				{
					LogResults.Error(*LOCTEXT("TooManyCustomEvents_Error", "Can use @@ as a template for creating the function, can only have a single custom event! Previously found @@").ToString(), CustomEvent, InterfaceTemplateNode);
				}
				else
				{
					// The custom event will be used as the template interface for the function.
					InterfaceTemplateNode = CustomEvent;
					if(InputConnection)
					{
						InputConnection = (EventExecPin->LinkedTo.Num() > 0) ? EventExecPin->LinkedTo[0] : nullptr;
					}
					continue;
				}
			}

			LogResults.Error(*LOCTEXT("CannotPasteNodeFunction_Error", "@@ cannot be placed in function graph").ToString(), Node);
			bBadConnection = true;
		}
		else
		{
			for (UEdGraphPin* NodePin : Node->Pins)
			{
				if (NodePin->PinType.PinCategory == K2Schema->PC_Exec)
				{
					if (NodePin->LinkedTo.Num() == 0 && NodePin->Direction == EGPD_Input)
					{
						EntryGatewayPins.Add(NodePin);
					}
					else
					{
						for (UEdGraphPin* ConnectedPin : NodePin->LinkedTo)
						{
							if (!InSelection.Contains(ConnectedPin->GetOwningNode()))
							{
								if (NodePin->Direction == EGPD_Input)
								{
									// For input pins, there must be a single connection 
									if ((InputConnection == nullptr) || (InputConnection == NodePin))
									{
										EntryGatewayPins.Add(NodePin);
										InputConnection = NodePin;
									}
									else
									{
										// Check if the input connection was linked, report what node it is connected to
										LogResults.Error(*LOCTEXT("TooManyPathsMultipleInput_Error", "Found too many input connections in selection! @@ is connected to @@, previously found @@ connected to @@").ToString(), Node, ConnectedPin->GetOwningNode(), InputConnection->GetOwningNode(), (InputConnection->LinkedTo.Num() > 0) ? InputConnection->LinkedTo[0]->GetOwningNode() : nullptr);
										bBadConnection = true;
									}
								}
								else
								{
									// For output pins, as long as they all connect to the same pin, we consider the selection valid for being made into a function
									if ((OutputConnection == nullptr) || (OutputConnection == ConnectedPin))
									{
										OutputConnection = ConnectedPin;
									}
									else
									{
										LogResults.Error(*LOCTEXT("TooManyPathsMultipleOutput_Error", "Found too many output connections in selection! @@ is connected to @@, previously found @@ connected to @@").ToString(), Node, ConnectedPin->GetOwningNode(), OutputConnection->GetOwningNode(), (OutputConnection->LinkedTo.Num() > 0) ? OutputConnection->LinkedTo[0]->GetOwningNode() : nullptr);
										bBadConnection = true;
									}
								}
							}
						}
					}
				}
			}
		}
	}

	if (!bBadConnection && (InputConnection == nullptr) && (EntryGatewayPins.Num() > 1))
	{
		// Too many input gateway pins with no connections.
		LogResults.Error(*LOCTEXT("AmbiguousEntryPaths_Error", "Multiple entry pin possibilities. Unable to convert to a function. Make sure that selection either has only 1 entry pin or exactly 1 entry pin has a connection.").ToString());
		bBadConnection = true;
	}

	// No need to check for cycling if the selection is invalid anyways.
	if(!bBadConnection && FBlueprintEditorUtils::CheckIfSelectionIsCycling(InSelection, LogResults))
	{
		bBadConnection = true;
	}

	FMessageLog MessageLog("BlueprintLog");
	MessageLog.NewPage(LOCTEXT("CollapseToFunctionPageLabel", "Collapse to Function"));
	MessageLog.AddMessages(LogResults.Messages);
	MessageLog.Notify(LOCTEXT("CollapseToFunctionError", "Collapsing to Function Failed!"));

	FBlueprintEditorUtils::RemoveGraph(GetBlueprintObj(), FunctionGraph);
	FunctionGraph->MarkAsGarbage();
	return !bBadConnection;
}

bool FBlueprintEditor::CanCollapseSelectionToFunction() const
{
	return !IsEditingAnimGraph();
}

void FBlueprintEditor::OnCollapseSelectionToMacro()
{
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	// Does the selection set contain anything that is legal to collapse?
	TSet<UEdGraphNode*> CollapsableNodes;
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		if (UEdGraphNode* SelectedNode = Cast<UEdGraphNode>(*NodeIt))
		{
			if (Schema->CanEncapuslateNode(*SelectedNode))
			{
				CollapsableNodes.Add(SelectedNode);
			}
		}
	}

	// Collapse them
	if (CollapsableNodes.Num() && CanCollapseSelectionToMacro(CollapsableNodes))
	{
		UBlueprint* BlueprintObj = GetBlueprintObj();
		const FScopedTransaction Transaction( FGraphEditorCommands::Get().CollapseNodes->GetDescription() );
		BlueprintObj->Modify();

		UEdGraphNode* MacroNode = nullptr;
		UEdGraph* MacroGraph = CollapseSelectionToMacro(FocusedGraphEdPtr.Pin(), CollapsableNodes, MacroNode);

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified( BlueprintObj );

		RenameNewlyAddedAction(MacroGraph->GetFName());
	}
}

bool FBlueprintEditor::CanCollapseSelectionToMacro(TSet<class UEdGraphNode*>& InSelection) const
{
	// Create a temporary macro graph
	UEdGraph* MacroGraph = FBlueprintEditorUtils::CreateNewGraph(GetBlueprintObj(), FBlueprintEditorUtils::FindUniqueKismetName(GetBlueprintObj(), TEXT("TempGraph")), UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());
	FBlueprintEditorUtils::AddMacroGraph(GetBlueprintObj(), MacroGraph, /*bIsUserCreated=*/ true, nullptr);

	// Does the selection set contain anything that is legal to collapse?
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	bool bCollapseAllowed = true;
	FCompilerResultsLog LogResults;
	LogResults.bAnnotateMentionedNodes = false;

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UEdGraphNode* Node = Cast<UEdGraphNode>(*NodeIt);

		if(!Node->CanPasteHere(MacroGraph))
		{
			LogResults.Error(*LOCTEXT("CannotPasteNodeMacro_Error", "@@ cannot be placed in macro graph").ToString(), Node);
			bCollapseAllowed = false;
		}
	}

	FMessageLog MessageLog("BlueprintLog");
	MessageLog.NewPage(LOCTEXT("CollapseToMacroPageLabel", "Collapse to Macro"));
	MessageLog.AddMessages(LogResults.Messages);
	MessageLog.Notify(LOCTEXT("CollapseToMacroError", "Collapsing to Macro Failed!"));

	FBlueprintEditorUtils::RemoveGraph(GetBlueprintObj(), MacroGraph);
	MacroGraph->MarkAsGarbage();
	return bCollapseAllowed;
}

bool FBlueprintEditor::CanCollapseSelectionToMacro() const
{
	if (FocusedGraphEdPtr.IsValid())
	{
		if(IsEditingAnimGraph())
		{
			return false;
		}
	}

	return true;
}

void FBlueprintEditor::OnPromoteSelectionToFunction()
{
	const FScopedTransaction Transaction( LOCTEXT("ConvertCollapsedGraphToFunction", "Convert Collapse Graph to Function") );
	UBlueprint* BP = GetBlueprintObj();
	BP->Modify();

	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();

	// Set of nodes to select when finished
	TSet<UEdGraphNode*> NodesToSelect;

	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator It(SelectedNodes); It; ++It)
	{
		if (UK2Node_Composite* CompositeNode = Cast<UK2Node_Composite>(*It))
		{
			// Check if there is only one input and one output connection
			TSet<UEdGraphNode*> NodesInGraph;
			NodesInGraph.Add(CompositeNode);

			if(CanCollapseSelectionToFunction(NodesInGraph))
			{
				DocumentManager->CleanInvalidTabs();

				UEdGraph* SourceGraph = CompositeNode->BoundGraph;
				
				TSet<UEdGraphNode*> NodesToMove;
				CollapseGraphUtils::GatherMoveableNodes(SourceGraph, /* Out */ NodesToMove);

				// Remove this node from selection
				FocusedGraphEd->SetNodeSelection(CompositeNode, false);

				UEdGraphNode* FunctionNode = nullptr;
				CollapseSelectionToFunction(FocusedGraphEd, NodesToMove, FunctionNode);
				NodesToSelect.Add(FunctionNode);

				// Connect the exec pin of the newly dropped function call node to any previously existing connections
				const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
				UEdGraphPin* ResultExecFunc = K2Schema->FindExecutionPin(*FunctionNode, EGPD_Input);
				UEdGraphPin* OriginalExecPin = K2Schema->FindExecutionPin(*CompositeNode, EGPD_Input);

				if (ResultExecFunc && OriginalExecPin)
				{
					for (UEdGraphPin* CurPin : OriginalExecPin->LinkedTo)
					{
						// Make a connection if this is an exec pin
						if (CurPin && CurPin->Direction == EGPD_Output && K2Schema->IsExecPin(*CurPin))
						{
							ResultExecFunc->MakeLinkTo(CurPin);
						}
					}
				}

				// Remove the old collapsed graph
				FBlueprintEditorUtils::RemoveNode(BP, CompositeNode);
			}
			else
			{
				NodesToSelect.Add(CompositeNode);
			}
		}
		else if(UEdGraphNode* Node = Cast<UEdGraphNode>(*It))
		{
			NodesToSelect.Add(Node);
		}
	}

	// Select all nodes that should still be part of selection
	for (UEdGraphNode* NodeToSelect : NodesToSelect)
	{
		FocusedGraphEd->SetNodeSelection(NodeToSelect, true);
	}
}

bool FBlueprintEditor::CanPromoteSelectionToFunction() const
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		if (UK2Node_Composite* CompositeNode = Cast<UK2Node_Composite>(*NodeIt))
		{
			// Check if there is only one input and one output connection
			TSet<class UEdGraphNode*> NodesInGraph;
			NodesInGraph.Add(CompositeNode);

			return true;
		}
	}
	return false;
}

void FBlueprintEditor::OnPromoteSelectionToMacro()
{
	const FScopedTransaction Transaction( LOCTEXT("ConvertCollapsedGraphToMacro", "Convert Collapse Graph to Macro") );
	UBlueprint* BP = GetBlueprintObj();
	BP->Modify();

	// Set of nodes to select when finished
	TSet<UEdGraphNode*> NodesToSelect;

	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();

	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator It(SelectedNodes); It; ++It)
	{
		if (UK2Node_Composite* CompositeNode = Cast<UK2Node_Composite>(*It))
		{
			TSet<UEdGraphNode*> NodesToMove;
			UEdGraph* SourceGraph = CompositeNode->BoundGraph;

			// Gather the entry and exit nodes of the collapsed graph so that we can create new macro inputs
			CollapseGraphUtils::GatherMoveableNodes(SourceGraph, /* Out */ NodesToMove);

			if(CanCollapseSelectionToMacro(NodesToMove))
			{
				DocumentManager->CleanInvalidTabs();				

				// Remove this node from selection
				FocusedGraphEd->SetNodeSelection(CompositeNode, false);

				UEdGraphNode* MacroNode = nullptr;
				CollapseSelectionToMacro(FocusedGraphEd, NodesToMove, MacroNode);
				NodesToSelect.Add(MacroNode);

				// Reconnect anything that the original composite node was connected to to this new macro instance
				if (MacroNode)
				{
					// Gather what connections were already on the composite node...
					TMap<FString, TSet<UEdGraphPin*>> OldToNewPinMap;
					FEdGraphUtilities::GetPinConnectionMap(CompositeNode, OldToNewPinMap);

					for (UEdGraphPin* const NewPin : MacroNode->Pins)
					{
						// Reconnect any new pins here that we can! 
						const FString& NewPinName = NewPin->GetName();
						if (OldToNewPinMap.Contains(NewPinName))
						{							
							for (UEdGraphPin* OldPin : OldToNewPinMap[NewPinName])
							{
								NewPin->MakeLinkTo(OldPin);
							}
						}
					}
				}

				FBlueprintEditorUtils::RemoveNode(BP, CompositeNode);
			}
			else
			{
				NodesToSelect.Add(CompositeNode);
			}
		}
		else if(UEdGraphNode* Node = Cast<UEdGraphNode>(*It))
		{
			NodesToSelect.Add(Node);
		}
	}

	// Select all nodes that should still be part of selection
	for (UEdGraphNode* NodeToSelect : NodesToSelect)
	{
		FocusedGraphEd->SetNodeSelection(NodeToSelect, true);
	}
}

bool FBlueprintEditor::CanPromoteSelectionToMacro() const
{
	if (FocusedGraphEdPtr.IsValid())
	{
		if(IsEditingAnimGraph())
		{
			return false;
		}
	}

	for (UObject* SelectedNode : GetSelectedNodes())
	{
		UK2Node_Composite* CompositeNode = Cast<UK2Node_Composite>(SelectedNode);
		if (CompositeNode && CompositeNode->BoundGraph)
		{
			TSet<class UEdGraphNode*> NodesInGraph;

			// Collect all the nodes to test if they can be made into a function
			for (UEdGraphNode* Node : CompositeNode->BoundGraph->Nodes)
			{
				// Ignore the tunnel nodes
				if (Node->GetClass() != UK2Node_Tunnel::StaticClass())
				{
					NodesInGraph.Add(Node);
				}
			}

			return true;
		}
	}
	return false;
}

void FBlueprintEditor::OnExpandNodes()
{
	const FScopedTransaction Transaction( FGraphEditorCommands::Get().ExpandNodes->GetLabel() );
	GetBlueprintObj()->Modify();

	TSet<UEdGraphNode*> ExpandedNodes;
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();

	// Expand selected nodes into the focused graph context.
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	
	if(FocusedGraphEd)
	{
		FocusedGraphEd->ClearSelectionSet();
	}

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		ExpandedNodes.Empty();
		bool bExpandedNodesNeedUniqueGuid = true;

		DocumentManager->CleanInvalidTabs();

		if (UK2Node_MacroInstance* SelectedMacroInstanceNode = Cast<UK2Node_MacroInstance>(*NodeIt))
		{
			UEdGraph* MacroGraph = SelectedMacroInstanceNode->GetMacroGraph();
			if(MacroGraph)
			{
				// Clone the graph so that we do not delete the original
				UEdGraph* ClonedGraph = FEdGraphUtilities::CloneGraph(MacroGraph, nullptr);
				ExpandNode(SelectedMacroInstanceNode, ClonedGraph, /*inout*/ ExpandedNodes);

				ClonedGraph->MarkAsGarbage();
			}
		}
		else if (UK2Node_Composite* SelectedCompositeNode = Cast<UK2Node_Composite>(*NodeIt))
		{
			// No need to assign unique GUIDs since the source graph will be removed.
			bExpandedNodesNeedUniqueGuid = false;

			// Expand the composite node back into the world
			UEdGraph* SourceGraph = SelectedCompositeNode->BoundGraph;
			ExpandNode(SelectedCompositeNode, SourceGraph, /*inout*/ ExpandedNodes);

			FBlueprintEditorUtils::RemoveGraph(GetBlueprintObj(), SourceGraph, EGraphRemoveFlags::Recompile);
		}
		else if (UK2Node_CallFunction* SelectedCallFunctionNode = Cast<UK2Node_CallFunction>(*NodeIt))
		{
			const UEdGraphNode* ResultEventNode = nullptr;
			UEdGraph* FunctionGraph = SelectedCallFunctionNode->GetFunctionGraph(ResultEventNode);

			// We should never get here when attempting to expand a call function that calls an event.
			check(!ResultEventNode);

			if(FunctionGraph)
			{
				// Clone the graph so that we do not delete the original
				UEdGraph* ClonedGraph = FEdGraphUtilities::CloneGraph(FunctionGraph, nullptr);
				ExpandNode(SelectedCallFunctionNode, ClonedGraph, ExpandedNodes);

				ClonedGraph->MarkAsGarbage();
			}
		}
		UEdGraphNode* SourceNode = CastChecked<UEdGraphNode>(*NodeIt);
		check(SourceNode);
		MoveNodesToAveragePos(ExpandedNodes, FVector2D(SourceNode->NodePosX, SourceNode->NodePosY), bExpandedNodesNeedUniqueGuid);
	}

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprintObj());
}

bool FBlueprintEditor::CanExpandNodes() const
{
	// Does the selection set contain any composite nodes that are legal to expand?
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		if (Cast<UK2Node_Composite>(*NodeIt))
		{
			return true;
		}
		else if (UK2Node_MacroInstance* SelectedMacroInstanceNode = Cast<UK2Node_MacroInstance>(*NodeIt))
		{
			return SelectedMacroInstanceNode->GetMacroGraph() != nullptr;
		}
		else if (UK2Node_CallFunction* SelectedCallFunctionNode = Cast<UK2Node_CallFunction>(*NodeIt))
		{
			// If ResultEventNode is non-nullptr, it means it is sourced by an event, we do not want to expand events
			const UEdGraphNode* ResultEventNode = nullptr;
			return SelectedCallFunctionNode->GetFunctionGraph(ResultEventNode) != nullptr && ResultEventNode == nullptr;
		}
	}

	return false;
}

void FBlueprintEditor::OnOpenSelectedNodesInNewTab()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		if (const UK2Node_Composite* CompositeNode = Cast<UK2Node_Composite>(*NodeIt))
		{
			if (CompositeNode->BoundGraph)
			{
				OpenDocument(CompositeNode->BoundGraph, FDocumentTracker::ForceOpenNewDocument);
			}
		}
		else if (const UK2Node_MacroInstance* SelectedMacroInstanceNode = Cast<UK2Node_MacroInstance>(*NodeIt))
		{
			if (SelectedMacroInstanceNode->GetMacroGraph())
			{
				OpenDocument(SelectedMacroInstanceNode->GetMacroGraph(), FDocumentTracker::ForceOpenNewDocument);
			}
		}
		else if (const UK2Node_CallFunction* SelectedCallFunctionNode = Cast<UK2Node_CallFunction>(*NodeIt))
		{
			const UEdGraphNode* ResultEventNode = nullptr;
			const UEdGraph* FunctionGraph = SelectedCallFunctionNode->GetFunctionGraph(ResultEventNode);

			if (FunctionGraph && !ResultEventNode)
			{
				OpenDocument(FunctionGraph, FDocumentTracker::ForceOpenNewDocument);
			}
		}
	}
}

bool FBlueprintEditor::CanOpenSelectedNodesInNewTab() const
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		if (const UK2Node_Composite* CompositeNode = Cast<UK2Node_Composite>(*NodeIt))
		{
			if (CompositeNode->BoundGraph)
			{
				return true;
			}
		}
		else if (const UK2Node_MacroInstance* SelectedMacroInstanceNode = Cast<UK2Node_MacroInstance>(*NodeIt))
		{
			if (SelectedMacroInstanceNode->GetMacroGraph())
			{
				return true;
			}
		}
		else if (const UK2Node_CallFunction* SelectedCallFunctionNode = Cast<UK2Node_CallFunction>(*NodeIt))
		{
			const UEdGraphNode* ResultEventNode = nullptr;
			const UEdGraph* FunctionGraph = SelectedCallFunctionNode->GetFunctionGraph(ResultEventNode);

			// Only allow opening a new tab if the call function isn't sourced by a custom event call,
			// because in that case it just opens the current graph in a new tab which can be confusing and doesn't add any benefit to the user 
			return FunctionGraph != nullptr && ResultEventNode == nullptr;
		}
	}

	return false;
}

void FBlueprintEditor::MoveNodesToAveragePos(TSet<UEdGraphNode*>& AverageNodes, FVector2D SourcePos, bool bExpandedNodesNeedUniqueGuid /* = false */) const
{
	if (AverageNodes.Num() > 0)
	{
		FVector2D AvgNodePosition(0.0f, 0.0f);

		for (TSet<UEdGraphNode*>::TIterator It(AverageNodes); It; ++It)
		{
			UEdGraphNode* Node = *It;
			AvgNodePosition.X += Node->NodePosX;
			AvgNodePosition.Y += Node->NodePosY;
		}

		float InvNumNodes = 1.0f / float(AverageNodes.Num());
		AvgNodePosition.X *= InvNumNodes;
		AvgNodePosition.Y *= InvNumNodes;

		TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();

		for (UEdGraphNode* ExpandedNode : AverageNodes)
		{
			ExpandedNode->NodePosX = static_cast<int32>((ExpandedNode->NodePosX - AvgNodePosition.X) + SourcePos.X);
			ExpandedNode->NodePosY = static_cast<int32>((ExpandedNode->NodePosY - AvgNodePosition.Y) + SourcePos.Y);

			ExpandedNode->SnapToGrid(SNodePanel::GetSnapGridSize());

			if (bExpandedNodesNeedUniqueGuid)
			{
				ExpandedNode->CreateNewGuid();
			}

			//Add expanded node to selection
			FocusedGraphEd->SetNodeSelection(ExpandedNode, true);
		}
	}
}

bool FBlueprintEditor::CanConvertFunctionToEvent() const
{
	UEdGraphNode* const SelectedNode = GetSingleSelectedNode();	

	if (UK2Node_FunctionEntry* const SelectedCallFunctionNode = Cast<UK2Node_FunctionEntry>(SelectedNode))
	{
		return FBlueprintEditorUtils::IsFunctionConvertableToEvent(GetBlueprintObj(), SelectedCallFunctionNode->FindSignatureFunction());
	}
	
	return false;
}

void FBlueprintEditor::OnConvertFunctionToEvent()
{
	UEdGraphNode* SelectedNode = GetSingleSelectedNode();

	if (UK2Node_FunctionEntry* SelectedCallFunctionNode = Cast<UK2Node_FunctionEntry>(SelectedNode))
	{
		ConvertFunctionIfValid(SelectedCallFunctionNode);
	}
}

bool FBlueprintEditor::ConvertFunctionIfValid(UK2Node_FunctionEntry* FuncEntryNode)
{
	FCompilerResultsLog LogResults;
	FMessageLog MessageLog("BlueprintLog");
	FText SpecificErrorMessage;
	
	UBlueprint* BlueprintObj = FuncEntryNode ? FuncEntryNode->GetBlueprint() : nullptr;

	if(BlueprintObj && FuncEntryNode)
	{
		UFunction* Func = FuncEntryNode->FindSignatureFunction();
		check(Func);

		// Make sure there are no output parameters
		if (UEdGraphSchema_K2::HasFunctionAnyOutputParameter(Func))
		{
			LogResults.Error(*LOCTEXT("FunctionHasOutput_Error", "A function can only be converted if it does not have any output parameters.").ToString());
			SpecificErrorMessage = LOCTEXT("FunctionHasOutput_Error_Title", "Function cannot have output parameters");
		}
		// Make sure this is not a blueprint/macro function library
		else if (BlueprintObj->BlueprintType == BPTYPE_FunctionLibrary || BlueprintObj->BlueprintType == BPTYPE_MacroLibrary)
		{
			LogResults.Error(*LOCTEXT("BlueprintFunctionLibarary_Error", "Cannot convert functions from blueprint or macro libraries.").ToString());
			SpecificErrorMessage = LOCTEXT("BlueprintFunctionLibarary_Error_Title", "Cannot convert blueprint or macro library functions");
		}
		// Ensure that this is no the construction script
		else if (FuncEntryNode->FunctionReference.GetMemberName() == UEdGraphSchema_K2::FN_UserConstructionScript)
		{
			LogResults.Error(*LOCTEXT("ConvertConstructionScript_Error", "Cannot convert the construction script!").ToString());
			SpecificErrorMessage = LOCTEXT("ConvertConstructionScript_Error_Title", "Cannot convert construction script");
		}
		// Make sure we are not on the animation graph
		else if (IsEditingAnimGraph())
		{
			LogResults.Error(*LOCTEXT("ConvertAnimGraph_Error", "Cannot convert functions on the anim graph!").ToString());
			SpecificErrorMessage = LOCTEXT("ConvertAnimGraph_Error_Title", "Cannot convert on the anim graph");
		}
		else
		{
			ConvertFunctionToEvent(FuncEntryNode);
		}		
	}
	else
	{
		LogResults.Error(*LOCTEXT("MultipleNodesSelectred_Error", "Only one node can be selected for conversion!").ToString());
	}

	// Show the log results if there were any errors
	if (LogResults.NumErrors)
	{
		MessageLog.NewPage(LOCTEXT("OnConvertEventToFunctionLabel", "Convert Event to Function"));
		MessageLog.AddMessages(LogResults.Messages);

		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("SpecificErrorMessage"), SpecificErrorMessage);
		MessageLog.Notify(FText::Format(LOCTEXT("OnConvertEventToFunctionErrorMsg", "Convert Event to Function Failed!\n{SpecificErrorMessage}"), Arguments));
	}

	// If there are no errors, we succeed and should return true
	return LogResults.NumErrors == 0;
}

void FBlueprintEditor::ConvertFunctionToEvent(UK2Node_FunctionEntry* SelectedCallFunctionNode)
{
	check(SelectedCallFunctionNode);
	UBlueprint* NodeBP = GetBlueprintObj();
	UEdGraph* FunctionGraph = SelectedCallFunctionNode->GetGraph();

	// Create a new event node with the old function name
	UFunction* Func = SelectedCallFunctionNode->FindSignatureFunction();
	UEdGraph* EventGraph = FBlueprintEditorUtils::FindEventGraph(NodeBP);

	if (Func && EventGraph && FunctionGraph)
	{
		const FScopedTransaction Transaction(FGraphEditorCommands::Get().ConvertFunctionToEvent->GetLabel());
		NodeBP->Modify();
		EventGraph->Modify();

		// Find all return nodes and break their connections
		TArray<UK2Node_FunctionResult*> FunctionReturnNodes;
		FunctionGraph->GetNodesOfClass(FunctionReturnNodes);
		for (UK2Node_FunctionResult* Node : FunctionReturnNodes)
		{
			if (Node)
			{
				Node->BreakAllNodeLinks();
			}
		}

		// Keep track of the old connections from the entry node
		TMap<FString, TSet<UEdGraphPin*>> EntryNodePinConnections;
		FEdGraphUtilities::GetPinConnectionMap(SelectedCallFunctionNode, EntryNodePinConnections);

		FName EventName = Func->GetFName();
		UClass* const OverrideFuncClass = CastChecked<UClass>(Func->GetOuter())->GetAuthoritativeClass();
		
		UK2Node_Event* NewEventNode = nullptr;
		FVector2D SpawnPos = EventGraph->GetGoodPlaceForNewNode();
		
		// Was this function implemented in as an override?
		UFunction* ParentFunction = FindUField<UFunction>(NodeBP->ParentClass, EventName);
		bool bIsOverrideFunc = Func->GetSuperFunction() || (ParentFunction != nullptr);
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

		// Allow an interface function to be converted in the case where the interface has changed
		// and the function should now be placed as an event (UE-85687)
		if (bIsOverrideFunc || FBlueprintEditorUtils::IsInterfaceFunction(NodeBP, Func))
		{
			NewEventNode = FEdGraphSchemaAction_K2NewNode::SpawnNode<UK2Node_Event>(
				EventGraph,
				SpawnPos,
				EK2NewNodeFlags::SelectNewNode,
				[EventName, OverrideFuncClass](UK2Node_Event* NewInstance)
				{
					NewInstance->EventReference.SetExternalMember(EventName, OverrideFuncClass);
					NewInstance->bOverrideFunction = true;
				}
			);
		}
		else
		{
			// Spawn a new node, we have to do it this way to avoid renaming on creation of the custom event
			UEdGraphNode* NewNode = nullptr;
			NewNode = NewObject<UEdGraphNode>(EventGraph, UK2Node_CustomEvent::StaticClass());
			NewNode->CreateNewGuid();

			NewNode->NodePosX = static_cast<int32>(SpawnPos.X);
			NewNode->NodePosY = static_cast<int32>(SpawnPos.Y);

			NewNode->SetFlags(RF_Transactional);
			NewNode->AllocateDefaultPins();
			NewNode->PostPlacedNewNode();
			EventGraph->Modify();
			// the FBlueprintMenuActionItem should do the selecting
			EventGraph->AddNode(NewNode, /*bFromUI =*/false, /*bSelectNewNode =*/false);
			
			NewEventNode = Cast<UK2Node_CustomEvent>(NewNode);
			NewEventNode->CustomFunctionName = EventName;
			NewEventNode->bOverrideFunction = false;

			// Add every type of user pin that we need to the new event node
			for (TSharedPtr<FUserPinInfo> Pin : SelectedCallFunctionNode->UserDefinedPins)
			{
				NewEventNode->CreateUserDefinedPin(Pin->PinName, Pin->PinType, Pin->DesiredPinDirection);
			}

			K2Schema->ReconstructNode(*NewEventNode);
		}

		check(NewEventNode);
		
		TArray<UK2Node_Variable*> AllVarNodes;
		TArray<UK2Node_Variable*> RemainingVarNodes;
		FunctionGraph->GetNodesOfClass<UK2Node_Variable>(AllVarNodes);

		// Parameter getter nodes aren't supported in the event graph, so they'll be destroyed.
		// In order to preserve their connections, we'll add their linked pins to the entry node's pin connection map.
		for (UK2Node_Variable* VarNode : AllVarNodes)
		{
			check(VarNode);
			const FProperty* Property = VarNode->GetPropertyForVariable();
			const bool bIsInputParameter =
				Property &&
				Property->HasAnyPropertyFlags(CPF_Parm) &&
				!Property->HasAnyPropertyFlags(CPF_ReturnParm)
			;

			if (bIsInputParameter)
			{
				TSet<UEdGraphPin*>* ConnectionsEntry = EntryNodePinConnections.Find(Property->GetName());
				if (ensure(ConnectionsEntry))
				{
					UEdGraphPin* ValuePin = VarNode->GetValuePin();
					if (ensure(ValuePin))
					{
						for (UEdGraphPin* ConnectedPin : ValuePin->LinkedTo)
						{
							if (ensure(ConnectedPin))
							{
								ConnectionsEntry->Add(ConnectedPin);
							}
						}
					}
				}

				VarNode->DestroyNode();
			}
			else
			{
				RemainingVarNodes.Add(VarNode);
			}
		}

		// If this function had any local scope variables, we need to convert them to global scope
		if (SelectedCallFunctionNode->LocalVariables.Num() > 0)
		{
			// Make a globally scoped version of any local variables
			for (const FBPVariableDescription& LocalVar : SelectedCallFunctionNode->LocalVariables)
			{
				// If a variable already exists of this name globally, then we need to use a unique name
				// Only use FindUniqueKismetName if one exists because otherwise it will always add a "_0" to the name
				FName NewVarName = FBlueprintEditorUtils::FindNewVariableIndex(NodeBP, LocalVar.VarName) == INDEX_NONE ?
					LocalVar.VarName : FBlueprintEditorUtils::FindUniqueKismetName(NodeBP, LocalVar.VarName.ToString());

				if (FBlueprintEditorUtils::AddMemberVariable(NodeBP, NewVarName, LocalVar.VarType, LocalVar.DefaultValue))
				{
					// Upon success, update the variable node's reference members
					const FGuid NewVarGuid = FBlueprintEditorUtils::FindMemberVariableGuidByName(NodeBP, NewVarName);

					for (UK2Node_Variable* VarNode : RemainingVarNodes)
					{
						if (VarNode->GetVarName() == LocalVar.VarName)
						{
							VarNode->Modify();
							VarNode->VariableReference.SetDirect(NewVarName, NewVarGuid, nullptr, true);
						}
					}
				}
			}
		}

		// Keep track of any nodes that have been expanded out of the function graph
		TSet<UEdGraphNode*> ExpandedNodes;

		UEdGraphNode* Entry = nullptr;
		UEdGraphNode* Result = nullptr;
		FunctionGraph->Modify();
		MoveNodesToGraph(MutableView(FunctionGraph->Nodes), EventGraph, ExpandedNodes, &Entry, &Result, false);

		MoveNodesToAveragePos(ExpandedNodes, FVector2D(SpawnPos.X + 500.0f, SpawnPos.Y));
		
		if (Entry)
		{
			Entry->DestroyNode();
		}

		if (Result)
		{
			Result->DestroyNode();
		}

		// Link the nodes from the original function entry node to the new event node
		FEdGraphUtilities::ReconnectPinMap(NewEventNode, EntryNodePinConnections);

		// Copy over parameter default values to the new event node
		FEdGraphUtilities::CopyPinDefaults(SelectedCallFunctionNode, NewEventNode);

		// Remove the old function graph
		FBlueprintEditorUtils::RemoveGraph(NodeBP, FunctionGraph, EGraphRemoveFlags::Recompile);
		FunctionGraph->MarkAsGarbage();
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(NodeBP);

		// Do this AFTER removing the function graph so that it's not opened into the existing function graph document tab.
		FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(NewEventNode, false);
	}
}

bool FBlueprintEditor::CanConvertEventToFunction() const
{
	// Only allow when only a _single_ event node is selected, not allowed on the anim graph
	UEdGraphNode* SelectedNode = GetSingleSelectedNode();
	if (SelectedNode && SelectedNode->IsA<UK2Node_Event>())
	{
		return true;
	}

	return false;
}

bool FBlueprintEditor::ConvertEventIfValid(UK2Node_Event* EventToConv)
{
	FCompilerResultsLog LogResults;
	FMessageLog MessageLog("BlueprintLog");
	FText SpecificErrorMessage;

	if(EventToConv)
	{
		UFunction* const Func = FFunctionFromNodeHelper::FunctionFromNode(EventToConv);

		// Ensure we are not trying to do this on an interface event
		if (FBlueprintEditorUtils::IsInterfaceFunction(GetBlueprintObj(), Func) || EventToConv->IsInterfaceEventNode())
		{
			LogResults.Error(*LOCTEXT("EventIsOnInterface_Error", "Only non-interface events can be converted to functions!").ToString());
			SpecificErrorMessage = LOCTEXT("EventIsOnInterface_Error_Title", "Cannot convert interface events");
		}
		// Make sure that we are not on the animation graph
		else if (IsEditingAnimGraph())
		{
			LogResults.Error(*LOCTEXT("ConvertAnimGraphEvent_Error", "Cannot convert events on the anim graph!").ToString());
			SpecificErrorMessage = LOCTEXT("ConvertAnimGraphEvent_Error_Title", "Cannot convert on the anim graph");
		}
		else
		{
			ConvertEventToFunction(EventToConv);
		}
	}
	else
	{
		SpecificErrorMessage = LOCTEXT("MultipleNodesSelectred_Error_Title", "Only one node can be selected for conversion");
		LogResults.Error(*LOCTEXT("MultipleNodesSelectred_Error", "Only one node can be selected for conversion!").ToString());
	}

	// Show the log results if there were any errors
	if (LogResults.NumErrors)
	{
		MessageLog.NewPage(LOCTEXT("OnConvertEventToFunctionLabel", "Convert Event to Function"));
		MessageLog.AddMessages(LogResults.Messages);

		// Format the title node
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("SpecificErrorMessage"), SpecificErrorMessage);
		MessageLog.Notify(FText::Format(LOCTEXT("OnConvertEventToFunctionErrorMsg", "Convert Event to Function Failed!\n{SpecificErrorMessage}"), Arguments));
	}

	// If there are no errors, we succeed and should return true
	return LogResults.NumErrors == 0;
}

void FBlueprintEditor::OnConvertEventToFunction()
{
	UEdGraphNode* SelectedNode = GetSingleSelectedNode();

	if (UK2Node_Event* SelectedEventNode = Cast<UK2Node_Event>(SelectedNode))
	{
		ConvertEventIfValid(SelectedEventNode);
	}
}

void FBlueprintEditor::ConvertEventToFunction(UK2Node_Event* SelectedEventNode)
{	
	if (SelectedEventNode)
	{
		const FScopedTransaction Transaction(FGraphEditorCommands::Get().ConvertEventToFunction->GetLabel());

		UBlueprint* const NodeBP = GetBlueprintObj();
		NodeBP->Modify();

		const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

		// Refresh this node before getting the pin names to ensure that we get the most up to date ones
		Schema->ReconstructNode(*SelectedEventNode);

		// Keep track of the old connections from the event node
		TMap<FString, TSet<UEdGraphPin*>> PinConnections;
		FEdGraphUtilities::GetPinConnectionMap(SelectedEventNode, PinConnections);

		TArray<UEdGraphNode*> CollapsableNodes = GetAllConnectedNodes(SelectedEventNode);

		UFunction* const FunctionSig = FFunctionFromNodeHelper::FunctionFromNode(SelectedEventNode);
		UEdGraph* const SourceGraph = SelectedEventNode->GetGraph();
		const FName OriginalEventName = SelectedEventNode->GetFunctionName();

		if (FunctionSig && SourceGraph)
		{
			SourceGraph->Modify(); 

			// Check if this is an override function
			UFunction* ParentFunction = FindUField<UFunction>(NodeBP->ParentClass, OriginalEventName);
			const bool bIsOverrideFunc = FunctionSig->GetSuperFunction() || (ParentFunction != nullptr);
			UClass* const OverrideFuncClass = FBlueprintEditorUtils::GetOverrideFunctionClass(NodeBP, OriginalEventName);

			// Create a new function graph
			// We have to start it out with a temporary graph name because otherwise the flags will get renamed uniquely
			UEdGraph* NewGraph = nullptr;
			static const FString TempFuncGraph = "TEMP_FUNC_GRAPH";
			const FName TempFuncGraphName = FBlueprintEditorUtils::GenerateUniqueGraphName(NodeBP, TempFuncGraph);
			NewGraph = FBlueprintEditorUtils::CreateNewGraph(NodeBP, TempFuncGraphName, SourceGraph->GetClass(), SourceGraph->GetSchema() ? SourceGraph->GetSchema()->GetClass() : Schema->GetClass());

			if (bIsOverrideFunc)
			{
				FBlueprintEditorUtils::AddFunctionGraph<UClass>(NodeBP, NewGraph, /*bIsUserCreated=*/ false, OverrideFuncClass);
			}
			else
			{
				FBlueprintEditorUtils::AddFunctionGraph<UFunction>(NodeBP, NewGraph, /*bIsUserCreated=*/ true, FunctionSig);
			}
			
			// Get the new entry node
			TArray<UK2Node_FunctionEntry*> EntryNodes;
			NewGraph->GetNodesOfClass<UK2Node_FunctionEntry>(EntryNodes);
			check(EntryNodes.Num() > 0);
			UK2Node_FunctionEntry* NewEntryNode = EntryNodes[0];

			// Reconstruct the pins and add a proper function reference to the parent function
			if (bIsOverrideFunc && NewEntryNode)
			{
				FGuid GraphGuid;
				FBlueprintEditorUtils::GetFunctionGuidFromClassByFieldName(FBlueprintEditorUtils::GetMostUpToDateClass(OverrideFuncClass), OriginalEventName, /* Out */GraphGuid);
		
				// Set the external members of this function appropriately
				NewEntryNode->FunctionReference.SetExternalMember(OriginalEventName, SelectedEventNode->EventReference.GetMemberParentClass(), GraphGuid);
				Schema->ReconstructNode(*NewEntryNode);
			}
			
			NewGraph->Modify();

			UEdGraphNode* InvalidNode = nullptr;
			FCompilerResultsLog LogResults;

			// Check to make sure that this node can actually be put in event graph
			for (UEdGraphNode* const Node : CollapsableNodes)
			{
				if (Node && (!Schema->CanEncapuslateNode(*Node) || !Node->CanPasteHere(NewGraph)))
				{
					// Tell the user why we can't send to function graph
					LogResults.Error(*LOCTEXT("InvalidNode_Error", "@@ cannot be placed in function graph").ToString(), Node);
					InvalidNode = Node;
				}
			}

			// If this is invalid, then cancel this operation and focus on the node that is invalid
			if (InvalidNode != nullptr)
			{
				FMessageLog MessageLog("BlueprintLog");
				MessageLog.NewPage(LOCTEXT("OnConvertEventToFunctionLabel", "Convert Event to Function"));
				MessageLog.AddMessages(LogResults.Messages);
				MessageLog.Notify(LOCTEXT("OnConvertEventToFunctionError", "Convert Event to Function Failed!"));

				FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(InvalidNode, false);

				// Get rid of the function graph
				FBlueprintEditorUtils::RemoveGraph(NodeBP, NewGraph);
				NewGraph->MarkAsGarbage();
				return;
			}

			// Remove any return nodes that may have been added @see UE-78785
			TArray<UK2Node_FunctionResult*> FunctionReturnNodes;
			NewGraph->GetNodesOfClass(FunctionReturnNodes);
			for (UK2Node_FunctionResult* Node : FunctionReturnNodes)
			{
				if (Node)
				{
					Node->DestroyNode();
				}
			}
			
			// Break any connections to the delegate pin, because that won't exist 
			{
				UEdGraphPin* DelegatePin = SelectedEventNode->FindPin(UK2Node_Event::DelegateOutputName);
				if (DelegatePin)
				{
					DelegatePin->BreakAllPinLinks();
				}
			}

			// Move the nodes to the new graph
			UEdGraphNode* Entry = nullptr;
			UEdGraphNode* Result = nullptr;
			TSet<UEdGraphNode*> ExpandedNodes;
			MoveNodesToGraph(CollapsableNodes, NewGraph, ExpandedNodes, &Entry, &Result);
			
			// Link the new nodes accordingly
			if (NewEntryNode)
			{
				FEdGraphUtilities::ReconnectPinMap(NewEntryNode, PinConnections);
				FEdGraphUtilities::CopyPinDefaults(SelectedEventNode, NewEntryNode);

				MoveNodesToAveragePos(ExpandedNodes, FVector2D(NewEntryNode->NodePosX + 500.0f, NewEntryNode->NodePosY));
			}
			else
			{
				MoveNodesToAveragePos(ExpandedNodes, NewGraph->GetGoodPlaceForNewNode());
			}

			FBlueprintEditorUtils::UpdateTransactionalFlags(NodeBP);
			
			FVector2D NewFuncCallSpawn( SelectedEventNode->NodePosX, SelectedEventNode->NodePosY );

			FBlueprintEditorUtils::RemoveNode(NodeBP, SelectedEventNode, /* bDontRecompile= */ true);

			// Cache the string conversion of the event name
			const FString OriginalEventNameString = OriginalEventName.ToString();

			// If there is already another object in the same scope with this name, rename it.
			UObject* ExistingObject = StaticFindObject(/*Class=*/ nullptr, NodeBP, *OriginalEventNameString, EFindObjectFlags::ExactClass);
			if (ExistingObject)
			{
				check(ExistingObject->GetOuter() == NewGraph->GetOuter());

				ExistingObject->Rename(nullptr, nullptr, REN_DontCreateRedirectors);
			}

			// Rename the function graph to the original name
			FBlueprintEditorUtils::RenameGraph(NewGraph, *OriginalEventNameString);

			// If this function is blueprint callable, then spawn a function call node to it in place of the old event
			UFunction* const NewFunction = FindUField<UFunction>(NodeBP->SkeletonGeneratedClass, OriginalEventName);
			if (NewFunction && NewFunction->HasAllFunctionFlags(FUNC_BlueprintCallable))
			{
				IBlueprintNodeBinder::FBindingSet Bindings;
				UEdGraphNode* OutFunctionNode = UBlueprintFunctionNodeSpawner::Create(NewFunction)->Invoke(SourceGraph, Bindings, NewFuncCallSpawn);
				if (NewEntryNode)
				{
					FEdGraphUtilities::CopyPinDefaults(NewEntryNode, OutFunctionNode);
				}
				FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(OutFunctionNode, false);
			}
			else if(NewEntryNode)
			{
				FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(NewEntryNode, false);
			}
		}

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(NodeBP);
	}
}

TArray<UEdGraphNode*> FBlueprintEditor::GetAllConnectedNodes(UEdGraphNode* const SourceNode) const
{
	TArray<UEdGraphNode*> OutNodes;

	if (SourceNode == nullptr)
	{
		return OutNodes;
	}

	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	TQueue<UEdGraphNode*> Queue;
	Queue.Enqueue(SourceNode);
	
	TMap<int32, UEdGraphNode*> VisitedNodes;
	VisitedNodes.Add(SourceNode->GetUniqueID(), SourceNode);

	while (!Queue.IsEmpty())
	{
		UEdGraphNode* Current = nullptr;
		Queue.Dequeue(Current);

		if (Current != nullptr)
		{
			// Don't add the original node in the list
			if (Current != SourceNode)
			{
				OutNodes.Add(Current);
			}
			
			// For every pin that this node has
			for (UEdGraphPin* CurrentPin : Current->Pins)
			{
				// Look at what pins are connected to that
				for (UEdGraphPin* LinkedPin : CurrentPin->LinkedTo)
				{
					UEdGraphNode* OwningNode = LinkedPin->GetOwningNode();

					// If we can encapsulate the owner and it has not been visited
					if (OwningNode && !VisitedNodes.Contains(OwningNode->GetUniqueID()))
					{
						// Mark as visited and add to the queue
						VisitedNodes.Add(OwningNode->GetUniqueID(), OwningNode);
						Queue.Enqueue(OwningNode);
					}
				}
			}
		}
	}

	return OutNodes;
}

void FBlueprintEditor::OnAlignTop()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		FocusedGraphEd->OnAlignTop();
	}
}

void FBlueprintEditor::OnAlignMiddle()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		FocusedGraphEd->OnAlignMiddle();
	}
}

void FBlueprintEditor::OnAlignBottom()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		FocusedGraphEd->OnAlignBottom();
	}
}

void FBlueprintEditor::OnAlignLeft()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		FocusedGraphEd->OnAlignLeft();
	}
}

void FBlueprintEditor::OnAlignCenter()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		FocusedGraphEd->OnAlignCenter();
	}
}

void FBlueprintEditor::OnAlignRight()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		FocusedGraphEd->OnAlignRight();
	}
}

void FBlueprintEditor::OnStraightenConnections()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		FocusedGraphEd->OnStraightenConnections();
	}
}

void FBlueprintEditor::OnDistributeNodesH()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		FocusedGraphEd->OnDistributeNodesH();
	}
}

void FBlueprintEditor::OnDistributeNodesV()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		FocusedGraphEd->OnDistributeNodesV();
	}
}

void FBlueprintEditor::OnStackNodesH()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		FocusedGraphEd->OnStackNodesH();
	}
}

void FBlueprintEditor::OnStackNodesV()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		FocusedGraphEd->OnStackNodesV();
	}
}

void FBlueprintEditor::SelectAllNodes()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		FocusedGraphEd->SelectAllNodes();
	}
}

bool FBlueprintEditor::CanSelectAllNodes() const
{
	return true;
}

void FBlueprintEditor::DeleteSelectedNodes()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (!FocusedGraphEd.IsValid())
	{
		return;
	}

	const FScopedTransaction Transaction( FGenericCommands::Get().Delete->GetDescription() );
	FocusedGraphEd->GetCurrentGraph()->Modify();

	bool bNeedToModifyStructurally = false;

	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	SetUISelectionState(NAME_None);

	if(FocusedGraphEd)
	{
		FocusedGraphEd->ClearSelectionSet();
	}

	// Some nodes have sub-objects that are represented as other tabs.
	// Close them here as a pre-pass before we remove their nodes. If the documents are left open they
	// may reference dangling data and function incorrectly in cases such as FindBlueprintforNodeChecked
	for (FGraphPanelSelectionSet::TConstIterator NodeIt( SelectedNodes ); NodeIt; ++NodeIt)
	{
		if (UEdGraphNode* Node = Cast<UEdGraphNode>(*NodeIt))
		{
			if (Node->CanUserDeleteNode())
			{
				auto CloseAllDocumentsTab = [this](const UEdGraphNode* InNode)
				{
					TArray<UObject*> NodesToClose;
					GetObjectsWithOuter(InNode, NodesToClose);
					for (UObject* Node : NodesToClose)
					{
						UEdGraph* NodeGraph = Cast<UEdGraph>(Node);
						if (NodeGraph)
						{
							CloseDocumentTab(NodeGraph);
						}
					}
				};


				if (Node->GetSubGraphs().Num() > 0)
				{
					CloseAllDocumentsTab(Node);
				}
			}
		}
	}

	// Now remove the selected nodes
	for (FGraphPanelSelectionSet::TConstIterator NodeIt( SelectedNodes ); NodeIt; ++NodeIt)
	{
		if (UEdGraphNode* Node = Cast<UEdGraphNode>(*NodeIt))
		{
			if (Node->CanUserDeleteNode())
			{
				UK2Node* K2Node = Cast<UK2Node>(Node);
				if (K2Node != nullptr && K2Node->NodeCausesStructuralBlueprintChange())
				{
					bNeedToModifyStructurally = true;
				}

				if (Node->GetSubGraphs().Num() > 0)
				{
					DocumentManager->CleanInvalidTabs();
				}
				else if (UK2Node_Timeline* TimelineNode = Cast<UK2Node_Timeline>(*NodeIt))
				{
					DocumentManager->CleanInvalidTabs();
				}
				AnalyticsTrackNodeEvent( GetBlueprintObj(), Node, true );
				
				// If the user is pressing shift then try and reconnect the pins
				if (FSlateApplication::Get().GetModifierKeys().IsShiftDown())
				{
					ReconnectExecPins(K2Node);
				}
				
				FBlueprintEditorUtils::RemoveNode(GetBlueprintObj(), Node, true);
			}
		}
	}

	if (bNeedToModifyStructurally)
	{
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprintObj());
	}
	else
	{
		FBlueprintEditorUtils::MarkBlueprintAsModified(GetBlueprintObj());
	}

	//@TODO: Reselect items that were not deleted
}

void FBlueprintEditor::ReconnectExecPins(UK2Node* Node)
{
	if(!Node)
	{
		return;
	}

	UEdGraphPin* ExecPin = nullptr;
	UEdGraphPin* ThenPin = nullptr;

	// Get pins for knot nodes or impure nodes only
	if(UK2Node_Knot* KnotNode = Cast<UK2Node_Knot>(Node))
	{
		ExecPin = KnotNode->GetInputPin();
		ThenPin = KnotNode->GetOutputPin();
	}
	else if(!Node->IsNodePure())
	{
		ExecPin = Node->GetExecPin();

		// Nodes with multiple "then" pins (branch, sequence, foreach, etc) will actually have the FindPin return nullptr
		ThenPin = Node->FindPin(UEdGraphSchema_K2::PN_Then);
	}

	// We don't want to try and auto connect nodes with multiple outputs, 	
	// because it's likely the user will not want those connections anyway
	if (ExecPin && ThenPin)
	{
		// Make a connection from every incoming exec pin to every outgoing then pin
		for (UEdGraphPin* const IncomingConnectionPin : ExecPin->LinkedTo)
		{
			if (IncomingConnectionPin)
			{
				for (UEdGraphPin* const ConnectedCompletePin : ThenPin->LinkedTo)
				{
					IncomingConnectionPin->MakeLinkTo(ConnectedCompletePin);
				}
			}
		}
	}	
}

bool FBlueprintEditor::CanDeleteNodes() const
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	bool bCanUserDeleteNode = false;

	if(IsEditable(GetFocusedGraph()) && SelectedNodes.Num() > 0)
	{
		for( UObject* NodeObject : SelectedNodes )
		{
			// If any nodes allow deleting, then do not disable the delete option
			UEdGraphNode* Node = Cast<UEdGraphNode>(NodeObject);
			if(Node->CanUserDeleteNode())
			{
				bCanUserDeleteNode = true;
				break;
			}
		}
	}

	return bCanUserDeleteNode;
}

void FBlueprintEditor::DeleteSelectedDuplicatableNodes()
{
	// Cache off the old selection
	const FGraphPanelSelectionSet OldSelectedNodes = GetSelectedNodes();
	
	// Clear the selection and only select the nodes that can be duplicated
	FGraphPanelSelectionSet CurrentSelection;
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		FocusedGraphEd->ClearSelectionSet();

		FGraphPanelSelectionSet RemainingNodes;
		for (FGraphPanelSelectionSet::TConstIterator SelectedIter(OldSelectedNodes); SelectedIter; ++SelectedIter)
		{
			UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter);
			if ((Node != nullptr) && Node->CanDuplicateNode())
			{
				FocusedGraphEd->SetNodeSelection(Node, true);
			}
			else
			{
				RemainingNodes.Add(Node);
			}
		}

		// Delete the duplicatable nodes
		DeleteSelectedNodes();

		// Reselect whatever's left from the original selection after the deletion
		FocusedGraphEd->ClearSelectionSet();

		for (FGraphPanelSelectionSet::TConstIterator SelectedIter(RemainingNodes); SelectedIter; ++SelectedIter)
		{
			if (UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter))
			{
				FocusedGraphEd->SetNodeSelection(Node, true);
			}
		}
	}
}

void FBlueprintEditor::CutSelectedNodes()
{
	CopySelectedNodes();
	// Cut should only delete nodes that can be duplicated
	DeleteSelectedDuplicatableNodes();
}

bool FBlueprintEditor::CanCutNodes() const
{
	return CanCopyNodes() && CanDeleteNodes();
}

void FBlueprintEditor::CopySelectedNodes()
{
	// Export the selected nodes and place the text on the clipboard
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	FString ExportedText;

	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(SelectedNodes); SelectedIter; ++SelectedIter)
	{
		if(UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter))
		{
			Node->PrepareForCopying();
		}
	}

	FEdGraphUtilities::ExportNodesToText(SelectedNodes, /*out*/ ExportedText);
	FPlatformApplicationMisc::ClipboardCopy(*ExportedText);
}

bool FBlueprintEditor::CanCopyNodes() const
{
	// If any of the nodes can be duplicated then we should allow copying
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(SelectedNodes); SelectedIter; ++SelectedIter)
	{
		UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter);
		if ((Node != nullptr) && Node->CanDuplicateNode())
		{
			return true;
		}
	}
	return false;
}

void FBlueprintEditor::PasteNodes()
{
	// Find the graph editor with focus
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (!FocusedGraphEd.IsValid())
	{
		return;
	}

	PasteNodesHere(FocusedGraphEd->GetCurrentGraph(), FocusedGraphEd->GetPasteLocation2f());

	// Dump any temporary pre-compile warnings to the compiler log.
	UBlueprint* BlueprintObj = GetBlueprintObj();
	if (BlueprintObj->PreCompileLog.IsValid())
	{
		DumpMessagesToCompilerLog(BlueprintObj->PreCompileLog->Messages, true);
	}
}


/**
 *	When copying and pasting functions from the LSBP operating on an instance to a class BP, 
 *	we should automatically transfer the functions from actors to the components
 */
struct FUpdatePastedNodes
{
	TSet<UK2Node_VariableGet*> AddedTargets;
	TSet<UK2Node_CallFunction*> AddedFunctions;
	TSet<UK2Node_Literal*> ReplacedTargets;
	TSet<UK2Node_CallFunctionOnMember*> ReplacedFunctions;

	UClass* CurrentClass;
	UEdGraph* Graph;
	TSet<UEdGraphNode*>& PastedNodes;
	const UEdGraphSchema_K2* K2Schema;

	FUpdatePastedNodes(UClass* InCurrentClass, TSet<UEdGraphNode*>& InPastedNodes, UEdGraph* InDestinationGraph)
		: CurrentClass(InCurrentClass)
		, Graph(InDestinationGraph)
		, PastedNodes(InPastedNodes)
		, K2Schema(GetDefault<UEdGraphSchema_K2>())
	{
		check(InCurrentClass && InDestinationGraph && K2Schema);
	}

	/**
	 *	Replace UK2Node_CallFunctionOnMember called on actor with a UK2Node_CallFunction. 
	 *	When the blueprint has the member.
	 */
	void ReplaceAll()
	{
		for(UEdGraphNode* PastedNode : PastedNodes)
		{
			if (UK2Node_CallFunctionOnMember* CallOnMember = Cast<UK2Node_CallFunctionOnMember>(PastedNode))
			{
				if (UEdGraphPin* TargetInPin = CallOnMember->FindPin(UEdGraphSchema_K2::PN_Self))
				{
					const UClass* TargetClass = Cast<const UClass>(TargetInPin->PinType.PinSubCategoryObject.Get());

					const bool bTargetIsNullOrSingleLinked = (TargetInPin->LinkedTo.Num() == 1) ||
						(!TargetInPin->LinkedTo.Num() && !TargetInPin->DefaultObject);

					const bool bCanCurrentBlueprintReplace = TargetClass
						&& CurrentClass->IsChildOf(TargetClass) // If current class if of the same type, it has the called member
						&& (!CallOnMember->MemberVariableToCallOn.IsSelfContext() && (TargetClass != CurrentClass)) // Make sure the class isn't self, using a explicit check in case the class hasn't been compiled since the member was added
						&& bTargetIsNullOrSingleLinked;

					if (bCanCurrentBlueprintReplace) 
					{
						UEdGraphNode* TargetNode = TargetInPin->LinkedTo.Num() ? TargetInPin->LinkedTo[0]->GetOwningNode() : nullptr;
						UK2Node_Literal* TargetLiteralNode = Cast<UK2Node_Literal>(TargetNode);

						const bool bPastedNodeShouldBeReplacedWithTarget = TargetLiteralNode
							&& !TargetLiteralNode->GetObjectRef() //The node delivering target actor is invalid
							&& PastedNodes.Contains(TargetLiteralNode);
						const bool bPastedNodeShouldBeReplacedWithoutTarget = !TargetNode || !PastedNodes.Contains(TargetNode);

						if (bPastedNodeShouldBeReplacedWithTarget || bPastedNodeShouldBeReplacedWithoutTarget)
						{
							Replace(TargetLiteralNode, CallOnMember);
						}
					}
				}
			}
		}

		UpdatePastedCollection();
	}

private:

	void UpdatePastedCollection()
	{
		for (UK2Node_Literal* ReplacedTarget : ReplacedTargets)
		{
			if (ReplacedTarget && ReplacedTarget->GetValuePin() && !ReplacedTarget->GetValuePin()->LinkedTo.Num())
			{
				PastedNodes.Remove(ReplacedTarget);
				Graph->RemoveNode(ReplacedTarget);
			}
		}
		for (UK2Node_CallFunctionOnMember* ReplacedFunction : ReplacedFunctions)
		{
			PastedNodes.Remove(ReplacedFunction);
			Graph->RemoveNode(ReplacedFunction);
		}
		for (UK2Node_VariableGet* AddedTarget : AddedTargets)
		{
			PastedNodes.Add(AddedTarget);
		}
		for (UK2Node_CallFunction* AddedFunction : AddedFunctions)
		{
			PastedNodes.Add(AddedFunction);
		}
	}

	bool MoveAllLinksExeptSelf(UK2Node* NewNode, UK2Node* OldNode)
	{
		bool bResult = true;
		for(UEdGraphPin* OldPin : OldNode->Pins)
		{
			if(OldPin && (OldPin->PinName != UEdGraphSchema_K2::PN_Self))
			{
				UEdGraphPin* NewPin = NewNode->FindPin(OldPin->PinName);
				if (NewPin)
				{
					if (!K2Schema->MovePinLinks(*OldPin, *NewPin).CanSafeConnect())
					{
						UE_LOGF(LogBlueprint, Error, "FUpdatePastedNodes: Cannot connect pin '%ls' node '%ls'",
							*OldPin->PinName.ToString(), *OldNode->GetName());
						bResult = false;
					}
				}
				else
				{
					UE_LOGF(LogBlueprint, Error, "FUpdatePastedNodes: Cannot find pin '%ls'", *OldPin->PinName.ToString());
					bResult = false;
				}
			}
		}
		return bResult;
	}

	void InitializeNewNode(UK2Node* NewNode, UK2Node* OldNode, int32 NodePosX = 0, int32 NodePosY = 0)
	{	
		NewNode->NodePosX = OldNode ? OldNode->NodePosX : NodePosX;
		NewNode->NodePosY = OldNode ? OldNode->NodePosY : NodePosY;
		NewNode->SetFlags(RF_Transactional);
		Graph->AddNode(NewNode, false, false);
		NewNode->PostPlacedNewNode();
		NewNode->AllocateDefaultPins();
	}

	bool Replace(UK2Node_Literal* OldTarget, UK2Node_CallFunctionOnMember* OldCall)
	{
		bool bResult = true;
		check(OldCall);

		UK2Node_VariableGet* NewTarget = nullptr;
		
		const FProperty* Property = OldCall->MemberVariableToCallOn.ResolveMember<FProperty>();
		for (UK2Node_VariableGet* AddedTarget : AddedTargets)
		{
			if (AddedTarget && (Property == AddedTarget->VariableReference.ResolveMember<FProperty>(CurrentClass)))
			{
				NewTarget = AddedTarget;
				break;
			}
		}

		if (!NewTarget)
		{
			NewTarget = NewObject<UK2Node_VariableGet>(Graph);
			check(NewTarget);
			NewTarget->SetFromProperty(Property, true, Property->GetOwnerClass());
			AddedTargets.Add(NewTarget);
			const int32 AutoNodeOffsetX = 160;
			InitializeNewNode(NewTarget, OldTarget, OldCall->NodePosX - AutoNodeOffsetX, OldCall->NodePosY);
		}

		if (OldTarget)
		{
			ReplacedTargets.Add(OldTarget);
		}

		UK2Node_CallFunction* NewCall = NewObject<UK2Node_CallFunction>(Graph);
		check(NewCall);
		NewCall->SetFromFunction(OldCall->GetTargetFunction());
		InitializeNewNode(NewCall, OldCall);
		AddedFunctions.Add(NewCall);

		if (!MoveAllLinksExeptSelf(NewCall, OldCall))
		{
			bResult = false;
		}

		if (NewTarget)
		{
			UEdGraphPin* SelfPin = NewCall->FindPinChecked(UEdGraphSchema_K2::PN_Self);
			if (!K2Schema->TryCreateConnection(SelfPin, NewTarget->GetValuePin()))
			{
				UE_LOGF(LogBlueprint, Error, "FUpdatePastedNodes: Cannot connect new self.");
				bResult = false;
			}
		}

		OldCall->BreakAllNodeLinks();

		ReplacedFunctions.Add(OldCall);
		return bResult;
	}
};

void FBlueprintEditor::PasteNodesHere(class UEdGraph* DestinationGraph, const FVector2f& GraphLocation)
{
	// Find the graph editor with focus
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (!FocusedGraphEd.IsValid())
	{
		return;
	}
	// Select the newly pasted stuff
	bool bNeedToModifyStructurally = false;
	{
		const FScopedTransaction Transaction(FGenericCommands::Get().Paste->GetDescription());
		DestinationGraph->Modify();

		// Clear the selection set (newly pasted stuff will be selected)
		SetUISelectionState(NAME_None);

		// Grab the text to paste from the clipboard.
		FString TextToImport;
		FPlatformApplicationMisc::ClipboardPaste(TextToImport);

		// Import the nodes
		TSet<UEdGraphNode*> PastedNodes;
		FEdGraphUtilities::ImportNodesFromText(DestinationGraph, TextToImport, /*out*/ PastedNodes);

		// Only do this step if we can create functions on the blueprint (i.e. not macro graphs, etc)
		if (NewDocument_IsVisibleForType(CGT_NewFunctionGraph))
		{
			// Spawn Deferred Fixup Modal window if necessary
			TArray<UK2Node_CallFunction*> FixupNodes;
			for (UEdGraphNode* PastedNode : PastedNodes)
			{
				if (UK2Node_CallFunction* Node = Cast<UK2Node_CallFunction>(PastedNode))
				{
					if (Node->FunctionReference.IsSelfContext() && !Node->GetTargetFunction())
					{
						FixupNodes.Add(Node);
					}
				}
			}
			if (FixupNodes.Num() > 0)
			{
				if (!SFixupSelfContextDialog::CreateModal(FixupNodes, Cast<UBlueprint>(DestinationGraph->GetOuter()), this, FixupNodes.Num() != PastedNodes.Num()))
				{
					for (UEdGraphNode* Node : PastedNodes)
					{
						DestinationGraph->RemoveNode(Node);
					}

					return;
				}
			}
		}

		// Update Paste Analytics
		AnalyticsStats.NodePasteCreateCount += PastedNodes.Num();

		{
			UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(DestinationGraph);
			UClass* CurrentClass = Blueprint ? *Blueprint->GeneratedClass : nullptr;
			if (CurrentClass)
			{
				FUpdatePastedNodes ReplaceNodes(CurrentClass, PastedNodes, DestinationGraph);
				ReplaceNodes.ReplaceAll();
			}
		}

		//Average position of nodes so we can move them while still maintaining relative distances to each other
		FVector2f AvgNodePosition(0.0f, 0.0f);

		for (TSet<UEdGraphNode*>::TIterator It(PastedNodes); It; ++It)
		{
			UEdGraphNode* Node = *It;
			AvgNodePosition.X += Node->NodePosX;
			AvgNodePosition.Y += Node->NodePosY;
		}

		float InvNumNodes = 1.0f / float(PastedNodes.Num());
		AvgNodePosition.X *= InvNumNodes;
		AvgNodePosition.Y *= InvNumNodes;

		TSet<FString> NamespacesToImport;

		for (TSet<UEdGraphNode*>::TIterator It(PastedNodes); It; ++It)
		{
			UEdGraphNode* Node = *It;
			FocusedGraphEd->SetNodeSelection(Node, true);

			Node->NodePosX = static_cast<int32>((Node->NodePosX - AvgNodePosition.X) + GraphLocation.X);
			Node->NodePosY = static_cast<int32>((Node->NodePosY - AvgNodePosition.Y) + GraphLocation.Y);

			Node->SnapToGrid(SNodePanel::GetSnapGridSize());

			// Give new node a different Guid from the old one
			Node->CreateNewGuid();

			// Collect any required imports from node dependencies
			TArray<UStruct*> ExternalDependencies;
			if (Node->HasExternalDependencies(&ExternalDependencies))
			{
				for (const UStruct* ExternalDependency : ExternalDependencies)
				{
					FBlueprintNamespaceUtilities::GetDefaultImportsForObject(ExternalDependency, DeferredNamespaceImports);
				}
			}

			UK2Node* K2Node = Cast<UK2Node>(Node);
			if ((K2Node != nullptr) && K2Node->NodeCausesStructuralBlueprintChange())
			{
				bNeedToModifyStructurally = true;
			}

			// For pasted Event nodes, we need to see if there is an already existing node in a ghost state that needs to be cleaned up
			if (UK2Node_Event* EventNode = Cast<UK2Node_Event>(Node))
			{
				// Gather all existing event nodes
				TArray<UK2Node_Event*> ExistingEventNodes;
				FBlueprintEditorUtils::GetAllNodesOfClass<UK2Node_Event>(GetBlueprintObj(), ExistingEventNodes);

				for (UK2Node_Event* ExistingEventNode : ExistingEventNodes)
				{
					check(ExistingEventNode);

					const bool bIdenticalNode = (EventNode != ExistingEventNode) && ExistingEventNode->bOverrideFunction && UK2Node_Event::AreEventNodesIdentical(EventNode, ExistingEventNode);

					// Check if the nodes are identical, if they are we need to delete the original because it is disabled. Identical nodes that are in an enabled state will never make it this far and still be enabled.
					if (bIdenticalNode)
					{
						// Should not have made it to being a pasted node if the pre-existing node wasn't disabled or was otherwise explicitly disabled by the user.
						ensure(ExistingEventNode->IsAutomaticallyPlacedGhostNode());

						// Destroy the pre-existing node, we do not need it.
						ExistingEventNode->DestroyNode();
					}
				}
			}
			// Log new node created to analytics
			AnalyticsTrackNodeEvent(GetBlueprintObj(), Node, false);
		}
	}

	if (bNeedToModifyStructurally)
	{
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprintObj());
	}
	else
	{
		FBlueprintEditorUtils::MarkBlueprintAsModified(GetBlueprintObj());
	}

	// Update UI
	FocusedGraphEd->NotifyGraphChanged();
}

void FBlueprintEditor::PasteGeneric()
{
	if (CanPasteNodes())
	{
		PasteNodes();
	}
	else if (MyBlueprintWidget.IsValid())
	{
		if (MyBlueprintWidget->CanPasteGeneric())
		{
			MyBlueprintWidget->OnPasteGeneric();
		}
	}
}

bool FBlueprintEditor::CanPasteGeneric() const
{
	if (CanPasteNodes())
	{
		return true;
	}
	else
	{
		return MyBlueprintWidget.IsValid() && MyBlueprintWidget->CanPasteGeneric();
	}
}


bool FBlueprintEditor::CanPasteNodes() const
{
	// Do not allow pasting into interface Blueprints
	if (GetBlueprintObj()->BlueprintType == BPTYPE_Interface)
	{
		return false;
	}

	// Find the graph editor with focus
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (!FocusedGraphEd.IsValid())
	{
		return false;
	}

	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);

	return IsEditable(GetFocusedGraph()) && FEdGraphUtilities::CanImportNodesFromText(FocusedGraphEd->GetCurrentGraph(), ClipboardContent);
}

void FBlueprintEditor::DuplicateNodes()
{
	// Copy and paste current selection
	CopySelectedNodes();
	PasteNodes();
}

bool FBlueprintEditor::CanDuplicateNodes() const
{
	return CanCopyNodes() && IsEditable(GetFocusedGraph());
}

void FBlueprintEditor::OnAssignReferencedActor()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	USelection* SelectedActors = GEditor->GetSelectedActors();
	if ( SelectedNodes.Num() > 0 && SelectedActors != nullptr && SelectedActors->Num() == 1 )
	{
		AActor* SelectedActor = Cast<AActor>( SelectedActors->GetSelectedObject(0) );
		if ( SelectedActor != nullptr )
		{
			TArray<UK2Node_ActorBoundEvent*> NodesToAlter;

			for ( FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt )
			{
				UK2Node_ActorBoundEvent* SelectedNode = Cast<UK2Node_ActorBoundEvent>(*NodeIt);
				if ( SelectedNode != nullptr )
				{
					NodesToAlter.Add( SelectedNode );
				}
			}

			// only create a transaction if there is a node that is affected.
			if ( NodesToAlter.Num() > 0 )
			{
				const FScopedTransaction Transaction( LOCTEXT("AssignReferencedActor", "Assign referenced Actor") );
				{
					for ( int32 NodeIndex = 0; NodeIndex < NodesToAlter.Num(); NodeIndex++ )
					{
						UK2Node_ActorBoundEvent* CurrentEvent = NodesToAlter[NodeIndex];

						// Store the node's current state and replace the referenced actor
						CurrentEvent->Modify();
						CurrentEvent->EventOwner = SelectedActor;
						if (!SelectedActor->IsA(CurrentEvent->DelegateOwnerClass))
						{
							CurrentEvent->DelegateOwnerClass = SelectedActor->GetClass();
						}
						CurrentEvent->ReconstructNode();
					}
					FBlueprintEditorUtils::MarkBlueprintAsModified(GetBlueprintObj());
				}
			}
		}
	}
}

bool FBlueprintEditor::CanAssignReferencedActor() const
{
	bool bWouldAssignActors = false;

	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	if ( SelectedNodes.Num() > 0 )
	{
		USelection* SelectedActors = GEditor->GetSelectedActors();

		// If there is only one actor selected and at least one Blueprint graph
		// node is able to receive the assignment then return true.
		if ( SelectedActors != nullptr && SelectedActors->Num() == 1 )
		{
			AActor* SelectedActor = Cast<AActor>( SelectedActors->GetSelectedObject(0) );
			if ( SelectedActor != nullptr )
			{
				for ( FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt )
				{
					UK2Node_ActorBoundEvent* SelectedNode = Cast<UK2Node_ActorBoundEvent>(*NodeIt);
					if ( SelectedNode != nullptr )
					{
						if ( SelectedNode->EventOwner != SelectedActor )
						{
							bWouldAssignActors = true;
							break;
						}
					}
				}
			}
		}
	}

	return bWouldAssignActors;
}

void FBlueprintEditor::OnSelectReferenceInLevel()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	if (SelectedNodes.Num() > 0)
	{
		TArray<AActor*> ActorsToSelect;

		// Iterate over all nodes, and select referenced actors.
		for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
		{
			UK2Node* SelectedNode = Cast<UK2Node>(*NodeIt);
			AActor* ReferencedActor = (SelectedNode) ? SelectedNode->GetReferencedLevelActor() : nullptr;

			if (ReferencedActor != nullptr)
			{
				ActorsToSelect.AddUnique(ReferencedActor);
			}
		}
		// If we found any actors to select clear the existing selection, select them and move the camera to show them.
		if( ActorsToSelect.Num() != 0 )
		{
			// First clear the previous selection
			GEditor->GetSelectedActors()->Modify();
			GEditor->SelectNone( false, true );

			// Now select the actors.
			for (int32 iActor = 0; iActor < ActorsToSelect.Num(); iActor++)
			{
				GEditor->SelectActor(ActorsToSelect[ iActor ], true, true, false);
			}

			// Execute the command to move camera to the object(s).
			GUnrealEd->Exec_Camera( TEXT("ALIGN ACTIVEVIEWPORTONLY"),*GLog); 
		}
	}
}

bool FBlueprintEditor::CanSelectReferenceInLevel() const
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	bool bCanSelectActors = false;
	if (SelectedNodes.Num() > 0)
	{
		// Iterate over all nodes, testing if they're pointing to actors.
		for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
		{
			UK2Node* SelectedNode = Cast<UK2Node>(*NodeIt);
			const AActor* ReferencedActor = (SelectedNode) ? SelectedNode->GetReferencedLevelActor() : nullptr;

			bCanSelectActors = (ReferencedActor != nullptr);
			if (ReferencedActor == nullptr)
			{
				// Bail early if the selected node isn't referencing an actor
				return false;
			}
		}
	}

	return bCanSelectActors;
}

// Utility helper to get the currently hovered pin in the currently visible graph, or nullptr if there isn't one
UEdGraphPin* FBlueprintEditor::GetCurrentlySelectedPin() const
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		return FocusedGraphEd->GetGraphPinForMenu();
	}

	return nullptr;
}

void FBlueprintEditor::SetDetailsCustomization(TSharedPtr<FDetailsViewObjectFilter> DetailsObjectFilter, TSharedPtr<IDetailRootObjectCustomization> DetailsRootCustomization)
{
	if (Inspector.IsValid())
	{
		if (TSharedPtr<IDetailsView> DetailsView = Inspector->GetPropertyView())
		{
			DetailsView->SetObjectFilter(DetailsObjectFilter);
			DetailsView->SetRootObjectCustomizationInstance(DetailsRootCustomization);
			DetailsView->ForceRefresh();
		}
	}
}

void FBlueprintEditor::SetSubobjectEditorUICustomization(TSharedPtr<ISCSEditorUICustomization> InSCSEditorUICustomization)
{
	SCSEditorUICustomization = InSCSEditorUICustomization;

	if(SubobjectEditor.IsValid())
	{
		SubobjectEditor->SetUICustomization(SCSEditorUICustomization);
	}
}

void FBlueprintEditor::RegisterSCSEditorCustomization(const FName& InComponentName, TSharedPtr<ISCSEditorCustomization> InCustomization)
{
	SubobjectEditorCustomizations.Add(InComponentName, InCustomization);
}

void FBlueprintEditor::UnregisterSCSEditorCustomization(const FName& InComponentName)
{
	SubobjectEditorCustomizations.Remove(InComponentName);
}

void FBlueprintEditor::CreateMergeToolTab()
{
	MergeTool = IMerge::Get().GenerateMergeWidget(*GetBlueprintObj(), SharedThis(this));
}

void FBlueprintEditor::CreateMergeToolTab(const UBlueprint* BaseBlueprint, const UBlueprint* RemoteBlueprint, const FOnMergeResolved& ResolutionCallback)
{
	OnMergeResolved = ResolutionCallback;
	MergeTool = IMerge::Get().GenerateMergeWidget(BaseBlueprint, RemoteBlueprint, GetBlueprintObj(), ResolutionCallback, SharedThis(this));
}

void FBlueprintEditor::CloseMergeTool()
{
	TSharedPtr<SDockTab> MergeToolPtr = MergeTool.Pin();
	if( MergeToolPtr.IsValid() )
	{
		UBlueprint* Blueprint = GetBlueprintObj();
		UPackage* BpPackage = (Blueprint == nullptr) ? nullptr : Blueprint->GetOutermost();
		// @TODO: right now crashes the editor on closing of the BP editor
		//OnMergeResolved.ExecuteIfBound(BpPackage, EMergeResult::Unknown);
		OnMergeResolved.Unbind();

		MergeToolPtr->RequestCloseTab();
	}
}

TArray<FSubobjectEditorTreeNodePtrType> FBlueprintEditor::GetSelectedSubobjectEditorTreeNodes() const
{
	TArray<FSubobjectEditorTreeNodePtrType>  Nodes;
	if (SubobjectEditor.IsValid())
	{
		Nodes = SubobjectEditor->GetSelectedNodes();
	}
	return Nodes;
}

FSubobjectEditorTreeNodePtrType FBlueprintEditor::FindAndSelectSubobjectEditorTreeNode(const UActorComponent* InComponent, bool IsCntrlDown)
{
	FSubobjectEditorTreeNodePtrType NodePtr;

	if (SubobjectEditor.IsValid())
	{
		NodePtr = SubobjectEditor->FindSlateNodeForObject(InComponent);
		if(NodePtr.IsValid())
		{
			SubobjectEditor->SelectNode(NodePtr, IsCntrlDown);
		}
	}

	return NodePtr;
}

void FBlueprintEditor::OnDisallowedPinConnection(const UEdGraphPin* PinA, const UEdGraphPin* PinB)
{
	FDisallowedPinConnection NewRecord;
	NewRecord.PinTypeCategoryA = PinA->PinType.PinCategory;
	NewRecord.bPinIsArrayA = PinA->PinType.IsArray();
	NewRecord.bPinIsReferenceA = PinA->PinType.bIsReference;
	NewRecord.bPinIsWeakPointerA = PinA->PinType.bIsWeakPointer;
	NewRecord.PinTypeCategoryB = PinB->PinType.PinCategory;
	NewRecord.bPinIsArrayB = PinB->PinType.IsArray();
	NewRecord.bPinIsReferenceB = PinB->PinType.bIsReference;
	NewRecord.bPinIsWeakPointerB = PinB->PinType.bIsWeakPointer;
	AnalyticsStats.GraphDisallowedPinConnections.Add(NewRecord);
}

void FBlueprintEditor::OnStartWatchingPin()
{
	if (UEdGraphPin* Pin = GetCurrentlySelectedPin())
	{
		// Follow an input back to it's output
		if ((Pin->Direction == EGPD_Input) && (Pin->LinkedTo.Num() > 0))
		{
			Pin = Pin->LinkedTo[0];
		}

		// Start watching it
		FKismetDebugUtilities::TogglePinWatch(GetBlueprintObj(), Pin);
	}
}

bool FBlueprintEditor::CanStartWatchingPin() const
{
	if (UEdGraphPin* Pin = GetCurrentlySelectedPin())
	{
		// Follow an input back to it's output
		if ((Pin->Direction == EGPD_Input) && (Pin->LinkedTo.Num() > 0))
		{
			Pin = Pin->LinkedTo[0];
		}

		return FKismetDebugUtilities::CanWatchPin(GetBlueprintObj(), Pin);
	}
	return false;
}

void FBlueprintEditor::OnStopWatchingPin()
{
	if (UEdGraphPin* Pin = GetCurrentlySelectedPin())
	{
		// Follow an input back to it's output
		if ((Pin->Direction == EGPD_Input) && (Pin->LinkedTo.Num() > 0))
		{
			Pin = Pin->LinkedTo[0];
		}

		FKismetDebugUtilities::TogglePinWatch(GetBlueprintObj(), Pin);
	}
}

bool FBlueprintEditor::CanStopWatchingPin() const
{
	if (UEdGraphPin* Pin = GetCurrentlySelectedPin())
	{
		// Follow an input back to it's output
		if ((Pin->Direction == EGPD_Input) && (Pin->LinkedTo.Num() > 0))
		{
			Pin = Pin->LinkedTo[0];
		}

		return FKismetDebugUtilities::IsPinBeingWatched(GetBlueprintObj(), Pin);
	}

	return false;
}

bool FBlueprintEditor::CanGoToDefinition() const
{
	const UEdGraphNode* Node = GetSingleSelectedNode();
	return (Node != nullptr) && Node->CanJumpToDefinition();
}

void FBlueprintEditor::OnGoToDefinition()
{
	if (UEdGraphNode* SelectedGraphNode = GetSingleSelectedNode())
	{
		OnNodeDoubleClicked(SelectedGraphNode);
	}
}

FString FBlueprintEditor::GetDocLinkForSelectedNode()
{
	FString DocumentationLink;

	if (const UEdGraphNode* SelectedGraphNode = GetSingleSelectedNode())
	{
		const FString DocLink = SelectedGraphNode->GetDocumentationLink();
		const FString DocExcerpt = SelectedGraphNode->GetDocumentationExcerptName();

		if (!DocLink.IsEmpty() && !DocExcerpt.IsEmpty())
		{
			DocumentationLink = FEditorClassUtils::GetDocumentationLinkFromExcerpt(DocLink, DocExcerpt);
		}
	}

	return DocumentationLink;
}

FString FBlueprintEditor::GetDocLinkBaseUrlForSelectedNode()
{
	FString DocumentationLinkBaseUrl;

	if (const UEdGraphNode* SelectedGraphNode = GetSingleSelectedNode())
	{
		const FString DocLink = SelectedGraphNode->GetDocumentationLink();
		const FString DocExcerpt = SelectedGraphNode->GetDocumentationExcerptName();

		if (!DocLink.IsEmpty() && !DocExcerpt.IsEmpty())
		{
			DocumentationLinkBaseUrl = FEditorClassUtils::GetDocumentationLinkBaseUrlFromExcerpt(DocLink, DocExcerpt);
		}
	}

	return DocumentationLinkBaseUrl;
}

void FBlueprintEditor::OnGoToDocumentation()
{
	const FString DocumentationLink = GetDocLinkForSelectedNode();
	const FString DocumentationLinkBaseUrl = GetDocLinkBaseUrlForSelectedNode();
	if (!DocumentationLink.IsEmpty())
	{
		IDocumentation::Get()->Open(DocumentationLink, FDocumentationSourceInfo(TEXT("rightclick_bpnode")), DocumentationLinkBaseUrl);
	}
}

bool FBlueprintEditor::CanGoToDocumentation()
{
	FString DocumentationLink = GetDocLinkForSelectedNode();
	return !DocumentationLink.IsEmpty();
}

void FBlueprintEditor::OnSetEnabledStateForSelectedNodes(ENodeEnabledState NewState)
{
	const FScopedTransaction Transaction(LOCTEXT("SetNodeEnabledState", "Set Node Enabled State"));

	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (UObject* SelectedNode : SelectedNodes)
	{
		if (UEdGraphNode* SelectedGraphNode = Cast<UEdGraphNode>(SelectedNode))
		{
			SelectedGraphNode->Modify();
			SelectedGraphNode->SetEnabledState(NewState);
		}
	}

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprintObj());
}

ECheckBoxState FBlueprintEditor::GetEnabledCheckBoxStateForSelectedNodes()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	ECheckBoxState Result = SelectedNodes.Num() > 0 ? ECheckBoxState::Undetermined : ECheckBoxState::Unchecked;
	for (UObject* SelectedNode : SelectedNodes)
	{
		UEdGraphNode* SelectedGraphNode = Cast<UEdGraphNode>(SelectedNode);
		if(SelectedGraphNode)
		{
			const bool bIsSelectedNodeEnabled = SelectedGraphNode->IsNodeEnabled();
			if(Result == ECheckBoxState::Undetermined)
			{
				Result = bIsSelectedNodeEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
			else if((!bIsSelectedNodeEnabled && Result == ECheckBoxState::Checked)
				|| (bIsSelectedNodeEnabled && Result == ECheckBoxState::Unchecked))
			{
				Result = ECheckBoxState::Undetermined;
				break;
			}
		}
	}

	return Result;
}

ECheckBoxState FBlueprintEditor::CheckEnabledStateForSelectedNodes(ENodeEnabledState CheckState)
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	ECheckBoxState Result = (SelectedNodes.Num() > 0) ? ECheckBoxState::Undetermined : ECheckBoxState::Unchecked;
	for (UObject* SelectedNode : SelectedNodes)
	{
		if (UEdGraphNode* SelectedGraphNode = Cast<UEdGraphNode>(SelectedNode))
		{
			const ENodeEnabledState NodeState = SelectedGraphNode->GetDesiredEnabledState();
			if (Result == ECheckBoxState::Undetermined)
			{
				Result = (NodeState == CheckState) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
			else if ((NodeState != CheckState && Result == ECheckBoxState::Checked) || (NodeState == CheckState && Result == ECheckBoxState::Unchecked))
			{
				Result = ECheckBoxState::Undetermined;
				break;
			}
		}
	}

	return Result;
}

void FBlueprintEditor::UpdateNodesUnrelatedStatesAfterGraphChange()
{
	if (bHideUnrelatedNodes && !bLockNodeFadeState && bSelectRegularNode)
	{
		ResetAllNodesUnrelatedStates();

		HideUnrelatedNodes();
	}
}

void FBlueprintEditor::ResetAllNodesUnrelatedStates()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();

	if (FocusedGraphEd.IsValid())
	{
		FocusedGraphEd->ResetAllNodesUnrelatedStates();
	}
}

void FBlueprintEditor::CollectExecDownstreamNodes(UEdGraphNode* CurrentNode, TArray<UEdGraphNode*>& CollectedNodes)
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	TArray<UEdGraphPin*> AllPins = CurrentNode->GetAllPins();

	for (auto& Pin : AllPins)
	{
		if (Pin->Direction == EGPD_Output && Pin->PinType.PinCategory == K2Schema->PC_Exec)
		{
			for (auto& Link : Pin->LinkedTo)
			{
				UEdGraphNode* LinkedNode = Cast<UEdGraphNode>(Link->GetOwningNode());
				if (LinkedNode && !CollectedNodes.Contains(LinkedNode))
				{
					CollectedNodes.Add(LinkedNode);
					CollectExecDownstreamNodes( LinkedNode, CollectedNodes );
				}
			}
		}
	}
}

void FBlueprintEditor::CollectExecUpstreamNodes(UEdGraphNode* CurrentNode, TArray<UEdGraphNode*>& CollectedNodes)
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	TArray<UEdGraphPin*> AllPins = CurrentNode->GetAllPins();

	for (auto& Pin : AllPins)
	{
		if (Pin->Direction == EGPD_Input && Pin->PinType.PinCategory == K2Schema->PC_Exec)
		{
			for (auto& Link : Pin->LinkedTo)
			{
				UEdGraphNode* LinkedNode = Cast<UEdGraphNode>(Link->GetOwningNode());
				if (LinkedNode && !CollectedNodes.Contains(LinkedNode))
				{
					CollectedNodes.Add(LinkedNode);
					CollectExecUpstreamNodes( LinkedNode, CollectedNodes );
				}
			}
		}
	}
}

void FBlueprintEditor::CollectPureDownstreamNodes(UEdGraphNode* CurrentNode, TArray<UEdGraphNode*>& CollectedNodes)
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	TArray<UEdGraphPin*> AllPins = CurrentNode->GetAllPins();

	for (auto& Pin : AllPins)
	{
		if (Pin->Direction == EGPD_Output && Pin->PinType.PinCategory != K2Schema->PC_Exec)
		{
			for (auto& Link : Pin->LinkedTo)
			{
				UK2Node* LinkedNode = Cast<UK2Node>(Link->GetOwningNode());
				if (LinkedNode && !CollectedNodes.Contains(LinkedNode))
				{
					CollectedNodes.Add(LinkedNode);
					if (LinkedNode->IsNodePure())
					{
						CollectPureDownstreamNodes( LinkedNode, CollectedNodes );
					}
				}
			}
		}
	}
}

void FBlueprintEditor::CollectPureUpstreamNodes(UEdGraphNode* CurrentNode, TArray<UEdGraphNode*>& CollectedNodes)
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	TArray<UEdGraphPin*> AllPins = CurrentNode->GetAllPins();

	for (auto& Pin : AllPins)
	{
		if (Pin->Direction == EGPD_Input && Pin->PinType.PinCategory != K2Schema->PC_Exec)
		{
			for (auto& Link : Pin->LinkedTo)
			{
				UK2Node* LinkedNode = Cast<UK2Node>(Link->GetOwningNode());
				if (LinkedNode && !CollectedNodes.Contains(LinkedNode))
				{
					CollectedNodes.Add(LinkedNode);
					if (LinkedNode->IsNodePure())
					{
						CollectPureUpstreamNodes( LinkedNode, CollectedNodes );
					}
				}
			}
		}
	}
}

void FBlueprintEditor::HideUnrelatedNodes()
{
	TArray<UEdGraphNode*> NodesToShow;

	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

		TArray<UObject*> ImpureNodes = SelectedNodes.Array().FilterByPredicate([](UObject* Node){
			UK2Node* K2Node = Cast<UK2Node>(Node);
			if (K2Node)
			{
				return !(K2Node->IsNodePure());
			}
			return false;
		});

		TArray<UObject*> PureNodes = SelectedNodes.Array().FilterByPredicate([](UObject* Node){
			UK2Node* K2Node = Cast<UK2Node>(Node);
			if (K2Node)
			{
				return K2Node->IsNodePure();
			}
			// Treat a node which can't cast to an UK2Node as a pure node (like a document node or a commment node)
			// Make sure all selected nodes are handled
			return true;
		});

		for (auto Node : ImpureNodes)
		{
			UEdGraphNode* SelectedNode = Cast<UEdGraphNode>(Node);

			if (SelectedNode)
			{
				NodesToShow.Add(SelectedNode);
				CollectExecDownstreamNodes( SelectedNode, NodesToShow );
				CollectExecUpstreamNodes( SelectedNode, NodesToShow );
				CollectPureDownstreamNodes( SelectedNode, NodesToShow );
				CollectPureUpstreamNodes( SelectedNode, NodesToShow );
			}
		}

		for (auto Node : PureNodes)
		{
			UEdGraphNode* SelectedNode = Cast<UEdGraphNode>(Node);

			if (SelectedNode)
			{
				NodesToShow.Add(SelectedNode);
				CollectPureDownstreamNodes( SelectedNode, NodesToShow );
				CollectPureUpstreamNodes( SelectedNode, NodesToShow );
			}
		}

		TArray<class UEdGraphNode*> AllNodes = FocusedGraphEd->GetCurrentGraph()->Nodes;

		TArray<UEdGraphNode*> CommentNodes;
		TArray<UEdGraphNode*> RelatedNodes;

		for (auto& Node : AllNodes)
		{
			if (NodesToShow.Contains(Cast<UEdGraphNode>(Node)))
			{
				Node->SetNodeUnrelated(false);
				RelatedNodes.Add(Node);
			}
			else
			{
				if (UEdGraphNode_Comment* CommentNode = Cast<UEdGraphNode_Comment>(Node))
				{
					CommentNodes.Add(Node);
				}
				else
				{
					Node->SetNodeUnrelated(true);
				}
			}
		}

		if (FocusedGraphEd.IsValid())
		{
			FocusedGraphEd->FocusCommentNodes(CommentNodes, RelatedNodes);
		}
	}
}

void FBlueprintEditor::ToggleHideUnrelatedNodes()
{
	bHideUnrelatedNodes = !bHideUnrelatedNodes;

	ResetAllNodesUnrelatedStates();

	if (bHideUnrelatedNodes && bSelectRegularNode)
	{
		HideUnrelatedNodes();
	}
	else
	{
		bLockNodeFadeState = false;
	}
}

bool FBlueprintEditor::IsToggleHideUnrelatedNodesChecked() const
{
	return bHideUnrelatedNodes == true;
}

TSharedRef<SWidget> FBlueprintEditor::MakeHideUnrelatedNodesOptionsMenu()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder( bShouldCloseWindowAfterMenuSelection, GetToolkitCommands() );

	TSharedRef<SWidget> OptionsHeading = SNew(SBox)
		.Padding(2.0f)
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("HideUnrelatedNodesOptions", "Hide Unrelated Nodes Options"))
					.TextStyle(FAppStyle::Get(), "Menu.Heading")
			]
		];

	TSharedRef<SWidget> LockNodeStateCheckBox = SNew(SBox)
		[
			SNew(SCheckBox)
				.IsChecked(bLockNodeFadeState ? ECheckBoxState::Checked : ECheckBoxState::Unchecked)
				.OnCheckStateChanged(this, &FBlueprintEditor::OnLockNodeStateCheckStateChanged)
				.Style(FAppStyle::Get(), "Menu.CheckBox")
				.ToolTipText(LOCTEXT("LockNodeStateCheckBoxToolTip", "Lock the current state of all nodes."))
				.Content()
				[
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.Padding(2.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(STextBlock)
							.Text(LOCTEXT("LockNodeState", "Lock Node State"))
					]
				]
		];

	MenuBuilder.AddWidget(OptionsHeading, FText::GetEmpty(), true);

	MenuBuilder.AddMenuEntry(FUIAction(), LockNodeStateCheckBox);

	return MenuBuilder.MakeWidget();
}

void FBlueprintEditor::LoadEditorSettings()
{
	UBlueprintEditorSettings* LocalSettings = GetMutableDefault<UBlueprintEditorSettings>();

	if (LocalSettings->bHideUnrelatedNodes)
	{
		ToggleHideUnrelatedNodes();
	}
}

void FBlueprintEditor::SaveEditorSettings()
{
	UBlueprintEditorSettings* LocalSettings = GetMutableDefault<UBlueprintEditorSettings>();
	
	LocalSettings->bHideUnrelatedNodes     = bHideUnrelatedNodes;
	LocalSettings->SaveConfig();
}

void FBlueprintEditor::OnLockNodeStateCheckStateChanged(ECheckBoxState NewCheckedState)
{
	bLockNodeFadeState = (NewCheckedState == ECheckBoxState::Checked) ? true : false;
}

void FBlueprintEditor::ToggleSaveIntermediateBuildProducts()
{
	bSaveIntermediateBuildProducts = !bSaveIntermediateBuildProducts;
}

bool FBlueprintEditor::GetSaveIntermediateBuildProducts() const
{
	return bSaveIntermediateBuildProducts;
}

void FBlueprintEditor::OnNodeDoubleClicked(UEdGraphNode* Node)
{
	if (Node->CanJumpToDefinition())
	{
		Node->JumpToDefinition();
	}
}

void FBlueprintEditor::ExtractEventTemplateForFunction(class UK2Node_CustomEvent* InCustomEvent, UEdGraphNode* InGatewayNode, class UK2Node_EditablePinBase* InEntryNode, class UK2Node_EditablePinBase* InResultNode, TSet<UEdGraphNode*>& InCollapsableNodes)
{
	check(InCustomEvent);

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	for(UEdGraphPin* Pin : InCustomEvent->Pins)
	{
		if(Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec)
		{
			TArray< UEdGraphPin* > PinLinkList = Pin->LinkedTo;
			for( UEdGraphPin* PinLink : PinLinkList)
			{
				if(!InCollapsableNodes.Contains(PinLink->GetOwningNode()))
				{
					InGatewayNode->Modify();
					Pin->Modify();
					PinLink->Modify();

					K2Schema->MovePinLinks(*Pin, *K2Schema->FindExecutionPin(*InGatewayNode, EGPD_Output));
				}
			}
		}
		else if(Pin->PinType.PinCategory != UEdGraphSchema_K2::PC_Delegate)
		{

			TArray< UEdGraphPin* > PinLinkList = Pin->LinkedTo;
			for( UEdGraphPin* PinLink : PinLinkList)
			{
				if(!InCollapsableNodes.Contains(PinLink->GetOwningNode()))
				{
					InGatewayNode->Modify();
					Pin->Modify();
					PinLink->Modify();

					const FName PortName = *FString::Printf(TEXT("%s_Out"), *Pin->PinName.ToString());
					UEdGraphPin* RemotePortPin = InGatewayNode->FindPin(PortName);
					// For nodes that are connected to the event but not collapsing into the graph, they need to create a pin on the result.
					if(RemotePortPin == nullptr)
					{
						FName UniquePortName = InGatewayNode->CreateUniquePinName(PortName);

						RemotePortPin = InGatewayNode->CreatePin(Pin->Direction, Pin->PinType, UniquePortName);
						InResultNode->CreateUserDefinedPin(UniquePortName, Pin->PinType, EGPD_Input);
					}
					PinLink->BreakAllPinLinks();
					PinLink->MakeLinkTo(RemotePortPin);
				}
				else
				{
					InEntryNode->Modify();

					const FName UniquePortName = InGatewayNode->CreateUniquePinName(Pin->PinName);
					InEntryNode->CreateUserDefinedPin(UniquePortName, Pin->PinType, EGPD_Output);
				}
			}
		}
	}
}

static void SortPinsByConnectionPosition(TArray<UEdGraphPin*>& Pins)
{
	// Maps a pin to it's index in the owning node
	TMap<UEdGraphPin*, int32> GetPinIndexCache;
	auto GetPinIndex = [&GetPinIndexCache](const UEdGraphPin* Pin)
	{
		if (!GetPinIndexCache.Contains(Pin))
		{
			const UEdGraphNode* Parent = Pin->GetOwningNode();
			for (int I = 0; I < Parent->Pins.Num(); ++I)
			{
				// since we're already iterating all the siblings,
				// go ahead and store them as well for faster lookup
				GetPinIndexCache.Add(Parent->Pins[I], I);
			}
		}
		return GetPinIndexCache[Pin];
	};

	// determines whether the Y value of PinA is less than that of PinB
	auto PinYComp = [&GetPinIndex](const UEdGraphPin* PinA, const UEdGraphPin* PinB){
		const float NodeAPosY = PinA->GetOwningNode()->NodePosY;
		const float NodeBPosY = PinB->GetOwningNode()->NodePosY;
		const float NodeAPosX = PinA->GetOwningNode()->NodePosX;
		const float NodeBPosX = PinB->GetOwningNode()->NodePosX;

		// Exec pins should always appear first
		if (UEdGraphSchema_K2::IsExecPin(*PinA))
		{
			return true;
		}
		if (UEdGraphSchema_K2::IsExecPin(*PinB))
		{
			return false;
		}

		// sort by Y, then X, then Pin index
		if (NodeAPosY != NodeBPosY)
		{
			return NodeAPosY < NodeBPosY;
		}
		if (NodeAPosX != NodeBPosX)
		{
			return NodeAPosX < NodeBPosX;
		}
		return GetPinIndex(PinA) < GetPinIndex(PinB);
	};

	Pins.Sort([&PinYComp](const UEdGraphPin& PinA, const UEdGraphPin& PinB)
	{
		// since we care more about making the calling convention pretty
		// and less about making the internal implementation pretty, we'll
		// sort by linked pins instead of the gateway pins directly.
		return PinYComp(*Algo::MinElement(PinA.LinkedTo, PinYComp), *Algo::MinElement(PinB.LinkedTo, PinYComp));
	});
}

void FBlueprintEditor::CollapseNodesIntoGraph(UEdGraphNode* InGatewayNode, UK2Node_EditablePinBase* InEntryNode, UK2Node_EditablePinBase* InResultNode, UEdGraph* InSourceGraph, UEdGraph* InDestinationGraph, TSet<UEdGraphNode*>& InCollapsableNodes, bool bCanDiscardEmptyReturnNode, bool bCanHaveWeakObjPtrParam)
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// Keep track of the statistics of the node positions so the new nodes can be located reasonably well
	int32 SumNodeX = 0;
	int32 SumNodeY = 0;
	int32 MinNodeX = std::numeric_limits<int32>::max();
	int32 MinNodeY = std::numeric_limits<int32>::max();
	int32 MaxNodeX = std::numeric_limits<int32>::min();
	int32 MaxNodeY = std::numeric_limits<int32>::min();

	UEdGraphNode* InterfaceTemplateNode = nullptr;

	// If our return node only contains an exec pin, then we don't need to add it
	// This helps to mitigate cases where it is unclear which exec pins should be connected to the return node
	bool bDiscardReturnNode = true;

	// For collapsing to functions can use a single event as a template for the function. This event MUST be deleted at the end, and the pins pre-generated. 
	if (InGatewayNode->GetClass() == UK2Node_CallFunction::StaticClass())
	{
		for (UEdGraphNode* Node : InCollapsableNodes)
		{
			if (UK2Node_CustomEvent* const CustomEvent = Cast<UK2Node_CustomEvent>(Node))
			{
				check(!InterfaceTemplateNode);

				InterfaceTemplateNode = CustomEvent;
				InterfaceTemplateNode->Modify();

				ExtractEventTemplateForFunction(CustomEvent, InGatewayNode, InEntryNode, InResultNode, InCollapsableNodes);

				FString GraphName = FBlueprintEditorUtils::GenerateUniqueGraphName(GetBlueprintObj(), CustomEvent->GetNodeTitle(ENodeTitleType::ListView).ToString()).ToString();
				FBlueprintEditorUtils::RenameGraph(InDestinationGraph, GraphName);

				// Remove the node, it has no place in the new graph
				InCollapsableNodes.Remove(Node);

				// Also break all links so that we don't try to reconnect to it from the new graph
				Node->BreakAllNodeLinks();
				break;
			}
		}
	}
	
	TArray<UEdGraphPin*> GatewayPins;

	// Move the nodes over, which may create cross-graph references that we need fix up ASAP
	for (TSet<UEdGraphNode*>::TConstIterator NodeIt(InCollapsableNodes); NodeIt; ++NodeIt)
	{
		UEdGraphNode* Node = *NodeIt;
		Node->Modify();

		// Update stats
		SumNodeX += Node->NodePosX;
		SumNodeY += Node->NodePosY;
		MinNodeX = FMath::Min<int32>(MinNodeX, Node->NodePosX);
		MinNodeY = FMath::Min<int32>(MinNodeY, Node->NodePosY);
		MaxNodeX = FMath::Max<int32>(MaxNodeX, Node->NodePosX);
		MaxNodeY = FMath::Max<int32>(MaxNodeY, Node->NodePosY);

		// Move the node over
		InSourceGraph->Nodes.Remove(Node);
		InDestinationGraph->Nodes.Add(Node);
		Node->Rename(/*NewName=*/ nullptr, /*NewOuter=*/ InDestinationGraph);

		// Move the sub-graph to the new graph
		if (UK2Node_Composite* Composite = Cast<UK2Node_Composite>(Node))
		{
			InSourceGraph->SubGraphs.Remove(Composite->BoundGraph);
			InDestinationGraph->SubGraphs.Add(Composite->BoundGraph);
		}

		TArray<UEdGraphPin*> OutputGatewayExecPins;

		// Find cross-graph links
		for (int32 PinIndex = 0; PinIndex < Node->Pins.Num(); ++PinIndex)
		{
			UEdGraphPin* LocalPin = Node->Pins[PinIndex];

			bool bIsGatewayPin = false;
			if(LocalPin->LinkedTo.Num())
			{
				for (int32 LinkIndex = 0; LinkIndex < LocalPin->LinkedTo.Num(); ++LinkIndex)
				{
					UEdGraphPin* TrialPin = LocalPin->LinkedTo[LinkIndex];
					if (!InCollapsableNodes.Contains(TrialPin->GetOwningNode()))
					{
						bIsGatewayPin = true;
						break;
					}
				}
			}
			// If the pin has no links but is an exec pin and this is a function graph, then it is a gateway pin
			else if (InGatewayNode->GetClass() == UK2Node_CallFunction::StaticClass() && K2Schema->IsExecPin(*LocalPin))
			{
				if (LocalPin->Direction == EGPD_Input)
				{
					// Connect the gateway pin to the node, there is no remote pin to hook up because the exec pin was not originally connected
					LocalPin->Modify();
					UK2Node_EditablePinBase* LocalPort = InEntryNode;
					UEdGraphPin* LocalPortPin = LocalPort->Pins[0];
					LocalPin->MakeLinkTo(LocalPortPin);
				}
				else
				{
					OutputGatewayExecPins.Add(LocalPin);
				}
			}

			if (bIsGatewayPin)
			{
				GatewayPins.Add(LocalPin);
			}
		}

		if (OutputGatewayExecPins.Num() > 0)
		{
			UEdGraphPin* LocalResultPortPin = K2Schema->FindExecutionPin(*InResultNode, EGPD_Input);

			// If the Result Node already contains links, then we don't need to make these connections as the intended connections have already been
			// transferred from original graph.
			if (LocalResultPortPin != nullptr && LocalResultPortPin->LinkedTo.Num() == 0)
			{
				// TODO: Some of these pins may not necessarily be terminal pins. We should prompt the user to choose which of these connections should
				// be made to the return node.
				for (UEdGraphPin* LocalPin : OutputGatewayExecPins)
				{
					// Connect the gateway pin to the node, there is no remote pin to hook up because the exec pin was not originally connected
					LocalPin->Modify();
					LocalPin->MakeLinkTo(LocalResultPortPin);
				}
			}
		}
	}

	// put all exec pins first, then sort by Y, then X, then Pin index
	SortPinsByConnectionPosition(GatewayPins);

	// Thunk cross-graph links thru the gateway
	for (UEdGraphPin* LocalPin : GatewayPins)
	{
		// Local port is either the entry or the result node in the collapsed graph
		// Remote port is the node placed in the source graph
		UK2Node_EditablePinBase* LocalPort = (LocalPin->Direction == EGPD_Input) ? InEntryNode : InResultNode;

		// Add a new pin to the entry/exit node and to the composite node
		UEdGraphPin* LocalPortPin = nullptr;
		UEdGraphPin* RemotePortPin = nullptr;

		// Function graphs have a single exec path through them, so only one exec pin for input and another for output. In this fashion, they must not be handled by name.
		if (InGatewayNode->GetClass() == UK2Node_CallFunction::StaticClass() && LocalPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec)
		{
			LocalPortPin = LocalPort->Pins[0];
			RemotePortPin = K2Schema->FindExecutionPin(*InGatewayNode, (LocalPortPin->Direction == EGPD_Input)? EGPD_Output : EGPD_Input);
		}
		else
		{
			// If there is a custom event being used as a template, we must check to see if any connected pins have already been built
			if (InterfaceTemplateNode && LocalPin->Direction == EGPD_Input)
			{
				// Find the pin on the entry node, we will use that pin's name to find the pin on the remote port
				UEdGraphPin* EntryNodePin = InEntryNode->FindPin(LocalPin->LinkedTo[0]->PinName);
				if(EntryNodePin)
				{
					LocalPin->BreakAllPinLinks();
					LocalPin->MakeLinkTo(EntryNodePin);
					continue;
				}
			}

			if (LocalPin->LinkedTo[0]->GetOwningNode() != InEntryNode)
			{
				const FName UniquePortName = InGatewayNode->CreateUniquePinName(LocalPin->PinName);

				if (!RemotePortPin && !LocalPortPin)
				{
					if (LocalPin->Direction == EGPD_Output)
					{
						bDiscardReturnNode = false;
					}
					
					FEdGraphPinType PinType = LocalPin->PinType;
					if (PinType.bIsWeakPointer && !PinType.IsContainer() && !bCanHaveWeakObjPtrParam)
					{
						PinType.bIsWeakPointer = false;
					}
					RemotePortPin = InGatewayNode->CreatePin(LocalPin->Direction, PinType, UniquePortName);
					LocalPortPin = LocalPort->CreateUserDefinedPin(UniquePortName, PinType, (LocalPin->Direction == EGPD_Input) ? EGPD_Output : EGPD_Input);
				}
			}
		}

		check(LocalPortPin);
		check(RemotePortPin);

		LocalPin->Modify();

		// Route the links
		for (int32 LinkIndex = 0; LinkIndex < LocalPin->LinkedTo.Num(); ++LinkIndex)
		{
			UEdGraphPin* RemotePin = LocalPin->LinkedTo[LinkIndex];
			RemotePin->Modify();

			if (!InCollapsableNodes.Contains(RemotePin->GetOwningNode()) && RemotePin->GetOwningNode() != InEntryNode && RemotePin->GetOwningNode() != InResultNode)
			{
				// Fix up the remote pin
				RemotePin->LinkedTo.Remove(LocalPin);
				// When given a composite node, we could possibly be given a pin with a different outer
				// which is bad! Then there would be a pin connecting to itself and cause an ensure
				if (RemotePin->GetOwningNode()->GetOuter() == RemotePortPin->GetOwningNode()->GetOuter())
				{
					RemotePin->MakeLinkTo(RemotePortPin);
				}

				// The Entry Node only supports a single link, so if we made links above
				// we need to break them now, to make room for the new link.
				if (LocalPort == InEntryNode)
				{
					LocalPortPin->BreakAllPinLinks();
				}

				// Fix up the local pin
				LocalPin->LinkedTo.Remove(RemotePin);
				--LinkIndex;
				LocalPin->MakeLinkTo(LocalPortPin);
			}
		}
	}

	// Reposition the newly created nodes
	const int32 NumNodes = InCollapsableNodes.Num();

	// Remove the template node if one was used for generating the function
	if (InterfaceTemplateNode)
	{
		if (NumNodes == 0)
		{
			SumNodeX = InterfaceTemplateNode->NodePosX;
			SumNodeY = InterfaceTemplateNode->NodePosY;
		}

		FBlueprintEditorUtils::RemoveNode(GetBlueprintObj(), InterfaceTemplateNode);
	}

	// Using the result pin, we will ensure that there is a path through the function by checking if it is connected. If it is not, link it to the entry node.
	if (UEdGraphPin* ResultExecFunc = K2Schema->FindExecutionPin(*InResultNode, EGPD_Input))
	{
		if (ResultExecFunc->LinkedTo.Num() == 0)
		{
			K2Schema->FindExecutionPin(*InEntryNode, EGPD_Output)->MakeLinkTo(K2Schema->FindExecutionPin(*InResultNode, EGPD_Input));
		}
	}

	if (UK2Node_FunctionEntry* EntryNode = Cast<UK2Node_FunctionEntry>(InEntryNode))
	{
		// If the source entry node is threadsafe, then our new entry node should also be threadsafe.
		// It's implied that the source graph only has threadsafe nodes.
		// This ensures that our newly collapsed graph can be still be used in the source graph.

		EntryNode->MetaData.bThreadSafe = K2Schema->IsGraphMarkedThreadSafe(InSourceGraph);
	}

	const int32 CenterX = (NumNodes == 0) ? SumNodeX : SumNodeX / NumNodes;
	const int32 CenterY = (NumNodes == 0) ? SumNodeY : SumNodeY / NumNodes;
	const int32 MinusOffsetX = 160; //@TODO: Random magic numbers
	const int32 PlusOffsetX = 300;

	// Put the gateway node at the center of the empty space in the old graph
	InGatewayNode->NodePosX = CenterX;
	InGatewayNode->NodePosY = CenterY;
	InGatewayNode->SnapToGrid(SNodePanel::GetSnapGridSize());

	// Put the entry and exit nodes on either side of the nodes in the new graph
	//@TODO: Should we recenter the whole ensemble?
	if (NumNodes != 0)
	{
		InEntryNode->NodePosX = MinNodeX - MinusOffsetX;
		InEntryNode->NodePosY = CenterY;
		InEntryNode->SnapToGrid(SNodePanel::GetSnapGridSize());

		InResultNode->NodePosX = MaxNodeX + PlusOffsetX;
		InResultNode->NodePosY = CenterY;
		InResultNode->SnapToGrid(SNodePanel::GetSnapGridSize());
	}

	if (bCanDiscardEmptyReturnNode && bDiscardReturnNode)
	{
		InResultNode->DestroyNode();
	}
}

void FBlueprintEditor::CollapseNodes(TSet<UEdGraphNode*>& InCollapsableNodes)
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (!FocusedGraphEd.IsValid())
	{
		return;
	}

	UEdGraph* SourceGraph = FocusedGraphEd->GetCurrentGraph();
	SourceGraph->Modify();

	// Create the composite node that will serve as the gateway into the subgraph
	UK2Node_Composite* GatewayNode = nullptr;
	{
		GatewayNode = FEdGraphSchemaAction_K2NewNode::SpawnNode<UK2Node_Composite>(SourceGraph, FVector2D(0,0), EK2NewNodeFlags::SelectNewNode);
		GatewayNode->bCanRenameNode = true;
		check(GatewayNode);
	}

	FName GraphName;
	GraphName = FBlueprintEditorUtils::FindUniqueKismetName(GetBlueprintObj(), TEXT("CollapseGraph"));

	// Rename the graph to the correct name
	UEdGraph* DestinationGraph = GatewayNode->BoundGraph;
	TSharedPtr<INameValidatorInterface> NameValidator = MakeShareable(new FKismetNameValidator(GetBlueprintObj(), GraphName));
	FBlueprintEditorUtils::RenameGraphWithSuggestion(DestinationGraph, NameValidator, GraphName.ToString());

	CollapseNodesIntoGraph(GatewayNode, GatewayNode->GetInputSink(), GatewayNode->GetOutputSource(), SourceGraph, DestinationGraph, InCollapsableNodes, false, true);
}

UEdGraph* FBlueprintEditor::CollapseSelectionToFunction(TSharedPtr<SGraphEditor> InRootGraph, TSet<UEdGraphNode*>& InCollapsableNodes, UEdGraphNode*& OutFunctionNode)
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = InRootGraph;
	if (!FocusedGraphEd.IsValid())
	{
		return nullptr;
	}

	UEdGraph* SourceGraph = FocusedGraphEd->GetCurrentGraph();
	SourceGraph->Modify();

	UEdGraph* NewGraph = nullptr;

	FName DocumentName = FBlueprintEditorUtils::FindUniqueKismetName(GetBlueprintObj(), TEXT("NewFunction"));

	NewGraph = FBlueprintEditorUtils::CreateNewGraph(GetBlueprintObj(), DocumentName, UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());
	FBlueprintEditorUtils::AddFunctionGraph<UClass>(GetBlueprintObj(), NewGraph, /*bIsUserCreated=*/ true, nullptr);

	TArray<UK2Node_FunctionEntry*> EntryNodes;
	NewGraph->GetNodesOfClass(EntryNodes);
	UK2Node_FunctionEntry* EntryNode = EntryNodes[0];
	UK2Node_FunctionResult* ResultNode = nullptr;

	// Create Result
	FGraphNodeCreator<UK2Node_FunctionResult> ResultNodeCreator(*NewGraph);
	UK2Node_FunctionResult* FunctionResult = ResultNodeCreator.CreateNode();

	const UEdGraphSchema_K2* Schema = Cast<UEdGraphSchema_K2>(FunctionResult->GetSchema());
	FunctionResult->NodePosX = EntryNode->NodePosX + EntryNode->NodeWidth + 256;
	FunctionResult->NodePosY = EntryNode->NodePosY;

	ResultNodeCreator.Finalize();

	ResultNode = FunctionResult;

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// make temp list builder
	FGraphActionListBuilderBase TempListBuilder;
	TempListBuilder.OwnerOfTemporaries = NewObject<UEdGraph>(GetBlueprintObj());
	TempListBuilder.OwnerOfTemporaries->SetFlags(RF_Transient);

	IBlueprintNodeBinder::FBindingSet Bindings;
	OutFunctionNode = UBlueprintFunctionNodeSpawner::Create(FindUField<UFunction>(GetBlueprintObj()->SkeletonGeneratedClass, DocumentName))->Invoke(SourceGraph, Bindings, FVector2D::ZeroVector);

	check(OutFunctionNode);

	CollapseNodesIntoGraph(OutFunctionNode, EntryNode, ResultNode, SourceGraph, NewGraph, InCollapsableNodes, true, false);

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprintObj());
	OutFunctionNode->ReconstructNode();

	return NewGraph;
}

UEdGraph* FBlueprintEditor::CollapseSelectionToMacro(TSharedPtr<SGraphEditor> InRootGraph, TSet<UEdGraphNode*>& InCollapsableNodes, UEdGraphNode*& OutMacroNode)
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = InRootGraph;
	if (!FocusedGraphEd.IsValid())
	{
		return nullptr;
	}

	UBlueprint* BP = GetBlueprintObj();

	UEdGraph* SourceGraph = FocusedGraphEd->GetCurrentGraph();
	SourceGraph->Modify();

	UEdGraph* DestinationGraph = nullptr;

	FName DocumentName = FBlueprintEditorUtils::FindUniqueKismetName(BP, TEXT("NewMacro"));

	DestinationGraph = FBlueprintEditorUtils::CreateNewGraph(BP, DocumentName, UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());
	FBlueprintEditorUtils::AddMacroGraph(BP, DestinationGraph, /*bIsUserCreated=*/ true, nullptr);

	UK2Node_MacroInstance* GatewayNode = FEdGraphSchemaAction_K2NewNode::SpawnNode<UK2Node_MacroInstance>(
		SourceGraph,
		FVector2D(0.0f, 0.0f),
		EK2NewNodeFlags::None,
		[DestinationGraph](UK2Node_MacroInstance* NewInstance)
		{
			NewInstance->SetMacroGraph(DestinationGraph);
		}
	);

	TArray<UK2Node_Tunnel*> TunnelNodes;
	GatewayNode->GetMacroGraph()->GetNodesOfClass(TunnelNodes);

	UK2Node_EditablePinBase* InputSink = nullptr;
	UK2Node_EditablePinBase* OutputSink = nullptr;

	// Retrieve the tunnel nodes to use them to match up pin links that connect to the gateway.
	for (UK2Node_Tunnel* Node : TunnelNodes)
	{
		if (Node->IsEditable())
		{
			if (Node->bCanHaveOutputs)
			{
				InputSink = Node;
			}
			else if (Node->bCanHaveInputs)
			{
				OutputSink = Node;
			}
		}
	}

	CollapseNodesIntoGraph(GatewayNode, InputSink, OutputSink, SourceGraph, DestinationGraph, InCollapsableNodes, /* bCanDiscardEmptyReturnNode */ false, /* bCanHaveWeakObjPtrParam */ false);

	OutMacroNode = GatewayNode;
	OutMacroNode->ReconstructNode();

	return DestinationGraph;
}

void FBlueprintEditor::ExpandNode(UEdGraphNode* InNodeToExpand, UEdGraph* InSourceGraph, TSet<UEdGraphNode*>& OutExpandedNodes)
{
 	UEdGraph* DestinationGraph = InNodeToExpand->GetGraph();
	UEdGraph* SourceGraph = InSourceGraph;
	check(SourceGraph);

	// Mark all edited objects so they will appear in the transaction record if needed.
	DestinationGraph->Modify();
	SourceGraph->Modify();
	InNodeToExpand->Modify();

	UEdGraphNode* Entry = nullptr;
	UEdGraphNode* Result = nullptr;

	const bool bIsCollapsedGraph = InNodeToExpand->IsA<UK2Node_Composite>();

	MoveNodesToGraph(MutableView(SourceGraph->Nodes), DestinationGraph, OutExpandedNodes, &Entry, &Result, bIsCollapsedGraph);

	UEdGraphPin* OutputExecPinReconnect = nullptr;
	if(UK2Node_CallFunction* CallFunction = Cast<UK2Node_CallFunction>(InNodeToExpand))
	{
		UEdGraphPin* ThenPin = CallFunction->GetThenPin();
		if (ThenPin && ThenPin->LinkedTo.Num())
		{
			OutputExecPinReconnect = ThenPin->LinkedTo[0];
		}
	}

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	K2Schema->CollapseGatewayNode(Cast<UK2Node>(InNodeToExpand), Entry, Result, nullptr, &OutExpandedNodes);

	if(Entry)
	{
		Entry->DestroyNode();
	}

	if(Result)
	{
		Result->DestroyNode();
	}

	// Make sure any subgraphs get propagated appropriately
	if (SourceGraph->SubGraphs.Num() > 0)
	{
		DestinationGraph->SubGraphs.Append(SourceGraph->SubGraphs);
		SourceGraph->SubGraphs.Empty();
	}

	// Remove the gateway node and source graph
	InNodeToExpand->DestroyNode();

	// This should be set for function nodes, all expanded nodes should connect their output exec pins to the original pin.
	if(OutputExecPinReconnect)
	{
		for (TSet<UEdGraphNode*>::TConstIterator NodeIt(OutExpandedNodes); NodeIt; ++NodeIt)
		{
			UEdGraphNode* Node = *NodeIt;
			for (int32 PinIndex = 0; PinIndex < Node->Pins.Num(); ++PinIndex)
			{
				// Only hookup output exec pins that do not have a connection
				if(Node->Pins[PinIndex]->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec && Node->Pins[PinIndex]->Direction == EGPD_Output && Node->Pins[PinIndex]->LinkedTo.Num() == 0)
				{
					Node->Pins[PinIndex]->MakeLinkTo(OutputExecPinReconnect);
				}
			}
		}
	}
}

void FBlueprintEditor::MoveNodesToGraph(TArray<UEdGraphNode*>& SourceNodes, UEdGraph* DestinationGraph, TSet<UEdGraphNode*>& OutExpandedNodes, UEdGraphNode** OutEntry, UEdGraphNode** OutResult, const bool bIsCollapsedGraph)
{
	// Move the nodes over, remembering any that are boundary nodes
	while (SourceNodes.Num())
	{
		UEdGraphNode* Node = SourceNodes.Pop();
		UEdGraph* OriginalGraph = Node->GetGraph();

		Node->Modify();
		OriginalGraph->Modify();
		Node->Rename(/*NewName=*/ nullptr, /*NewOuter=*/ DestinationGraph, REN_DontCreateRedirectors);

		// Remove the node from the original graph
		OriginalGraph->RemoveNode(Node, false);
		
		// We do not check CanPasteHere when determining CanCollapseNodes, unlike CanCollapseSelectionToFunction/Macro,
		// so when expanding a collapsed graph we don't want to check the CanPasteHere function:
		if (!bIsCollapsedGraph && !Node->CanPasteHere(DestinationGraph))
		{
			Node->BreakAllNodeLinks();
			continue;
		}

		// Successfully added the node to the graph, we may need to remove flags
		if (Node->HasAllFlags(RF_Transient) && !DestinationGraph->HasAllFlags(RF_Transient))
		{
			Node->SetFlags(RF_Transactional);
			Node->ClearFlags(RF_Transient);
			TArray<UObject*> Subobjects;
			GetObjectsWithOuter(Node, Subobjects);
			for (UObject* Subobject : Subobjects)
			{
				Subobject->ClearFlags(RF_Transient);
				Subobject->SetFlags(RF_Transactional);
			}
		}
		
		DestinationGraph->AddNode(Node, /* bFromUI */ false, /* bSelectNewNode */ false);
		
		if(UK2Node_Composite* Composite = Cast<UK2Node_Composite>(Node))
		{
			OriginalGraph->SubGraphs.Remove(Composite->BoundGraph);
			DestinationGraph->SubGraphs.Add(Composite->BoundGraph);
		}

		// Want to test exactly against tunnel, we shouldn't collapse embedded collapsed
		// nodes or macros, only the tunnels in/out of the collapsed graph
		if (Node->GetClass() == UK2Node_Tunnel::StaticClass())
		{
			UK2Node_Tunnel* TunnelNode = Cast<UK2Node_Tunnel>(Node);
			if (TunnelNode->bCanHaveOutputs)
			{
				*OutEntry = Node;
			}
			else if (TunnelNode->bCanHaveInputs)
			{
				*OutResult = Node;
			}
		}
		else if (Node->GetClass() == UK2Node_FunctionEntry::StaticClass())
		{
			*OutEntry = Node;
		}
		else if (Node->GetClass() == UK2Node_FunctionResult::StaticClass())
		{
			*OutResult = Node;
		}
		else
		{
			OutExpandedNodes.Add(Node);
		}
	}
}

void FBlueprintEditor::SaveEditedObjectState()
{
	check(IsEditingSingleBlueprint());

	// Clear currently edited documents
	GetBlueprintObj()->LastEditedDocuments.Empty();

	// Ask all open documents to save their state, which will update LastEditedDocuments
	DocumentManager->SaveAllState();
}

void FBlueprintEditor::RequestSaveEditedObjectState()
{
	bRequestedSavingOpenDocumentState = true;
}

void FBlueprintEditor::GetBoundsForNode(const UEdGraphNode* InNode, class FSlateRect& OutRect, float InPadding) const
{
	if (FocusedGraphEdPtr.IsValid())
	{
		FocusedGraphEdPtr.Pin()->GetBoundsForNode(InNode, OutRect, InPadding);
	}
}

void FBlueprintEditor::GetViewBookmark(FGuid& BookmarkId)
{
	BookmarkId.Invalidate();

	if (FocusedGraphEdPtr.IsValid())
	{
		FocusedGraphEdPtr.Pin()->GetViewBookmark(BookmarkId);
	}
}

void FBlueprintEditor::GetViewLocation(FVector2D& Location, float& ZoomAmount)
{
	Location = FVector2D::ZeroVector;
	ZoomAmount = 0.0f;

	if (FocusedGraphEdPtr.IsValid())
	{
		FVector2f TempLocation;
		FocusedGraphEdPtr.Pin()->GetViewLocation(TempLocation, ZoomAmount);
		Location = FVector2D(TempLocation);
	}
}

void FBlueprintEditor::SetViewLocation(const FVector2D& Location, float ZoomAmount, const FGuid& BookmarkId)
{
	if (FocusedGraphEdPtr.IsValid())
	{
		FocusedGraphEdPtr.Pin()->SetViewLocation(UE::Slate::CastToVector2f(Location), ZoomAmount, BookmarkId);
	}
}

void FBlueprintEditor::Tick(float DeltaTime)
{
	PreviewScene.UpdateCaptureContents();

	// Create or update the Blueprint actor instance in the preview scene
	if ( GetPreviewActor() == nullptr )
	{
		UpdatePreviewActor(GetBlueprintObj(), true);
	}

	if (bRequestedSavingOpenDocumentState)
	{
		bRequestedSavingOpenDocumentState = false;

		SaveEditedObjectState();
	}

	if (InstructionsFadeCountdown > 0.f)
	{
		InstructionsFadeCountdown -= DeltaTime;
	}

	if (bPendingDeferredClose)
	{
		IAssetEditorInstance* EditorInst = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(GetBlueprintObj(), /*bFocusIfOpen =*/false);
		check(EditorInst != nullptr);
		EditorInst->CloseWindow(EAssetEditorCloseReason::AssetUnloadingOrInvalid);
	}
	else
	{
		// Auto-import any namespaces we've collected as a result of compound events that may have occurred within this frame.
		if (!DeferredNamespaceImports.IsEmpty())
		{
			FImportNamespaceExParameters Params;
			Params.NamespacesToImport = MoveTemp(DeferredNamespaceImports);
			ImportNamespaceEx(Params);

			// Assert that this was reset by the move above.
			check(DeferredNamespaceImports.IsEmpty());
		}
	}
}
TStatId FBlueprintEditor::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FBlueprintEditor, STATGROUP_Tickables);
}


void FBlueprintEditor::OnStartEditingDefaultsClicked()
{
	StartEditingDefaults(/*bAutoFocus=*/ true);
}

void FBlueprintEditor::OnListObjectsReferencedByClass()
{
	ObjectTools::ShowReferencedObjs(GetBlueprintObj()->GeneratedClass);
}

void FBlueprintEditor::OnListObjectsReferencedByBlueprint()
{
	ObjectTools::ShowReferencedObjs(GetBlueprintObj());
}

void FBlueprintEditor::OnRepairCorruptedBlueprint()
{
	IKismetCompilerInterface& Compiler = FModuleManager::LoadModuleChecked<IKismetCompilerInterface>(KISMET_COMPILER_MODULENAME);
	Compiler.RecoverCorruptedBlueprint(GetBlueprintObj());
}

void FBlueprintEditor::StartEditingDefaults(bool bAutoFocus, bool bForceRefresh)
{
	SetUISelectionState(FBlueprintEditor::SelectionState_ClassDefaults);

	if (IsEditingSingleBlueprint())
	{
		if (GetBlueprintObj()->GeneratedClass != nullptr)
		{
			if (SubobjectEditor.IsValid() && GetBlueprintObj()->GeneratedClass->IsChildOf<AActor>())
			{
				SubobjectEditor->SelectRoot();
			}
			else
			{
				UObject* DefaultObject = GetBlueprintObj()->GeneratedClass->GetDefaultObject();

				// Update the details panel
				FString Title;
				DefaultObject->GetName(Title);
				SKismetInspector::FShowDetailsOptions Options(FText::FromString(Title), bForceRefresh);
				Options.bShowComponents = false;

				Inspector->ShowDetailsForSingleObject(DefaultObject, Options);

				if ( bAutoFocus )
				{
					TryInvokingDetailsTab();
				}
			}
		}
	}
	
	RefreshStandAloneDefaultsEditor();
}

void FBlueprintEditor::RefreshStandAloneDefaultsEditor()
{
	// Update the details panel
	SKismetInspector::FShowDetailsOptions Options(FText::GetEmpty(), true);

	TArray<UObject*> DefaultObjects;
	for ( int32 i = 0; i < GetEditingObjects().Num(); ++i )
	{
		if (UBlueprint* Blueprint = Cast<UBlueprint>(GetEditingObjects()[i]))
		{
			if (CurrentUISelection == FBlueprintEditor::SelectionState_ClassSettings)
			{
				DefaultObjects.Add(Blueprint);
			}
			else if (Blueprint->GeneratedClass)
			{
				DefaultObjects.Add(Blueprint->GeneratedClass->GetDefaultObject());
			}
		}
	}

	if ( DefaultObjects.Num() && DefaultEditor.IsValid() )
	{
		DefaultEditor->ShowDetailsForObjects(DefaultObjects);
	}
}

void FBlueprintEditor::RenameNewlyAddedAction(FName InActionName)
{
	TabManager->TryInvokeTab(FBlueprintEditorTabs::MyBlueprintID);
	TryInvokingDetailsTab(/*Flash*/false);

	if (MyBlueprintWidget.IsValid())
	{
		// Force a refresh immediately, the item has to be present in the list for the rename requests to be successful.
		MyBlueprintWidget->Refresh();
		MyBlueprintWidget->SelectItemByName(InActionName,ESelectInfo::OnMouseClick);
		MyBlueprintWidget->OnRequestRenameOnActionNode();
	}
}

void FBlueprintEditor::OnAddNewVariable()
{
	const FScopedTransaction Transaction( LOCTEXT("AddVariable", "Add Variable") );

	FName VarName = FBlueprintEditorUtils::FindUniqueKismetName(GetBlueprintObj(), TEXT("NewVar"));

	bool bSuccess = MyBlueprintWidget.IsValid() && FBlueprintEditorUtils::AddMemberVariable(GetBlueprintObj(), VarName, MyBlueprintWidget->GetLastPinTypeUsed());

	if(!bSuccess)
	{
		LogSimpleMessage( LOCTEXT("AddVariable_Error", "Adding new variable failed.") );
	}
	else
	{
		RenameNewlyAddedAction(VarName);
	}
}

bool FBlueprintEditor::CanAddNewLocalVariable() const
{
	if (InEditingMode())
	{
		TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
		if (!FocusedGraphEd.IsValid())
		{
			return false;
		}

		UEdGraph* TargetGraph = FBlueprintEditorUtils::GetTopLevelGraph(FocusedGraphEd->GetCurrentGraph());
		return TargetGraph->GetSchema()->GetGraphType(TargetGraph) == GT_Function;
	}
	return false;
}

void FBlueprintEditor::OnAddNewLocalVariable()
{
	if (!CanAddNewLocalVariable())
	{
		return;
	}

	// Find the top level graph to place the local variables into
	UEdGraph* TargetGraph = FBlueprintEditorUtils::GetTopLevelGraph(FocusedGraphEdPtr.Pin()->GetCurrentGraph());
	check(TargetGraph->GetSchema()->GetGraphType(TargetGraph) == GT_Function);

	FName VarName = FBlueprintEditorUtils::FindUniqueKismetName(GetBlueprintObj(), TEXT("NewLocalVar"), FindUField<UFunction>(GetBlueprintObj()->SkeletonGeneratedClass, TargetGraph->GetFName()));

	bool bSuccess = MyBlueprintWidget.IsValid() && FBlueprintEditorUtils::AddLocalVariable(GetBlueprintObj(), TargetGraph, VarName, MyBlueprintWidget->GetLastPinTypeUsed());

	if(!bSuccess)
	{
		LogSimpleMessage( LOCTEXT("AddLocalVariable_Error", "Adding new local variable failed.") );
	}
	else
	{
		RenameNewlyAddedAction(VarName);
	}
}

void FBlueprintEditor::OnPasteNewLocalVariable(const FBPVariableDescription& VariableDescription)
{
	if (!CanAddNewLocalVariable())
	{
		return;
	}

	// Find the top level graph to place the local variables into
	UEdGraph* TargetGraph = FBlueprintEditorUtils::GetTopLevelGraph(FocusedGraphEdPtr.Pin()->GetCurrentGraph());
	check(TargetGraph->GetSchema()->GetGraphType(TargetGraph) == GT_Function);

	bool bSuccess = MyBlueprintWidget.IsValid() && FBlueprintEditorUtils::AddLocalVariable(GetBlueprintObj(), TargetGraph, VariableDescription.VarName, VariableDescription.VarType, VariableDescription.DefaultValue);

	if(!bSuccess)
	{
		LogSimpleMessage( LOCTEXT("PasteLocalVariable_Error", "Pasting new local variable failed.") );
	}
}

void FBlueprintEditor::OnAddNewDelegate()
{
	if (!AddNewDelegateIsVisible())
	{
		return;
	}

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	check(nullptr != K2Schema);
	UBlueprint* const Blueprint = GetBlueprintObj();
	check(nullptr != Blueprint);

	FName Name = FBlueprintEditorUtils::FindUniqueKismetName(GetBlueprintObj(), TEXT("NewEventDispatcher"));


	const FScopedTransaction Transaction( LOCTEXT("AddNewDelegate", "Add New Event Dispatcher") ); 
	Blueprint->Modify();

	FEdGraphPinType DelegateType;
	DelegateType.PinCategory = UEdGraphSchema_K2::PC_MCDelegate;
	const bool bVarCreatedSuccess = FBlueprintEditorUtils::AddMemberVariable(Blueprint, Name, DelegateType);
	if(!bVarCreatedSuccess)
	{
		LogSimpleMessage( LOCTEXT("AddDelegateVariable_Error", "Adding new delegate variable failed.") );
		return;
	}

	UEdGraph* const NewGraph = FBlueprintEditorUtils::CreateNewGraph(Blueprint, Name, UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());
	if(!NewGraph)
	{
		FBlueprintEditorUtils::RemoveMemberVariable(Blueprint, Name);
		LogSimpleMessage( LOCTEXT("AddDelegateVariable_Error", "Adding new delegate variable failed.") );
		return;
	}

	NewGraph->bEditable = false;

	K2Schema->CreateDefaultNodesForGraph(*NewGraph);
	K2Schema->CreateFunctionGraphTerminators(*NewGraph, (UClass*)nullptr);
	K2Schema->AddExtraFunctionFlags(NewGraph, (FUNC_BlueprintCallable|FUNC_BlueprintEvent|FUNC_Public));
	K2Schema->MarkFunctionEntryAsEditable(NewGraph, true);

	Blueprint->DelegateSignatureGraphs.Add(NewGraph);
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);

	RenameNewlyAddedAction(Name);
}

void FBlueprintEditor::NewDocument_OnClicked(ECreatedDocumentType GraphType)
{
	if (!NewDocument_IsVisibleForType(GraphType))
	{
		return;
	}

	FText DocumentNameText;
	bool bResetMyBlueprintFilter = false;

	switch (GraphType)
	{
	case CGT_NewFunctionGraph:
		DocumentNameText = LOCTEXT("NewDocFuncName", "NewFunction");
		bResetMyBlueprintFilter = true;
		break;
	case CGT_NewEventGraph:
		DocumentNameText = LOCTEXT("NewDocEventGraphName", "NewEventGraph");
		bResetMyBlueprintFilter = true;
		break;
	case CGT_NewMacroGraph:
		DocumentNameText = LOCTEXT("NewDocMacroName", "NewMacro");
		bResetMyBlueprintFilter = true;
		break;
	case CGT_NewAnimationLayer:
		DocumentNameText = LOCTEXT("NewDocAnimationLayerName", "NewAnimationLayer");
		bResetMyBlueprintFilter = true;
		break;
	default:
		DocumentNameText = LOCTEXT("NewDocNewName", "NewDocument");
		break;
	}

	FName DocumentName = FName(*DocumentNameText.ToString());

	// Make sure the new name is valid
	DocumentName = FBlueprintEditorUtils::FindUniqueKismetName(GetBlueprintObj(), DocumentNameText.ToString());

	check(IsEditingSingleBlueprint());

	const FScopedTransaction Transaction( LOCTEXT("AddNewFunction", "Add New Function") ); 
	GetBlueprintObj()->Modify();

	UEdGraph* NewGraph = nullptr;

	if (GraphType == CGT_NewFunctionGraph)
	{
		NewGraph = FBlueprintEditorUtils::CreateNewGraph(GetBlueprintObj(), DocumentName, UEdGraph::StaticClass(), GetDefaultSchemaClass());
		FBlueprintEditorUtils::AddFunctionGraph<UClass>(GetBlueprintObj(), NewGraph, /*bIsUserCreated=*/ true, nullptr);
	}
	else if (GraphType == CGT_NewMacroGraph)
	{
		NewGraph = FBlueprintEditorUtils::CreateNewGraph(GetBlueprintObj(), DocumentName, UEdGraph::StaticClass(), GetDefaultSchemaClass());
		FBlueprintEditorUtils::AddMacroGraph(GetBlueprintObj(), NewGraph, /*bIsUserCreated=*/ true, nullptr);
	}
	else if (GraphType == CGT_NewEventGraph)
	{
		NewGraph = FBlueprintEditorUtils::CreateNewGraph(GetBlueprintObj(), DocumentName, UEdGraph::StaticClass(), GetDefaultSchemaClass());
		FBlueprintEditorUtils::AddUbergraphPage(GetBlueprintObj(), NewGraph);
	}
	else if (GraphType == CGT_NewAnimationLayer)
	{
		//@TODO: ANIMREFACTOR: This code belongs in Persona, not in BlueprintEditor
		NewGraph = FBlueprintEditorUtils::CreateNewGraph(GetBlueprintObj(), DocumentName, UAnimationGraph::StaticClass(), UAnimationGraphSchema::StaticClass());
		FBlueprintEditorUtils::AddDomainSpecificGraph(GetBlueprintObj(), NewGraph);
	}
	else
	{
		ensureMsgf(false, TEXT("GraphType is invalid") );
	}

	// Now open the new graph
	if (NewGraph)
	{
		OpenDocument(NewGraph, FDocumentTracker::OpenNewDocument);

		RenameNewlyAddedAction(DocumentName);
	}
	else
	{
		LogSimpleMessage( LOCTEXT("AddDocument_Error", "Adding new document failed.") );
	}
}

bool FBlueprintEditor::NewDocument_IsVisibleForType(ECreatedDocumentType GraphType) const
{
	switch (GraphType)
	{
	case CGT_NewVariable:
		return (GetBlueprintObj()->BlueprintType != BPTYPE_FunctionLibrary) 
			&& (GetBlueprintObj()->BlueprintType != BPTYPE_Interface) 
			&& (GetBlueprintObj()->BlueprintType != BPTYPE_MacroLibrary);
	case CGT_NewFunctionGraph:
		{
			if(UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(GetBlueprintObj()))
			{
				return (GetBlueprintObj()->BlueprintType != BPTYPE_Interface);
			}
			else
			{
				return (GetBlueprintObj()->BlueprintType != BPTYPE_MacroLibrary);
			}
		}
	case CGT_NewMacroGraph:
		return (GetBlueprintObj()->BlueprintType == BPTYPE_MacroLibrary) || (GetBlueprintObj()->BlueprintType == BPTYPE_Normal) || (GetBlueprintObj()->BlueprintType == BPTYPE_LevelScript);
	case CGT_NewAnimationLayer:
		{	
			if(UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(GetBlueprintObj()))
			{
				UAnimBlueprint* RootBlueprint = UAnimBlueprint::FindRootAnimBlueprint(AnimBlueprint);
				if(RootBlueprint == nullptr)
				{
					return true;
				}
			}
		}
		break;
	case CGT_NewEventGraph:
		return FBlueprintEditorUtils::DoesSupportEventGraphs(GetBlueprintObj());
	case CGT_NewLocalVariable:
		return FBlueprintEditorUtils::DoesSupportLocalVariables(GetFocusedGraph()) 
			&& IsFocusedGraphEditable();
	}

	return false;
}

TSubclassOf<UEdGraphSchema> FBlueprintEditor::GetDefaultSchemaClass() const
{
	return UEdGraphSchema_K2::StaticClass();
}

bool FBlueprintEditor::AddNewDelegateIsVisible() const
{
	const UBlueprint* Blueprint = GetBlueprintObj();
	return (nullptr != Blueprint)
		&& (Blueprint->BlueprintType != BPTYPE_Interface) 
		&& (Blueprint->BlueprintType != BPTYPE_MacroLibrary)
		&& (Blueprint->BlueprintType != BPTYPE_FunctionLibrary);
}

void FBlueprintEditor::NotifyPreChange(FProperty* PropertyAboutToChange)
{
	// this only delivers message to the "FOCUSED" one, not every one
	// internally it will only deliver the message to the selected node, not all nodes
	FString PropertyName = PropertyAboutToChange->GetName();
	if (FocusedGraphEdPtr.IsValid())
	{
		FocusedGraphEdPtr.Pin()->NotifyPrePropertyChange(PropertyName);
	}
}

void FBlueprintEditor::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	FString PropertyName = PropertyThatChanged->GetName();
	if (FocusedGraphEdPtr.IsValid())
	{
		FocusedGraphEdPtr.Pin()->NotifyPostPropertyChange(PropertyChangedEvent, PropertyName);
	}
	
	if (IsEditingSingleBlueprint())
	{
		UBlueprint* Blueprint = GetBlueprintObj();
		UPackage* BlueprintPackage = Blueprint->GetOutermost();

		// if any of the objects being edited are in our package, mark us as dirty
		bool bPropertyInBlueprint = false;
		for (int32 ObjectIndex = 0; ObjectIndex < PropertyChangedEvent.GetNumObjectsBeingEdited(); ++ObjectIndex)
		{
			const UObject* Object = PropertyChangedEvent.GetObjectBeingEdited(ObjectIndex);
			if (Object && Object->GetOutermost() == BlueprintPackage)
			{
				bPropertyInBlueprint = true;
				break;
			}
		}

		if (bPropertyInBlueprint)
		{
			// Note: if change type is "interactive," hold off on applying the change (e.g. this will occur if the user is scrubbing a spinbox value; we don't want to apply the change until the mouse is released, for performance reasons)
			if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
			{
				FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint, PropertyChangedEvent);

				// Call PostEditChange() on any Actors that might be based on this Blueprint
				FBlueprintEditorUtils::PostEditChangeBlueprintActors(Blueprint);
			}

			// Force updates to occur immediately during interactive mode (otherwise the preview won't refresh because it won't be ticking)
			UpdateSubobjectPreview(PropertyChangedEvent.ChangeType == EPropertyChangeType::Interactive);
		}
	}
}

bool FBlueprintEditor::ShouldShowPublicViewControl() const
{
	// In defaults-only mode, hide the "Public View" checkbox when Class Settings is selected into the Details view.
	return !bWasOpenedInDefaultsMode || CurrentUISelection != FBlueprintEditor::SelectionState_ClassSettings;
}

void FBlueprintEditor::OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent)
{
	FName PropertyName = PropertyChangedEvent.GetPropertyName();
	FName MemberPropertyName = PropertyChangedEvent.MemberProperty ? PropertyChangedEvent.MemberProperty->GetFName() : PropertyName;

	//@TODO: This code does not belong here (might not even be necessary anymore as they seem to have PostEditChangeProperty impls now)!
	if ((PropertyName == GET_MEMBER_NAME_CHECKED(UK2Node_Switch, bHasDefaultPin)) ||
		(PropertyName == GET_MEMBER_NAME_CHECKED(UK2Node_SwitchInteger, StartIndex)) ||
		(PropertyName == GET_MEMBER_NAME_CHECKED(UK2Node_SwitchString, PinNames)) ||
		(PropertyName == GET_MEMBER_NAME_CHECKED(UK2Node_SwitchName, PinNames)) ||
		(PropertyName == GET_MEMBER_NAME_CHECKED(UK2Node_SwitchString, bIsCaseSensitive)))
	{
		DocumentManager->RefreshAllTabs();
	}
}

void FBlueprintEditor::OnPreObjectPropertyChanged(UObject* InObject, const FEditPropertyChain& EditPropertyChain)
{
	if (GetDefault<UBlueprintEditorSettings>()->bDoNotMarkAllInstancesDirtyOnDefaultValueChange)
	{
		// Determine which inspector widget is currently in use.
		TSharedPtr<SKismetInspector> CurrentInspectorWidget;
		if (GetCurrentMode() == FBlueprintEditorApplicationModes::BlueprintDefaultsMode)
		{
			CurrentInspectorWidget = DefaultEditor;
		}
		else
		{
			CurrentInspectorWidget = Inspector;
		}

		// Note: While we could rely on our notify hook to determine whether an incoming change belongs to this
		// context, NotifyPreChange() may end up getting called after this event is broadcast (e.g. from inside
		// of a details customization). So it's not safe to assume they will get called in any specific order.
		// It is, however, safe to assume this will get called for an archetype/CDO prior to change propagation.
		if (CurrentInspectorWidget.IsValid())
		{
			check(InObject);

			auto IsSubobjectOfAnySelectedObject = [](const UObject* InObject, const TArray<TWeakObjectPtr<UObject>>& SelectedObjects)
			{
				for (const TWeakObjectPtr<UObject>& SelectedObject : SelectedObjects)
				{
					if (SelectedObject.IsValid() && InObject->IsInOuter(SelectedObject.Get()))
					{
						return true;
					}
				}

				return false;
			};

			// If we're modifying an object that's selected into our property editor context (e.g. the Blueprint CDO, or an SCS
			// component template), set up change propagation so that instances do not always mark their outer package as dirty.
			bool bIsObjectSelectedForEditing = false;
			const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = CurrentInspectorWidget->GetSelectedObjects();
			if (SelectedObjects.Contains(InObject) || IsSubobjectOfAnySelectedObject(InObject, SelectedObjects))
			{
				bIsObjectSelectedForEditing = true;
			}
			else if (IsEditingSingleBlueprint())
			{
				// Resolve the property that's about to be changed.
				FProperty* PropertyAboutToChange = nullptr;
				if (FEditPropertyChain::TDoubleLinkedListNode* PropertyNode = EditPropertyChain.GetActiveNode())
				{
					PropertyAboutToChange = PropertyNode->GetValue();
				}

				// When actions from the "My Blueprint" tab are selected (e.g. class variable actions), a property "wrapper"
				// object is selected to the inspector view instead, and the details customization will then pass the object
				// instance that's about to be changed (e.g. the Blueprint CDO) as the input parameter to this delegate.
				if (PropertyAboutToChange)
				{
					const UBlueprint* BlueprintContext = GetBlueprintObj();
					check(BlueprintContext);

					// Redirect to the skeleton class if the property's owner class was generated by the Blueprint that's
					// associated with this editing context. For details customization purposes, the property that's wrapped
					// belongs to the skeleton class and not the actual generated class (see SMyBlueprint::CollectAllActions).
					const UClass* OwnerClass = PropertyAboutToChange->GetOwnerClass();
					if (OwnerClass && OwnerClass->ClassGeneratedBy == BlueprintContext && BlueprintContext->SkeletonGeneratedClass)
					{
						PropertyAboutToChange = BlueprintContext->SkeletonGeneratedClass->FindPropertyByName(PropertyAboutToChange->GetFName());
						if (PropertyAboutToChange && SelectedObjects.Contains(PropertyAboutToChange->GetUPropertyWrapper()) && InObject == OwnerClass->GetDefaultObject())
						{
							bIsObjectSelectedForEditing = true;
						}
					}
				}
			}
			
			if (bIsObjectSelectedForEditing)
			{
				InObject->SetEditChangePropagationFlags(EEditChangePropagationFlags::OnlyMarkRealignedInstancesAsDirty);
			}
		}
	}
}

void FBlueprintEditor::OnPostObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& PropertyChangedEvent)
{
	check(InObject);

	// Reset propagation flags so that any changes external to our editing context will use the standard (default) behavior.
	InObject->SetEditChangePropagationFlags(EEditChangePropagationFlags::None);
}

FName FBlueprintEditor::GetToolkitFName() const
{
	return FName("BlueprintEditor");
}

FName FBlueprintEditor::GetContextFromBlueprintType(EBlueprintType InType)
{
	switch (InType)
	{
	default:
	case BPTYPE_Normal:
		return FName("BlueprintEditor");
	case BPTYPE_MacroLibrary:
		return FName("BlueprintEditor.MacroLibrary");
	case BPTYPE_Interface:
		return FName("BlueprintEditor.Interface");
	case BPTYPE_LevelScript:
		return FName("BlueprintEditor.LevelScript");
	}
}

FName FBlueprintEditor::GetToolkitContextFName() const
{
	if(GetBlueprintObj())
	{
		return GetContextFromBlueprintType(GetBlueprintObj()->BlueprintType);
	}

	return FName("BlueprintEditor");
}

FText FBlueprintEditor::GetBaseToolkitName() const
{
	return LOCTEXT( "AppLabel", "Blueprint Editor" );
}

FText FBlueprintEditor::GetToolkitName() const
{
	const auto& EditingObjs = GetEditingObjects();

	if( IsEditingSingleBlueprint() )
	{
		if (FBlueprintEditorUtils::IsLevelScriptBlueprint(GetBlueprintObj()))
		{
			const FString& LevelName = FPackageName::GetShortFName( GetBlueprintObj()->GetOutermost()->GetFName().GetPlainNameString() ).GetPlainNameString();	
			return FText::FromString(LevelName);
		}
		else
		{
			return FText::FromString(GetBlueprintObj()->GetName());
		}
	}

	TSubclassOf< UObject > SharedParentClass = nullptr;

	for (UObject* EditingObj : EditingObjs)
	{
		UBlueprint* Blueprint = Cast<UBlueprint>(EditingObj);
		check( Blueprint );

		// Initialize with the class of the first object we encounter.
		if( *SharedParentClass == nullptr )
		{
			SharedParentClass = Blueprint->ParentClass;
		}

		// If we've encountered an object that's not a subclass of the current best baseclass,
		// climb up a step in the class hierarchy.
		while( !Blueprint->ParentClass->IsChildOf( SharedParentClass ) )
		{
			SharedParentClass = SharedParentClass->GetSuperClass();
		}
	}

	FFormatNamedArguments Args;
	Args.Add( TEXT("NumberOfObjects"), EditingObjs.Num() );
	Args.Add( TEXT("ObjectName"), FText::FromString( SharedParentClass->GetName() ) );
	return FText::Format( NSLOCTEXT("KismetEditor", "ToolkitTitle_UniqueLayerName", "{NumberOfObjects} {ClassName} - Class Defaults"), Args );
}

FText FBlueprintEditor::GetToolkitToolTipText() const
{
	const auto& EditingObjs = GetEditingObjects();

	if( IsEditingSingleBlueprint() )
	{
		if (FBlueprintEditorUtils::IsLevelScriptBlueprint(GetBlueprintObj()))
		{
			const FString& LevelName = FPackageName::GetShortFName( GetBlueprintObj()->GetOutermost()->GetFName().GetPlainNameString() ).GetPlainNameString();	

			FFormatNamedArguments Args;
			Args.Add( TEXT("LevelName"), FText::FromString( LevelName ) );
			return FText::Format( NSLOCTEXT("KismetEditor", "LevelScriptAppToolTip", "{LevelName} - Level Blueprint Editor"), Args );
		}
		else
		{
			return FAssetEditorToolkit::GetToolTipTextForObject( GetBlueprintObj() );
		}
	}

	TSubclassOf< UObject > SharedParentClass = nullptr;

	for (UObject* EditingObj : EditingObjs)
	{
		UBlueprint* Blueprint = Cast<UBlueprint>(EditingObj);
		check( Blueprint );

		// Initialize with the class of the first object we encounter.
		if( *SharedParentClass == nullptr )
		{
			SharedParentClass = Blueprint->ParentClass;
		}
		 
		// If we've encountered an object that's not a subclass of the current best baseclass,
		// climb up a step in the class hierarchy.
		while( !Blueprint->ParentClass->IsChildOf( SharedParentClass ) )
		{
			SharedParentClass = SharedParentClass->GetSuperClass();
		}
	}

	FFormatNamedArguments Args;
	Args.Add( TEXT("NumberOfObjects"), EditingObjs.Num() );
	Args.Add( TEXT("ClassName"), FText::FromString( SharedParentClass->GetName() ) );
	return FText::Format( NSLOCTEXT("KismetEditor", "ToolkitTitle_UniqueLayerName", "{NumberOfObjects} {ClassName} - Class Defaults"), Args );
}

FLinearColor FBlueprintEditor::GetWorldCentricTabColorScale() const
{
	if ((IsEditingSingleBlueprint()) && FBlueprintEditorUtils::IsLevelScriptBlueprint(GetBlueprintObj()))
	{
		return FLinearColor( 0.0f, 0.2f, 0.3f, 0.5f );
	}
	else
	{
		return FLinearColor( 0.0f, 0.0f, 0.3f, 0.5f );
	}
}

bool FBlueprintEditor::IsBlueprintEditor() const
{
	return true;
}

FString FBlueprintEditor::GetWorldCentricTabPrefix() const
{
	check(IsEditingSingleBlueprint());

	if (FBlueprintEditorUtils::IsLevelScriptBlueprint(GetBlueprintObj()))
	{
		return NSLOCTEXT("KismetEditor", "WorldCentricTabPrefix_LevelScript", "Script ").ToString();
	}
	else
	{
		return NSLOCTEXT("KismetEditor", "WorldCentricTabPrefix_Blueprint", "Blueprint ").ToString();
	}
}

void FBlueprintEditor::VariableListWasUpdated()
{
	StartEditingDefaults(/*bAutoFocus=*/ false);
}

bool FBlueprintEditor::GetBoundsForSelectedNodes(class FSlateRect& Rect, float Padding)
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		return FocusedGraphEd->GetBoundsForSelectedNodes(Rect, Padding);
	}
	return false;
}

void FBlueprintEditor::OnRenameNode()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if(FocusedGraphEd.IsValid())
	{
		const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
		for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
		{
			UEdGraphNode* SelectedNode = Cast<UEdGraphNode>(*NodeIt);
			if (SelectedNode != nullptr && SelectedNode->GetCanRenameNode())
			{
				FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(SelectedNode, true);
				break;
			}
		}
	}
}

bool FBlueprintEditor::CanRenameNodes() const
{
	if (IsEditable(GetFocusedGraph()))
	{
		if (const UEdGraphNode* SelectedNode = GetSingleSelectedNode())
		{
			return SelectedNode->GetCanRenameNode();
		}
	}
	return false;
}

bool FBlueprintEditor::OnNodeVerifyTitleCommit(const FText& NewText, UEdGraphNode* NodeBeingChanged, FText& OutErrorMessage)
{
	bool bValid(false);

	if (NodeBeingChanged && NodeBeingChanged->GetCanRenameNode())
	{
		// Clear off any existing error message 
		NodeBeingChanged->ErrorMsg.Empty();
		NodeBeingChanged->bHasCompilerMessage = false;

		if (!NameEntryValidator.IsValid())
		{
			NameEntryValidator = FNameValidatorFactory::MakeValidator(NodeBeingChanged);
		}

		EValidatorResult VResult = NameEntryValidator->IsValid(NewText.ToString(), true);
		if (VResult == EValidatorResult::Ok)
		{
			bValid = true;
		}
		else if (FocusedGraphEdPtr.IsValid()) 
		{
			EValidatorResult Valid = NameEntryValidator->IsValid(NewText.ToString(), false);
			
			NodeBeingChanged->bHasCompilerMessage = true;
			NodeBeingChanged->ErrorMsg = NameEntryValidator->GetErrorString(NewText.ToString(), Valid);
			NodeBeingChanged->ErrorType = EMessageSeverity::Error;
		}
	}
	NameEntryValidator.Reset();

	return bValid;
}

void FBlueprintEditor::OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged)
{
	if (NodeBeingChanged)
	{
		const FScopedTransaction Transaction( NSLOCTEXT( "K2_RenameNode", "RenameNode", "Rename Node" ) );
		NodeBeingChanged->Modify();
		NodeBeingChanged->OnRenameNode(NewText.ToString());
		if (BookmarksWidget.IsValid())
		{
			BookmarksWidget->RefreshBookmarksTree();
		}
	}
}



/////////////////////////////////////////////////////

void FBlueprintEditor::OnEditTabClosed(TSharedRef<SDockTab> Tab)
{
	// Update the edited object state
	if (GetBlueprintObj())
	{
		SaveEditedObjectState();
	}
}

// Tries to open the specified graph and bring it's document to the front
TSharedPtr<SGraphEditor> FBlueprintEditor::OpenGraphAndBringToFront(UEdGraph* Graph, bool bSetFocus)
{
	if (!IsValid(Graph))
	{
		return TSharedPtr<SGraphEditor>();
	}

	// First, switch back to standard mode
	SetCurrentMode(FBlueprintEditorApplicationModes::StandardBlueprintEditorMode);

	// This will either reuse an existing tab or spawn a new one
	TSharedPtr<SDockTab> TabWithGraph = OpenDocument(Graph, FDocumentTracker::OpenNewDocument);
	if (TabWithGraph.IsValid())
	{

		// We know that the contents of the opened tabs will be a graph editor.
		TSharedRef<SGraphEditor> NewGraphEditor = StaticCastSharedRef<SGraphEditor>(TabWithGraph->GetContent());

		// Handover the keyboard focus to the new graph editor widget.
		if (bSetFocus)
		{
			NewGraphEditor->CaptureKeyboard();
		}

		return NewGraphEditor;
	}
	else
	{
		return TSharedPtr<SGraphEditor>();
	}
}

TSharedPtr<SDockTab> FBlueprintEditor::OpenDocument(const UObject* DocumentID, FDocumentTracker::EOpenDocumentCause Cause)
{
	TSharedRef<FTabPayload_UObject> Payload = FTabPayload_UObject::Make(DocumentID);
	return DocumentManager->OpenDocument(Payload, Cause);
}

void FBlueprintEditor::NavigateTab(FDocumentTracker::EOpenDocumentCause InCause)
{
	OpenDocument(nullptr, InCause);
}

void FBlueprintEditor::CloseDocumentTab(const UObject* DocumentID)
{
	TSharedRef<FTabPayload_UObject> Payload = FTabPayload_UObject::Make(DocumentID);
	DocumentManager->CloseTab(Payload);
}

// Finds any open tabs containing the specified document and adds them to the specified array; returns true if at least one is found
bool FBlueprintEditor::FindOpenTabsContainingDocument(const UObject* DocumentID, /*inout*/ TArray< TSharedPtr<SDockTab> >& Results)
{
	int32 StartingCount = Results.Num();

	TSharedRef<FTabPayload_UObject> Payload = FTabPayload_UObject::Make(DocumentID);

	DocumentManager->FindMatchingTabs( Payload, /*inout*/ Results);

	// Did we add anything new?
	return (StartingCount != Results.Num());
}

void FBlueprintEditor::RestoreEditedObjectState()
{
	check(IsEditingSingleBlueprint());	

	UBlueprint* Blueprint = GetBlueprintObj();
	if (Blueprint->LastEditedDocuments.Num() == 0)
	{
		if (FBlueprintEditorUtils::SupportsConstructionScript(Blueprint))
		{
			Blueprint->LastEditedDocuments.Add(FBlueprintEditorUtils::FindUserConstructionScript(Blueprint));
		}

		if (Blueprint->SupportsEventGraphs())
		{
			Blueprint->LastEditedDocuments.Add(FBlueprintEditorUtils::FindEventGraph(Blueprint));
		}
	}

	TSet<FSoftObjectPath> PathsToRemove;
	for (int32 i = 0; i < Blueprint->LastEditedDocuments.Num(); i++)
	{
		if (UObject* Obj = Blueprint->LastEditedDocuments[i].EditedObjectPath.ResolveObject())
		{
			if(UEdGraph* Graph = Cast<UEdGraph>(Obj))
			{
				if (FBlueprintEditorUtils::IsEventGraph(Graph) && !Blueprint->SupportsEventGraphs())
				{
					continue;
				}

				struct LocalStruct
				{
					static TSharedPtr<SDockTab> OpenGraphTree(FBlueprintEditor* InBlueprintEditor, UEdGraph* InGraph)
					{
						FDocumentTracker::EOpenDocumentCause OpenCause = FDocumentTracker::QuickNavigateCurrentDocument;

						for (UObject* OuterObject = InGraph->GetOuter(); OuterObject; OuterObject = OuterObject->GetOuter())
						{
							if (OuterObject->IsA<UBlueprint>())
							{
								// reached up to the blueprint for the graph, we are done climbing the tree
								OpenCause = FDocumentTracker::RestorePreviousDocument;
								break;
							}
							else if(UEdGraph* OuterGraph = Cast<UEdGraph>(OuterObject))
							{
								// Found another graph, open it up
								OpenGraphTree(InBlueprintEditor, OuterGraph);
								break;
							}
						}

						return InBlueprintEditor->OpenDocument(InGraph, OpenCause);
					}
				};
				TSharedPtr<SDockTab> TabWithGraph = LocalStruct::OpenGraphTree(this, Graph);
				if (TabWithGraph.IsValid())
				{
					TSharedRef<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(TabWithGraph->GetContent());
					GraphEditor->SetViewLocation(UE::Slate::CastToVector2f(Blueprint->LastEditedDocuments[i].SavedViewOffset), Blueprint->LastEditedDocuments[i].SavedZoomAmount);
				}
			}
			else
			{
				TSharedPtr<SDockTab> TabWithGraph = OpenDocument(Obj, FDocumentTracker::RestorePreviousDocument);
			}
		}
		else
		{
			PathsToRemove.Add(Blueprint->LastEditedDocuments[i].EditedObjectPath);
		}
	}

	// Older assets may have neglected to clean up this array when referenced objects were deleted, so
	// we'll check for that now. This is done to ensure we don't store invalid object paths indefinitely.
	if (PathsToRemove.Num() > 0)
	{
		Blueprint->LastEditedDocuments.RemoveAll([&PathsToRemove](const FEditedDocumentInfo& Entry)
		{
			return PathsToRemove.Contains(Entry.EditedObjectPath);
		});
	}
}

bool FBlueprintEditor::CanRecompileModules()
{
	// We're not able to recompile if a compile is already in progress!
	return !IHotReloadModule::Get().IsCurrentlyCompiling();
}

void FBlueprintEditor::OnCreateComment()
{
	TSharedPtr<SGraphEditor> GraphEditor = FocusedGraphEdPtr.Pin();
	if (GraphEditor.IsValid())
	{
		if (UEdGraph* Graph = GraphEditor->GetCurrentGraph())
		{
			if (const UEdGraphSchema* Schema = Graph->GetSchema())
			{
				if (Schema->IsA(UEdGraphSchema_K2::StaticClass()))
				{
					FEdGraphSchemaAction_K2AddComment CommentAction;
					CommentAction.PerformAction(Graph, nullptr, GraphEditor->GetPasteLocation2f());
				}
			}
		}
	}
}

void FBlueprintEditor::OnCreateCustomEvent()
{
	TSharedPtr<SGraphEditor> GraphEditor = FocusedGraphEdPtr.Pin();
	if (GraphEditor.IsValid())
	{
		if (UEdGraph* Graph = GraphEditor->GetCurrentGraph())
		{
			if (const UEdGraphSchema* Schema = Graph->GetSchema())
			{
				if (Schema->IsA(UEdGraphSchema_K2::StaticClass()))
				{
					// BlueprintEventNodeSpawner seems better but we'll use FEdGraphSchemaAction_K2AddCustomEvent
					FEdGraphSchemaAction_K2AddCustomEvent EventAction;
					EventAction.NodeTemplate = NewObject<UK2Node_CustomEvent>();
					EventAction.PerformAction(Graph, nullptr, GraphEditor->GetPasteLocation2f());
				}
			}
		}
	}
}

void FBlueprintEditor::SetPinVisibility(SGraphEditor::EPinVisibility Visibility)
{
	PinVisibility = Visibility;
	OnSetPinVisibility.Broadcast(PinVisibility);
}

void FBlueprintEditor::OnFindReferences(bool bSearchAllBlueprints, const EGetFindReferenceSearchStringFlags Flags)
{
	TSharedPtr<SGraphEditor> GraphEditor = FocusedGraphEdPtr.Pin();
	if (GraphEditor.IsValid())
	{
		const FGraphPanelSelectionSet SelectedNodes = GraphEditor->GetSelectedNodes();
		for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
		{
			if (UEdGraphNode* SelectedNode = Cast<UEdGraphNode>(*NodeIt))
			{
				FString SearchTerm = SelectedNode->GetFindReferenceSearchString(Flags);
				if (!SearchTerm.IsEmpty())
				{
					// Start the search
					const bool bSetFindWithinBlueprint = !bSearchAllBlueprints;
					SummonSearchUI(bSetFindWithinBlueprint, SearchTerm);
				}
			}
		}
	}
}

bool FBlueprintEditor::CanFindReferences()
{
	return GetSingleSelectedNode() != nullptr;
}

AActor* FBlueprintEditor::GetPreviewActor() const
{
	if (!PreviewScene.IsInitialized())
	{
		// On editor shutdown the PreviewScene might uninitialize before the BlueprintEditor.
		// The PreviewScene nulls its world and starts the gc, which would theoretically collect the PreviewActor too.
		// In practice however, the actor might be kept alive through references (e.g. on Linux, see UE-296301).
		// Thus, we test manually for the state of the PreviewScene, to not return an invalid actor.
		if (AActor* Actor = PreviewActorPtr.Get())
		{
			UE_LOGF(LogBlueprint, Warning, "Expected preview actor '%ls' to be garbage collected, but it was not:", *Actor->GetName());
			GEngine->Exec(Actor->GetWorld(), *FString::Printf(TEXT("obj refs name=\"%s\""), *Actor->GetPathName()));
		}
		PreviewActorPtr.Reset();
		return nullptr;
	}

	UBlueprint* PreviewBlueprint = GetBlueprintObj();

	// Note: The weak ptr can become stale if the actor is reinstanced due to a Blueprint change, etc. In that 
	// case we look to see if we can find the new instance in the preview world and then update the weak ptr.
	if ( PreviewActorPtr.IsStale(true) && PreviewBlueprint )
	{
		UWorld* PreviewWorld = PreviewScene.GetWorld();
		for ( TActorIterator<AActor> It(PreviewWorld); It; ++It )
		{
			AActor* Actor = *It;
			if ( !Actor->IsPendingKillPending()
				&& Actor->GetClass()->ClassGeneratedBy == PreviewBlueprint )
			{
				PreviewActorPtr = Actor;
				break;
			}
		}
	}

	return PreviewActorPtr.Get();
}

void FBlueprintEditor::UpdatePreviewActor(UBlueprint* InBlueprint, bool bInForceFullUpdate/* = false*/)
{
	// If the components mode isn't available there's no reason to update the preview actor.
	if ( !CanAccessComponentsMode() )
	{
		return;
	}

	AActor* PreviewActor = GetPreviewActor();

	// Signal that we're going to be constructing editor components
	if ( InBlueprint != nullptr && InBlueprint->SimpleConstructionScript != nullptr )
	{
		InBlueprint->SimpleConstructionScript->BeginEditorComponentConstruction();
	}

	UBlueprint* PreviewBlueprint = GetBlueprintObj();

	// If the Blueprint is changing
	if ( InBlueprint != PreviewBlueprint || bInForceFullUpdate )
	{
		// Destroy the previous actor instance
		DestroyPreview();

		// Save the Blueprint we're creating a preview for
		PreviewBlueprint = InBlueprint;

		// Spawn a new preview actor based on the Blueprint's generated class if it's Actor-based
		if ( PreviewBlueprint && PreviewBlueprint->GeneratedClass && PreviewBlueprint->GeneratedClass->IsChildOf(AActor::StaticClass()) )
		{
			FVector SpawnLocation = FVector::ZeroVector;
			FRotator SpawnRotation = FRotator::ZeroRotator;

			// Spawn an Actor based on the Blueprint's generated class
			FActorSpawnParameters SpawnInfo;
			SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
			SpawnInfo.bNoFail = true;
			SpawnInfo.ObjectFlags = RF_Transient|RF_Transactional;

			{
				FMakeClassSpawnableOnScope TemporarilySpawnable(PreviewBlueprint->GeneratedClass);
				PreviewActorPtr = PreviewActor = PreviewScene.GetWorld()->SpawnActor(PreviewBlueprint->GeneratedClass, &SpawnLocation, &SpawnRotation, SpawnInfo);
			}

			check(PreviewActor);

			// Ensure that the actor is visible
			if ( PreviewActor->IsHidden() )
			{
				PreviewActor->SetHidden(false);
				PreviewActor->MarkComponentsRenderStateDirty();				
			}

			// Prevent any audio from playing as a result of spawning
			if (FAudioDeviceHandle AudioDevice = GEngine->GetMainAudioDevice())
			{
				AudioDevice->Flush(PreviewScene.GetWorld());
			}

			// Set the reference to the preview actor for component editing purposes
			if ( PreviewBlueprint->SimpleConstructionScript != nullptr )
			{
				PreviewBlueprint->SimpleConstructionScript->SetComponentEditorActorInstance(PreviewActor);
			}
		}
	}
	else if ( PreviewActor )
	{
		PreviewActor->ReregisterAllComponents();
		PreviewActor->RerunConstructionScripts();
	}

	// Signal that we're done constructing editor components
	if ( InBlueprint != nullptr && InBlueprint->SimpleConstructionScript != nullptr )
	{
		InBlueprint->SimpleConstructionScript->EndEditorComponentConstruction();
	}
}

void FBlueprintEditor::DestroyPreview()
{
	// If the components mode isn't available there's no reason to delete the preview actor.
	if ( !CanAccessComponentsMode() )
	{
		return;
	}

	AActor* PreviewActor = GetPreviewActor();
	if ( PreviewActor != nullptr )
	{
		check(PreviewScene.GetWorld());
		PreviewScene.GetWorld()->EditorDestroyActor(PreviewActor, false);
	}

	UBlueprint* PreviewBlueprint = GetBlueprintObj();

	if ( PreviewBlueprint != nullptr )
	{
		if ( PreviewBlueprint->SimpleConstructionScript != nullptr
			&& PreviewActor == PreviewBlueprint->SimpleConstructionScript->GetComponentEditorActorInstance() )
		{
			// Ensure that all editable component references are cleared
			PreviewBlueprint->SimpleConstructionScript->ClearEditorComponentReferences();

			// Clear the reference to the preview actor instance
			PreviewBlueprint->SimpleConstructionScript->SetComponentEditorActorInstance(nullptr);
		}

		PreviewBlueprint = nullptr;
	}

	PreviewActorPtr = nullptr;
}


FReply FBlueprintEditor::OnSpawnGraphNodeByShortcut(FInputChord InChord, const FVector2f& InPosition, UEdGraph* InGraph)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return OnSpawnGraphNodeByShortcut(InChord, FVector2D(InPosition), InGraph);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FReply FBlueprintEditor::OnSpawnGraphNodeByShortcut(FInputChord InChord, const FVector2D& InPosition, UEdGraph* InGraph)
{
	UEdGraph* Graph = InGraph;
	if (Graph == nullptr)
	{
		return FReply::Handled();
	}

	FScopedTransaction Transaction(LOCTEXT("AddNode", "Add Node"));

	TArray<UEdGraphNode*> OutNodes;
	FVector2D NodeSpawnPos = InPosition;
	FBlueprintSpawnNodeCommands::Get().GetGraphActionByChord(InChord, InGraph, NodeSpawnPos, OutNodes);

	TSet<const UEdGraphNode*> NodesToSelect;

	for (UEdGraphNode* CurrentNode : OutNodes)
	{
		NodesToSelect.Add(CurrentNode);
	}

	// Do not change node selection if no actions were performed
	if(OutNodes.Num() > 0)
	{
		Graph->SelectNodeSet(NodesToSelect, /*bFromUI =*/true);
	}
	else
	{
		Transaction.Cancel();
	}

	return FReply::Handled();
}

void FBlueprintEditor::OnNodeSpawnedByKeymap()
{
	UpdateNodeCreationStats( ENodeCreateAction::Keymap );
}

void FBlueprintEditor::UpdateNodeCreationStats( const ENodeCreateAction::Type CreateAction )
{
	switch( CreateAction )
	{
	case ENodeCreateAction::MyBlueprintDragPlacement:
		AnalyticsStats.MyBlueprintNodeDragPlacementCount++;
		break;
	case ENodeCreateAction::PaletteDragPlacement:
		AnalyticsStats.PaletteNodeDragPlacementCount++;
		break;
	case ENodeCreateAction::GraphContext:
		AnalyticsStats.NodeGraphContextCreateCount++;
		break;
	case ENodeCreateAction::PinContext:
		AnalyticsStats.NodePinContextCreateCount++;
		break;
	case ENodeCreateAction::Keymap:
		AnalyticsStats.NodeKeymapCreateCount++;
		break;
	}
}

TSharedPtr<ISCSEditorCustomization> FBlueprintEditor::CustomizeSubobjectEditor(const USceneComponent* InComponentToCustomize) const
{
	check(InComponentToCustomize);
	const TSharedPtr<ISCSEditorCustomization>* FoundCustomization = SubobjectEditorCustomizations.Find(InComponentToCustomize->GetClass()->GetFName());
	if(FoundCustomization)
	{
		return *FoundCustomization;
	}

	return TSharedPtr<ISCSEditorCustomization>();
}

FText FBlueprintEditor::GetPIEStatus() const
{
	UBlueprint* CurrentBlueprint = GetBlueprintObj();
	UWorld *DebugWorld = nullptr;
	ENetMode NetMode = NM_Standalone;
	if (CurrentBlueprint)
	{
		DebugWorld = CurrentBlueprint->GetWorldBeingDebugged();
		if (DebugWorld)
		{
			NetMode = DebugWorld->GetNetMode();
		}
		else
		{
			UObject* ObjOuter = CurrentBlueprint->GetObjectBeingDebugged();
			while(DebugWorld == nullptr && ObjOuter != nullptr)
			{
				ObjOuter = ObjOuter->GetOuter();
				DebugWorld = Cast<UWorld>(ObjOuter);
			}

			if (DebugWorld)
			{
				// Redirect through streaming levels to find the owning world; this ensures that we always use the appropriate NetMode for the context string below.
				if (DebugWorld->PersistentLevel != nullptr && DebugWorld->PersistentLevel->OwningWorld != nullptr)
				{
					DebugWorld = DebugWorld->PersistentLevel->OwningWorld;
				}

				NetMode = DebugWorld->GetNetMode();
			}
		}
	}

	if (NetMode == NM_ListenServer || NetMode == NM_DedicatedServer)
	{
		return LOCTEXT("PIEStatusServerSimulating", "SERVER - SIMULATING");
	}
	else if (NetMode == NM_Client)
	{
		FWorldContext* PIEContext = GEngine->GetWorldContextFromWorld(DebugWorld);
		if (PIEContext && PIEContext->PIEInstance > 1)
		{
			return FText::Format(LOCTEXT("PIEStatusClientSimulatingFormat", "CLIENT {0} - SIMULATING"), FText::AsNumber(PIEContext->PIEInstance - 1));
		}
		
		return LOCTEXT("PIEStatusClientSimulating", "CLIENT - SIMULATING");
	}

	return LOCTEXT("PIEStatusSimulating", "SIMULATING");
}

bool FBlueprintEditor::IsEditingAnimGraph() const
{
	if (FocusedGraphEdPtr.IsValid())
	{
		if (UEdGraph* CurrentGraph = FocusedGraphEdPtr.Pin()->GetCurrentGraph())
		{
			if (CurrentGraph->Schema->IsChildOf(UAnimationGraphSchema::StaticClass()) || (CurrentGraph->Schema == UAnimationStateMachineSchema::StaticClass()))
			{
				return true;
			}
		}
	}

	return false;
}

UEdGraph* FBlueprintEditor::GetFocusedGraph() const
{
	if (FocusedGraphEdPtr.IsValid())
	{
		if (UEdGraph* Graph = FocusedGraphEdPtr.Pin()->GetCurrentGraph())
		{
			if (IsValid(Graph))
			{
				return Graph;
			}
		}
	}
	return nullptr;
}

bool FBlueprintEditor::IsEditable(UEdGraph* InGraph) const
{
	return InEditingMode() && !FBlueprintEditorUtils::IsGraphReadOnly(InGraph);
}

bool FBlueprintEditor::IsGraphReadOnly(UEdGraph* InGraph) const
{
	return FBlueprintEditorUtils::IsGraphReadOnly(InGraph);
}

float FBlueprintEditor::GetInstructionTextOpacity(UEdGraph* InGraph) const
{
	UBlueprintEditorSettings const* Settings = GetDefault<UBlueprintEditorSettings>();
	if ((InGraph == nullptr) || !IsEditable(InGraph) || FBlueprintEditorUtils::IsGraphReadOnly(InGraph) || !Settings->bShowGraphInstructionText)
	{
		return 0.0f;
	}
	else if ((InstructionsFadeCountdown > 0.0f) || (HasOpenActionMenu == InGraph))
	{
		return InstructionsFadeCountdown / BlueprintEditorImpl::InstructionFadeDuration;
	}
	else if (BlueprintEditorImpl::GraphHasUserPlacedNodes(InGraph))
	{
		return 0.0f;
	}
	return 1.0f;
}

FText FBlueprintEditor::GetGraphDisplayName(const UEdGraph* Graph)
{
	return FLocalKismetCallbacks::GetGraphDisplayName(Graph);
}


FText FBlueprintEditor::GetGraphDecorationString(UEdGraph* InGraph) const
{
	return FText::GetEmpty();
}

bool FBlueprintEditor::IsGraphInCurrentBlueprint(const UEdGraph* InGraph) const
{
	bool bEditable = true;

	UBlueprint* EditingBP = GetBlueprintObj();
	if(EditingBP)
	{
		TArray<UEdGraph*> Graphs;
		EditingBP->GetAllGraphs(Graphs);
		bEditable &= Graphs.Contains(InGraph);
	}

	return bEditable;
}

bool FBlueprintEditor::IsFocusedGraphEditable() const
{
	UEdGraph* FocusedGraph = GetFocusedGraph();
	if (FocusedGraph != nullptr)
	{
		return IsEditable(FocusedGraph);
	}
	return true;
}

void FBlueprintEditor::TryInvokingDetailsTab(bool bFlash)
{
	if ( TabManager->HasTabSpawner(FBlueprintEditorTabs::DetailsID) )
	{
		TSharedPtr<SDockTab> BlueprintTab = FGlobalTabmanager::Get()->GetMajorTabForTabManager(TabManager.ToSharedRef());

		// We don't want to force this tab into existence when the blueprint editor isn't in the foreground and actively
		// being interacted with.  So we make sure the window it's in is focused and the tab is in the foreground.
		if ( BlueprintTab.IsValid() && BlueprintTab->IsForeground() )
		{
			TSharedPtr<SWindow> ParentWindow = BlueprintTab->GetParentWindow();
			if ( ParentWindow.IsValid() && ParentWindow->HasFocusedDescendants() )
			{
				if ( !Inspector.IsValid() || !Inspector->GetOwnerTab().IsValid() || Inspector->GetOwnerTab()->GetDockArea().IsValid() )
				{
					// Show the details panel if it doesn't exist.
					TabManager->TryInvokeTab(FBlueprintEditorTabs::DetailsID);

					if ( bFlash )
					{
						TSharedPtr<SDockTab> OwnerTab = Inspector->GetOwnerTab();
						if ( OwnerTab.IsValid() )
						{
							OwnerTab->FlashTab();
						}
					}
				}
			}
		}
	}
}

void FBlueprintEditor::SelectGraphActionItemByName(const FName& ItemName, ESelectInfo::Type SelectInfo, int32 SectionId, bool bIsCategory)
{
	if (MyBlueprintWidget.IsValid() && Inspector.IsValid())
	{
		// Select Item in "My Blueprint"
		MyBlueprintWidget->SelectItemByName(ItemName, SelectInfo, SectionId, bIsCategory);

		// Find associated variable
		if (FEdGraphSchemaAction_K2Var* SelectedVar = MyBlueprintWidget->SelectionAsVar())
		{
			if (FProperty* SelectedProperty = SelectedVar->GetProperty())
			{
				// Update Details Panel
				Inspector->ShowDetailsForSingleObject(SelectedProperty->GetUPropertyWrapper());
			}
		}
	}
}

FBPEditorBookmarkNode* FBlueprintEditor::AddBookmark(const FText& DisplayName, const FEditedDocumentInfo& BookmarkInfo, bool bSharedBookmark)
{
	FBPEditorBookmarkNode* NewNode = nullptr;

	if (bSharedBookmark)
	{
		if (UBlueprint* Blueprint = GetBlueprintObj())
		{
			NewNode = new(Blueprint->BookmarkNodes) FBPEditorBookmarkNode;
			NewNode->NodeGuid = FGuid::NewGuid();
			NewNode->DisplayName = DisplayName;

			Blueprint->Modify();
			Blueprint->Bookmarks.Add(NewNode->NodeGuid, BookmarkInfo);
		}
	}
	else if(UBlueprintEditorSettings* LocalSettings = GetMutableDefault<UBlueprintEditorSettings>())
	{
		NewNode = new(LocalSettings->BookmarkNodes) FBPEditorBookmarkNode;
		NewNode->NodeGuid = FGuid::NewGuid();
		NewNode->DisplayName = DisplayName;

		LocalSettings->Bookmarks.Add(NewNode->NodeGuid, BookmarkInfo);
		LocalSettings->SaveConfig();
	}

	if (NewNode && BookmarksWidget.IsValid())
	{
		BookmarksWidget->RefreshBookmarksTree();
	}

	return NewNode;
}

void FBlueprintEditor::RenameBookmark(const FGuid& BookmarkNodeId, const FText& NewName)
{
	bool bFoundSharedBookmark = false;
	if (UBlueprint* Blueprint = GetBlueprintObj())
	{
		for (FBPEditorBookmarkNode& BookmarkNode : Blueprint->BookmarkNodes)
		{
			if (BookmarkNode.NodeGuid == BookmarkNodeId)
			{
				Blueprint->Modify();
				BookmarkNode.DisplayName = NewName;

				bFoundSharedBookmark = true;
				break;
			}
		}
	}

	if (!bFoundSharedBookmark)
	{
		UBlueprintEditorSettings* LocalSettings = GetMutableDefault<UBlueprintEditorSettings>();
		for (FBPEditorBookmarkNode& BookmarkNode : LocalSettings->BookmarkNodes)
		{
			if (BookmarkNode.NodeGuid == BookmarkNodeId)
			{
				BookmarkNode.DisplayName = NewName;
				LocalSettings->SaveConfig();

				break;
			}
		}
	}

	if (BookmarksWidget.IsValid())
	{
		BookmarksWidget->RefreshBookmarksTree();
	}
}

void FBlueprintEditor::RemoveBookmark(const FGuid& BookmarkNodeId, bool bRefreshUI)
{
	bool bFoundSharedBookmark = false;
	if (UBlueprint* Blueprint = GetBlueprintObj())
	{
		for (int32 i = 0; i < Blueprint->BookmarkNodes.Num(); ++i)
		{
			const FBPEditorBookmarkNode& BookmarkNode = Blueprint->BookmarkNodes[i];
			if (BookmarkNode.NodeGuid == BookmarkNodeId)
			{
				Blueprint->Modify();
				Blueprint->BookmarkNodes.RemoveAtSwap(i);
				FEditedDocumentInfo BookmarkInfo = Blueprint->Bookmarks.FindAndRemoveChecked(BookmarkNodeId);

				FGuid CurrentBookmarkId;
				GetViewBookmark(CurrentBookmarkId);
				if (CurrentBookmarkId == BookmarkNodeId)
				{
					SetViewLocation(BookmarkInfo.SavedViewOffset, BookmarkInfo.SavedZoomAmount);
				}

				bFoundSharedBookmark = true;
				break;
			}
		}
	}

	if (!bFoundSharedBookmark)
	{
		UBlueprintEditorSettings* LocalSettings = GetMutableDefault<UBlueprintEditorSettings>();
		for (int32 i = 0; i < LocalSettings->BookmarkNodes.Num(); ++i)
		{
			const FBPEditorBookmarkNode& BookmarkNode = LocalSettings->BookmarkNodes[i];
			if (BookmarkNode.NodeGuid == BookmarkNodeId)
			{
				LocalSettings->BookmarkNodes.RemoveAtSwap(i);
				FEditedDocumentInfo BookmarkInfo = LocalSettings->Bookmarks.FindAndRemoveChecked(BookmarkNodeId);
				LocalSettings->SaveConfig();

				FGuid CurrentBookmarkId;
				GetViewBookmark(CurrentBookmarkId);
				if (CurrentBookmarkId == BookmarkNodeId)
				{
					SetViewLocation(BookmarkInfo.SavedViewOffset, BookmarkInfo.SavedZoomAmount);
				}

				break;
			}
		}
	}

	if (bRefreshUI && BookmarksWidget.IsValid())
	{
		BookmarksWidget->RefreshBookmarksTree();
	}
}

void FBlueprintEditor::SetGraphEditorQuickJump(int32 QuickJumpIndex)
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		if (UEdGraph* GraphObject = FocusedGraphEd->GetCurrentGraph())
		{
			UBlueprintEditorSettings* LocalSettings = GetMutableDefault<UBlueprintEditorSettings>();
			FEditedDocumentInfo& QuickJumpInfo = LocalSettings->GraphEditorQuickJumps.FindOrAdd(QuickJumpIndex);

			QuickJumpInfo.EditedObjectPath = GraphObject;
			FVector2f TempViewOffset = UE::Slate::CastToVector2f(QuickJumpInfo.SavedViewOffset);
			FocusedGraphEd->GetViewLocation(TempViewOffset, QuickJumpInfo.SavedZoomAmount);
			QuickJumpInfo.SavedViewOffset = FDeprecateSlateVector2D(TempViewOffset);

			LocalSettings->SaveConfig();
		}
	}
}

void FBlueprintEditor::ClearGraphEditorQuickJump(int32 QuickJumpIndex)
{
	UBlueprintEditorSettings* LocalSettings = GetMutableDefault<UBlueprintEditorSettings>();
	LocalSettings->GraphEditorQuickJumps.Remove(QuickJumpIndex);
	LocalSettings->SaveConfig();
}

void FBlueprintEditor::OnGraphEditorQuickJump(int32 QuickJumpIndex)
{
	const UBlueprintEditorSettings* LocalSettings = GetDefault<UBlueprintEditorSettings>();
	if (const FEditedDocumentInfo* QuickJumpInfo = LocalSettings->GraphEditorQuickJumps.Find(QuickJumpIndex))
	{
		if (UObject* EditedObject = QuickJumpInfo->EditedObjectPath.TryLoad())
		{
			TSharedPtr<IBlueprintEditor> IBlueprintEditorPtr = FKismetEditorUtilities::GetIBlueprintEditorForObject(EditedObject, true);
			if (IBlueprintEditorPtr.IsValid())
			{
				IBlueprintEditorPtr->FocusWindow();
				TSharedPtr<SGraphEditor> GraphEditorPtr = IBlueprintEditorPtr->OpenGraphAndBringToFront(Cast<UEdGraph>(EditedObject));
				if (GraphEditorPtr.IsValid())
				{
					GraphEditorPtr->SetViewLocation(UE::Slate::CastToVector2f(QuickJumpInfo->SavedViewOffset), QuickJumpInfo->SavedZoomAmount);
				}
			}
		}
	}
}

void FBlueprintEditor::ClearAllGraphEditorQuickJumps()
{
	UBlueprintEditorSettings* LocalSettings = GetMutableDefault<UBlueprintEditorSettings>();
	LocalSettings->GraphEditorQuickJumps.Empty();
	LocalSettings->SaveConfig();
}

bool FBlueprintEditor::IsNonImportedObject(const UObject* InObject) const
{
	if (ImportedNamespaceHelper.IsValid() && !ImportedNamespaceHelper->IsImportedObject(InObject))
	{
		return true;
	}

	return false;
}

bool FBlueprintEditor::IsNonImportedObject(const FSoftObjectPath& InObject) const
{
	if (ImportedNamespaceHelper.IsValid() && !ImportedNamespaceHelper->IsImportedObject(InObject))
	{
		return true;
	}

	return false;
}

void FBlueprintEditor::OnBlueprintProjectSettingsChanged(UObject*, struct FPropertyChangedEvent&)
{
	ModifyDuringPIEStatus = ESafeToModifyDuringPIEStatus::Unknown;
}

void FBlueprintEditor::OnBlueprintEditorPreferencesChanged(UObject*, struct FPropertyChangedEvent&)
{
	ModifyDuringPIEStatus = ESafeToModifyDuringPIEStatus::Unknown;
}

bool FBlueprintEditor::AreEventGraphsAllowed() const
{
	return true;
}

bool FBlueprintEditor::AreMacrosAllowed() const
{
	return true;
}

bool FBlueprintEditor::AreDelegatesAllowed() const
{
	return true;
}

FEditorViewportSelectabilityBridge* FBlueprintEditor::GetViewportSelectabilityBridge()
{
	return ViewportSelectabilityBridge.Get();
}

/////////////////////////////////////////////////////
/////////////////////////////////////////////////////
/////////////////////////////////////////////////////

UBlueprint* UBlueprintEditorToolMenuContext::GetBlueprintObj() const
{
	return BlueprintEditor.IsValid() ? BlueprintEditor.Pin()->GetBlueprintObj() : nullptr;
}

#undef LOCTEXT_NAMESPACE
