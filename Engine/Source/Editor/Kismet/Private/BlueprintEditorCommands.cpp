// Copyright Epic Games, Inc. All Rights Reserved.


#include "BlueprintEditorCommands.h"

#include "BlueprintEventNodeSpawner.h"
#include "BlueprintFunctionNodeSpawner.h"
#include "BlueprintNodeBinder.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Blueprint.h"
#include "Engine/LevelScriptActor.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "GenericPlatform/GenericApplication.h"
#include "HAL/PlatformCrt.h"
#include "InputCoreTypes.h"
#include "Internationalization/Text.h"
#include "K2Node_Literal.h"
#include "K2Node_MacroInstance.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Logging/LogVerbosity.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/PackageName.h"
#include "Misc/Parse.h"
#include "Selection.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Textures/SlateIcon.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"

#define LOCTEXT_NAMESPACE "BlueprintEditorCommands"

/** UI_COMMAND takes long for the compile to optimize */
UE_DISABLE_OPTIMIZATION_SHIP
void FBlueprintEditorCommands::RegisterCommands()
{
	// Edit commands
	UI_COMMAND( FindInBlueprint, "Find", "Finds references to functions, events, variables, and pins in the current Blueprint (use Ctrl+Shift+F to search in all Blueprints)", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::F) );
	UI_COMMAND( FindInBlueprints, "Find in Blueprints", "Find references to functions, events and variables in ALL Blueprints", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::F) );
	UI_COMMAND( ReparentBlueprint, "Reparent Blueprint", "Change the parent of this Blueprint", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND( CompileBlueprint, "Compile", "Compile the blueprint", EUserInterfaceActionType::Button, FInputChord(EKeys::F7) );
	UI_COMMAND( RefreshAllNodes, "Refresh All nodes", "Refreshes all nodes in the graph to account for external changes", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( DeleteUnusedVariables, "Delete Unused Variables", "Deletes any variables that are never used", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND( FindReferencesFromClass, "List references (from class)", "Find all objects that the class references", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( FindReferencesFromBlueprint, "List referenced (from blueprint)", "Find all objects that the blueprint references", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( RepairCorruptedBlueprint, "Repair corrupted blueprint", "Attempts to repair a corrupted blueprint that cannot be saved", EUserInterfaceActionType::Button, FInputChord() );

	// View commands 
	UI_COMMAND( ZoomToWindow, "Zoom to Graph Extents", "Fit the current view to the entire graph", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ZoomToSelection, "Zoom to Selection", "Fit the current view to the selection", EUserInterfaceActionType::Button, FInputChord(EKeys::Home) );
	UI_COMMAND( NavigateToParent, "Go to parent graph", "Open the parent graph", EUserInterfaceActionType::Button, FInputChord(EKeys::PageUp) );
	UI_COMMAND( NavigateToParentBackspace, "Go to parent graph", "Open the parent graph", EUserInterfaceActionType::Button, FInputChord(EKeys::BackSpace) );
	UI_COMMAND( NavigateToChild, "Go to child graph", "Open the child graph", EUserInterfaceActionType::Button, FInputChord(EKeys::PageDown) );

	// Preview commands
	UI_COMMAND( ResetCamera, "Reset Camera", "Resets the camera to focus on the mesh", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( EnableSimulation, "Simulation", "Enables simulation of components in the viewport", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ShowFloor, "Show Floor", "Toggles a ground mesh for collision", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ShowGrid, "Show Grid", "Toggles viewport grid", EUserInterfaceActionType::ToggleButton, FInputChord() );

	// Debugging commands
	UI_COMMAND( EnableAllBreakpoints,"Enable All Breakpoints", "Enable all breakpoints", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( DisableAllBreakpoints, "Disable All Breakpoints", "Disable all breakpoints", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ClearAllBreakpoints, "Delete All Breakpoints", "Delete all breakpoints", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::F9) );
	UI_COMMAND(	ClearAllWatches, "Delete All Watches", "Delete all watches", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( OpenBlueprintDebugger, "Blueprint Debugger", "Open the Blueprint Debugger tab.", EUserInterfaceActionType::Button, FInputChord() );

	// New documents
	UI_COMMAND( AddNewVariable, "Variable", "Adds a new variable to this blueprint.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( AddNewLocalVariable, "Local Variable", "Adds a new local variable to this graph.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( AddNewFunction, "Function", "Add a new function graph", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND(	AddNewMacroDeclaration, "Macro", "Add a new macro declaration graph", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( AddNewAnimationLayer, "Animation Layer", "Add a new animation layer", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( AddNewEventGraph, "Graph", "Add a new event graph", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( AddNewDelegate, "Event Dispatcher", "Add a new event dispatcher", EUserInterfaceActionType::Button, FInputChord() );

	// Development commands
	UI_COMMAND( SaveIntermediateBuildProducts, "Save Intermediate Build Products", "Should the compiler save intermediate build products for debugging.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND(GenerateSearchIndex, "Generate Search Index", "Generate the search index for this blueprint.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DumpCachedIndexData, "Dump Cached Index Data", "Dump the cached index data for this blueprint.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ShowActionMenuItemSignatures, "Show Action Menu Item Signatures", "If enabled, tooltips on action menu items will show the associated action's signature id (can be used to setup custom favorites menus).", EUserInterfaceActionType::ToggleButton, FInputChord());
	
	// SCC commands
	UI_COMMAND( BeginBlueprintMerge, "Merge", "Shows the Blueprint merge panel and toolbar, allowing the user to resolve conflicted blueprints", EUserInterfaceActionType::Button, FInputChord() );
	
	// Hide unrelated nodes
	UI_COMMAND( ToggleHideUnrelatedNodes, "Hide Unrelated", "Toggles automatically hiding nodes which are unrelated to the selected nodes.", EUserInterfaceActionType::ToggleButton, FInputChord() );
}

UE_ENABLE_OPTIMIZATION_SHIP

namespace NodeSpawnInfoHelpers
{
	/**
	 * Checks if the passed in function is available as an event for the Blueprint
	 *
	 * @param InBlueprint		The Blueprint to check for validity with
	 * @param InFunction		The Function to check for being available as an event
	 *
	 * @return					Returns true if the function is available as an event in the Blueprint
	 */
	bool IsFunctionAvailableAsEvent(const UBlueprint* InBlueprint, const UFunction* InFunction)
	{
		// Build a list of all interface classes either implemented by this blueprint or through inheritance
		TArray<UClass*> InterfaceClasses;
		FBlueprintEditorUtils::FindImplementedInterfaces(InBlueprint, true, InterfaceClasses);
		InterfaceClasses.Add(InBlueprint->ParentClass);

		// Grab the list of events to be excluded from the override list
		const FString ExclusionListKeyName = TEXT("KismetHideOverrides");
		TArray<FString> ExcludedEventNames;
		if( InBlueprint->ParentClass->HasMetaData(*ExclusionListKeyName) )
		{
			const FString& ExcludedEventNameString = InBlueprint->ParentClass->GetMetaData(*ExclusionListKeyName);
			ExcludedEventNameString.ParseIntoArray(ExcludedEventNames, TEXT(","), true);
		}

		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		if (K2Schema->FunctionCanBePlacedAsEvent(InFunction) && !ExcludedEventNames.Contains(InFunction->GetName()))
		{
			// Check all potential interface events using the class list we build above
			for(auto It = InterfaceClasses.CreateConstIterator(); It; It++)
			{
				const UClass* Interface = (*It);
				for (TFieldIterator<UFunction> FunctionIt(Interface, EFieldIteratorFlags::IncludeSuper); FunctionIt; ++FunctionIt)
				{
					UFunction* Function = *FunctionIt;

					if(Function->GetName() == InFunction->GetName())
					{
						return true;
					}
				}
			}
		}

		return false;
	}

	/**
	 * Finds the event version of a UFunction for a given Blueprint
	 *
	 * @param InBlueprint			The Blueprint to find the function within
	 * @param InFunction			The Function to find an event version of
	 *
	 * @return						Event Function used by the given Blueprint for the given Function
	 */
	UFunction* FindEventFunctionForClass(const UBlueprint* InBlueprint, const UFunction* InFunction)
	{
		// Look at all of the Blueprint parent's functions for an event
		for (TFieldIterator<UFunction> FunctionIt(InBlueprint->ParentClass, EFieldIteratorFlags::IncludeSuper); FunctionIt; ++FunctionIt)
		{
			UFunction* Function = *FunctionIt;
			if(Function->GetName() == InFunction->GetName())
			{
				return Function;
			}
		}

		return NULL;
	}
}

class FNodeSpawnInfo
{
public:
	virtual ~FNodeSpawnInfo () {}

	/** Holds the UI Command to verify chords for this action are held */
	TSharedPtr< FUICommandInfo > CommandInfo;

	/**
	 * Creates an action to be used for placing a node into the graph
	 *
	 * @param	InDestGraph					The graph the action should be created for
	 * @param	InOutDestPosition			Position to start placing nodes, will be updated to be at the next safe position for node placement
	 * @param	OutNodes					All nodes spawned by this operation
	 *
	 * @return								A fully prepared action containing the information to spawn the node
	 */
	virtual void GetActions(UEdGraph* InDestGraph, FVector2D& InOutDestPosition, TArray<UEdGraphNode*>& OutNodes) = 0;
};

class FEdGraphNodeSpawnInfo : public FNodeSpawnInfo
{
public:
	FEdGraphNodeSpawnInfo(UClass* InClass) : NodeClass(InClass) {}

	// FNodeSpawnInfo interface
	virtual void GetActions(UEdGraph* InDestGraph, FVector2D& InOutDestPosition, TArray<UEdGraphNode*>& OutNodes) override
	{
		TSharedPtr<FEdGraphSchemaAction_NewNode> NewActionNode = TSharedPtr<FEdGraphSchemaAction_NewNode>(new FEdGraphSchemaAction_NewNode);

		IBlueprintNodeBinder::FBindingSet Bindings;
		UEdGraphNode* GraphNode = UBlueprintNodeSpawner::Create(NodeClass)->Invoke(InDestGraph, Bindings, InOutDestPosition);

		InOutDestPosition.Y = FMath::Max(InOutDestPosition.Y, GraphNode->NodePosY + UEdGraphSchema_K2::EstimateNodeHeight(GraphNode));
		OutNodes.Add(GraphNode);
	}
	// End of FNodeSpawnInfo interface

private:
	/** The class type the node should be */
	UClass* NodeClass;
};

class FFunctionNodeSpawnInfo : public FNodeSpawnInfo
{
public:
	FFunctionNodeSpawnInfo(UFunction* InFunctionPtr) : FunctionPtr(InFunctionPtr) {}

	// FNodeSpawnInfo interface
	virtual void GetActions(UEdGraph* InDestGraph, FVector2D& InOutDestPosition, TArray<UEdGraphNode*>& OutNodes) override
	{
		const UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(InDestGraph);

		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		if(NodeSpawnInfoHelpers::IsFunctionAvailableAsEvent(Blueprint, FunctionPtr))
		{
			UFunction* FunctionEvent = NodeSpawnInfoHelpers::FindEventFunctionForClass(Blueprint, FunctionPtr);
			if(FunctionEvent)
			{
				IBlueprintNodeBinder::FBindingSet Bindings;
				UEdGraphNode* GraphNode = UBlueprintEventNodeSpawner::Create(FunctionEvent)->Invoke(InDestGraph, Bindings, InOutDestPosition);

				InOutDestPosition.Y = FMath::Max(InOutDestPosition.Y, GraphNode->NodePosY + UEdGraphSchema_K2::EstimateNodeHeight(GraphNode));
				OutNodes.Add(GraphNode);
			}
		}
		else
		{
			const bool bAllowImpureFuncs = K2Schema->DoesGraphSupportImpureFunctions(InDestGraph);

			uint32 FunctionTypes = UEdGraphSchema_K2::EFunctionType::FT_Pure | UEdGraphSchema_K2::EFunctionType::FT_Const | UEdGraphSchema_K2::EFunctionType::FT_Protected;
			if(bAllowImpureFuncs)
			{
				FunctionTypes |= UEdGraphSchema_K2::EFunctionType::FT_Imperative;
			}

			if(K2Schema->CanFunctionBeUsedInGraph(Blueprint->GeneratedClass, FunctionPtr, InDestGraph, FunctionTypes, false))
			{
				IBlueprintNodeBinder::FBindingSet Bindings;
				UEdGraphNode* GraphNode = UBlueprintFunctionNodeSpawner::Create(FunctionPtr)->Invoke(InDestGraph, Bindings, InOutDestPosition);

				InOutDestPosition.Y = FMath::Max(InOutDestPosition.Y, GraphNode->NodePosY + UEdGraphSchema_K2::EstimateNodeHeight(GraphNode));
				OutNodes.Add(GraphNode);
			}
		}
	}
	// End of FNodeSpawnInfo interface

private:
	/** The function used to create the action to spawn the graph node */
	UFunction* FunctionPtr;
};

class FMacroNodeSpawnInfo : public FNodeSpawnInfo
{
public:
	FMacroNodeSpawnInfo(UEdGraph* InMacroGraph) : MacroGraph(InMacroGraph) {}

	// FNodeSpawnInfo interface
	virtual void GetActions(UEdGraph* InDestGraph, FVector2D& InOutDestPosition, TArray<UEdGraphNode*>& OutNodes) override
	{
		IBlueprintNodeBinder::FBindingSet Bindings;
		UK2Node_MacroInstance* MacroInstanceNode = Cast<UK2Node_MacroInstance>(UBlueprintNodeSpawner::Create(UK2Node_MacroInstance::StaticClass())->Invoke(InDestGraph, Bindings, InOutDestPosition));
		MacroInstanceNode->SetMacroGraph(MacroGraph);
		MacroInstanceNode->ReconstructNode();
		OutNodes.Add(MacroInstanceNode);
	}
	// End of FNodeSpawnInfo interface

private:
	/** The macro graph used to create the action to spawn the graph node */
	UEdGraph* MacroGraph;
};

class FActorRefSpawnInfo : public FNodeSpawnInfo
{
public:
	// FNodeSpawnInfo interface
	virtual void GetActions(UEdGraph* InDestGraph, FVector2D& InOutDestPosition, TArray<UEdGraphNode*>& OutNodes) override
	{
		const UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(InDestGraph);
		USelection* SelectedLvlActors = GEditor->GetSelectedActors();

		if ((Blueprint != nullptr) && Blueprint->ParentClass->IsChildOf<ALevelScriptActor>() && (SelectedLvlActors->Num() > 0))
		{
			for (FSelectionIterator LvlActorIt(*SelectedLvlActors); LvlActorIt; ++LvlActorIt)
			{
				IBlueprintNodeBinder::FBindingSet Bindings;
				UK2Node_Literal* TemplateRefNode = Cast<UK2Node_Literal>(UBlueprintNodeSpawner::Create(UK2Node_Literal::StaticClass())->Invoke(InDestGraph, Bindings, InOutDestPosition));
				TemplateRefNode->SetObjectRef(*LvlActorIt);

				OutNodes.Add(TemplateRefNode);
			}
		}
	}
	// End of FNodeSpawnInfo interface
};

//////////////////////////////////////////////////////////////////////////
// FBlueprintSpawnNodeCommands

void FBlueprintSpawnNodeCommands::RegisterCommands()
{
	const FString ConfigSection = TEXT("BlueprintSpawnNodes");
	const FString SettingName = TEXT("Node");
	TArray< FString > NodeSpawns;
	GConfig->GetArray(*ConfigSection, *SettingName, NodeSpawns, GEditorPerProjectIni);

	for(int32 x = 0; x < NodeSpawns.Num(); ++x)
	{
		FString ClassName;
		if(!FParse::Value(*NodeSpawns[x], TEXT("Class="), ClassName))
		{
			// Could not find a class name, cannot continue with this line
			continue;
		}

		UE_CLOGF(FPackageName::IsShortPackageName(ClassName), LogClass, Warning, "Short class name found when parsing ini: [%ls] %ls", *ConfigSection, *SettingName);

		FString CommandLabel;
		UClass* FoundClass = FindFirstObject<UClass>(*ClassName, EFindFirstObjectOptions::ExactClass, ELogVerbosity::Warning, TEXT("looking for SpawnNodes"));
		TSharedPtr< FNodeSpawnInfo > InfoPtr;

		if(FoundClass && FoundClass->IsChildOf(UEdGraphNode::StaticClass()))
		{
			// The class name matches that of a UEdGraphNode, so setup a spawn info that can generate UEdGraphNode graph actions
			UEdGraphNode* GraphNode = Cast<UEdGraphNode>(FoundClass->GetDefaultObject());

			CommandLabel = GraphNode->GetNodeTitle(ENodeTitleType::ListView).ToString();

			if(CommandLabel.Len() == 0)
			{
				CommandLabel = FoundClass->GetName();
			}

			InfoPtr = MakeShareable( new FEdGraphNodeSpawnInfo( FoundClass ) );
		}
		else if(UFunction* FoundFunction = FindFirstObject<UFunction>(*ClassName, EFindFirstObjectOptions::ExactClass, ELogVerbosity::Warning, TEXT("looking for SpawnNodes (function)")))
		{
			// The class name matches that of a function, so setup a spawn info that can generate function graph actions
			InfoPtr = MakeShareable( new FFunctionNodeSpawnInfo((UFunction*)FoundFunction));

			CommandLabel = FoundFunction->GetName();
		}
		else if(UEdGraph* FoundGraph = FindFirstObject<UEdGraph>(*ClassName, EFindFirstObjectOptions::ExactClass, ELogVerbosity::Warning, TEXT("looking for SpawnNodes (macro)")))
		{
			// If the name matches that of a macro graph, set up a spawn info that can generate macro graph actions
			if (const UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(FoundGraph))
			{
				if (Blueprint->BlueprintType == BPTYPE_MacroLibrary)
				{
					CommandLabel = FoundGraph->GetName();

					InfoPtr = MakeShareable(new FMacroNodeSpawnInfo(FoundGraph));
				}
			}
		}

		// If spawn info was created, setup a UI Command for keybinding.
		if(InfoPtr.IsValid())
		{
			TSharedPtr< FUICommandInfo > CommandInfo;

			FKey Key;
			bool bShift = false;
			bool bCtrl = false;
			bool bAlt = false;
			bool bCmd = false;
			
			// Parse the keybinding information
			FString KeyString;
			if( FParse::Value(*NodeSpawns[x], TEXT("Key="), KeyString) )
			{
				Key = *KeyString;
			}

			if( Key.IsValid() )
			{
				FParse::Bool(*NodeSpawns[x], TEXT("Shift="), bShift);
				FParse::Bool(*NodeSpawns[x], TEXT("Alt="), bAlt);
				FParse::Bool(*NodeSpawns[x], TEXT("Ctrl="), bCtrl);
				FParse::Bool(*NodeSpawns[x], TEXT("Cmd="), bCmd);
			}

			FInputChord Chord(Key, EModifierKey::FromBools(bCtrl, bAlt, bShift, bCmd));

			FText CommandLabelText = FText::FromString( CommandLabel );
			FText Description = FText::Format( NSLOCTEXT("BlueprintEditor", "NodeSpawnDescription", "Hold down the bound keys and left click in the graph panel to spawn a {0} node."), CommandLabelText );

			FUICommandInfo::MakeCommandInfo( this->AsShared(), CommandInfo, FName(*NodeSpawns[x]), CommandLabelText, Description, FSlateIcon(FAppStyle::GetAppStyleSetName(), *FString::Printf(TEXT("%s.%s"), *this->GetContextName().ToString(), *NodeSpawns[x])), EUserInterfaceActionType::Button, Chord );

			InfoPtr->CommandInfo = CommandInfo;

			NodeCommands.Add(InfoPtr);
		}
	}

	TSharedPtr<FNodeSpawnInfo> AddActorRefAction = MakeShareable(new FActorRefSpawnInfo);
	UI_COMMAND(AddActorRefAction->CommandInfo, "Add Selected Actor Reference(s)", "Spawns node(s) which reference the currently selected actor(s) in the level.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::R));
	NodeCommands.Add(AddActorRefAction);
}


void FBlueprintSpawnNodeCommands::GetGraphActionByChord(FInputChord& InChord, UEdGraph* InDestGraph, FVector2D& InOutDestPosition, TArray<UEdGraphNode*>& OutNodes) const
{
	if(InChord.IsValidChord())
	{
		for(int32 x = 0; x < NodeCommands.Num(); ++x)
		{
			if(NodeCommands[x]->CommandInfo->HasActiveChord(InChord))
			{
				NodeCommands[x]->GetActions(InDestGraph, InOutDestPosition, OutNodes);
			}
		}
	}
}

//////////////////////////////////////////////////////////////////////////
// FSCSEditorViewportCommands

void FSCSEditorViewportCommands::RegisterCommands()
{
	UI_COMMAND(DeleteComponent, "Delete", "Delete current selection", EUserInterfaceActionType::Button, FInputChord(EKeys::Platform_Delete));
}

#undef LOCTEXT_NAMESPACE
