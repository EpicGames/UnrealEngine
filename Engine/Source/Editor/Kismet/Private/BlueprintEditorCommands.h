// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Framework/Commands/Commands.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Math/Vector2D.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;
class UEdGraph;
class UEdGraphNode;
struct FInputChord;

//////////////////////////////////////////////////////////////////////////
// FBlueprintEditorCommands

/** Set of kismet 2 wide commands */
class FBlueprintEditorCommands : public TCommands<FBlueprintEditorCommands>
{
public:

	FBlueprintEditorCommands()
		: TCommands<FBlueprintEditorCommands>( TEXT("BlueprintEditor"), NSLOCTEXT("Contexts", "BlueprintEditor", "Blueprint Editor"), NAME_None, FAppStyle::GetAppStyleSetName() )
	{
	}	

	virtual void RegisterCommands() override;


	// File-ish commands
	TSharedPtr< FUICommandInfo > CompileBlueprint;
	TSharedPtr< FUICommandInfo > RefreshAllNodes;
	TSharedPtr< FUICommandInfo > DeleteUnusedVariables;
	TSharedPtr< FUICommandInfo > FindInBlueprints;

	TSharedPtr< FUICommandInfo > FindReferencesFromClass;
	TSharedPtr< FUICommandInfo > FindReferencesFromBlueprint;
	TSharedPtr< FUICommandInfo > RepairCorruptedBlueprint;

	// Edit commands
	TSharedPtr< FUICommandInfo > FindInBlueprint;
	TSharedPtr< FUICommandInfo > ReparentBlueprint;

	// View commands
	TSharedPtr< FUICommandInfo > ZoomToWindow;
	TSharedPtr< FUICommandInfo > ZoomToSelection;
	TSharedPtr< FUICommandInfo > NavigateToParent;
	TSharedPtr< FUICommandInfo > NavigateToParentBackspace;
	TSharedPtr< FUICommandInfo > NavigateToChild;

	// Preview commands
	TSharedPtr< FUICommandInfo > ResetCamera;
	TSharedPtr< FUICommandInfo > EnableSimulation;
	TSharedPtr< FUICommandInfo > ShowFloor;
	TSharedPtr< FUICommandInfo > ShowGrid;

	// Debugging commands
	TSharedPtr< FUICommandInfo > EnableAllBreakpoints;
	TSharedPtr< FUICommandInfo > DisableAllBreakpoints;
	TSharedPtr< FUICommandInfo > ClearAllBreakpoints;
	TSharedPtr< FUICommandInfo > ClearAllWatches;
	TSharedPtr< FUICommandInfo > OpenBlueprintDebugger;
	
	// New documents
	TSharedPtr< FUICommandInfo > AddNewVariable;
	TSharedPtr< FUICommandInfo > AddNewLocalVariable;
	TSharedPtr< FUICommandInfo > AddNewFunction;
	TSharedPtr< FUICommandInfo > AddNewMacroDeclaration;
	TSharedPtr< FUICommandInfo > AddNewAnimationLayer;
	TSharedPtr< FUICommandInfo > AddNewEventGraph;
	TSharedPtr< FUICommandInfo > AddNewDelegate;

	// Development commands
	TSharedPtr< FUICommandInfo > SaveIntermediateBuildProducts;
	TSharedPtr< FUICommandInfo > GenerateSearchIndex;
	TSharedPtr< FUICommandInfo > DumpCachedIndexData;
	TSharedPtr< FUICommandInfo > ShowActionMenuItemSignatures;

	// SSC commands
	TSharedPtr< FUICommandInfo > BeginBlueprintMerge;
	
	// Toggle focusing nodes which are related to the selected nodes
	TSharedPtr< FUICommandInfo > ToggleHideUnrelatedNodes;
};

//////////////////////////////////////////////////////////////////////////
// FBlueprintSpawnNodeCommands

/** Handles spawn node commands for the Blueprint Editor */
class FBlueprintSpawnNodeCommands : public TCommands<FBlueprintSpawnNodeCommands>
{
public:

	FBlueprintSpawnNodeCommands()
		: TCommands<FBlueprintSpawnNodeCommands>(TEXT("BlueprintEditorSpawnNodes"), NSLOCTEXT("Contexts", "BlueprintEditor_SpawnNodes", "Blueprint Editor - Spawn Nodes by chord"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}	

	virtual void RegisterCommands() override;

	/**
	 * Returns a graph action assigned to the passed in chord
	 *
	 * @param InChord					The chord to use for lookup
	 * @param InDestGraph				The graph to create the graph action for, used for validation purposes and to link any important node data to the graph
	 * @param InOutDestPosition			Position to start placing nodes, will be updated to be at the next safe position for node placement
	 * @param OutNodes					All nodes spawned by this operation
	 */
	void GetGraphActionByChord(FInputChord& InChord, UEdGraph* InDestGraph, FVector2D& InOutDestPosition, TArray<UEdGraphNode*>& OutNodes) const;

private:
	/** An array of all the possible commands for spawning nodes */
	TArray< TSharedPtr< class FNodeSpawnInfo > > NodeCommands;
};

//////////////////////////////////////////////////////////////////////////
// FSCSEditorViewportCommands

class FSCSEditorViewportCommands : public TCommands<FSCSEditorViewportCommands>
{
public:

	FSCSEditorViewportCommands()
		: TCommands<FSCSEditorViewportCommands>(TEXT("SCSEditorViewport"), NSLOCTEXT("Contexts", "SCSEditorViewport", "SCS Editor Viewport"), NAME_None, FAppStyle::GetAppStyleSetName())
	{}

	virtual void RegisterCommands() override;

	TSharedPtr< FUICommandInfo > DeleteComponent;
};
