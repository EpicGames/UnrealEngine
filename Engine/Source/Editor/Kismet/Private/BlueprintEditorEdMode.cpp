// Copyright Epic Games, Inc. All Rights Reserved.


#include "BlueprintEditorEdMode.h"

#include "UnrealEdGlobals.h"
#include "Editor/UnrealEdEngine.h"

UBlueprintEditorEdMode::UBlueprintEditorEdMode()
{
	Info = FEditorModeInfo(
		Id,
		NSLOCTEXT("BlueprintEditorEdMode", "DisplayName", "AssetSelection"),
		FSlateIcon("EditorStyle", "LevelEditor.SelectMode", "LevelEditor.SelectMode.Small"),
		false, // Visibility
		0 // Priority
	);
}

bool UBlueprintEditorEdMode::ShouldDrawWidget() const
{
	if (Super::ShouldDrawWidget())
	{
		return true;
	}
	
	if (GUnrealEd && GUnrealEd->ComponentVisManager.IsActive() && GUnrealEd->ComponentVisManager.IsVisualizingArchetype())
	{
		return true;
	}
	
	return false;
}

EEditAction::Type UBlueprintEditorEdMode::GetActionDragDuplicate()
{
	const EEditAction::Type BaseAction = Super::GetActionDragDuplicate();
	
	if (BaseAction == EEditAction::Skip && GUnrealEd && GUnrealEd->ComponentVisManager.IsActive() && GUnrealEd->ComponentVisManager.IsVisualizingArchetype())
	{
		// A drag action should start, and the component vis manager will perform it
		return EEditAction::Process;
	}
	
	return BaseAction;
}
