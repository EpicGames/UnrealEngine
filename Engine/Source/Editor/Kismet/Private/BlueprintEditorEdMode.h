// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Tools/DefaultEdMode.h"
#include "BlueprintEditorEdMode.generated.h"

UCLASS()
class UBlueprintEditorEdMode : public UAssetEdModeDefault
{
	GENERATED_BODY()
	
public:
	static inline const FEditorModeID Id = TEXT("BlueprintEditorEdMode");
	
	UBlueprintEditorEdMode();
	
	virtual bool ShouldDrawWidget() const override;
	virtual EEditAction::Type GetActionDragDuplicate() override;
};
