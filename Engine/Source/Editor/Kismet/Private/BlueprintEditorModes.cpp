// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintEditorModes.h"

#include "BlueprintEditor.h"
#include "BlueprintEditorModule.h"
#include "Components/ActorComponent.h"
#include "Framework/Docking/TabManager.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Modules/ModuleManager.h"
#include "SKismetInspector.h"
#include "SMyBlueprint.h"
// Core kismet tabs
#include "SSCSEditorViewport.h"
#include "Types/SlateEnums.h"
// End of core kismet tabs

// Debugging
// End of debugging

#include "BlueprintEditorSharedTabFactories.h"
#include "BlueprintEditorTabFactories.h"
#include "BlueprintEditorTabs.h"
#include "Framework/Docking/LayoutExtender.h"
#include "SBlueprintEditorToolbar.h"
#include "SSubobjectEditor.h"

class FSCSEditorTreeNode;
class UToolMenu;

#define LOCTEXT_NAMESPACE "BlueprintEditor"


const FName FBlueprintEditorApplicationModes::StandardBlueprintEditorMode( TEXT("GraphName") );
const FName FBlueprintEditorApplicationModes::BlueprintDefaultsMode( TEXT("DefaultsName") );
const FName FBlueprintEditorApplicationModes::BlueprintComponentsMode( TEXT("ComponentsName") );
const FName FBlueprintEditorApplicationModes::BlueprintInterfaceMode( TEXT("InterfaceName") );
const FName FBlueprintEditorApplicationModes::BlueprintMacroMode( TEXT("MacroName") );

TSharedPtr<FTabManager::FLayout> GetDefaltEditorLayout(TSharedPtr<class FBlueprintEditor> InBlueprintEditor)
{
	return FTabManager::NewLayout( "Standalone_BlueprintEditor_Layout_v7" )
	->AddArea
	(
		FTabManager::NewPrimaryArea() ->SetOrientation(Orient_Vertical)
		->Split
		(
			FTabManager::NewSplitter() ->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewSplitter() ->SetOrientation( Orient_Vertical )
				->SetSizeCoefficient(0.70f)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient( 0.80f )
					->AddTab( "Document", ETabState::ClosedTab )
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient( 0.20f )
					->AddTab( FBlueprintEditorTabs::CompilerResultsID, ETabState::ClosedTab )
					->AddTab( FBlueprintEditorTabs::FindResultsID, ETabState::ClosedTab )
					->AddTab( FBlueprintEditorTabs::BookmarksID, ETabState::ClosedTab )
				)
			)
			->Split
			(
				FTabManager::NewSplitter() ->SetOrientation( Orient_Vertical )
				->SetSizeCoefficient(0.15f)
				->Split
				(
					FTabManager::NewStack() ->SetSizeCoefficient(0.5f)
					->AddTab( FBlueprintEditorTabs::MyBlueprintID, ETabState::OpenedTab )
				)
				->Split
				(
					FTabManager::NewStack() ->SetSizeCoefficient(0.5f)
					->AddTab( FBlueprintEditorTabs::DetailsID, ETabState::OpenedTab )
					->AddTab( FBlueprintEditorTabs::PaletteID, ETabState::ClosedTab )
					->AddTab( FBlueprintEditorTabs::DefaultEditorID, ETabState::ClosedTab )
				)
			)
		)
	);
}

FBlueprintEditorApplicationMode::FBlueprintEditorApplicationMode(TSharedPtr<class FBlueprintEditor> InBlueprintEditor, FName InModeName, FText(*GetLocalizedMode)(const FName), const bool bRegisterViewport, const bool bRegisterDefaultsTab)
	: FApplicationMode(InModeName, GetLocalizedMode)
{
	MyBlueprintEditor = InBlueprintEditor;

	// Create the tab factories
	BlueprintEditorTabFactories.RegisterFactory(MakeShareable(new FPaletteSummoner(InBlueprintEditor)));
	BlueprintEditorTabFactories.RegisterFactory(MakeShareable(new FMyBlueprintSummoner(InBlueprintEditor)));
	BlueprintEditorTabFactories.RegisterFactory(MakeShareable(new FReplaceNodeReferencesSummoner(InBlueprintEditor)));
	BlueprintEditorTabFactories.RegisterFactory(MakeShareable(new FCompilerResultsSummoner(InBlueprintEditor)));
	BlueprintEditorTabFactories.RegisterFactory(MakeShareable(new FFindResultsSummoner(InBlueprintEditor)));
	BlueprintEditorTabFactories.RegisterFactory(MakeShareable(new FBookmarksSummoner(InBlueprintEditor)));
	
	if( bRegisterViewport )
	{
		BlueprintEditorTabFactories.RegisterFactory(MakeShareable(new FSCSViewportSummoner(InBlueprintEditor)));
	}
	if( bRegisterDefaultsTab )
	{
		BlueprintEditorTabFactories.RegisterFactory(MakeShareable(new FDefaultsEditorSummoner(InBlueprintEditor)));
	}
	CoreTabFactories.RegisterFactory(MakeShareable(new FSelectionDetailsSummoner(InBlueprintEditor)));

	TabLayout = GetDefaltEditorLayout(InBlueprintEditor);

	FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>("Kismet");
	BlueprintEditorModule.OnRegisterTabsForEditor().Broadcast(BlueprintEditorTabFactories, InModeName, InBlueprintEditor);

	LayoutExtender = MakeShared<FLayoutExtender>();
	BlueprintEditorModule.OnRegisterLayoutExtensions().Broadcast(*LayoutExtender);
}

void FBlueprintEditorApplicationMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	TSharedPtr<FBlueprintEditor> BP = MyBlueprintEditor.Pin();
	
	BP->RegisterToolbarTab(InTabManager.ToSharedRef());

	// Mode-specific setup
	BP->PushTabFactories(CoreTabFactories);
	BP->PushTabFactories(BlueprintEditorOnlyTabFactories);
	BP->PushTabFactories(BlueprintEditorTabFactories);
}

void FBlueprintEditorApplicationMode::PreDeactivateMode()
{
	FApplicationMode::PreDeactivateMode();

	TSharedPtr<FBlueprintEditor> BP = MyBlueprintEditor.Pin();
	
	BP->SaveEditedObjectState();
	BP->GetMyBlueprintWidget()->ClearGraphActionMenuSelection();
}

void FBlueprintEditorApplicationMode::PostActivateMode()
{
	// Reopen any documents that were open when the blueprint was last saved
	TSharedPtr<FBlueprintEditor> BP = MyBlueprintEditor.Pin();
	BP->RestoreEditedObjectState();
	BP->SetupViewForBlueprintEditingMode();

	FApplicationMode::PostActivateMode();
}


FBlueprintDefaultsApplicationMode::FBlueprintDefaultsApplicationMode(TSharedPtr<class FBlueprintEditor> InBlueprintEditor)
	: FApplicationMode(FBlueprintEditorApplicationModes::BlueprintDefaultsMode, FBlueprintEditorApplicationModes::GetLocalizedMode)
{
	MyBlueprintEditor = InBlueprintEditor;
	
	BlueprintDefaultsTabFactories.RegisterFactory(MakeShareable(new FDefaultsEditorSummoner(InBlueprintEditor)));
	BlueprintDefaultsTabFactories.RegisterFactory(MakeShareable(new FFindResultsSummoner(InBlueprintEditor)));
	BlueprintDefaultsTabFactories.RegisterFactory(MakeShareable(new FMyBlueprintSummoner(InBlueprintEditor)));

	TabLayout = FTabManager::NewLayout( "Standalone_BlueprintDefaults_Layout_v7" )
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewStack()
				->AddTab( FBlueprintEditorTabs::DefaultEditorID, ETabState::OpenedTab )
				->SetHideTabWell(true)

			)
		);

	// setup toolbar
	if (UToolMenu* Toolbar = InBlueprintEditor->RegisterModeToolbarIfUnregistered(GetModeName()))
	{
		InBlueprintEditor->GetToolbarBuilder()->AddCompileToolbar(Toolbar);
		InBlueprintEditor->GetToolbarBuilder()->AddBlueprintGlobalOptionsToolbar(Toolbar);
	}
}

void FBlueprintDefaultsApplicationMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	TSharedPtr<FBlueprintEditor> BP = MyBlueprintEditor.Pin();
	
	BP->RegisterToolbarTab(InTabManager.ToSharedRef());

	// Mode-specific setup
	BP->PushTabFactories(BlueprintDefaultsTabFactories);
}

void FBlueprintDefaultsApplicationMode::PostActivateMode()
{
	// Reopen any documents that were open when the blueprint was last saved
	TSharedPtr<FBlueprintEditor> BP = MyBlueprintEditor.Pin();
	BP->StartEditingDefaults();

	FApplicationMode::PostActivateMode();
}


FBlueprintComponentsApplicationMode::FBlueprintComponentsApplicationMode(TSharedPtr<class FBlueprintEditor> InBlueprintEditor)
	: FApplicationMode(FBlueprintEditorApplicationModes::BlueprintComponentsMode, FBlueprintEditorApplicationModes::GetLocalizedMode)
{
	MyBlueprintEditor = InBlueprintEditor;
	
	BlueprintComponentsTabFactories.RegisterFactory(MakeShareable(new FConstructionScriptEditorSummoner(InBlueprintEditor)));
	BlueprintComponentsTabFactories.RegisterFactory(MakeShareable(new FSCSViewportSummoner(InBlueprintEditor)));
	BlueprintComponentsTabFactories.RegisterFactory(MakeShareable(new FSelectionDetailsSummoner(InBlueprintEditor)));
	BlueprintComponentsTabFactories.RegisterFactory(MakeShareable(new FDefaultsEditorSummoner(InBlueprintEditor)));
	BlueprintComponentsTabFactories.RegisterFactory(MakeShareable(new FFindResultsSummoner(InBlueprintEditor)));

	TabLayout = FTabManager::NewLayout( "Standalone_BlueprintComponents_Layout_v6" )
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewSplitter()
					->SetSizeCoefficient( 0.15f )
					->SetOrientation(Orient_Vertical)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient( 0.35f )
						->AddTab( FBlueprintEditorTabs::ConstructionScriptEditorID, ETabState::OpenedTab )
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient( 0.65f )
						->AddTab( FBlueprintEditorTabs::DetailsID, ETabState::OpenedTab )
					)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient( 0.85f )
					->SetHideTabWell(true)
					->AddTab( FBlueprintEditorTabs::SCSViewportID, ETabState::OpenedTab )
				)
			)
		);

	// setup toolbar
	if (UToolMenu* Toolbar = InBlueprintEditor->RegisterModeToolbarIfUnregistered(GetModeName()))
	{
		InBlueprintEditor->GetToolbarBuilder()->AddCompileToolbar(Toolbar);
		InBlueprintEditor->GetToolbarBuilder()->AddBlueprintGlobalOptionsToolbar(Toolbar, false);
	}
}

void FBlueprintComponentsApplicationMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	TSharedPtr<FBlueprintEditor> BP = MyBlueprintEditor.Pin();
	
	BP->RegisterToolbarTab(InTabManager.ToSharedRef());

	// Mode-specific setup
	BP->PushTabFactories(BlueprintComponentsTabFactories);
}

void FBlueprintComponentsApplicationMode::PreDeactivateMode()
{
	FApplicationMode::PreDeactivateMode();

	TSharedPtr<FBlueprintEditor> BP = MyBlueprintEditor.Pin();
	BP->GetSubobjectEditor()->SetEnabled(true);
    BP->GetSubobjectEditor()->UpdateTree();
	BP->GetInspector()->SetEnabled(true);
	BP->GetInspector()->EnableComponentDetailsCustomization(false);
	BP->EnableSubobjectPreview(false);

	// Cache component selection before clearing so it can be restored
	for(FSubobjectEditorTreeNodePtrType& Node : BP->GetSubobjectEditor()->GetSelectedNodes())
	{
		CachedComponentSelection.AddUnique(Node->GetComponentTemplate());
	}
	
	BP->GetSubobjectEditor()->ClearSelection();
}

void FBlueprintComponentsApplicationMode::PostActivateMode()
{
	TSharedPtr<FBlueprintEditor> BP = MyBlueprintEditor.Pin();
	if (BP.IsValid())
	{
		TSharedPtr<SSubobjectEditor> SubobjectEditor = BP->GetSubobjectEditor();
		SubobjectEditor->UpdateTree();
		BP->EnableSubobjectPreview(true);
		BP->UpdateSubobjectPreview();
		BP->GetInspector()->EnableComponentDetailsCustomization(true);

		// Reselect the cached components
		TArray<TSharedPtr<FSCSEditorTreeNode>> Selection;
		for (TWeakObjectPtr<const UActorComponent>& Component : CachedComponentSelection)
		{
			if (Component.IsValid())
			{
				SubobjectEditor->GetDragDropTree()->SetItemSelection(SubobjectEditor->FindSlateNodeForObject(Component.Get()), true);
			}
		}

		if (BP->GetSubobjectViewport()->GetIsSimulateEnabled())
		{
			SubobjectEditor->SetEnabled(false);
			BP->GetInspector()->SetEnabled(false);
		}
	}

	FApplicationMode::PostActivateMode();
}

////////////////////////////////////////
//
FBlueprintInterfaceApplicationMode::FBlueprintInterfaceApplicationMode(TSharedPtr<class FBlueprintEditor> InBlueprintEditor, FName InModeName, FText(*GetLocalizedMode)(const FName))
	: FApplicationMode(InModeName, GetLocalizedMode)
{
	MyBlueprintEditor = InBlueprintEditor;
	
	// Create the tab factories
	BlueprintInterfaceTabFactories.RegisterFactory(MakeShareable(new FMyBlueprintSummoner(InBlueprintEditor)));
	BlueprintInterfaceTabFactories.RegisterFactory(MakeShareable(new FReplaceNodeReferencesSummoner(InBlueprintEditor)));
	BlueprintInterfaceTabFactories.RegisterFactory(MakeShareable(new FCompilerResultsSummoner(InBlueprintEditor)));
	BlueprintInterfaceTabFactories.RegisterFactory(MakeShareable(new FBookmarksSummoner(InBlueprintEditor)));
	BlueprintInterfaceTabFactories.RegisterFactory(MakeShareable(new FFindResultsSummoner(InBlueprintEditor)));
	BlueprintInterfaceTabFactories.RegisterFactory(MakeShareable(new FSelectionDetailsSummoner(InBlueprintEditor)));

	TabLayout = FTabManager::NewLayout( "Standalone_BlueprintInterface_Layout_v4" )
		->AddArea
		(
			FTabManager::NewPrimaryArea() ->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewSplitter()->SetOrientation( Orient_Vertical )
					->SetSizeCoefficient(0.70f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient( 0.80f )
						->AddTab( "Document", ETabState::ClosedTab )
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient( 0.20f )
						->AddTab( FBlueprintEditorTabs::CompilerResultsID, ETabState::ClosedTab )
						->AddTab( FBlueprintEditorTabs::FindResultsID, ETabState::ClosedTab )
						->AddTab( FBlueprintEditorTabs::BookmarksID, ETabState::ClosedTab )
					)
				)
				->Split
				(
					FTabManager::NewSplitter() ->SetOrientation( Orient_Vertical )
					->SetSizeCoefficient(0.15f)
					->Split
					(
						FTabManager::NewStack() ->SetSizeCoefficient(0.5f)
						->AddTab( FBlueprintEditorTabs::MyBlueprintID, ETabState::OpenedTab )
					)
					->Split
					(
						FTabManager::NewStack() ->SetSizeCoefficient(0.5f)
						->AddTab( FBlueprintEditorTabs::DetailsID, ETabState::OpenedTab )
						->AddTab( FBlueprintEditorTabs::PaletteID, ETabState::ClosedTab )
					)
				)
			)
		);

	// setup toolbar
	if (UToolMenu* Toolbar = InBlueprintEditor->RegisterModeToolbarIfUnregistered(GetModeName()))
	{
		InBlueprintEditor->GetToolbarBuilder()->AddCompileToolbar(Toolbar);
		InBlueprintEditor->GetToolbarBuilder()->AddBlueprintGlobalOptionsToolbar(Toolbar);
	}
}

void FBlueprintInterfaceApplicationMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	TSharedPtr<FBlueprintEditor> BP = MyBlueprintEditor.Pin();
	
	BP->RegisterToolbarTab(InTabManager.ToSharedRef());

	// Mode-specific setup
	BP->PushTabFactories(BlueprintInterfaceTabFactories);
}

void FBlueprintInterfaceApplicationMode::PreDeactivateMode()
{
	FApplicationMode::PreDeactivateMode();

	TSharedPtr<FBlueprintEditor> BP = MyBlueprintEditor.Pin();

	BP->SaveEditedObjectState();
}

void FBlueprintInterfaceApplicationMode::PostActivateMode()
{
	// Reopen any documents that were open when the blueprint was last saved
	TSharedPtr<FBlueprintEditor> BP = MyBlueprintEditor.Pin();
	BP->RestoreEditedObjectState();

	FApplicationMode::PostActivateMode();
}

////////////////////////////////////////
//
FBlueprintMacroApplicationMode::FBlueprintMacroApplicationMode(TSharedPtr<class FBlueprintEditor> InBlueprintEditor)
	: FApplicationMode(FBlueprintEditorApplicationModes::BlueprintMacroMode, FBlueprintEditorApplicationModes::GetLocalizedMode)
{
	MyBlueprintEditor = InBlueprintEditor;
	
	// Create the tab factories
	BlueprintMacroTabFactories.RegisterFactory(MakeShareable(new FMyBlueprintSummoner(InBlueprintEditor)));
	BlueprintMacroTabFactories.RegisterFactory(MakeShareable(new FReplaceNodeReferencesSummoner(InBlueprintEditor)));
	BlueprintMacroTabFactories.RegisterFactory(MakeShareable(new FPaletteSummoner(InBlueprintEditor)));
	BlueprintMacroTabFactories.RegisterFactory(MakeShareable(new FBookmarksSummoner(InBlueprintEditor)));
	BlueprintMacroTabFactories.RegisterFactory(MakeShareable(new FFindResultsSummoner(InBlueprintEditor)));
	BlueprintMacroTabFactories.RegisterFactory(MakeShareable(new FSelectionDetailsSummoner(InBlueprintEditor)));

	TabLayout = FTabManager::NewLayout( "Standalone_BlueprintMacro_Layout_v4" )
		->AddArea
		(
			FTabManager::NewPrimaryArea() ->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewSplitter()->SetOrientation( Orient_Vertical )
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient( 0.80f )
						->AddTab( "Document", ETabState::ClosedTab )
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient( 0.20f )
						->AddTab( FBlueprintEditorTabs::FindResultsID, ETabState::ClosedTab )
						->AddTab( FBlueprintEditorTabs::BookmarksID, ETabState::ClosedTab )
					)
				)
				->Split
				(
					FTabManager::NewSplitter() ->SetOrientation( Orient_Vertical )
					->SetSizeCoefficient(0.15f)
					->Split
					(
						FTabManager::NewStack() ->SetSizeCoefficient(0.5f)
						->AddTab( FBlueprintEditorTabs::MyBlueprintID, ETabState::OpenedTab )
					)
					->Split
					(
						FTabManager::NewStack() ->SetSizeCoefficient(0.5f)
						->AddTab( FBlueprintEditorTabs::DetailsID, ETabState::OpenedTab )
						->AddTab( FBlueprintEditorTabs::PaletteID, ETabState::ClosedTab )
					)
				)
			)
		);

	// setup toolbar
	if (UToolMenu* Toolbar = InBlueprintEditor->RegisterModeToolbarIfUnregistered(GetModeName()))
	{
		InBlueprintEditor->GetToolbarBuilder()->AddCompileToolbar(Toolbar);
		InBlueprintEditor->GetToolbarBuilder()->AddScriptingToolbar(Toolbar);
		InBlueprintEditor->GetToolbarBuilder()->AddBlueprintGlobalOptionsToolbar(Toolbar);
		InBlueprintEditor->GetToolbarBuilder()->AddDebuggingToolbar(Toolbar);
	}
}
void FBlueprintMacroApplicationMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	TSharedPtr<FBlueprintEditor> BP = MyBlueprintEditor.Pin();
	
	BP->RegisterToolbarTab(InTabManager.ToSharedRef());

	// Mode-specific setup
	BP->PushTabFactories(BlueprintMacroTabFactories);
}

void FBlueprintMacroApplicationMode::PreDeactivateMode()
{
	FApplicationMode::PreDeactivateMode();

	TSharedPtr<FBlueprintEditor> BP = MyBlueprintEditor.Pin();

	BP->SaveEditedObjectState();
}

void FBlueprintMacroApplicationMode::PostActivateMode()
{
	// Reopen any documents that were open when the blueprint was last saved
	TSharedPtr<FBlueprintEditor> BP = MyBlueprintEditor.Pin();
	BP->RestoreEditedObjectState();

	FApplicationMode::PostActivateMode();
}

////////////////////////////////////////
//
FBlueprintEditorUnifiedMode::FBlueprintEditorUnifiedMode(TSharedPtr<class FBlueprintEditor> InBlueprintEditor, FName InModeName, FText(*GetLocalizedMode)( const FName ), const bool bRegisterViewport)
	: FApplicationMode(InModeName, GetLocalizedMode)
{
	MyBlueprintEditor = InBlueprintEditor;

	// Create the tab factories
	BlueprintEditorTabFactories.RegisterFactory(MakeShareable(new FPaletteSummoner(InBlueprintEditor)));
	BlueprintEditorTabFactories.RegisterFactory(MakeShareable(new FMyBlueprintSummoner(InBlueprintEditor)));
	BlueprintEditorTabFactories.RegisterFactory(MakeShareable(new FReplaceNodeReferencesSummoner(InBlueprintEditor)));
	BlueprintEditorTabFactories.RegisterFactory(MakeShareable(new FCompilerResultsSummoner(InBlueprintEditor)));
	BlueprintEditorTabFactories.RegisterFactory(MakeShareable(new FFindResultsSummoner(InBlueprintEditor)));
	BlueprintEditorTabFactories.RegisterFactory(MakeShareable(new FBookmarksSummoner(InBlueprintEditor)));
	
	if( bRegisterViewport )
	{
		BlueprintEditorTabFactories.RegisterFactory(MakeShareable(new FConstructionScriptEditorSummoner(InBlueprintEditor)));
		BlueprintEditorTabFactories.RegisterFactory(MakeShareable(new FSCSViewportSummoner(InBlueprintEditor)));
	}

	CoreTabFactories.RegisterFactory(MakeShareable(new FSelectionDetailsSummoner(InBlueprintEditor)));

	if ( bRegisterViewport )
	{
		TabLayout = FTabManager::NewLayout( "Blueprints_Unified_Components_v7" )
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.15f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.30f)
						->AddTab( FBlueprintEditorTabs::ConstructionScriptEditorID, ETabState::OpenedTab )
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.70f)
						->AddTab( FBlueprintEditorTabs::MyBlueprintID, ETabState::OpenedTab )
					)
				)
				->Split
				(
					FTabManager::NewSplitter()->SetOrientation( Orient_Vertical )
					->SetSizeCoefficient(0.60f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient( 0.80f )
						->AddTab(FBlueprintEditorTabs::SCSViewportID, ETabState::OpenedTab)
						->AddTab( "Document", ETabState::ClosedTab )
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient( 0.20f )
						->AddTab( FBlueprintEditorTabs::CompilerResultsID, ETabState::ClosedTab )
						->AddTab( FBlueprintEditorTabs::FindResultsID, ETabState::ClosedTab )
						->AddTab( FBlueprintEditorTabs::BookmarksID, ETabState::ClosedTab )
					)
				)
				->Split
				(
					FTabManager::NewSplitter() ->SetOrientation( Orient_Vertical )
					->SetSizeCoefficient(0.25f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.50f)
						->AddTab( FBlueprintEditorTabs::DetailsID, ETabState::OpenedTab )
						->AddTab( FBlueprintEditorTabs::PaletteID, ETabState::ClosedTab )
					)
				)
			)
		);
	}
	else
	{
		TabLayout = FTabManager::NewLayout( "Blueprints_Unified_v5" )
		->AddArea
		(
		FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
		->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.15f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.50f)
						->AddTab( FBlueprintEditorTabs::MyBlueprintID, ETabState::OpenedTab )
					)
				)
				->Split
				(
					FTabManager::NewSplitter()->SetOrientation( Orient_Vertical )
					->SetSizeCoefficient(0.60f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient( 0.80f )
						->AddTab( "Document", ETabState::ClosedTab )
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient( 0.20f )
						->AddTab( FBlueprintEditorTabs::CompilerResultsID, ETabState::ClosedTab )
						->AddTab( FBlueprintEditorTabs::FindResultsID, ETabState::ClosedTab )
						->AddTab( FBlueprintEditorTabs::BookmarksID, ETabState::ClosedTab )
					)
				)
				->Split
				(
					FTabManager::NewSplitter() ->SetOrientation( Orient_Vertical )
					->SetSizeCoefficient(0.25f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.60f)
						->AddTab( FBlueprintEditorTabs::DetailsID, ETabState::OpenedTab )
						->AddTab( FBlueprintEditorTabs::PaletteID, ETabState::ClosedTab )
					)
				)
			)
		);
	}
	
	// setup toolbar
	//@TODO: Keep this in sync with AnimBlueprintMode.cpp
	if (UToolMenu* Toolbar = InBlueprintEditor->RegisterModeToolbarIfUnregistered(GetModeName()))
	{
		InBlueprintEditor->GetToolbarBuilder()->AddCompileToolbar(Toolbar);
		InBlueprintEditor->GetToolbarBuilder()->AddScriptingToolbar(Toolbar);
		InBlueprintEditor->GetToolbarBuilder()->AddBlueprintGlobalOptionsToolbar(Toolbar, bRegisterViewport);
		InBlueprintEditor->GetToolbarBuilder()->AddDebuggingToolbar(Toolbar);
	}

	FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>("Kismet");
	BlueprintEditorModule.OnRegisterTabsForEditor().Broadcast(BlueprintEditorTabFactories, InModeName, InBlueprintEditor);

	LayoutExtender = MakeShared<FLayoutExtender>();
	BlueprintEditorModule.OnRegisterLayoutExtensions().Broadcast(*LayoutExtender);
}

void FBlueprintEditorUnifiedMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	TSharedPtr<FBlueprintEditor> BP = MyBlueprintEditor.Pin();
	
	BP->RegisterToolbarTab(InTabManager.ToSharedRef());

	// Mode-specific setup
	BP->PushTabFactories(CoreTabFactories);
	BP->PushTabFactories(BlueprintEditorOnlyTabFactories);
	BP->PushTabFactories(BlueprintEditorTabFactories);
}

void FBlueprintEditorUnifiedMode::PreDeactivateMode()
{
	FApplicationMode::PreDeactivateMode();

	TSharedPtr<FBlueprintEditor> BP = MyBlueprintEditor.Pin();
	
	BP->SaveEditedObjectState();
	BP->GetMyBlueprintWidget()->ClearGraphActionMenuSelection();
}

void FBlueprintEditorUnifiedMode::PostActivateMode()
{
	// Reopen any documents that were open when the blueprint was last saved
	TSharedPtr<FBlueprintEditor> BP = MyBlueprintEditor.Pin();
	BP->RestoreEditedObjectState();
	BP->SetupViewForBlueprintEditingMode();

	FApplicationMode::PostActivateMode();
}

#undef LOCTEXT_NAMESPACE
