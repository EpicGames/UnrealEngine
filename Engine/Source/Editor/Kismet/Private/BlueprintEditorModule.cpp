// Copyright Epic Games, Inc. All Rights Reserved.


#include "BlueprintEditorModule.h"

#include "BlueprintDebugger.h"
#include "BlueprintEditorSettings.h"
#include "BlueprintEditor.h"
#include "BlueprintEditorSCSEditorUICustomization.h"
#include "BlueprintGraphPanelPinFactory.h"
#include "BlueprintNamespaceRegistry.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Set.h"
#include "CoreGlobals.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphUtilities.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Editor/Transactor.h"
#include "EditorUndoClient.h"
#include "Framework/Commands/UICommandList.h"
#include "HAL/PlatformCrt.h"
#include "IMessageLogListing.h"
#include "ISettingsModule.h"
#include "InstancedStaticMeshSCSEditorCustomization.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "KismetWidgets.h"
#include "LevelEditor.h"
#include "Logging/LogVerbosity.h"
#include "Logging/TokenizedMessage.h"
#include "MessageLogInitializationOptions.h"
#include "MessageLogModule.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Parse.h"
#include "Misc/UObjectToken.h"
#include "Modules/ModuleManager.h"
#include "SPinValueInspector.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "Templates/Casts.h"
#include "Templates/TypeHash.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectHash.h"
#include "UObject/WeakObjectPtr.h"
#include "UserDefinedEnumEditor.h"
#include "UserDefinedStructureEditor.h"

class AActor;
class FDetailsViewObjectFilter;
class FExtender;
class IDetailRootObjectCustomization;
class IToolkitHost;
class SWidget;

#define LOCTEXT_NAMESPACE "BlueprintEditor"

IMPLEMENT_MODULE( FBlueprintEditorModule, Kismet );

//////////////////////////////////////////////////////////////////////////
// FBlueprintEditorModule

TSharedRef<FExtender> ExtendLevelViewportContextMenuForBlueprints(const TSharedRef<FUICommandList> CommandList, TArray<AActor*> SelectedActors);

FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors LevelViewportContextMenuBlueprintExtender;

static void FocusBlueprintEditorOnObject(const TSharedRef<IMessageToken>& Token)
{
	if( Token->GetType() == EMessageToken::Object )
	{
		const TSharedRef<FUObjectToken> UObjectToken = StaticCastSharedRef<FUObjectToken>(Token);
		if(UObjectToken->GetObject().IsValid())
		{
			FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(UObjectToken->GetObject().Get());
		}
	}
}

struct FBlueprintUndoRedoHandler : public FEditorUndoClient
{	
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
};
static FBlueprintUndoRedoHandler* UndoRedoHandler = nullptr;

void FixSubObjectReferencesPostUndoRedo(UObject* InObject)
{
	// Post undo/redo, these may have the correct Outer but are not referenced by the CDO's UProperties
	TArray<UObject*> SubObjects;
	GetObjectsWithOuter(InObject, SubObjects, EGetObjectsFlags::None);

	// Post undo/redo, these may have the in-correct Outer but are incorrectly referenced by the CDO's UProperties
	// @todo - The original use case (UE-18137) no longer seems to rely on this code path for fixup, can we remove it?
	TMap<UObject*, UObject*> OldToNewInstanceMap;
	for (TPropertyValueIterator<FObjectProperty> It(InObject->GetClass(), InObject); It; ++It)
	{
		// This is a legacy path that was intentionally excluding non-"persistent" instanced properties. We'll keep the
		// same logic in place to avoid unintentional side effects and to avoid introducing any new regressive edge cases.
		const FObjectProperty* Property = It.Key();
		if (!Property->HasAllPropertyFlags(CPF_InstancedReference | CPF_PersistentInstance))
		{
			continue;
		}

		// Ignore values that are NULL (e.g. child class overrides that exclude the property from instancing).
		UObject* PropertySubObject = Property->GetObjectPropertyValue(It.Value());
		if (!PropertySubObject)
		{
			continue;
		}

		bool bFoundMatchingSubObject = false;
		for (UObject* SubObject : SubObjects)
		{
			// The property and sub-objects should have the same name.
			if (PropertySubObject->GetFName() == SubObject->GetFName())
			{
				// We found a matching property, we do not want to re-make the property
				bFoundMatchingSubObject = true;

				// Check if the properties have different outers so we can map old-to-new
				if (PropertySubObject->GetOuter() != InObject)
				{
					OldToNewInstanceMap.Add(PropertySubObject, SubObject);
				}
				// Recurse on the SubObject to correct any sub-object/property references
				FixSubObjectReferencesPostUndoRedo(SubObject);
				break;
			}
		}

		// If the property referenced does not exist in the current context as a subobject, we need to duplicate it and fix up references
		// This will occur during post-undo/redo of deletions
		if (!bFoundMatchingSubObject)
		{
			UObject* NewSubObject = DuplicateObject(PropertySubObject, InObject, PropertySubObject->GetFName());

			// Don't forget to fix up all references and sub-object references
			OldToNewInstanceMap.Add(PropertySubObject, NewSubObject);
		}
	}

	FArchiveReplaceObjectRef<UObject> Replacer(InObject, OldToNewInstanceMap);
}

void FixSubObjectReferencesPostUndoRedo(const FTransaction* Transaction)
{
	TArray<UBlueprint*> ModifiedBlueprints;

	// Look at the transaction this function is responding to, see if any object in it has an outermost of the Blueprint
	if (Transaction != nullptr)
	{
		TArray<UObject*> TransactionObjects;
		Transaction->GetTransactionObjects(TransactionObjects);
		for (UObject* Object : TransactionObjects)
		{
			UBlueprint* Blueprint = nullptr;

			while (Object != nullptr && Blueprint == nullptr)
			{
				Blueprint = Cast<UBlueprint>(Object);
				Object = Object->GetOuter();
			}

			if (Blueprint != nullptr)
			{
				if (Blueprint->ShouldBeMarkedDirtyUponTransaction())
				{
					ModifiedBlueprints.AddUnique(Blueprint);
				}
			}
		}
	}

	// Transaction affects the Blueprints this editor handles, so react as necessary
	for (UBlueprint* Blueprint : ModifiedBlueprints)
	{
		FixSubObjectReferencesPostUndoRedo(Blueprint->GeneratedClass->GetDefaultObject());
		// Will cause a call to RefreshEditors()
		if (Blueprint->ShouldBeMarkedDirtyUponTransaction())
		{
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
		}
		else
		{
			Blueprint->MarkPackageDirty();
		}
	}
}

void FBlueprintUndoRedoHandler::PostUndo(bool bSuccess)
{
	FixSubObjectReferencesPostUndoRedo(GEditor->Trans->GetTransaction(GEditor->Trans->GetQueueLength() - GEditor->Trans->GetUndoCount()));
}

void FBlueprintUndoRedoHandler::PostRedo(bool bSuccess)
{
	// Note: We add 1 to get the correct slot, because the transaction buffer will have decremented the UndoCount prior to getting here.
	if( GEditor->Trans->GetQueueLength() > 0 )
	{
		FixSubObjectReferencesPostUndoRedo(GEditor->Trans->GetTransaction(GEditor->Trans->GetQueueLength() - (GEditor->Trans->GetUndoCount() + 1)));
	}
}

void FBlueprintEditorModule::StartupModule()
{
	check(GEditor);

	delete UndoRedoHandler;
	UndoRedoHandler = new FBlueprintUndoRedoHandler();
	GEditor->RegisterForUndo(UndoRedoHandler);

	MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
	SharedBlueprintEditorCommands = MakeShareable(new FUICommandList);

	BlueprintDebugger = MakeUnique<FBlueprintDebugger>();

	FBlueprintNamespaceRegistry::Get().Initialize();

	// Have to check GIsEditor because right now editor modules can be loaded by the game
	// Once LoadModule is guaranteed to return NULL for editor modules in game, this can be removed
	// Without this check, loading the level editor in the game will crash
	if (GIsEditor)
	{
		// Extend the level viewport context menu to handle blueprints
		LevelViewportContextMenuBlueprintExtender = FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors::CreateStatic(&ExtendLevelViewportContextMenuForBlueprints);
		FLevelEditorModule& LevelEditorModule = FModuleManager::Get().LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		auto& MenuExtenders = LevelEditorModule.GetAllLevelViewportContextMenuExtenders();
		MenuExtenders.Add(LevelViewportContextMenuBlueprintExtender);
		LevelViewportContextMenuBlueprintExtenderDelegateHandle = MenuExtenders.Last().GetHandle();

		FModuleManager::Get().LoadModuleChecked<FKismetWidgetsModule>("KismetWidgets");
	}

	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	FMessageLogInitializationOptions InitOptions;
	InitOptions.bShowFilters = true;
	InitOptions.bShowPages = true;
	MessageLogModule.RegisterLogListing("BlueprintLog", LOCTEXT("BlueprintLog", "Blueprint Log"), InitOptions);

	// Listen for clicks in log so we can focus on the object, might have to restart K2 if the K2 tab has been closed
	MessageLogModule.GetLogListing("BlueprintLog")->OnMessageTokenClicked().AddStatic( &FocusBlueprintEditorOnObject );
	
	// Also listen for clicks in the PIE log, runtime errors with Blueprints may post clickable links there
	MessageLogModule.GetLogListing("PIE")->OnMessageTokenClicked().AddStatic( &FocusBlueprintEditorOnObject );

	// Add a page for pre-loading of the editor
	MessageLogModule.GetLogListing("BlueprintLog")->NewPage(LOCTEXT("PreloadLogPageLabel", "Editor Load"));

	SCSEditorUICustomization = FFBlueprintEditorSCSEditorUICustomization::GetInstance();

	// Register internal SCS editor customizations
	RegisterSCSEditorCustomization("InstancedStaticMeshComponent", FSCSEditorCustomizationBuilder::CreateStatic(&FInstancedStaticMeshSCSEditorCustomization::MakeInstance));
	RegisterSCSEditorCustomization("HierarchicalInstancedStaticMeshComponent", FSCSEditorCustomizationBuilder::CreateStatic(&FInstancedStaticMeshSCSEditorCustomization::MakeInstance));

	TSharedPtr<FBlueprintGraphPanelPinFactory> BlueprintGraphPanelPinFactory = MakeShareable(new FBlueprintGraphPanelPinFactory());
	FEdGraphUtilities::RegisterVisualPinFactory(BlueprintGraphPanelPinFactory);

	PrepareAutoGeneratedDefaultEvents();

	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
	}
}

void FBlueprintEditorModule::ShutdownModule()
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->UnregisterSettings("Project", "Engine", "Blueprints");
		ConfigurationPanel = TSharedPtr<SWidget>();
	}
	// we're intentionally leaking UndoRedoHandler because the GEditor may be garbage when ShutdownModule is called:

	// Cleanup all information for auto generated default event nodes by this module
	FKismetEditorUtilities::UnregisterAutoBlueprintNodeCreation(this);

	SharedBlueprintEditorCommands.Reset();
	MenuExtensibilityManager.Reset();

	// Remove level viewport context menu extenders
	if ( FModuleManager::Get().IsModuleLoaded( "LevelEditor" ) )
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>( "LevelEditor" );
		LevelEditorModule.GetAllLevelViewportContextMenuExtenders().RemoveAll([&](const FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors& Delegate) {
			return Delegate.GetHandle() == LevelViewportContextMenuBlueprintExtenderDelegateHandle;
		});
	}

	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	MessageLogModule.UnregisterLogListing("BlueprintLog");

	// Unregister internal SCS editor customizations
	UnregisterSCSEditorCustomization("InstancedStaticMeshComponent");

	UEdGraphPin::ShutdownVerification();
	FPinValueInspectorTooltip::ShutdownTooltip();

	FBlueprintNamespaceRegistry::Get().Shutdown();
}

FBlueprintEditorModule::FAdditionalCategoryHandle FBlueprintEditorModule::RegisterAdditionalBlueprintCategory(UE::BlueprintEditor::FAdditionalCategory InCategory)
{
	const uint32 RawHandle = NextAdditionalCategoryHandle++;
	const FAdditionalCategoryHandle Handle(RawHandle);
	RegisteredAdditionalCategories.Emplace(Handle, MoveTemp(InCategory));
	return Handle;
}

void FBlueprintEditorModule::UnregisterAdditionalBlueprintCategory(FAdditionalCategoryHandle Handle)
{
	if (Handle.IsValid())
	{
		RegisteredAdditionalCategories.RemoveAllSwap(
			[Handle](const TTuple<FAdditionalCategoryHandle, UE::BlueprintEditor::FAdditionalCategory>& Category)
			{
				return Category.Get<0>() == Handle;
			});
	}
}

TArray<UE::BlueprintEditor::FAdditionalCategory> FBlueprintEditorModule::GetAdditionalBlueprintCategories() const
{
	TArray<UE::BlueprintEditor::FAdditionalCategory> Result;

	for (const TTuple<FAdditionalCategoryHandle, UE::BlueprintEditor::FAdditionalCategory>& Category : RegisteredAdditionalCategories)
	{
		Result.Add(Category.Get<1>());
	}

	// Merge entries from UBlueprintEditorSettings
	for (const FAdditionalBlueprintCategory& Settings : GetDefault<UBlueprintEditorSettings>()->AdditionalBlueprintCategories)
	{
		if (!Settings.Name.IsEmpty())
		{
			UE::BlueprintEditor::FAdditionalCategory Converted;
			// Empty array is for all classes.
			if (!Settings.ClassFilter.IsNull())
			{
				Converted.ClassFilters.Add(Settings.ClassFilter);
			}
			Converted.Category = Settings.Name;
			Result.Add(MoveTemp(Converted));
		}
	}
	return Result;
}

TArray<UE::BlueprintEditor::FAdditionalCategory> FBlueprintEditorModule::GetAdditionalBlueprintCategories(TNotNull<const UBlueprint*> Blueprint) const
{
	TArray<UE::BlueprintEditor::FAdditionalCategory> Result;

	auto Filter = [Blueprint](const FSoftClassPath& Filter)
		{
			// If null, then the class is not loaded.
			//The Blueprint can't be loaded if the parent is not loaded. Then no need to load the class to test IsChildOf.
			const UClass* FilterClass = Filter.ResolveClass();
			return Blueprint->ParentClass && FilterClass && Blueprint->ParentClass->IsChildOf(FilterClass);
		};

	for (const TTuple<FAdditionalCategoryHandle, UE::BlueprintEditor::FAdditionalCategory>& Category : RegisteredAdditionalCategories)
	{
		const bool bPassesFilter = Category.Get<1>().ClassFilters.IsEmpty() || Category.Get<1>().ClassFilters.ContainsByPredicate(Filter);
		if (bPassesFilter)
		{
			Result.Add(Category.Get<1>());
		}
	}

	// Merge entries from UBlueprintEditorSettings
	for (const FAdditionalBlueprintCategory& Settings : GetDefault<UBlueprintEditorSettings>()->AdditionalBlueprintCategories)
	{
		if (!Settings.Name.IsEmpty())
		{
			if (Settings.ClassFilter.IsNull() || Filter(Settings.ClassFilter))
			{
				UE::BlueprintEditor::FAdditionalCategory Converted;
				Converted.Category = Settings.Name;
				Converted.ClassFilters.Add(Settings.ClassFilter);
				Result.Add(MoveTemp(Converted));
			}
		}
	}

	return Result;
}

TSharedRef<IBlueprintEditor> FBlueprintEditorModule::CreateBlueprintEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UBlueprint* Blueprint, bool bShouldOpenInDefaultsMode)
{
	TArray<UBlueprint*> BlueprintsToEdit = { Blueprint };
	return CreateBlueprintEditor(Mode, InitToolkitHost, BlueprintsToEdit, bShouldOpenInDefaultsMode);
}

TSharedRef<IBlueprintEditor> FBlueprintEditorModule::CreateBlueprintEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, const TArray< UBlueprint* >& BlueprintsToEdit, bool bShouldOpenInDefaultsMode)
{
	TSharedRef< FBlueprintEditor > NewBlueprintEditor( new FBlueprintEditor() );

	NewBlueprintEditor->InitBlueprintEditor(Mode, InitToolkitHost, BlueprintsToEdit, bShouldOpenInDefaultsMode, SCSEditorUICustomization);

	NewBlueprintEditor->SetDetailsCustomization(DetailsObjectFilter, DetailsRootCustomization);

	for(auto It(SCSEditorCustomizations.CreateConstIterator()); It; ++It)
	{
		NewBlueprintEditor->RegisterSCSEditorCustomization(It->Key, It->Value.Execute(NewBlueprintEditor));
	}

	EBlueprintType const BPType = ( (BlueprintsToEdit.Num() > 0) && (BlueprintsToEdit[0] != NULL) ) 
		? (EBlueprintType) BlueprintsToEdit[0]->BlueprintType
		: BPTYPE_Normal;

	BlueprintEditorOpened.Broadcast(BPType);

	BlueprintEditors.Add(NewBlueprintEditor);

	return NewBlueprintEditor;
}

TArray<TSharedRef<IBlueprintEditor>> FBlueprintEditorModule::GetBlueprintEditors() const
{
	TArray<TSharedRef<IBlueprintEditor>> ValidBlueprintEditors;
	ValidBlueprintEditors.Reserve(BlueprintEditors.Num());

	for (TWeakPtr<FBlueprintEditor> BlueprintEditor : BlueprintEditors)
	{
		if (TSharedPtr<FBlueprintEditor> BlueprintEditorPinned = BlueprintEditor.Pin())
		{
			ValidBlueprintEditors.Add(BlueprintEditorPinned.ToSharedRef());
		}
	}

	if (BlueprintEditors.Num() > ValidBlueprintEditors.Num())
	{
		TArray<TWeakPtr<FBlueprintEditor>>& BlueprintEditorsNonConst = const_cast<TArray<TWeakPtr<FBlueprintEditor>>&>(BlueprintEditors);
		BlueprintEditorsNonConst.Reset(ValidBlueprintEditors.Num());
		for (const TSharedRef<IBlueprintEditor>& ValidBlueprintEditor : ValidBlueprintEditors)
		{
			BlueprintEditorsNonConst.Add(StaticCastSharedRef<FBlueprintEditor>(ValidBlueprintEditor));
		}
	}

	return ValidBlueprintEditors;
}

TSharedRef<IUserDefinedEnumEditor> FBlueprintEditorModule::CreateUserDefinedEnumEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UUserDefinedEnum* UDEnum)
{
	TSharedRef<FUserDefinedEnumEditor> UserDefinedEnumEditor(new FUserDefinedEnumEditor());
	UserDefinedEnumEditor->InitEditor(Mode, InitToolkitHost, UDEnum);
	return UserDefinedEnumEditor;
}

TSharedRef<IUserDefinedStructureEditor> FBlueprintEditorModule::CreateUserDefinedStructEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UUserDefinedStruct* UDStruct)
{
	TSharedRef<FUserDefinedStructureEditor> UserDefinedStructureEditor(new FUserDefinedStructureEditor());
	UserDefinedStructureEditor->InitEditor(Mode, InitToolkitHost, UDStruct);
	return UserDefinedStructureEditor;
}

void FBlueprintEditorModule::SetDetailsCustomization(TSharedPtr<FDetailsViewObjectFilter> InDetailsObjectFilter, TSharedPtr<IDetailRootObjectCustomization> InDetailsRootCustomization)
{
	DetailsObjectFilter = InDetailsObjectFilter;
	DetailsRootCustomization = InDetailsRootCustomization;

	for (const TSharedRef<IBlueprintEditor>& BlueprintEditor : GetBlueprintEditors())
	{
		StaticCastSharedRef<FBlueprintEditor>(BlueprintEditor)->SetDetailsCustomization(DetailsObjectFilter, DetailsRootCustomization);
	}
}

void FBlueprintEditorModule::SetSubobjectEditorUICustomization(TSharedPtr<ISCSEditorUICustomization> InSCSEditorUICustomization)
{
	AddCustomization(InSCSEditorUICustomization);

	for (const TSharedRef<IBlueprintEditor>& BlueprintEditor : GetBlueprintEditors())
	{
		StaticCastSharedRef<FBlueprintEditor>(BlueprintEditor)->SetSubobjectEditorUICustomization(SCSEditorUICustomization);
	}
}

void FBlueprintEditorModule::AddCustomization(TSharedPtr<ISCSEditorUICustomization> InSCSEditorUICustomization)
{
	if (SCSEditorUICustomization)
	{
		SCSEditorUICustomization->AddCustomization(InSCSEditorUICustomization);
	}
}

void FBlueprintEditorModule::RemoveCustomization(TSharedPtr<ISCSEditorUICustomization> InSCSEditorUICustomization)
{
	if (SCSEditorUICustomization)
	{
		SCSEditorUICustomization->RemoveCustomization(InSCSEditorUICustomization);
	}
}

void FBlueprintEditorModule::RegisterSCSEditorCustomization(const FName& InComponentName, FSCSEditorCustomizationBuilder InCustomizationBuilder)
{
	SCSEditorCustomizations.Add(InComponentName, InCustomizationBuilder);
}

void FBlueprintEditorModule::UnregisterSCSEditorCustomization(const FName& InComponentName)
{
	SCSEditorCustomizations.Remove(InComponentName);
}

FDelegateHandle FBlueprintEditorModule::RegisterVariableCustomization(FFieldClass* InFieldClass, FOnGetVariableCustomizationInstance InOnGetVariableCustomization)
{
	FDelegateHandle Result = InOnGetVariableCustomization.GetHandle();
	VariableCustomizations.Add(InFieldClass, InOnGetVariableCustomization);
	return Result;
}

void FBlueprintEditorModule::UnregisterVariableCustomization(FFieldClass* InFieldClass, FDelegateHandle InHandle)
{
	for (TMultiMap<FFieldClass*, FOnGetVariableCustomizationInstance>::TKeyIterator It = VariableCustomizations.CreateKeyIterator(InFieldClass); It; ++It)
	{
		if (It.Value().GetHandle() == InHandle)
		{
			It.RemoveCurrent();
		}
	}
}

FDelegateHandle FBlueprintEditorModule::RegisterLocalVariableCustomization(FFieldClass* InFieldClass, FOnGetLocalVariableCustomizationInstance InOnGetLocalVariableCustomization)
{
	FDelegateHandle Result = InOnGetLocalVariableCustomization.GetHandle();
	LocalVariableCustomizations.Add(InFieldClass, InOnGetLocalVariableCustomization);
	return Result;
}

void FBlueprintEditorModule::UnregisterLocalVariableCustomization(FFieldClass* InFieldClass, FDelegateHandle InHandle)
{
	for (TMultiMap<FFieldClass*, FOnGetVariableCustomizationInstance>::TKeyIterator It = LocalVariableCustomizations.CreateKeyIterator(InFieldClass); It; ++It)
	{
		if (It.Value().GetHandle() == InHandle)
		{
			It.RemoveCurrent();
		}
	}
}

void FBlueprintEditorModule::RegisterGraphCustomization(const UEdGraphSchema* InGraphSchema, FOnGetGraphCustomizationInstance InOnGetGraphCustomization)
{
	GraphCustomizations.Add(InGraphSchema, InOnGetGraphCustomization);
}

void FBlueprintEditorModule::UnregisterGraphCustomization(const UEdGraphSchema* InGraphSchema)
{
	GraphCustomizations.Remove(InGraphSchema);
}

FDelegateHandle FBlueprintEditorModule::RegisterFunctionCustomization(TSubclassOf<UK2Node_EditablePinBase> InFieldClass, FOnGetFunctionCustomizationInstance InOnGetFunctionCustomization)
{
	FDelegateHandle Result = InOnGetFunctionCustomization.GetHandle();
	FunctionCustomizations.Add(InFieldClass, InOnGetFunctionCustomization);
	return Result;
}

void FBlueprintEditorModule::UnregisterFunctionCustomization(TSubclassOf<UK2Node_EditablePinBase> InFieldClass, FDelegateHandle InHandle)
{
	for (TMultiMap<TSubclassOf<UK2Node_EditablePinBase>, FOnGetFunctionCustomizationInstance>::TKeyIterator It = FunctionCustomizations.CreateKeyIterator(InFieldClass); It; ++It)
	{
		if (It.Value().GetHandle() == InHandle)
		{
			It.RemoveCurrent();
		}
	}
}

TArray<TSharedPtr<IDetailCustomization>> FBlueprintEditorModule::CustomizeVariable(FFieldClass* InFieldClass, TSharedPtr<IBlueprintEditor> InBlueprintEditor)
{
	TArray<TSharedPtr<IDetailCustomization>> DetailsCustomizations;
	TArray<FFieldClass*> ParentClassesToQuery;
	if (InFieldClass)
	{
		ParentClassesToQuery.Add(InFieldClass);

		FFieldClass* ParentClass = InFieldClass->GetSuperClass();
		while (ParentClass)
		{
			ParentClassesToQuery.Add(ParentClass);
			ParentClass = ParentClass->GetSuperClass();
		}

		for (FFieldClass* ClassToQuery : ParentClassesToQuery)
		{
			TArray<FOnGetVariableCustomizationInstance*, TInlineAllocator<4>> CustomizationDelegates;
			VariableCustomizations.MultiFindPointer(ClassToQuery, CustomizationDelegates, false);
			for (FOnGetVariableCustomizationInstance* CustomizationDelegate : CustomizationDelegates)
			{
				if (CustomizationDelegate && CustomizationDelegate->IsBound())
				{
					TSharedPtr<IDetailCustomization> Customization = CustomizationDelegate->Execute(InBlueprintEditor);
					if (Customization.IsValid())
					{
						DetailsCustomizations.Add(Customization);
					}
				}
			}
		}
	}

	return DetailsCustomizations;
}

TArray<TSharedPtr<IDetailCustomization>> FBlueprintEditorModule::CustomizeGraph(const UEdGraphSchema* InGraphSchema, TSharedPtr<IBlueprintEditor> InBlueprintEditor)
{
	TArray<TSharedPtr<IDetailCustomization>> DetailsCustomizations;
	TArray<UClass*> ParentSchemaClassesToQuery;
	if (InGraphSchema)
	{
		UClass* GraphSchemaClass = InGraphSchema->GetClass();
		ParentSchemaClassesToQuery.Add(InGraphSchema->GetClass());

		UClass* ParentSchemaClass = GraphSchemaClass->GetSuperClass();
		while (ParentSchemaClass && ParentSchemaClass->IsChildOf(UEdGraphSchema::StaticClass()))
		{
			ParentSchemaClassesToQuery.Add(ParentSchemaClass);
			ParentSchemaClass = ParentSchemaClass->GetSuperClass();
		}

		for (UClass* ClassToQuery : ParentSchemaClassesToQuery)
		{
			UEdGraphSchema* SchemaToQuery = CastChecked<UEdGraphSchema>(ClassToQuery->GetDefaultObject());
			FOnGetGraphCustomizationInstance* CustomizationDelegate = GraphCustomizations.Find(SchemaToQuery);
			if (CustomizationDelegate && CustomizationDelegate->IsBound())
			{
				TSharedPtr<IDetailCustomization> Customization = CustomizationDelegate->Execute(InBlueprintEditor);
				if(Customization.IsValid())
				{ 
					DetailsCustomizations.Add(Customization);
				}
			}
		}
	}

	return DetailsCustomizations;
}

TArray<TSharedPtr<IDetailCustomization>> FBlueprintEditorModule::CustomizeFunction(const TSubclassOf<UK2Node_EditablePinBase> InFunctionClass, TSharedPtr<IBlueprintEditor> InBlueprintEditor)
{
	TArray<TSharedPtr<IDetailCustomization>> DetailsCustomizations;
	TArray<UClass*> ParentPinClassesToQuery;
	if (InFunctionClass)
	{
		UClass* FunctionClass = InFunctionClass.Get();
		ParentPinClassesToQuery.Add(FunctionClass);

		UClass* ParentSchemaClass = FunctionClass->GetSuperClass();
		while (ParentSchemaClass && ParentSchemaClass->IsChildOf(UK2Node_EditablePinBase::StaticClass()))
		{
			ParentPinClassesToQuery.Add(ParentSchemaClass);
			ParentSchemaClass = ParentSchemaClass->GetSuperClass();
		}

		for (UClass* ClassToQuery : ParentPinClassesToQuery)
		{
			TArray<FOnGetFunctionCustomizationInstance*, TInlineAllocator<4>> CustomizationDelegates;
			FunctionCustomizations.MultiFindPointer(ClassToQuery, CustomizationDelegates, false);
			for (FOnGetFunctionCustomizationInstance* CustomizationDelegate : CustomizationDelegates)
			{
				if (CustomizationDelegate && CustomizationDelegate->IsBound())
				{
					TSharedPtr<IDetailCustomization> Customization = CustomizationDelegate->Execute(InBlueprintEditor);
					if (Customization.IsValid())
					{
						DetailsCustomizations.Add(Customization);
					}
				}
			}
		}
	}

	return DetailsCustomizations;
}

void FBlueprintEditorModule::PrepareAutoGeneratedDefaultEvents()
{
	// Load up all default events that should be spawned for Blueprints that are children of specific classes
	const FString ConfigSection = TEXT("DefaultEventNodes");
	const FString SettingName = TEXT("Node");
	TArray< FString > NodeSpawns;
	GConfig->GetArray(*ConfigSection, *SettingName, NodeSpawns, GEditorPerProjectIni);

	for(FString CurrentNodeSpawn : NodeSpawns)
	{
		FString TargetClassName;
		if(!FParse::Value(*CurrentNodeSpawn, TEXT("TargetClass="), TargetClassName))
		{
			// Could not find a class name, cannot continue with this line
			continue;
		}

		UClass* FoundTargetClass = FindFirstObject<UClass>(*TargetClassName, EFindFirstObjectOptions::None, ELogVerbosity::Fatal, TEXT("looking for DefaultEventNodes"));
		if(FoundTargetClass)
		{
			FString TargetEventFunction;
			if(!FParse::Value(*CurrentNodeSpawn, TEXT("TargetEvent="), TargetEventFunction))
			{
				// Could not find a class name, cannot continue with this line
				continue;
			}

			FName TargetEventFunctionName(*TargetEventFunction);
			if ( FoundTargetClass->FindFunctionByName(TargetEventFunctionName) )
			{
				FKismetEditorUtilities::RegisterAutoGeneratedDefaultEvent(this, FoundTargetClass, FName(*TargetEventFunction));
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
