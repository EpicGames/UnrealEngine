// Copyright Epic Games, Inc. All Rights Reserved.
#include "BlueprintEditorSCSEditorUICustomization.h"

#include "SubobjectData.h"
#include "SubobjectDataHandle.h"

TSharedPtr<FFBlueprintEditorSCSEditorUICustomization> FFBlueprintEditorSCSEditorUICustomization::Instance;

TSharedPtr<FFBlueprintEditorSCSEditorUICustomization> FFBlueprintEditorSCSEditorUICustomization::GetInstance()
{
	if (!Instance)
	{
		Instance = MakeShareable(new FFBlueprintEditorSCSEditorUICustomization());
	}

	return Instance;
}

bool FFBlueprintEditorSCSEditorUICustomization::HideComponentsTree(TArrayView<UObject*> Context) const
{
	for (const TSharedPtr<ISCSEditorUICustomization>& Customization : Customizations)
	{
		if (Customization->HideComponentsTree(Context))
		{
			return true;
		}
	}

	return ISCSEditorUICustomization::HideComponentsTree(Context);
}

bool FFBlueprintEditorSCSEditorUICustomization::HideComponentsFilterBox(TArrayView<UObject*> Context) const
{
	for (const TSharedPtr<ISCSEditorUICustomization>& Customization : Customizations)
	{
		if (Customization->HideComponentsFilterBox(Context))
		{
			return true;
		}
	}

	return ISCSEditorUICustomization::HideComponentsFilterBox(Context);
}

bool FFBlueprintEditorSCSEditorUICustomization::HideAddComponentButton(TArrayView<UObject*> Context) const
{
	for (const TSharedPtr<ISCSEditorUICustomization>& Customization : Customizations)
	{
		if (Customization->HideAddComponentButton(Context))
		{
			return true;
		}
	}

	return ISCSEditorUICustomization::HideAddComponentButton(Context);
}

bool FFBlueprintEditorSCSEditorUICustomization::HideBlueprintButtons(TArrayView<UObject*> Context) const
{
	for (const TSharedPtr<ISCSEditorUICustomization>& Customization : Customizations)
	{
		if (Customization->HideBlueprintButtons(Context))
		{
			return true;
		}
	}

	return ISCSEditorUICustomization::HideBlueprintButtons(Context);
}

const FSlateBrush* FFBlueprintEditorSCSEditorUICustomization::GetIconBrush(const FSubobjectData& SubobjectData) const
{
	for (const TSharedPtr<ISCSEditorUICustomization>& Customization : Customizations)
	{
		if (const FSlateBrush* IconBrush = Customization->GetIconBrush(SubobjectData))
		{
			return IconBrush;
		}
	}

	return ISCSEditorUICustomization::GetIconBrush(SubobjectData);
}

TSharedPtr<SWidget> FFBlueprintEditorSCSEditorUICustomization::GetControlsWidget(TSharedRef<SSubobjectEditor>& SubobjectEditor, const FSubobjectData& SubobjectData) const
{
	for (const TSharedPtr<ISCSEditorUICustomization>& Customization : Customizations)
	{
		if (TSharedPtr<SWidget> ControlsWidget = Customization->GetControlsWidget(SubobjectEditor, SubobjectData))
		{
			return ControlsWidget;
		}
	}

	return ISCSEditorUICustomization::GetControlsWidget(SubobjectEditor, SubobjectData);
}

EChildActorComponentTreeViewVisualizationMode FFBlueprintEditorSCSEditorUICustomization::GetChildActorVisualizationMode() const
{
	for (const TSharedPtr<ISCSEditorUICustomization>& Customization : Customizations)
	{
		EChildActorComponentTreeViewVisualizationMode VisualizationMode = Customization->GetChildActorVisualizationMode();
		if (VisualizationMode != EChildActorComponentTreeViewVisualizationMode::UseDefault)
		{
			return VisualizationMode;
		}
	}

	return ISCSEditorUICustomization::GetChildActorVisualizationMode();
}

TSubclassOf<UActorComponent> FFBlueprintEditorSCSEditorUICustomization::GetComponentTypeFilter(TArrayView<UObject*> Context) const
{
	for (const TSharedPtr<ISCSEditorUICustomization>& Customization : Customizations)
	{
		if(TSubclassOf<UActorComponent> SubClass = Customization->GetComponentTypeFilter(Context))
		{
			return SubClass;
		}
	}

	return ISCSEditorUICustomization::GetComponentTypeFilter(Context);
}

void FFBlueprintEditorSCSEditorUICustomization::AddCustomization(TSharedPtr<ISCSEditorUICustomization> Customization)
{
	Customizations.AddUnique(Customization);
}

void FFBlueprintEditorSCSEditorUICustomization::RemoveCustomization(TSharedPtr<ISCSEditorUICustomization> Customization)
{
	Customizations.Remove(Customization);
}

bool FFBlueprintEditorSCSEditorUICustomization::SortSubobjectData(TArray<FSubobjectDataHandle>& SubobjectData)
{
	for (const TSharedPtr<ISCSEditorUICustomization>& Customization : Customizations)
	{
		if (Customization->SortSubobjectData(SubobjectData))
		{
			return true;
		}
	}

	return ISCSEditorUICustomization::SortSubobjectData(SubobjectData);
}

FFBlueprintEditorSCSEditorUICustomization::FFBlueprintEditorSCSEditorUICustomization()
{
	Customizations.Add(FFBlueprintEditorDefaultSortUICustomization::GetInstance());
}

TSharedPtr<FFBlueprintEditorDefaultSortUICustomization> FFBlueprintEditorDefaultSortUICustomization::Instance;

TSharedPtr<FFBlueprintEditorDefaultSortUICustomization> FFBlueprintEditorDefaultSortUICustomization::GetInstance()
{
	if (!Instance)
	{
		Instance = MakeShareable(new FFBlueprintEditorDefaultSortUICustomization());
	}

	return Instance;
}

bool FFBlueprintEditorDefaultSortUICustomization::SortSubobjectData(TArray<FSubobjectDataHandle>& SubobjectData)
{
	// Sort subobjects by parentage, type (always put scene components first in the tree), then display name
	SubobjectData.Sort([](const FSubobjectDataHandle& A, const FSubobjectDataHandle& B)
	{
		// If A is a child of B, sort B ahead
		if (A.GetData()->GetParentData() == B.GetData())
		{
			return false;
		}

		// Scene components should sort ahead of non-scene components
		if (A.GetData()->IsSceneComponent())
		{
			// If both are scene components, sort by display name
			if (B.GetData()->IsSceneComponent())
			{
				return A.GetData()->GetDisplayName().CompareTo(B.GetData()->GetDisplayName()) < 0;
			}

			return !B.GetData()->IsActor();
		}

		// Actors should go next
		if (A.GetData()->IsActor())
		{
			// If both are actors, sort by display name
			if (B.GetData()->IsActor())
			{
				return A.GetData()->GetDisplayName().CompareTo(B.GetData()->GetDisplayName()) < 0;
			}

			return true;
		}

		// Otherwise just sort by display name
		return A.GetData()->GetDisplayName().CompareTo(B.GetData()->GetDisplayName()) < 0;
	});

	return true;
}


