// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISCSEditorUICustomization.h"
#include "Containers/Array.h"
#include "Templates/SharedPointer.h"

class FFBlueprintEditorSCSEditorUICustomization : public ISCSEditorUICustomization
{
public:
	static TSharedPtr<FFBlueprintEditorSCSEditorUICustomization> GetInstance();
	
	virtual ~FFBlueprintEditorSCSEditorUICustomization() override = default;

	virtual bool HideComponentsTree(TArrayView<UObject*> Context) const override;

	virtual bool HideComponentsFilterBox(TArrayView<UObject*> Context) const override;

	virtual bool HideAddComponentButton(TArrayView<UObject*> Context) const override;

	virtual bool HideBlueprintButtons(TArrayView<UObject*> Context) const override;

	virtual const FSlateBrush* GetIconBrush(const FSubobjectData&) const override;
	virtual TSharedPtr<SWidget> GetControlsWidget(TSharedRef<SSubobjectEditor>& SubobjectEditor, const FSubobjectData&) const override;

	virtual EChildActorComponentTreeViewVisualizationMode GetChildActorVisualizationMode() const override;

	virtual TSubclassOf<UActorComponent> GetComponentTypeFilter(TArrayView<UObject*> Context) const override;

	void AddCustomization(TSharedPtr<ISCSEditorUICustomization> Customization);
	void RemoveCustomization(TSharedPtr<ISCSEditorUICustomization> Customization);

	virtual bool SortSubobjectData(TArray<FSubobjectDataHandle>& SubobjectData) override;

private:
	FFBlueprintEditorSCSEditorUICustomization();

	static TSharedPtr<FFBlueprintEditorSCSEditorUICustomization> Instance;
	TArray<TSharedPtr<ISCSEditorUICustomization>> Customizations;
};

class FFBlueprintEditorDefaultSortUICustomization : public ISCSEditorUICustomization
{
public:
	static TSharedPtr<FFBlueprintEditorDefaultSortUICustomization> GetInstance();

	virtual bool SortSubobjectData(TArray<FSubobjectDataHandle>& SubobjectData) override;

private:
	FFBlueprintEditorDefaultSortUICustomization() = default;

	static TSharedPtr<FFBlueprintEditorDefaultSortUICustomization> Instance;
};