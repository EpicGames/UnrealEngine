// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintEditorTabFactories.h"

#include "BlueprintEditor.h"
#include "BlueprintEditorSharedTabFactories.h"
#include "BlueprintEditorTabs.h"
#include "Components/ActorComponent.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Blueprint.h"
#include "Fonts/SlateFontInfo.h"
#include "GameFramework/Actor.h"
#include "GraphEditor.h"
#include "HAL/PlatformCrt.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "SBlueprintBookmarks.h"
#include "SBlueprintPalette.h"
#include "SMyBlueprint.h"
#include "SlotBase.h"
#include "SReplaceNodeReferences.h"
#include "SSCSEditorViewport.h"
#include "STimelineEditor.h"
#include "SSubobjectEditor.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateEnums.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "BlueprintEditor"

void FGraphEditorSummoner::OnTabActivated(TSharedPtr<SDockTab> Tab) const
{
	TSharedRef<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(Tab->GetContent());
	BlueprintEditorPtr.Pin()->OnGraphEditorFocused(GraphEditor);
}

void FGraphEditorSummoner::OnTabBackgrounded(TSharedPtr<SDockTab> Tab) const
{
	TSharedRef<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(Tab->GetContent());
	BlueprintEditorPtr.Pin()->OnGraphEditorBackgrounded(GraphEditor);
}

void FGraphEditorSummoner::OnTabRefreshed(TSharedPtr<SDockTab> Tab) const
{
	TSharedRef<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(Tab->GetContent());
	GraphEditor->NotifyGraphChanged();
}

void FGraphEditorSummoner::SaveState(TSharedPtr<SDockTab> Tab, TSharedPtr<FTabPayload> Payload) const
{
	TSharedRef<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(Tab->GetContent());

	FVector2f ViewLocation;
	float ZoomAmount;
	GraphEditor->GetViewLocation(ViewLocation, ZoomAmount);

	UEdGraph* Graph = Payload->IsValid() ? FTabPayload_UObject::CastChecked<UEdGraph>(Payload) : nullptr;

	if (Graph && BlueprintEditorPtr.Pin()->IsGraphInCurrentBlueprint(Graph))
	{
		// Don't save references to external graphs.
		BlueprintEditorPtr.Pin()->GetBlueprintObj()->LastEditedDocuments.Add(FEditedDocumentInfo(Graph, ViewLocation, ZoomAmount));
	}
}

FGraphEditorSummoner::FGraphEditorSummoner(TSharedPtr<class FBlueprintEditor> InBlueprintEditorPtr, FOnCreateGraphEditorWidget CreateGraphEditorWidgetCallback) : FDocumentTabFactoryForObjects<UEdGraph>(FBlueprintEditorTabs::GraphEditorID, InBlueprintEditorPtr)
, BlueprintEditorPtr(InBlueprintEditorPtr)
, OnCreateGraphEditorWidget(CreateGraphEditorWidgetCallback)
{

}

TSharedRef<SWidget> FGraphEditorSummoner::CreateTabBodyForObject(const FWorkflowTabSpawnInfo& Info, UEdGraph* DocumentID) const
{
	check(Info.TabInfo.IsValid());
	return OnCreateGraphEditorWidget.Execute(Info.TabInfo.ToSharedRef(), DocumentID);
}

const FSlateBrush* FGraphEditorSummoner::GetTabIconForObject(const FWorkflowTabSpawnInfo& Info, UEdGraph* DocumentID) const
{
	return FBlueprintEditor::GetGlyphForGraph(DocumentID, false);
}

TSharedRef<FGenericTabHistory> FGraphEditorSummoner::CreateTabHistoryNode(TSharedPtr<FTabPayload> Payload)
{
	return MakeShareable(new FGraphTabHistory(SharedThis(this), Payload));
}

void FTimelineEditorSummoner::OnTabRefreshed(TSharedPtr<SDockTab> Tab) const
{
	TSharedRef<STimelineEditor> TimelineEditor = StaticCastSharedRef<STimelineEditor>(Tab->GetContent());
	TimelineEditor->OnTimelineChanged();
}

FTimelineEditorSummoner::FTimelineEditorSummoner(TSharedPtr<class FBlueprintEditor> InBlueprintEditorPtr)
	: FDocumentTabFactoryForObjects<UTimelineTemplate>(FBlueprintEditorTabs::TimelineEditorID, InBlueprintEditorPtr)
, BlueprintEditorPtr(InBlueprintEditorPtr)
{

}

TSharedRef<SWidget> FTimelineEditorSummoner::CreateTabBodyForObject(const FWorkflowTabSpawnInfo& Info, UTimelineTemplate* DocumentID) const
{
	return SNew(STimelineEditor, BlueprintEditorPtr.Pin(), DocumentID);
}

const FSlateBrush* FTimelineEditorSummoner::GetTabIconForObject(const FWorkflowTabSpawnInfo& Info, UTimelineTemplate* DocumentID) const
{
	return FAppStyle::GetBrush("GraphEditor.Timeline_16x");
}

void FTimelineEditorSummoner::SaveState(TSharedPtr<SDockTab> Tab, TSharedPtr<FTabPayload> Payload) const
{
	UTimelineTemplate* Timeline = FTabPayload_UObject::CastChecked<UTimelineTemplate>(Payload);
	BlueprintEditorPtr.Pin()->GetBlueprintObj()->LastEditedDocuments.Add(FEditedDocumentInfo(Timeline));
}

TAttribute<FText> FTimelineEditorSummoner::ConstructTabNameForObject(UTimelineTemplate* DocumentID) const
{
	return TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&FLocalKismetCallbacks::GetObjectName, (UObject*)DocumentID));
}

FDefaultsEditorSummoner::FDefaultsEditorSummoner(TSharedPtr<FBlueprintEditor> InHostingApp)
	: FWorkflowTabFactory(FBlueprintEditorTabs::DefaultEditorID, InHostingApp)
	, EditingBlueprint(InHostingApp->GetBlueprintObj())
{
	TabLabel = LOCTEXT("ClassDefaultsTabTitle", "Class Defaults");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.BlueprintDefaults");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("DefaultEditorView", "Defaults");
	ViewMenuTooltip = LOCTEXT("DefaultEditorView_ToolTip", "Shows the default editor view");
}

TSharedRef<SWidget> FDefaultsEditorSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FBlueprintEditor> BlueprintEditorPtr = StaticCastSharedPtr<FBlueprintEditor>(HostingApp.Pin());

	TSharedRef<SWidget> Message = CreateOptionalDataOnlyMessage();
	TSharedRef<SWidget> EditableWarning = CreateOptionalEditableWarning();

	return SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(0,0,0,1))
		[
			Message
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(0,0,0,1))
		[
			EditableWarning
		]

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			BlueprintEditorPtr->GetDefaultEditor()
		];
}

TSharedRef<SWidget> FDefaultsEditorSummoner::CreateOptionalEditableWarning() const
{
	bool bHasUneditableBlueprintComponent = false;
	if (EditingBlueprint.IsValid())
	{
		if (EditingBlueprint->GeneratedClass && FBlueprintEditorUtils::IsDataOnlyBlueprint(EditingBlueprint.Get()))
		{
			if (AActor* Actor = Cast<AActor>(EditingBlueprint->GeneratedClass->GetDefaultObject()))
			{
				for (UActorComponent* Component : Actor->GetComponents())
				{
					if (Component && !Component->IsCreatedByConstructionScript())
					{
						UActorComponent* ComponentArchetype = Cast<UActorComponent>(Component->GetArchetype());
						if (ComponentArchetype && !ComponentArchetype->bEditableWhenInherited)
						{
							bHasUneditableBlueprintComponent = true;
							break;
						}
					}
				}
			}
		}
	}

	TSharedRef<SWidget> Message = SNullWidget::NullWidget;
	if (bHasUneditableBlueprintComponent)
	{
		Message = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(2)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.Warning"))
			]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(2)
			[
				SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle("BoldFont"))
				.Text(LOCTEXT("BlueprintUneditableInheritedComponentWarning", "Some properties are not editable due to belonging to a Component flagged as not editable when inherited."))
			];
	}

	return Message;
}

TSharedRef<SWidget> FDefaultsEditorSummoner::CreateOptionalDataOnlyMessage() const
{
	TSharedRef<SWidget> Message = SNullWidget::NullWidget;
	if (EditingBlueprint.IsValid() && FBlueprintEditorUtils::IsDataOnlyBlueprint(EditingBlueprint.Get()))
	{
		Message = SNew(SBorder)
			.Padding(FMargin(5))
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SWrapBox)
				.UseAllottedSize(true)

			+ SWrapBox::Slot()
			[
				SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle("BoldFont"))
				.Text(LOCTEXT("DataOnlyMessage_Part1", "NOTE: This is a data only blueprint, so only the default values are shown.  It does not have any script or variables.  If you want to add some, "))
			]

			+ SWrapBox::Slot()
			[
				SNew(SHyperlink)
				.Style(FAppStyle::Get(), "Common.GotoBlueprintHyperlink")
				.OnNavigate(const_cast<FDefaultsEditorSummoner*>(this), &FDefaultsEditorSummoner::OnChangeBlueprintToNotDataOnly)
				.Text(LOCTEXT("FullEditor", "Open Full Blueprint Editor"))
				.ToolTipText(LOCTEXT("FullEditorToolTip", "This opens the blueprint in the full editor."))
			]
		];
	}

	return Message;
}

void FDefaultsEditorSummoner::OnChangeBlueprintToNotDataOnly()
{
	if (!GEditor || !EditingBlueprint.IsValid())
	{
		return;
	}

	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	if (!AssetEditorSubsystem)
	{
		return;
	}

	AssetEditorSubsystem->CloseAllEditorsForAsset(EditingBlueprint.Get());
	EditingBlueprint->bForceFullEditor = true;
	AssetEditorSubsystem->OpenEditorForAsset(EditingBlueprint.Get());
}

FConstructionScriptEditorSummoner::FConstructionScriptEditorSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp)
	: FWorkflowTabFactory(FBlueprintEditorTabs::ConstructionScriptEditorID, InHostingApp)
{
	TabLabel = LOCTEXT("ComponentsTabLabel", "Components");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.Components");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("ComponentsView", "Components");
	ViewMenuTooltip = LOCTEXT("ComponentsView_ToolTip", "Show the components view");
}

TSharedRef<SWidget> FConstructionScriptEditorSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FBlueprintEditor> BlueprintEditorPtr = StaticCastSharedPtr<FBlueprintEditor>(HostingApp.Pin());

	return BlueprintEditorPtr->GetSubobjectEditor().ToSharedRef();
}

FSCSViewportSummoner::FSCSViewportSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp)
	: FWorkflowTabFactory(FBlueprintEditorTabs::SCSViewportID, InHostingApp)
{
	TabLabel = LOCTEXT("SCSViewportTabLabel", "Viewport");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports");

	bIsSingleton = true;
	TabRole = ETabRole::DocumentTab;

	ViewMenuDescription = LOCTEXT("SCSViewportView", "Viewport");
	ViewMenuTooltip = LOCTEXT("SCSViewportView_ToolTip", "Show the viewport view");
}

TSharedRef<SWidget> FSCSViewportSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FBlueprintEditor> BlueprintEditorPtr = StaticCastSharedPtr<FBlueprintEditor>(HostingApp.Pin());

	TSharedPtr<SWidget> Result;
	if (BlueprintEditorPtr->CanAccessComponentsMode())
	{
		Result = BlueprintEditorPtr->GetSubobjectViewport();
	}

	if (Result.IsValid())
	{
		return Result.ToSharedRef();
	}
	else
	{
		return SNew(SErrorText)
			.BackgroundColor(FLinearColor::Transparent)
			.ErrorText(LOCTEXT("SCSViewportView_Unavailable", "Viewport is not available for this Blueprint."));
	}
}

TSharedRef<SDockTab> FSCSViewportSummoner::SpawnTab(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedRef<SDockTab> Tab = FWorkflowTabFactory::SpawnTab(Info);

	TSharedPtr<FBlueprintEditor> BlueprintEditorPtr = StaticCastSharedPtr<FBlueprintEditor>(HostingApp.Pin());
	BlueprintEditorPtr->GetSubobjectViewport()->SetOwnerTab(Tab);

	return Tab;
}

FPaletteSummoner::FPaletteSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp)
	: FWorkflowTabFactory(FBlueprintEditorTabs::PaletteID, InHostingApp)
{
	TabLabel = LOCTEXT("PaletteTabTitle", "Palette");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.Palette");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("PaletteView", "Palette");
	ViewMenuTooltip = LOCTEXT("PaletteView_ToolTip", "Show palette of all functions and variables");
}

TSharedRef<SWidget> FPaletteSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FBlueprintEditor> BlueprintEditorPtr = StaticCastSharedPtr<FBlueprintEditor>(HostingApp.Pin());

	return BlueprintEditorPtr->GetPalette();
}

FBookmarksSummoner::FBookmarksSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp)
	: FWorkflowTabFactory(FBlueprintEditorTabs::BookmarksID, InHostingApp)
{
	TabLabel = LOCTEXT("BookmarksTabTitle", "Bookmarks");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.Bookmarks");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("BookmarksView", "Bookmarks");
	ViewMenuTooltip = LOCTEXT("BookmarksView_ToolTip", "Show bookmarks associated with this Blueprint");
}

TSharedRef<SWidget> FBookmarksSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FBlueprintEditor> BlueprintEditorPtr = StaticCastSharedPtr<FBlueprintEditor>(HostingApp.Pin());

	return BlueprintEditorPtr->GetBookmarksWidget();
}

FMyBlueprintSummoner::FMyBlueprintSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp)
	: FWorkflowTabFactory(FBlueprintEditorTabs::MyBlueprintID, InHostingApp)
{
	TabLabel = LOCTEXT("MyBlueprintTabLabel", "My Blueprint");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.BlueprintCore");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("MyBlueprintTabView", "My Blueprint");
	ViewMenuTooltip = LOCTEXT("MyBlueprintTabView_ToolTip", "Show the my blueprint view");
}

TSharedRef<SWidget> FMyBlueprintSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FBlueprintEditor> BlueprintEditorPtr = StaticCastSharedPtr<FBlueprintEditor>(HostingApp.Pin());

	return BlueprintEditorPtr->GetMyBlueprintWidget().ToSharedRef();
}

FReplaceNodeReferencesSummoner::FReplaceNodeReferencesSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp)
	: FWorkflowTabFactory(FBlueprintEditorTabs::ReplaceNodeReferencesID, InHostingApp)
{
	TabLabel = LOCTEXT("ReplaceNodeReferences", "Replace References");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.BlueprintCore");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("ReplaceNodeReferences", "Replace References");
	ViewMenuTooltip = LOCTEXT("ReplaceNodeReferences_Tooltip", "Show Replace References");
}

TSharedRef<SWidget> FReplaceNodeReferencesSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FBlueprintEditor> BlueprintEditorPtr = StaticCastSharedPtr<FBlueprintEditor>(HostingApp.Pin());

	return BlueprintEditorPtr->GetReplaceReferencesWidget().ToSharedRef();
}

FCompilerResultsSummoner::FCompilerResultsSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp)
	: FWorkflowTabFactory(FBlueprintEditorTabs::CompilerResultsID, InHostingApp)
{
	TabLabel = LOCTEXT("CompilerResultsTabTitle", "Compiler Results");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.CompilerResults");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("CompilerResultsView", "Compiler Results");
	ViewMenuTooltip = LOCTEXT("CompilerResultsView_ToolTip", "Show compiler results of all functions and variables");
}

TSharedRef<SWidget> FCompilerResultsSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FBlueprintEditor> BlueprintEditorPtr = StaticCastSharedPtr<FBlueprintEditor>(HostingApp.Pin());

	return BlueprintEditorPtr->GetCompilerResults();
}

FFindResultsSummoner::FFindResultsSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp)
	: FWorkflowTabFactory(FBlueprintEditorTabs::FindResultsID, InHostingApp)
{
	TabLabel = LOCTEXT("FindResultsTabTitle", "Find Results");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.FindResults");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("FindResultsView", "Find Results");
	ViewMenuTooltip = LOCTEXT("FindResultsView_ToolTip", "Show find results for searching in this blueprint");
}

TSharedRef<SWidget> FFindResultsSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FBlueprintEditor> BlueprintEditorPtr = StaticCastSharedPtr<FBlueprintEditor>(HostingApp.Pin());

	return BlueprintEditorPtr->GetFindResults();
}

void FGraphTabHistory::EvokeHistory(TSharedPtr<FTabInfo> InTabInfo, bool bPrevTabMatches)
{
	FWorkflowTabSpawnInfo SpawnInfo;
	SpawnInfo.Payload = Payload;
	SpawnInfo.TabInfo = InTabInfo;

	if(bPrevTabMatches)
	{
		TSharedPtr<SDockTab> DockTab = InTabInfo->GetTab().Pin();
		GraphEditor = StaticCastSharedRef<SGraphEditor>(DockTab->GetContent());
	}
	else
	{
		TSharedRef< SGraphEditor > GraphEditorRef = StaticCastSharedRef< SGraphEditor >(FactoryPtr.Pin()->CreateTabBody(SpawnInfo));
		GraphEditor = GraphEditorRef;
		FactoryPtr.Pin()->UpdateTab(InTabInfo->GetTab().Pin(), SpawnInfo, GraphEditorRef);
	}
}

void FGraphTabHistory::SaveHistory()
{
	if (IsHistoryValid())
	{
		check(GraphEditor.IsValid());
		GraphEditor.Pin()->GetViewLocation(SavedLocation, SavedZoomAmount);
		GraphEditor.Pin()->GetViewBookmark(SavedBookmarkId);
	}
}

void FGraphTabHistory::RestoreHistory()
{
	if (IsHistoryValid())
	{
		check(GraphEditor.IsValid());
		GraphEditor.Pin()->SetViewLocation(SavedLocation, SavedZoomAmount, SavedBookmarkId);
	}
}


#undef LOCTEXT_NAMESPACE
