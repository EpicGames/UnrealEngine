// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintEditor.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphSchema.h"
#include "Engine/TimelineTemplate.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"
#include "WorkflowOrientedApp/WorkflowUObjectDocuments.h"

class SDockTab;
class SGraphEditor;
class SWidget;
class UBlueprint;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "BlueprintEditor"

/////////////////////////////////////////////////////
// FLocalKismetCallbacks

struct FLocalKismetCallbacks
{
	static FText GetObjectName(UObject* Object)
	{
		return (Object != NULL) ? FText::FromString( Object->GetName() ) : LOCTEXT("UnknownObjectName", "UNKNOWN");
	}

	static FText GetGraphDisplayName(const UEdGraph* Graph)
	{
		if (Graph)
		{
			if (const UEdGraphSchema* Schema = Graph->GetSchema())
			{
				FGraphDisplayInfo Info;
				Schema->GetGraphDisplayInformation(*Graph, /*out*/ Info);

				return Info.DisplayName;
			}
			else
			{
				// if we don't have a schema, we're dealing with a malformed (or incomplete graph)...
				// possibly in the midst of some transaction - here we return the object's outer path 
				// so we can at least get some context as to which graph we're referring
				return FText::FromString(Graph->GetPathName());
			}
		}

		return LOCTEXT("UnknownGraphName", "UNKNOWN");
	}
};

/////////////////////////////////////////////////////
// FDefaultsEditorSummoner

struct FDefaultsEditorSummoner : public FWorkflowTabFactory
{
public:
	FDefaultsEditorSummoner(TSharedPtr<class FBlueprintEditor> InHostingApp);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override
	{
		return LOCTEXT("DefaultsEditorTooltip", "The class defaults editor lets you set the default value for all variables in your Blueprint.");
	}

private:
	TSharedRef<SWidget> CreateOptionalDataOnlyMessage() const;
	TSharedRef<SWidget> CreateOptionalEditableWarning() const;

	void OnChangeBlueprintToNotDataOnly();

	TWeakObjectPtr<UBlueprint> EditingBlueprint;
};

/////////////////////////////////////////////////////
// FGraphTabHistory

struct FGraphTabHistory : public FGenericTabHistory
{
public:
	/**
	 * @param InFactory		The factory used to regenerate the content
	 * @param InPayload		The payload object used to regenerate the content
	 */
	FGraphTabHistory(TSharedPtr<FDocumentTabFactory> InFactory, TSharedPtr<FTabPayload> InPayload)
		: FGenericTabHistory(InFactory, InPayload)
		, SavedLocation(FVector2f::ZeroVector)
		, SavedZoomAmount(INDEX_NONE)
	{

	}

	virtual void EvokeHistory(TSharedPtr<FTabInfo> InTabInfo, bool bPrevTabMatches) override;

	virtual void SaveHistory() override;

	virtual void RestoreHistory() override;

private:
	/** The graph editor represented by this history node. While this node is inactive, the graph editor is invalid */
	TWeakPtr< class SGraphEditor > GraphEditor;
	/** Saved location the graph editor was at when this history node was last visited */
	FDeprecateSlateVector2D SavedLocation;
	/** Saved zoom the graph editor was at when this history node was last visited */
	float SavedZoomAmount;
	/** Saved bookmark ID the graph editor was at when this history node was last visited */
	FGuid SavedBookmarkId;
};

/////////////////////////////////////////////////////
// FGraphEditorSummoner

struct FGraphEditorSummoner : public FDocumentTabFactoryForObjects<UEdGraph>
{
public:
	DECLARE_DELEGATE_RetVal_TwoParams(TSharedRef<SGraphEditor>, FOnCreateGraphEditorWidget, TSharedRef<FTabInfo>, UEdGraph*);
public:
	FGraphEditorSummoner(TSharedPtr<class FBlueprintEditor> InBlueprintEditorPtr, FOnCreateGraphEditorWidget CreateGraphEditorWidgetCallback);

	virtual void OnTabActivated(TSharedPtr<SDockTab> Tab) const override;

	virtual void OnTabBackgrounded(TSharedPtr<SDockTab> Tab) const override;

	virtual void OnTabRefreshed(TSharedPtr<SDockTab> Tab) const override;

	virtual void SaveState(TSharedPtr<SDockTab> Tab, TSharedPtr<FTabPayload> Payload) const override;

protected:
	virtual TAttribute<FText> ConstructTabNameForObject(UEdGraph* DocumentID) const override
	{
		return TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&FLocalKismetCallbacks::GetGraphDisplayName, (const UEdGraph*)DocumentID));
	}

	virtual TSharedRef<SWidget> CreateTabBodyForObject(const FWorkflowTabSpawnInfo& Info, UEdGraph* DocumentID) const override;

	virtual const FSlateBrush* GetTabIconForObject(const FWorkflowTabSpawnInfo& Info, UEdGraph* DocumentID) const override;

	virtual TSharedRef<FGenericTabHistory> CreateTabHistoryNode(TSharedPtr<FTabPayload> Payload) override;

protected:
	TWeakPtr<class FBlueprintEditor> BlueprintEditorPtr;
	FOnCreateGraphEditorWidget OnCreateGraphEditorWidget;
};

/////////////////////////////////////////////////////
// FTimelineEditorSummoner

struct FTimelineEditorSummoner : public FDocumentTabFactoryForObjects<UTimelineTemplate>
{
public:
	FTimelineEditorSummoner(TSharedPtr<class FBlueprintEditor> InBlueprintEditorPtr);
protected:
	virtual TAttribute<FText> ConstructTabNameForObject(UTimelineTemplate* DocumentID) const override;

	virtual TSharedRef<SWidget> CreateTabBodyForObject(const FWorkflowTabSpawnInfo& Info, UTimelineTemplate* DocumentID) const override;

	virtual const FSlateBrush* GetTabIconForObject(const FWorkflowTabSpawnInfo& Info, UTimelineTemplate* DocumentID) const override;

	virtual void SaveState(TSharedPtr<SDockTab> Tab, TSharedPtr<FTabPayload> Payload) const override;

// 	virtual void OnTabClosed(TSharedRef<SDockTab> Tab, TSharedPtr<FTabPayload> Payload) const override
// 	{
// 		BlueprintEditorPtr.Pin()->RequestSaveEditedObjectState();
// 	}

	virtual void OnTabRefreshed(TSharedPtr<SDockTab> Tab) const override;

	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override
	{
		return LOCTEXT("TimelineEditorTooltip", "The Timeline editor lets you add tracks and visually keyframe values over time, to be used inside a Blueprint graph.");
	}

protected:
	TWeakPtr<class FBlueprintEditor> BlueprintEditorPtr;
};

/////////////////////////////////////////////////////
// FConstructionScriptEditorSummoner

struct FConstructionScriptEditorSummoner : public FWorkflowTabFactory
{
public:
	FConstructionScriptEditorSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override
	{
		return LOCTEXT("ComponentsTabTooltip", "The Components tab is for easily adding, selecting and attaching Components within your Blueprint.");
	}
};

/////////////////////////////////////////////////////
// FConstructionScriptEditorSummoner

struct FSCSViewportSummoner : public FWorkflowTabFactory
{
public:
	FSCSViewportSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual TSharedRef<SDockTab> SpawnTab(const FWorkflowTabSpawnInfo& Info) const override;
};

//////////////////////////////////////////////////////////////////////////
// FPaletteSummoner

struct FPaletteSummoner : public FWorkflowTabFactory
{
public:
	FPaletteSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override
	{
		return LOCTEXT("PaletteTooltip", "The Palette tab provides access to _all_ nodes that can be placed (functions, variables, events etc).");
	}
};

//////////////////////////////////////////////////////////////////////////
// FBookmarksSummoner

struct FBookmarksSummoner : public FWorkflowTabFactory
{
public:
	FBookmarksSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override
	{
		return LOCTEXT("BookmarksTooltip", "The Bookmarks tab allows you to create, edit and select bookmarks associated with this Blueprint.");
	}
};

/////////////////////////////////////////////////////
// FMyBlueprintSummoner

struct FMyBlueprintSummoner : public FWorkflowTabFactory
{
public:
	FMyBlueprintSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override
	{
		return LOCTEXT("MyBlueprintTooltip", "The MyBlueprint tab shows you elements that belong to _this_ Blueprint (variables, graphs etc.)");
	}
};

/////////////////////////////////////////////////////
// FReplaceNodeReferencesSummoner

struct FReplaceNodeReferencesSummoner : public FWorkflowTabFactory
{
public:
	FReplaceNodeReferencesSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override
	{
		return LOCTEXT("ReplaceNodeReferencesTooltip", "Allows the user to replace node references in all Blueprint graphs with another node reference.");
	}
};

//////////////////////////////////////////////////////////////////////////
// FFindResultsSummoner

struct FFindResultsSummoner : public FWorkflowTabFactory
{
public:
	FFindResultsSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override
	{
		return LOCTEXT("FindResultsTooltip", "The find results tab shows results of searching this Blueprint (or all Blueprints).");
	}
};

#undef LOCTEXT_NAMESPACE
