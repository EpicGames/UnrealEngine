// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintEditorTabs.h"

#include "HAL/Platform.h"
#include "UObject/NameTypes.h"

const FName FBlueprintEditorTabs::DetailsID( TEXT( "Inspector" ) );
const FName FBlueprintEditorTabs::DefaultEditorID( TEXT( "DefaultEditor" ) );
const FName FBlueprintEditorTabs::DebugID( TEXT( "Debug" ) );
const FName FBlueprintEditorTabs::PaletteID( TEXT( "PaletteList" ) );
const FName FBlueprintEditorTabs::BookmarksID( TEXT( "BookmarkList" ) );
const FName FBlueprintEditorTabs::CompilerResultsID( TEXT( "CompilerResults" ) );
const FName FBlueprintEditorTabs::FindResultsID( TEXT( "FindResults" ) );
const FName FBlueprintEditorTabs::ConstructionScriptEditorID( TEXT( "ConstructionScriptEditor" ) );
const FName FBlueprintEditorTabs::SCSViewportID( TEXT( "SCSViewport" ) );
const FName FBlueprintEditorTabs::MyBlueprintID( TEXT( "MyBlueprint" ) );
const FName FBlueprintEditorTabs::ReplaceNodeReferencesID( TEXT( "ReplaceNodeReferences" ) );
const FName FBlueprintEditorTabs::BlueprintDebuggerID( TEXT( "BlueprintDebugger" ) );

const FName FBlueprintEditorTabs::GraphEditorID( TEXT( "GraphEditor" ) );
const FName FBlueprintEditorTabs::TimelineEditorID( TEXT( "TimelineEditor" ) );

const FName FBlueprintEditorTabs::UserDefinedStructureID( TEXT( "UserDefinedStructure" ) );
