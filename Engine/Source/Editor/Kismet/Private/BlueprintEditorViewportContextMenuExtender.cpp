// Copyright Epic Games, Inc. All Rights Reserved.


#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "CreateBlueprintFromActorDialog.h"
#include "Delegates/Delegate.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Blueprint.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Selection.h"
#include "Styling/AppStyle.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FUICommandList;

DEFINE_LOG_CATEGORY_STATIC(LogViewportBlueprintMenu, Log, All);

#define LOCTEXT_NAMESPACE "LevelViewportContextMenuBlueprints"

/** Blueprint class info for context menu */
struct FMenuBlueprintClass
{
	/** Name of the class */
	FString Name;

	/** Blueprint for a kismet graph */
	TWeakObjectPtr<UBlueprint> Blueprint;
};

/**
 * Called to edit code for the specified function symbol name
 *
 * @param	Blueprint		Blueprint to edit code for
 */
void EditKismetCodeFor( TWeakObjectPtr<UBlueprint> BlueprintRef )
{
	// Navigate to this function (implemented in Kismet 2)!
	if (UBlueprint* Blueprint = BlueprintRef.Get())
	{
		// Open the blueprint
		// @todo toolkit major: Needs world-centric support (pass in LevelEditor.  See FLevelEditorActionCallbacks::OpenLevelBlueprint)
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset( Blueprint );
	}
	else
	{
		UE_LOGF(LogViewportBlueprintMenu, Warning, "Failed to find blueprint");
	}
}

/**
* Fills in a sub-menu that shows all of the classes that can be edited
*
* @param	MenuBuilder		The sub-menu we're building up
*/
void FillEditCodeMenu( class FMenuBuilder& MenuBuilder, TArray< FMenuBlueprintClass > Classes)
{
	for( int32 CurClassIndex = 0; CurClassIndex < Classes.Num(); ++CurClassIndex )
	{
		FMenuBlueprintClass& CurClass = Classes[ CurClassIndex ];

		FText LabelName = FText::FromString( CurClass.Name );

		const FText ToolTipName = LOCTEXT("EditCodeMenu_ClassToolTip", "Opens this Blueprint in the Blueprint Editor");

		FUIAction UIAction;
		UIAction.ExecuteAction.BindStatic(
			&EditKismetCodeFor,
			CurClass.Blueprint );

		MenuBuilder.AddMenuEntry( LabelName, ToolTipName, FSlateIcon(), UIAction );
	}
}

/**
* Called to recompile the out of date blueprint for the current selection set
*/
void RecompileOutOfDateKismetForSelection()
{
	int32 BlueprintFailures = 0;

	// Run thru all selected actors, looking for out of date blueprints
	FSelectionIterator SelectedActorItr( GEditor->GetSelectedActorIterator() );
	for ( ; SelectedActorItr; ++SelectedActorItr)
	{
		AActor* CurrentActor = Cast<AActor>(*SelectedActorItr);

		UBlueprint* Blueprint = Cast<UBlueprint>(CurrentActor->GetClass()->ClassGeneratedBy);
		if ((Blueprint != NULL) && (!Blueprint->IsUpToDate()))
		{
			FKismetEditorUtilities::CompileBlueprint(Blueprint);
			if (Blueprint->Status == BS_Error)
			{
				++BlueprintFailures;
			}
		}
	}

	if (BlueprintFailures)
	{
		UE_LOGF(LogViewportBlueprintMenu, Warning, "%d blueprints failed to be recompiled", BlueprintFailures);
	}
}

/**
 * Gathers all blueprints for the actors in question, outputting them to the classes array
 */
void GatherBlueprintsForActors( TArray< AActor* >& Actors, TArray< FMenuBlueprintClass >& Classes )
{
	struct Local
	{
		static void AddBlueprint( TArray< FMenuBlueprintClass >& InClasses, const FString& ClassName, UBlueprint* Blueprint = NULL )
		{
			check( !ClassName.IsEmpty() );

			// Check to see if we already have this class name in our list
			FMenuBlueprintClass* FoundClass = NULL;
			for( int32 CurClassIndex = 0; CurClassIndex < InClasses.Num(); ++CurClassIndex )
			{
				FMenuBlueprintClass& CurClass = InClasses[ CurClassIndex ];
				if( CurClass.Name == ClassName )
				{
					FoundClass = &CurClass;
					break;
				}
			}

			// Add a new class to our list if we need to
			if( FoundClass == NULL )
			{
				FoundClass = new( InClasses ) FMenuBlueprintClass();
				FoundClass->Name = ClassName;
				FoundClass->Blueprint = Blueprint;
			}
			else
			{
				check(FoundClass->Blueprint.Get() == Blueprint);
			}
		}
	};


	for( TArray< AActor* >::TIterator It( Actors ); It; ++It )
	{
		AActor* Actor = static_cast<AActor*>( *It );
		checkSlow( Actor->IsA(AActor::StaticClass()) );

		// Grab the class of this actor
		UClass* ActorClass = Actor->GetClass();
		check( ActorClass != NULL );

		// Walk the inheritance hierarchy for this class
		for( UClass* CurClass = ActorClass; CurClass != NULL; CurClass = CurClass->GetSuperClass() )
		{
			if (UBlueprint* Blueprint = Cast<UBlueprint>(CurClass->ClassGeneratedBy))
			{
				// Class was created by a blueprint, so don't offer C++ editing of functions declared in it
				// Instead offer to edit the events and graphs of the blueprint

				Local::AddBlueprint( Classes, CurClass->GetName(), Blueprint );
			}
		}
	}
}

/**
 * Fills the Blueprint menu with extra options
 */
void FillBlueprintOptions(FMenuBuilder& MenuBuilder, TArray<AActor*> SelectedActors)
{
	// Gather Blueprint classes for this actor
	TArray< FMenuBlueprintClass > BlueprintClasses;
	GatherBlueprintsForActors( SelectedActors, BlueprintClasses );

	MenuBuilder.BeginSection("ActorBlueprint", LOCTEXT("BlueprintsHeading", "Blueprints") );

	// Adds the "Create Blueprint..." menu option if valid.
	{
		int NumBlueprintableActors = 0;
		bool IsBlueprintBased = BlueprintClasses.Num() > 1;

		if(!BlueprintClasses.Num())
		{
			for(auto It(SelectedActors.CreateIterator());It;++It)
			{
				AActor* Actor = *It;
				if( FKismetEditorUtilities::CanCreateBlueprintOfClass(Actor->GetClass()))
				{
					NumBlueprintableActors++;
				}
			}
		}

		const bool bCanHarvestComponentsForBlueprint = (!IsBlueprintBased && (NumBlueprintableActors > 0));

		if(bCanHarvestComponentsForBlueprint)
		{
			AActor* ActorOverride = nullptr;
			FUIAction CreateBlueprintAction( FExecuteAction::CreateStatic( &FCreateBlueprintFromActorDialog::OpenDialog, ECreateBlueprintFromActorMode::Harvest, ActorOverride, true ) );
			MenuBuilder.AddMenuEntry(LOCTEXT("CreateBlueprint", "Create Blueprint..."), LOCTEXT("CreateBlueprint_Tooltip", "Harvest Components from Selected Actors and create Blueprint"), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.HarvestBlueprintFromActors"), CreateBlueprintAction);
		}
	}
	
	// Check to see if we have any classes with functions to display
	if( BlueprintClasses.Num() > 0 )
	{
		{
			UBlueprint* FirstBlueprint = BlueprintClasses[0].Blueprint.Get();

			// Determine if the selected objects that have blueprints are all of the same class, and if they are all up to date
			bool bAllAreSameType = true;
			bool bAreAnyNotUpToDate = false;
			for (int32 ClassIndex = 0; ClassIndex < BlueprintClasses.Num(); ++ClassIndex)
			{
				UBlueprint* CurrentBlueprint = BlueprintClasses[ClassIndex].Blueprint.Get();

				bAllAreSameType = bAllAreSameType && (CurrentBlueprint == FirstBlueprint);

				if (CurrentBlueprint != NULL)
				{
					bAreAnyNotUpToDate |= !CurrentBlueprint->IsUpToDate();
				}
			}

			// For a single selected class, we show a top level item (saves 2 clicks); otherwise we show the full hierarchy
			if (bAllAreSameType && (FirstBlueprint != NULL))
			{
				// Shortcut to edit the blueprint directly, saves two clicks
				FUIAction UIAction;
				UIAction.ExecuteAction.BindStatic(
					&EditKismetCodeFor,
					/*Blueprint=*/ MakeWeakObjectPtr(FirstBlueprint) );

				const FText Label = LOCTEXT("EditBlueprint", "Edit Blueprint");
				const FText Description = FText::Format( LOCTEXT("EditBlueprint_ToolTip", "Opens {0} in the Blueprint editor"), FText::FromString( FirstBlueprint->GetName() ) );

				MenuBuilder.AddMenuEntry( Label, Description, FSlateIcon(), UIAction );
			}
			else
			{
				// More than one type of blueprint is selected, so add a sub-menu for "Edit Kismet Code"
				MenuBuilder.AddSubMenu(
					LOCTEXT("EditBlueprintSubMenu", "Edit Blueprint"),
					LOCTEXT("EditBlueprintSubMenu_ToolTip", "Shows Blueprints that can be opened for editing"),
					FNewMenuDelegate::CreateStatic( &FillEditCodeMenu, BlueprintClasses ) );
			}

			// For any that aren't up to date, we offer a compile blueprints button
			if (bAreAnyNotUpToDate)
			{
				// Shortcut to edit the blueprint directly, saves two clicks
				FUIAction UIAction;
				UIAction.ExecuteAction.BindStatic(&RecompileOutOfDateKismetForSelection);

				const FText Label = LOCTEXT("CompileOutOfDateBPs", "Compile Out-of-Date Blueprints");
				const FText Description = LOCTEXT("CompileOutOfDateBPs_ToolTip", "Compiles out-of-date blueprints for selected actors");

				MenuBuilder.AddMenuEntry( Label, Description, FSlateIcon(), UIAction );
			}
		}
	}
	MenuBuilder.EndSection();
}

/**
 * Extends the level viewport context menu with blueprint-specific menu items
 */
TSharedRef<FExtender> ExtendLevelViewportContextMenuForBlueprints(const TSharedRef<FUICommandList> CommandList, TArray<AActor*> SelectedActors)
{
	TSharedPtr<FExtender> Extender = MakeShareable(new FExtender);

	Extender->AddMenuExtension("LevelViewportEdit", EExtensionHook::Before, CommandList,
		FMenuExtensionDelegate::CreateStatic(&FillBlueprintOptions, SelectedActors));

	return Extender.ToSharedRef();
}

#undef LOCTEXT_NAMESPACE
