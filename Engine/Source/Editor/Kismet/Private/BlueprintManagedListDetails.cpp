// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintManagedListDetails.h"

#include "Algo/Sort.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/SlateDelegates.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"

void FBlueprintManagedListDetails::GenerateHeaderRowContent(FDetailWidgetRow& HeaderRow)
{
	TSharedPtr<SWidget> AddItemWidget = MakeAddItemWidget();
	if (!AddItemWidget.IsValid())
	{
		AddItemWidget = SNullWidget::NullWidget;
	}

	HeaderRow
	.NameContent()
	[
		SNew(STextBlock)
		.Text(DisplayOptions.TitleText)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.ToolTipText(DisplayOptions.TitleTooltipText)
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.AutoWidth()
		[
			AddItemWidget.ToSharedRef()
		]
	]
	.FilterString(
		DisplayOptions.TitleText
	)
	.EditCondition(
		DisplayOptions.EditCondition,
		FOnBooleanValueChanged()
	);
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FBlueprintManagedListDetails::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	TArray<FManagedListItem> Items;
	GetManagedListItems(Items);

	Algo::Sort(Items, [](const FManagedListItem& ItemA, const FManagedListItem& ItemB)
	{
		return ItemA.ItemName.Compare(ItemB.ItemName) < 0;
	});

	if (Items.Num() > 0)
	{
		for (const FManagedListItem& Item : Items)
		{
			TSharedPtr<SHorizontalBox> Box;
			ChildrenBuilder.AddCustomRow(FText::GetEmpty())
			[
				SAssignNew(Box, SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(Item.DisplayName)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			]
			.FilterString(
				Item.DisplayName
			)
			.EditCondition(
				DisplayOptions.EditCondition,
				FOnBooleanValueChanged()
			)
			.PropertyHandleList(Item.PropertyHandles);

			if (Item.AssetPtr.IsValid())
			{
				TSharedRef<SWidget> BrowseButton = PropertyCustomizationHelpers::MakeBrowseButton(FSimpleDelegate::CreateLambda([this, AssetPtr = Item.AssetPtr]() -> void
				{
					if (AssetPtr.IsValid())
					{
						GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(AssetPtr.Get());
					}
				}));

				BrowseButton->SetToolTipText(DisplayOptions.BrowseButtonToolTipText);

				Box->AddSlot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(2.0f, 0.0f)
				[
					BrowseButton
				];
			}

			if (Item.bIsRemovable)
			{
				TSharedRef<SWidget> RemoveButton = PropertyCustomizationHelpers::MakeClearButton(FSimpleDelegate::CreateLambda([this, Item]() -> void
				{
					OnRemoveItem(Item);
				}));

				RemoveButton->SetToolTipText(DisplayOptions.RemoveButtonToolTipText);

				Box->AddSlot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(2.0f, 0.0f)
				[
					RemoveButton
				];
			}
		}
	}
	else
	{
		ChildrenBuilder.AddCustomRow(FText::GetEmpty())
		[
			SNew(SBox)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(DisplayOptions.NoItemsLabelText)
				.Font(IDetailLayoutBuilder::GetDetailFontItalic())
			]
		]
		.FilterString(
			DisplayOptions.NoItemsLabelText
		)
		.EditCondition(
			DisplayOptions.EditCondition,
			FOnBooleanValueChanged()
		);
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void FBlueprintManagedListDetails::RegenerateChildContent()
{
	RegenerateChildrenDelegate.ExecuteIfBound();
}