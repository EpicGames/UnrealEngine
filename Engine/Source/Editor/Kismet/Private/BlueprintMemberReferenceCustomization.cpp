// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintMemberReferenceCustomization.h"

#include "BlueprintEditor.h"
#include "BlueprintEditorModule.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailWidgetRow.h"
#include "EdGraph/EdGraph.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Blueprint.h"
#include "Engine/MemberReference.h"
#include "Features/IModularFeatures.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IPropertyAccessEditor.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "K2Node_EditablePinBase.h"
#include "K2Node_FunctionEntry.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "SMyBlueprint.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBox.h"

class UObject;

#define LOCTEXT_NAMESPACE "BlueprintMemberReferenceCustomization"

void FBlueprintMemberReferenceDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InStructCustomizationUtils)
{
	if(IModularFeatures::Get().IsModularFeatureAvailable("PropertyAccessEditor"))
	{
		UBlueprint* Blueprint = MyBlueprint.IsValid() ? MyBlueprint.Pin()->GetBlueprintObj() : nullptr;
		if(Blueprint == nullptr)
		{
			// Try to get the BP from a node that we are member of
			TArray<UObject*> OuterObjects;
			InStructPropertyHandle->GetOuterObjects(OuterObjects);
			if(OuterObjects.Num() > 0)
			{
				if(UK2Node* Node = Cast<UK2Node>(OuterObjects[0]))
				{
					Blueprint = Node->HasValidBlueprint() ? Node->GetBlueprint() : nullptr;
				}
			}
		}

		if(Blueprint == nullptr)
		{
			return;
		}
		
		const bool bAllowOnlyVisibleProperties = InStructPropertyHandle->HasMetaData("AllowOnlyVisibleProperties");
		const bool bFunctionReference = InStructPropertyHandle->HasMetaData("FunctionReference");
		const bool bPropertyReference = InStructPropertyHandle->HasMetaData("PropertyReference");
		if(!bPropertyReference && !bFunctionReference)
		{
			// Only property and function references are supported right now
			return;
		}

		const bool bAllowFunctionLibraryReferences = InStructPropertyHandle->HasMetaData("AllowFunctionLibraries");
		
		// Try to find the prototype function
		const FString& PrototypeFunctionName = InStructPropertyHandle->GetMetaData("PrototypeFunction");
		UFunction* PrototypeFunction = PrototypeFunctionName.IsEmpty() ? nullptr : FindObject<UFunction>(nullptr, *PrototypeFunctionName);

		FString DefaultBindingName = InStructPropertyHandle->HasMetaData("DefaultBindingName") ? InStructPropertyHandle->GetMetaData("DefaultBindingName") : TEXT("NewFunction");

		// Binding widget re-adds the 'On' prefix because of legacy support for UMG events, so we remove it here
		DefaultBindingName.RemoveFromStart(TEXT("On"));

		auto OnGenerateBindingName = [DefaultBindingName]() -> FString
		{
			return DefaultBindingName;
		};
		
		auto OnCanBindProperty = [bPropertyReference, bAllowOnlyVisibleProperties](FProperty* InPropertyOnCanBindProperty, TConstArrayView<FBindingChainElement> InBindingChain)
		{
			if (bPropertyReference)
			{
				if (bAllowOnlyVisibleProperties && InPropertyOnCanBindProperty)
				{
					return InPropertyOnCanBindProperty->HasAnyPropertyFlags(CPF_Edit)
							&& (!InPropertyOnCanBindProperty->HasAnyPropertyFlags(CPF_NativeAccessSpecifierPrivate | CPF_NativeAccessSpecifierProtected)
								|| InPropertyOnCanBindProperty->GetBoolMetaData(FBlueprintMetadata::MD_AllowPrivateAccess));
				}
				return true;
			}
			return false;
		};

		auto OnGoToBinding = [InStructPropertyHandle, Blueprint](FName InPropertyName)
		{
			void* StructData = nullptr;
			const FPropertyAccess::Result Result = InStructPropertyHandle->GetValueData(StructData);
			if(Result == FPropertyAccess::Success)
			{
				check(StructData);
				FMemberReference* MemberReference = static_cast<FMemberReference*>(StructData);
				UFunction* Function = MemberReference->ResolveMember<UFunction>(Blueprint->SkeletonGeneratedClass);
				if(Function)
				{
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Blueprint);
					
					if(IBlueprintEditor* BlueprintEditor = static_cast<IBlueprintEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(Blueprint, true)))
					{
						BlueprintEditor->JumpToHyperlink(Function, false);
						return true;
					}
				}
			}

			return false;
		};
		
		auto OnCanGotoBinding = [InStructPropertyHandle, Blueprint](FName InPropertyName)
		{
			void* StructData = nullptr;
			const FPropertyAccess::Result Result = InStructPropertyHandle->GetValueData(StructData);
			if(Result == FPropertyAccess::Success)
			{
				check(StructData);
				const FMemberReference* MemberReference = static_cast<FMemberReference*>(StructData);
				// Only support to go to functions for now.
				const bool bHasValidFunction = MemberReference->ResolveMember<UFunction>(Blueprint->SkeletonGeneratedClass) != nullptr;
				
				return bHasValidFunction;
			}
			
			return false;
		};
		
		auto OnCanBindFunction = [PrototypeFunction, bFunctionReference](UFunction* InFunction)
		{
			if (PrototypeFunction != nullptr
				&& bFunctionReference)
			{
				return PrototypeFunction->IsSignatureCompatibleWith(InFunction)
					&& FBlueprintEditorUtils::HasFunctionBlueprintThreadSafeMetaData(PrototypeFunction) == FBlueprintEditorUtils::HasFunctionBlueprintThreadSafeMetaData(InFunction);
			}
			
			return false;
		};

		auto OnAddBinding = [InStructPropertyHandle, Blueprint](FName InPropertyName, const TArray<FBindingChainElement>& InBindingChain)
		{
			void* StructData = nullptr;
			const FPropertyAccess::Result Result = InStructPropertyHandle->GetValueData(StructData);
			if(Result == FPropertyAccess::Success)
			{
				InStructPropertyHandle->NotifyPreChange();
				
				check(StructData);
				FMemberReference* MemberReference = static_cast<FMemberReference*>(StructData);
				
				if (UFunction* Function = InBindingChain[0].Field.Get<UFunction>())
				{
					UClass* OwnerClass = Function ? Function->GetOwnerClass() : nullptr;
					bool bSelfContext = false;
					if(OwnerClass != nullptr)
					{
						bSelfContext = (Blueprint->GeneratedClass != nullptr && Blueprint->GeneratedClass->IsChildOf(OwnerClass)) ||
										(Blueprint->SkeletonGeneratedClass != nullptr && Blueprint->SkeletonGeneratedClass->IsChildOf(OwnerClass));
					}
					MemberReference->SetFromField<UFunction>(Function, bSelfContext);
				}
				else if (const FProperty* Property = InBindingChain[0].Field.Get<FProperty>())
				{
					UClass* OwnerClass = Property ? Property->GetOwnerClass() : nullptr;
					bool bSelfContext = false;
					if(OwnerClass != nullptr)
					{
						bSelfContext = (Blueprint->GeneratedClass != nullptr && Blueprint->GeneratedClass->IsChildOf(OwnerClass)) ||
										(Blueprint->SkeletonGeneratedClass != nullptr && Blueprint->SkeletonGeneratedClass->IsChildOf(OwnerClass));
					}
					MemberReference->SetFromField<FProperty>(Property, bSelfContext);
				}

				InStructPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
			}
		};

		auto OnRemoveBinding = [InStructPropertyHandle](FName InPropertyName)
		{
			void* StructData = nullptr;
			const FPropertyAccess::Result Result = InStructPropertyHandle->GetValueData(StructData);
			if(Result == FPropertyAccess::Success)
			{
				InStructPropertyHandle->NotifyPreChange();
				
				check(StructData);
				FMemberReference* MemberReference = static_cast<FMemberReference*>(StructData);
				*MemberReference = FMemberReference();

				InStructPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
			}
		};

		auto CanRemoveBinding = [InStructPropertyHandle](FName InPropertyName)
		{
			void* StructData = nullptr;
			const FPropertyAccess::Result Result = InStructPropertyHandle->GetValueData(StructData);
			if(Result == FPropertyAccess::Success)
			{
				check(StructData);
				FMemberReference* MemberReference = static_cast<FMemberReference*>(StructData);
				return MemberReference->GetMemberName() != NAME_None;
			}
			
			return false;
		};

		auto OnNewFunctionBindingCreated = [PrototypeFunction](UEdGraph* InFunctionGraph, UFunction* InFunction)
		{
			// Ensure newly created function is thread safe 
			if(PrototypeFunction && FBlueprintEditorUtils::HasFunctionBlueprintThreadSafeMetaData(PrototypeFunction))
			{
				TArray<UK2Node_FunctionEntry*> EntryNodes;
				InFunctionGraph->GetNodesOfClass<UK2Node_FunctionEntry>(EntryNodes);
				if(EntryNodes.Num() > 0)
				{
					EntryNodes[0]->MetaData.bThreadSafe = true;
				}
			}
		};

		auto CurrentBindingText = [Blueprint, InStructPropertyHandle]()
		{
			void* StructData = nullptr;
			const FPropertyAccess::Result Result = InStructPropertyHandle->GetValueData(StructData);
			if(Result == FPropertyAccess::Success)
			{
				check(StructData);
				FMemberReference* MemberReference = static_cast<FMemberReference*>(StructData);

				if (UFunction* Function = MemberReference->ResolveMember<UFunction>(Blueprint->SkeletonGeneratedClass))
				{
					return FText::FromName(Function->GetFName());
				}
				else if (const FProperty* Property = MemberReference->ResolveMember<FProperty>(Blueprint->SkeletonGeneratedClass))
				{
					return FText::FromName(Property->GetFName());
				}
				return FText::FromName(MemberReference->GetMemberName());
			}
			else if(Result == FPropertyAccess::MultipleValues)
			{
				return LOCTEXT("MultipleValues", "Multiple Values");
			}

			return FText::GetEmpty();
		};

		auto CurrentBindingToolTipText = [InStructPropertyHandle]()
		{
			void* StructData = nullptr;
			const FPropertyAccess::Result Result = InStructPropertyHandle->GetValueData(StructData);
			if(Result == FPropertyAccess::Success)
			{
				check(StructData);
				FMemberReference* MemberReference = static_cast<FMemberReference*>(StructData);
				return FText::FromName(MemberReference->GetMemberName());
			}
			else if(Result == FPropertyAccess::MultipleValues)
			{
				return LOCTEXT("MultipleValues", "Multiple Values");
			}

			return FText::GetEmpty();
		};

		auto CurrentBindingImage = [Blueprint, InStructPropertyHandle]() -> const FSlateBrush*
		{
			void* StructData = nullptr;
			const FPropertyAccess::Result Result = InStructPropertyHandle->GetValueData(StructData);
			if(Result == FPropertyAccess::Success)
			{
				check(StructData);
				FMemberReference* MemberReference = static_cast<FMemberReference*>(StructData);

				if (MemberReference->ResolveMember<UFunction>(Blueprint->SkeletonGeneratedClass))
				{
					return FAppStyle::GetBrush("GraphEditor.Function_16x");
				}
				else if (MemberReference->ResolveMember<FProperty>(Blueprint->SkeletonGeneratedClass))
				{
					return FAppStyle::GetBrush("Kismet.Tabs.Variables");
				}
			}
			return nullptr;
		};

		auto CurrentBindingColor = [Blueprint, InStructPropertyHandle]() -> FLinearColor
		{
			void* StructData = nullptr;
			const FPropertyAccess::Result Result = InStructPropertyHandle->GetValueData(StructData);
			if(Result == FPropertyAccess::Success)
			{
				check(StructData);
				FMemberReference* MemberReference = static_cast<FMemberReference*>(StructData);

				if (MemberReference->ResolveMember<UFunction>(Blueprint->SkeletonGeneratedClass))
				{
					return FAppStyle::GetSlateColor("Colors.Foreground").GetSpecifiedColor();
				}
				else if (const FProperty* Property = MemberReference->ResolveMember<FProperty>(Blueprint->SkeletonGeneratedClass))
				{
					const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
					check(Schema);
					FEdGraphPinType PinType;
					Schema->ConvertPropertyToPinType(Property, PinType);
					return Schema->GetPinTypeColor(PinType);
				}
			}
			return FAppStyle::GetSlateColor("Colors.Foreground").GetSpecifiedColor();
		};
	
		FPropertyBindingWidgetArgs Args;
		Args.BindableSignature = PrototypeFunction;
		Args.OnGenerateBindingName = FOnGenerateBindingName::CreateLambda(OnGenerateBindingName);
		Args.OnCanBindPropertyWithBindingChain = FOnCanBindPropertyWithBindingChain::CreateLambda(OnCanBindProperty);
		Args.OnGotoBinding = FOnGotoBinding::CreateLambda(OnGoToBinding);
		Args.OnCanGotoBinding = FOnCanGotoBinding::CreateLambda(OnCanGotoBinding);
		Args.OnCanBindFunction = FOnCanBindFunction::CreateLambda(OnCanBindFunction);
		Args.OnCanBindToClass = FOnCanBindToClass::CreateLambda([](UClass* InClass){ return true; });
		Args.OnAddBinding = FOnAddBinding::CreateLambda(OnAddBinding);
		Args.OnRemoveBinding = FOnRemoveBinding::CreateLambda(OnRemoveBinding);
		Args.OnCanRemoveBinding = FOnCanRemoveBinding::CreateLambda(CanRemoveBinding);
		Args.OnNewFunctionBindingCreated = FOnNewFunctionBindingCreated::CreateLambda(OnNewFunctionBindingCreated);
		Args.CurrentBindingText = MakeAttributeLambda(CurrentBindingText);
		Args.CurrentBindingToolTipText = MakeAttributeLambda(CurrentBindingToolTipText);
		Args.CurrentBindingImage = MakeAttributeLambda(CurrentBindingImage);
		Args.CurrentBindingColor = MakeAttributeLambda(CurrentBindingColor);
		Args.bGeneratePureBindings = bPropertyReference;
		Args.bAllowFunctionBindings = bFunctionReference;
		Args.bAllowFunctionLibraryBindings = bAllowFunctionLibraryReferences;
		Args.bAllowPropertyBindings = bPropertyReference;
		Args.bAllowNewBindings = bFunctionReference;
		Args.bAllowArrayElementBindings = false;
		Args.bAllowUObjectFunctions = false;
		Args.bAllowStructFunctions = false;
		Args.bAllowStructMemberBindings = false;

		IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");
		InHeaderRow
		.NameContent()
		[
			InStructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SBox)
			.MaxDesiredWidth(200.0f)
			[
				PropertyAccessEditor.MakePropertyBindingWidget(Blueprint, Args)
			]
		];
	}
}

#undef LOCTEXT_NAMESPACE
