// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintNamespaceHelper.h"

#include "BlueprintEditorSettings.h"
#include "BlueprintNamespacePathTree.h"
#include "BlueprintNamespaceUtilities.h"
#include "ClassViewerFilter.h"
#include "Containers/Set.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "HAL/IConsoleManager.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "SPinTypeSelector.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FClassViewerInitializationOptions;
class UObject;

#define LOCTEXT_NAMESPACE "BlueprintNamespaceHelper"

// ---
// @todo_namespaces - Remove CVar flags/sink below after converting to editable 'config' properties
// ---

static TAutoConsoleVariable<bool> CVarBPEnableNamespaceFilteringFeatures(
	TEXT("BP.EnableNamespaceFilteringFeatures"),
	false,
	TEXT("Enables namespace filtering features in the Blueprint editor (experimental).")
);

static TAutoConsoleVariable<bool> CVarBPEnableNamespaceImportingFeatures(
	TEXT("BP.EnableNamespaceImportingFeatures"),
	false,
	TEXT("Enables namespace importing features in the Blueprint editor (experimental)."));

static TAutoConsoleVariable<bool> CVarBPImportParentClassNamespaces(
	TEXT("BP.ImportParentClassNamespaces"),
	false,
	TEXT("Enables import of parent class namespaces when opening a Blueprint for editing."));

// ---

class FClassViewerNamespaceFilter : public IClassViewerFilter
{
public:
	FClassViewerNamespaceFilter(const FBlueprintNamespaceHelper* InNamespaceHelper)
		: CachedNamespaceHelper(InNamespaceHelper)
	{
	}

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
	{
		if (!CachedNamespaceHelper)
		{
			return true;
		}

		return CachedNamespaceHelper->IsImportedObject(InClass);
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<const IUnloadedBlueprintData> InBlueprint, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
	{
		if (!CachedNamespaceHelper)
		{
			return true;
		}

		FSoftObjectPath ClassPath(InBlueprint->GetClassPathName());
		return CachedNamespaceHelper->IsImportedObject(ClassPath);
	}

private:
	/** Associated namespace helper object. */
	const FBlueprintNamespaceHelper* CachedNamespaceHelper;
};

// ---

class FPinTypeSelectorNamespaceFilter : public IPinTypeSelectorFilter
{
	DECLARE_MULTICAST_DELEGATE(FOnFilterChanged);

public:
	FPinTypeSelectorNamespaceFilter(const FBlueprintNamespaceHelper* InNamespaceHelper)
		: CachedNamespaceHelper(InNamespaceHelper)
	{
	}

	virtual bool ShouldShowPinTypeTreeItem(FPinTypeTreeItem InItem) const override
	{
		if (!CachedNamespaceHelper)
		{
			return true;
		}

		const FEdGraphPinType& PinType = InItem->GetPinTypeNoResolve();

		if (PinType.PinSubCategoryObject.IsValid() && !CachedNamespaceHelper->IsImportedObject(PinType.PinSubCategoryObject.Get()))
		{
			// A pin type whose underlying object is loaded, but not imported.
			return false;
		}
		else
		{
			if (!CachedNamespaceHelper->IsImportedAsset(InItem->GetCachedAssetData()))
			{
				// A pin type whose underlying asset may be either loaded or unloaded, but is not imported.
				return false;
			}
		}

		return true;
	}

private:
	/** Associated namespace helper object. */
	const FBlueprintNamespaceHelper* CachedNamespaceHelper;
};

// ---

FBlueprintNamespaceHelper::FBlueprintNamespaceHelper()
{
	// Instance the path tree used to store/retrieve namespaces.
	NamespacePathTree = MakeUnique<FBlueprintNamespacePathTree>();

	// Add global namespace paths implicitly imported by every Blueprint.
	TSet<FString> GlobalImports;
	FBlueprintNamespaceUtilities::GetSharedGlobalImports(GlobalImports);
	AddNamespaces(GlobalImports);

	// Instance the filters that can be used with type pickers, etc.
	ClassViewerFilter = MakeShared<FClassViewerNamespaceFilter>(this);
	PinTypeSelectorFilter = MakeShared<FPinTypeSelectorNamespaceFilter>(this);
}

FBlueprintNamespaceHelper::~FBlueprintNamespaceHelper()
{
}

void FBlueprintNamespaceHelper::AddBlueprint(const UBlueprint* InBlueprint)
{
	if (!InBlueprint)
	{
		return;
	}

	// Add the default import set for the given Blueprint.
	TSet<FString> DefaultImports;
	FBlueprintNamespaceUtilities::GetDefaultImportsForObject(InBlueprint, DefaultImports);
	AddNamespaces(DefaultImports);

	// Additional namespaces that are explicitly imported by this Blueprint.
	AddNamespaces(InBlueprint->ImportedNamespaces);
}

void FBlueprintNamespaceHelper::AddNamespace(const FString& Namespace)
{
	if (!Namespace.IsEmpty())
	{
		// Add the path corresponding to the given namespace identifier.
		NamespacePathTree->AddPath(Namespace);
	}
}

void FBlueprintNamespaceHelper::RemoveNamespace(const FString& Namespace)
{
	if (!Namespace.IsEmpty())
	{
		// Remove the path corresponding to the given namespace identifier.
		NamespacePathTree->RemovePath(Namespace);
	}
}

bool FBlueprintNamespaceHelper::IsIncludedInNamespaceList(const FString& TestNamespace) const
{
	// Empty namespace == global namespace
	if (TestNamespace.IsEmpty())
	{
		return true;
	}

	// Check to see if X is added, followed by X.Y (which contains X.Y.Z), and so on until we run out of path segments
	const bool bMatchFirstInclusivePath = true;
	TSharedPtr<FBlueprintNamespacePathTree::FNode> PathNode = NamespacePathTree->FindPathNode(TestNamespace, bMatchFirstInclusivePath);

	// Return true if this is a valid path that was explicitly added
	return PathNode.IsValid();
}

bool FBlueprintNamespaceHelper::IsImportedObject(const UObject* InObject) const
{
	// Determine the object's namespace identifier.
	FString Namespace = FBlueprintNamespaceUtilities::GetObjectNamespace(InObject);

	// Return whether or not the namespace was added, explicitly or otherwise.
	return IsIncludedInNamespaceList(Namespace);
}

bool FBlueprintNamespaceHelper::IsImportedObject(const FSoftObjectPath& InObjectPath) const
{
	// Determine the object's namespace identifier.
	FString Namespace = FBlueprintNamespaceUtilities::GetObjectNamespace(InObjectPath);

	// Return whether or not the namespace was added, explicitly or otherwise.
	return IsIncludedInNamespaceList(Namespace);
}

bool FBlueprintNamespaceHelper::IsImportedAsset(const FAssetData& InAssetData) const
{
	// Determine the asset's namespace identifier.
	FString Namespace = FBlueprintNamespaceUtilities::GetAssetNamespace(InAssetData);

	// Return whether or not the namespace was added, explicitly or otherwise.
	return IsIncludedInNamespaceList(Namespace);
}

namespace UE::Editor::Kismet::Private
{
	static void OnUpdateNamespaceEditorFeatureConsoleFlag(IConsoleVariable* InCVar, bool* InValuePtr)
	{
		check(InCVar);

		// Skip if not set by console command; in that case we're updating the flag directly.
		if ((InCVar->GetFlags() & ECVF_SetByMask) != ECVF_SetByConsole)
		{
			return;
		}

		// Update the editor setting (referenced) to match the console variable's new setting.
		check(InValuePtr);
		*InValuePtr = InCVar->GetBool();

		// Refresh the Blueprint editor UI environment in response to the console variable change.
		FBlueprintNamespaceUtilities::RefreshBlueprintEditorFeatures();
	}
}

void FBlueprintNamespaceHelper::RefreshEditorFeatureConsoleFlags()
{
	UBlueprintEditorSettings* BlueprintEditorSettings = GetMutableDefault<UBlueprintEditorSettings>();

	// Register callbacks to respond to flag changes via console.
	static bool bIsInitialized = false;
	if (!bIsInitialized)
	{
		auto InitCVarFlag = [](IConsoleVariable* InCVar, bool& InValueRef)
		{
			using namespace UE::Editor::Kismet::Private;
			InCVar->OnChangedDelegate().AddStatic(&OnUpdateNamespaceEditorFeatureConsoleFlag, &InValueRef);
		};

		InitCVarFlag(CVarBPEnableNamespaceFilteringFeatures.AsVariable(), BlueprintEditorSettings->bEnableNamespaceFilteringFeatures);
		InitCVarFlag(CVarBPEnableNamespaceImportingFeatures.AsVariable(), BlueprintEditorSettings->bEnableNamespaceImportingFeatures);
		InitCVarFlag(CVarBPImportParentClassNamespaces.AsVariable(), BlueprintEditorSettings->bInheritImportedNamespacesFromParentBP);

		bIsInitialized = true;
	}

	// Update console variables to match current Blueprint editor settings.
	static bool bIsUpdating = false;
	if (!bIsUpdating)
	{
		TGuardValue<bool> ScopeGuard(bIsUpdating, true);

		auto SetCVarFlag = [](IConsoleVariable* InCVar, bool& InValueRef)
		{
			InCVar->Set(InValueRef);
		};

		SetCVarFlag(CVarBPEnableNamespaceFilteringFeatures.AsVariable(), BlueprintEditorSettings->bEnableNamespaceFilteringFeatures);
		SetCVarFlag(CVarBPEnableNamespaceImportingFeatures.AsVariable(), BlueprintEditorSettings->bEnableNamespaceImportingFeatures);
		SetCVarFlag(CVarBPImportParentClassNamespaces.AsVariable(), BlueprintEditorSettings->bInheritImportedNamespacesFromParentBP);
	}
}

#undef LOCTEXT_NAMESPACE