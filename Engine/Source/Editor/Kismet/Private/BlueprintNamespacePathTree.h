// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Data type used to store and retrieve Blueprint namespace path components.
 * Note: Namespace identifier strings are expected to be of the form: "X.Y.Z"
 */
struct FBlueprintNamespacePathTree
{
public:
	/** Path separator; can be used for path parsing as well as path construction. */
	inline static const TCHAR PathSeparator[] = TEXT(".");

	/** Path tree node structure. */
	struct FNode : TSharedFromThis<FNode>
	{
		/** If TRUE, this node marks the end of an explicitly-added path string. Allows for "wildcard" paths which are inclusive of all subtrees. */
		bool bIsAddedPath = false;

		/** Maps path component names to any added child nodes (subtrees). */
		TMap<FName, TSharedPtr<struct FNode>> Children;

		/** Find or add the subtree associated with the given path component name as the key. */
		TSharedRef<FNode> FindOrAddChild(const FName& InKey)
		{
			if (const TSharedPtr<FNode>* NodePtr = Children.Find(InKey))
			{
				return NodePtr->ToSharedRef();
			}

			TSharedPtr<FNode> NewNode = MakeShared<FNode>();
			Children.Add(InKey, NewNode);

			return NewNode.ToSharedRef();
		}
	};

	FBlueprintNamespacePathTree()
	{
		// All added path identifier strings are rooted to this node.
		RootNode = MakeShared<FNode>();
	}

	TSharedRef<FNode> GetRootNode() const
	{
		check(RootNode.IsValid());
		return RootNode.ToSharedRef();
	}

	/**
	 * Attempts to locate an added path node that represents the given identifier string.
	 * 
	 * @param InPath					A Blueprint namespace path identifier string (e.g. "X.Y.Z").
	 * @param bMatchFirstInclusivePath	Whether to match on any prefix that represents an explicitly-added path (e.g. "X.Y.*").
	 * 
	 * @return A valid path node if the search was successful.
	 */
	TSharedPtr<FNode> FindPathNode(const FString& InPath, bool bMatchFirstInclusivePath = false) const
	{
		TSharedPtr<FNode> Node = GetRootNode();

		TArray<FString> PathSegments;
		InPath.ParseIntoArray(PathSegments, PathSeparator);
		for (const FString& PathSegment : PathSegments)
		{
			if (const TSharedPtr<FNode>* ChildNodePtr = Node->Children.Find(FName(*PathSegment)))
			{
				Node = *ChildNodePtr;

				if (bMatchFirstInclusivePath && Node->bIsAddedPath)
				{
					break;
				}
			}
			else
			{
				Node = nullptr;
				break;
			}
		}

		return Node;
	}

	/**
	 * Adds the given namespace identifier string as an explicitly-added path.
	 * 
	 * @param InPath	A Blueprint namespace path identifier string (e.g. "X.Y.Z").
	 */
	void AddPath(const FString& InPath)
	{
		TSharedRef<FNode> Node = GetRootNode();

		TArray<FString> PathSegments;
		InPath.ParseIntoArray(PathSegments, PathSeparator);
		for (const FString& PathSegment : PathSegments)
		{
			Node = Node->FindOrAddChild(FName(*PathSegment));
		}

		Node->bIsAddedPath = true;
	}

	/**
	 * Removes the given namespace identifier string as an explicitly-added path.
	 *
	 * @param InPath	A Blueprint namespace path identifier string (e.g. "X.Y.Z").
	 */
	void RemovePath(const FString& InPath)
	{
		TArray<FString> PathSegments;
		InPath.ParseIntoArray(PathSegments, PathSeparator);

		using FPathTraceNode = TTuple<FName, TSharedPtr<FNode>>;
		TArray<FPathTraceNode> PathTrace;
		PathTrace.Reserve(PathSegments.Num() + 1);

		TSharedPtr<FNode> Node = GetRootNode();
		PathTrace.Push(MakeTuple(NAME_None, Node));
		for (const FString& PathSegment : PathSegments)
		{
			FName SubPathName = FName(*PathSegment);
			if (const TSharedPtr<FNode>* NodePtr = Node->Children.Find(SubPathName))
			{
				Node = *NodePtr;
				PathTrace.Push(MakeTuple(SubPathName, Node));
			}
		}

		FName LastKey = NAME_None;
		TSharedPtr<FNode> LastNode;
		while (PathTrace.Num() > 0)
		{
			const FPathTraceNode& PathTraceNode = PathTrace.Pop();

			TSharedPtr<FNode> CurrNode = PathTraceNode.Get<1>();
			check(CurrNode.IsValid());
			
			// On the first pass, LastNode will not be valid, so we clear the added flag on the leaf node. On the
			// next pass, the current node equates to the parent of the leaf node, and we remove the leaf node only
			// if it has an empty subtree. All subsequent passes will iterate from there back up to the root node,
			// where at each level we examine the last child node to see if it can be removed (no children and
			// not explicitly added; i.e. - not a wildcard path (e.g. X.Y.*)). This means that we will not be left
			// with a subtree that contains only non-explicitly added paths (i.e. we're also minimizing the tree).
			//
			// Example: Remove path "X.Y.Z" from the following tree:
			//
			//                         Root        <-- The root node is never removed.
			//                          |
			//                          +- X        <-- Node "X" will not be removed even if "Y" is removed, b/c its subtree is still not empty (i.e. "X.W" would still be a valid path). 
			//                             |
			//                             +- Y        <-- Node "Y" will be removed (*unless* "X.Y" was explicitly added) only if it's not left with any children after node "Z" is removed.
			//                             |  |
			//                             |  +- Z        <-- Node "Z" will not be removed if its subtree is not empty. Since this is the leaf node, we clear the "added" flag and shift up.
			//                             |     |
			//                             |     +- T
			//                             +- W
			//
			if (!LastNode.IsValid())
			{
				CurrNode->bIsAddedPath = false;
			}
			else if(!LastNode->bIsAddedPath && LastNode->Children.Num() == 0)
			{
				TSharedPtr<FNode> RemovedNode = CurrNode->Children.FindAndRemoveChecked(LastKey);
				check(RemovedNode == LastNode);
			}

			LastKey = PathTraceNode.Get<0>();
			LastNode = CurrNode;
		}
	}

	/** Path node visitor function signature.
	 * 
	 * @param CurrentPath	Current path (represented as a stack of names).
	 * @param Node			A read-only reference to the node at the current visitor level.
	 */
	typedef TFunctionRef<void(const TArray<FName>& /* CurrentPath */, TSharedRef<const FBlueprintNamespacePathTree::FNode> /* Node */)> FNodeVisitorFunc;

	/**
	 * A utility method that will recursively visit all added nodes.
	 * 
	 * @param VisitorFunc	A function that will be called for each visited node.
	 */
	void ForeachNode(FNodeVisitorFunc VisitorFunc) const
	{
		TArray<FName> CurrentPath;
		RecursiveNodeVisitor(GetRootNode(), CurrentPath, VisitorFunc);
	}

protected:
	/** Helper method for recursively visiting all nodes. */
	void RecursiveNodeVisitor(TSharedPtr<FBlueprintNamespacePathTree::FNode> Node, TArray<FName>& CurrentPath, FNodeVisitorFunc VisitorFunc) const
	{
		for (auto ChildIt = Node->Children.CreateConstIterator(); ChildIt; ++ChildIt)
		{
			CurrentPath.Push(ChildIt.Key());

			TSharedPtr<FNode> ChildNode = ChildIt.Value();
			VisitorFunc(CurrentPath, ChildNode.ToSharedRef());

			RecursiveNodeVisitor(ChildNode, CurrentPath, VisitorFunc);
			CurrentPath.Pop();
		}
	}

private:
	TSharedPtr<FNode> RootNode;
};