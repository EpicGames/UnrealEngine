// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintNamespaceRegistry.h"

#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "BlueprintNamespacePathTree.h"
#include "BlueprintNamespaceUtilities.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/UserDefinedEnum.h"
#include "StructUtils/UserDefinedStruct.h"
#include "HAL/IConsoleManager.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/StringBuilder.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/UObjectIterator.h"

class UObject;

DEFINE_LOG_CATEGORY_STATIC(LogNamespace, Log, All);

FBlueprintNamespaceRegistry::FBlueprintNamespaceRegistry()
	: bIsInitialized(false)
{
	IConsoleManager::Get().RegisterConsoleCommand
	(
		TEXT("BP.ToggleUsePackagePathAsDefaultNamespace"),
		TEXT("Toggle the use of a type's package path as its default namespace when not explicitly assigned. Otherwise, all types default to the global namespace."),
		FConsoleCommandDelegate::CreateRaw(this, &FBlueprintNamespaceRegistry::ToggleDefaultNamespace),
		ECVF_Default
	);

	IConsoleManager::Get().RegisterConsoleCommand
	(
		TEXT("BP.DumpAllRegisteredNamespacePaths"),
		TEXT("Dumps all registered namespace paths."),
		FConsoleCommandDelegate::CreateRaw(this, &FBlueprintNamespaceRegistry::DumpAllRegisteredPaths),
		ECVF_Default
	);
}

FBlueprintNamespaceRegistry::~FBlueprintNamespaceRegistry()
{
	Shutdown();
}

void FBlueprintNamespaceRegistry::Initialize()
{
	if (bIsInitialized)
	{
		return;
	}

	PathTree = MakeUnique<FBlueprintNamespacePathTree>();

	// Skip namespace harvesting if we're not inside an interactive editor context.
	if(GIsEditor && !IsRunningCommandlet())
	{
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
		if (AssetRegistry.IsLoadingAssets())
		{
			OnFilesLoadedDelegateHandle = AssetRegistry.OnFilesLoaded().AddRaw(this, &FBlueprintNamespaceRegistry::OnAssetRegistryFilesLoaded);
		}
		else
		{
			OnAssetRegistryFilesLoaded();
		}

		OnReloadCompleteDelegateHandle = FCoreUObjectDelegates::ReloadCompleteDelegate.AddRaw(this, &FBlueprintNamespaceRegistry::OnReloadComplete);
		OnDefaultNamespaceTypeChangedDelegateHandle = FBlueprintNamespaceUtilities::OnDefaultBlueprintNamespaceTypeChanged().AddRaw(this, &FBlueprintNamespaceRegistry::OnDefaultNamespaceTypeChanged);
	}

	bIsInitialized = true;
}

void FBlueprintNamespaceRegistry::Shutdown()
{
	if (!bIsInitialized)
	{
		return;
	}

	FCoreUObjectDelegates::ReloadCompleteDelegate.Remove(OnReloadCompleteDelegateHandle);
	FBlueprintNamespaceUtilities::OnDefaultBlueprintNamespaceTypeChanged().Remove(OnDefaultNamespaceTypeChangedDelegateHandle);

	if (FModuleManager::Get().IsModuleLoaded(TEXT("AssetRegistry")))
	{
		IAssetRegistry* AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).TryGet();
		if (AssetRegistry)
		{
			AssetRegistry->OnAssetAdded().Remove(OnAssetAddedDelegateHandle);
			AssetRegistry->OnAssetRemoved().Remove(OnAssetRemovedDelegateHandle);
			AssetRegistry->OnAssetRenamed().Remove(OnAssetRenamedDelegateHandle);
			AssetRegistry->OnFilesLoaded().Remove(OnFilesLoadedDelegateHandle);
		}
	}

	bIsInitialized = false;
}

void FBlueprintNamespaceRegistry::OnAssetAdded(const FAssetData& AssetData)
{
	if(const UClass* AssetClass = AssetData.GetClass())
	{
		if (AssetClass->IsChildOf<UBlueprint>()
			|| AssetClass->IsChildOf<UBlueprintGeneratedClass>()
			|| AssetClass->IsChildOf<UUserDefinedEnum>()
			|| AssetClass->IsChildOf<UUserDefinedStruct>()
			|| AssetClass->IsChildOf<UBlueprintFunctionLibrary>())
		{
			RegisterNamespace(AssetData);
		}
	}
}

void FBlueprintNamespaceRegistry::OnAssetRemoved(const FAssetData& AssetData)
{
	// Remove the asset's current namespace if it's non-empty and was previously registered.
	FString AssetNamespace = FBlueprintNamespaceUtilities::GetAssetNamespace(AssetData);
	if (!AssetNamespace.IsEmpty() && IsRegisteredPath(AssetNamespace))
	{
		PathTree->RemovePath(AssetNamespace);

		// Add the removed asset to the exclusion set so we don't re-register it.
		FSoftObjectPath RemovedObjectPath = AssetData.ToSoftObjectPath();
		ExcludedObjectPaths.Add(RemovedObjectPath);

		// Refresh the registry in case the same namespace is in use by another asset.
		FindAndRegisterAllNamespaces();
		
		// Clear the removed asset from the exclusion set.
		ExcludedObjectPaths.Remove(RemovedObjectPath);
	}
}

void FBlueprintNamespaceRegistry::OnAssetRenamed(const FAssetData& AssetData, const FString& InOldPath)
{
	FString OldDefaultNamespacePath;
	FBlueprintNamespaceUtilities::ConvertPackagePathToNamespacePath(InOldPath, OldDefaultNamespacePath);

	// Remove the old path if it was explicitly registered as the default namespace.
	if (IsRegisteredPath(OldDefaultNamespacePath))
	{
		PathTree->RemovePath(OldDefaultNamespacePath);
	}

	// Register the asset's new package path if we are using it as the default namespace.
	if (FBlueprintNamespaceUtilities::GetDefaultBlueprintNamespaceType() == EDefaultBlueprintNamespaceType::UsePackagePathAsDefaultNamespace)
	{
		FString NewDefaultNamespacePath;
		FBlueprintNamespaceUtilities::ConvertPackagePathToNamespacePath(AssetData.PackageName.ToString(), NewDefaultNamespacePath);

		PathTree->AddPath(NewDefaultNamespacePath);
	}
}

void FBlueprintNamespaceRegistry::OnAssetRegistryFilesLoaded()
{
	// Find available namespaces once all assets are registered.
	FindAndRegisterAllNamespaces();

	// Get notified for new asset events that occur going forward.
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	OnAssetAddedDelegateHandle = AssetRegistry.OnAssetAdded().AddRaw(this, &FBlueprintNamespaceRegistry::OnAssetAdded);
	OnAssetRemovedDelegateHandle = AssetRegistry.OnAssetRemoved().AddRaw(this, &FBlueprintNamespaceRegistry::OnAssetRemoved);
	OnAssetRenamedDelegateHandle = AssetRegistry.OnAssetRenamed().AddRaw(this, &FBlueprintNamespaceRegistry::OnAssetRenamed);
}

void FBlueprintNamespaceRegistry::OnReloadComplete(EReloadCompleteReason InReason)
{
	// Rebuild the registry to reflect the appropriate default namespace identifiers for any reloaded types.
	Rebuild();
}

bool FBlueprintNamespaceRegistry::IsInclusivePath(const FString& InPath) const
{
	// A path is considered inclusive if it represents any valid subpath in the tree.
	TSharedPtr<FBlueprintNamespacePathTree::FNode> Node = PathTree->FindPathNode(InPath);
	return Node.IsValid();
}

bool FBlueprintNamespaceRegistry::IsRegisteredPath(const FString& InPath) const
{
	// A path is considered to be registered only if it was explicitly added to the tree.
	TSharedPtr<FBlueprintNamespacePathTree::FNode> Node = PathTree->FindPathNode(InPath);
	return Node.IsValid() && Node->bIsAddedPath;
}

void FBlueprintNamespaceRegistry::GetNamesUnderPath(const FString& InPath, TArray<FName>& OutNames) const
{
	TSharedPtr<FBlueprintNamespacePathTree::FNode> Node = PathTree->FindPathNode(InPath);
	if (Node.IsValid())
	{
		for (auto ChildIt = Node->Children.CreateConstIterator(); ChildIt; ++ChildIt)
		{
			OutNames.Add(ChildIt.Key());
		}
	}
}

void FBlueprintNamespaceRegistry::GetAllRegisteredPaths(TArray<FString>& OutPaths) const
{
	PathTree->ForeachNode([&OutPaths](const TArray<FName>& CurrentPath, TSharedRef<const FBlueprintNamespacePathTree::FNode> Node)
	{
		if (Node->bIsAddedPath)
		{
			// Note: This is not a hard limit on namespace path identifier string length, it's an optimization to try and avoid reallocation during path construction.
			TStringBuilder<128> PathBuilder;
			for (FName PathSegment : CurrentPath)
			{
				if (PathBuilder.Len() > 0)
				{
					PathBuilder += FBlueprintNamespacePathTree::PathSeparator;
				}
				PathBuilder += PathSegment.ToString();
			}

			FString FullPath = PathBuilder.ToString();
			OutPaths.Add(MoveTemp(FullPath));
		}
	});
}

void FBlueprintNamespaceRegistry::FindAndRegisterAllNamespaces()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FBlueprintNamespaceRegistry::FindAndRegisterAllNamespaces);

	// Resolve the subset of excluded objects that are loaded.
	TSet<const UObject*> ExcludedObjects;
	for (const FSoftObjectPath& ExcludedObjectPath : ExcludedObjectPaths)
	{
		if (const UObject* ExcludedObject = ExcludedObjectPath.ResolveObject())
		{
			ExcludedObjects.Add(ExcludedObject);
		}
	}

	// Register loaded class type namespace identifiers.
	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		const UClass* ClassObject = *ClassIt;

		// Include native function library class objects as well.
		const bool bShouldRegisterClassObject =
			UEdGraphSchema_K2::IsAllowableBlueprintVariableType(ClassObject)
			|| ClassObject->IsChildOf<UBlueprintFunctionLibrary>();

		const UBlueprint* BlueprintObject = UBlueprint::GetBlueprintFromClass(ClassObject);

		if (bShouldRegisterClassObject && !ExcludedObjects.Contains(ClassObject) && (!BlueprintObject || !ExcludedObjects.Contains(BlueprintObject)))
		{
			RegisterNamespace(ClassObject);
		}
	}

	// Register loaded macro asset namespace identifiers.
	for (TObjectIterator<UBlueprint> BlueprintIt; BlueprintIt; ++BlueprintIt)
	{
		const UBlueprint* BlueprintObject = *BlueprintIt;

		if (BlueprintObject->BlueprintType == BPTYPE_MacroLibrary && !ExcludedObjects.Contains(BlueprintObject))
		{
			RegisterNamespace(BlueprintObject);
		}
	}

	// Register loaded struct type namespace identifiers.
	for (TObjectIterator<UScriptStruct> StructIt; StructIt; ++StructIt)
	{
		const UScriptStruct* StructObject = *StructIt;
		if (UEdGraphSchema_K2::IsAllowableBlueprintVariableType(StructObject) && !ExcludedObjects.Contains(StructObject))
		{
			RegisterNamespace(StructObject);
		}
	}

	// Register loaded enum type namespace identifiers.
	for (TObjectIterator<UEnum> EnumIt; EnumIt; ++EnumIt)
	{
		const UEnum* EnumObject = *EnumIt;
		if (UEdGraphSchema_K2::IsAllowableBlueprintVariableType(EnumObject) && !ExcludedObjects.Contains(EnumObject))
		{
			RegisterNamespace(EnumObject);
		}
	}

	FARFilter ClassFilter;
	ClassFilter.ClassPaths.Add(UBlueprint::StaticClass()->GetClassPathName());
	ClassFilter.ClassPaths.Add(UBlueprintGeneratedClass::StaticClass()->GetClassPathName());
	ClassFilter.ClassPaths.Add(UUserDefinedStruct::StaticClass()->GetClassPathName());
	ClassFilter.ClassPaths.Add(UUserDefinedEnum::StaticClass()->GetClassPathName());
	ClassFilter.ClassPaths.Add(UBlueprintFunctionLibrary::StaticClass()->GetClassPathName());
	ClassFilter.bRecursiveClasses = true;

	// Register unloaded type namespace identifiers.
	TArray<FAssetData> BlueprintAssets;
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	AssetRegistry.GetAssets(ClassFilter, BlueprintAssets);
	for (const FAssetData& BlueprintAsset : BlueprintAssets)
	{
		if (!BlueprintAsset.IsAssetLoaded() && !ExcludedObjectPaths.Contains(BlueprintAsset.ToSoftObjectPath()))
		{
			RegisterNamespace(BlueprintAsset);
		}
	}
}

void FBlueprintNamespaceRegistry::RegisterNamespace(const FString& InPath)
{
	if (!InPath.IsEmpty())
	{
		PathTree->AddPath(InPath);
	}
}

void FBlueprintNamespaceRegistry::RegisterNamespace(const UObject* InObject)
{
	FString ObjectNamespace = FBlueprintNamespaceUtilities::GetObjectNamespace(InObject);
	RegisterNamespace(ObjectNamespace);
}

void FBlueprintNamespaceRegistry::RegisterNamespace(const FAssetData& AssetData)
{
	FString AssetNamespace = FBlueprintNamespaceUtilities::GetAssetNamespace(AssetData);
	RegisterNamespace(AssetNamespace);
}

void FBlueprintNamespaceRegistry::ToggleDefaultNamespace()
{
	const EDefaultBlueprintNamespaceType OldType = FBlueprintNamespaceUtilities::GetDefaultBlueprintNamespaceType();
	if (OldType == EDefaultBlueprintNamespaceType::DefaultToGlobalNamespace)
	{
		FBlueprintNamespaceUtilities::SetDefaultBlueprintNamespaceType(EDefaultBlueprintNamespaceType::UsePackagePathAsDefaultNamespace);
	}
	else if(OldType == EDefaultBlueprintNamespaceType::UsePackagePathAsDefaultNamespace)
	{
		FBlueprintNamespaceUtilities::SetDefaultBlueprintNamespaceType(EDefaultBlueprintNamespaceType::DefaultToGlobalNamespace);
	}
}

void FBlueprintNamespaceRegistry::DumpAllRegisteredPaths()
{
	if (!bIsInitialized)
	{
		Initialize();
	}

	TArray<FString> AllPaths;
	GetAllRegisteredPaths(AllPaths);

	UE_LOGF(LogNamespace, Log, "=== Registered Blueprint namespace paths:");

	for (const FString& Path : AllPaths)
	{
		UE_LOGF(LogNamespace, Log, "%ls", *Path);
	}

	UE_LOGF(LogNamespace, Log, "=== (end) %d total paths ===", AllPaths.Num());
}

void FBlueprintNamespaceRegistry::OnDefaultNamespaceTypeChanged()
{
	// Rebuild the registry to reflect the appropriate default namespace identifiers for all known types.
	Rebuild();
}

void FBlueprintNamespaceRegistry::Rebuild()
{
	PathTree = MakeUnique<FBlueprintNamespacePathTree>();
	FindAndRegisterAllNamespaces();
}

FBlueprintNamespaceRegistry& FBlueprintNamespaceRegistry::Get()
{
	static FBlueprintNamespaceRegistry* Singleton = new FBlueprintNamespaceRegistry();
	return *Singleton;
}