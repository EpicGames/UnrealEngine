// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintNamespaceUtilities.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "BlueprintEditorModule.h"
#include "BlueprintEditorSettings.h"
#include "BlueprintNamespacePathTree.h"
#include "Containers/Array.h"
#include "EdGraphSchema_K2.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Blueprint.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Modules/ModuleManager.h"
#include "Settings/BlueprintEditorProjectSettings.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "Templates/UnrealTemplate.h"
#include "Toolkits/IToolkit.h"
#include "Toolkits/ToolkitManager.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"

namespace UE::Editor::Kismet::Private
{
	// The default Blueprint namespace to use for objects/assets if not explicitly assigned.
	static EDefaultBlueprintNamespaceType DefaultBlueprintNamespaceType = EDefaultBlueprintNamespaceType::DefaultToGlobalNamespace;

	// Delegate invoked whenever the default Blueprint namespace type is set to a different value.
	static FBlueprintNamespaceUtilities::FOnDefaultBlueprintNamespaceTypeChanged OnDefaultBlueprintNamespaceTypeChangedDelegate;
}

void FBlueprintNamespaceUtilities::ConvertPackagePathToNamespacePath(const FString& InPackagePath, FString& OutNamespacePath)
{
	OutNamespacePath = InPackagePath;
	OutNamespacePath.ReplaceCharInline(TEXT('/'), FBlueprintNamespacePathTree::PathSeparator[0]);
	if (OutNamespacePath.StartsWith(FBlueprintNamespacePathTree::PathSeparator))
	{
		OutNamespacePath.RemoveAt(0);
	}
}

FString FBlueprintNamespaceUtilities::GetAssetNamespace(const FAssetData& InAssetData)
{
	// All assets will default to the global namespace (empty string). This will be returned if no other value is explicitly set.
	FString Namespace;

	if (InAssetData.IsValid())
	{
		using namespace UE::Editor::Kismet::Private;

		// @todo_namespaces - Add cases for unloaded UDS/UDE assets once they have a searchable namespace tag or property.
		FString TagValue;
		if (InAssetData.GetTagValue<FString>(GET_MEMBER_NAME_STRING_CHECKED(UBlueprint, BlueprintNamespace), TagValue))
		{
			Namespace = MoveTemp(TagValue);
		}
		else if (DefaultBlueprintNamespaceType == EDefaultBlueprintNamespaceType::UsePackagePathAsDefaultNamespace)
		{
			ConvertPackagePathToNamespacePath(InAssetData.PackageName.ToString(), Namespace);
		}
	}

	return Namespace;
}

FString FBlueprintNamespaceUtilities::GetObjectNamespace(const UObject* InObject)
{
	// All objects default to the global namespace (empty string). This will be returned if no other paths are set.
	FString Namespace;

	if (const UField* Field = Cast<UField>(InObject))
	{
		const UStruct* OwnerStruct = Field->GetOwnerStruct();

		// If the field's owner is a function (e.g. parameter), continue up the chain until we find the outer class type.
		if (const UFunction* OwnerAsUFunction = Cast<UFunction>(OwnerStruct))
		{
			OwnerStruct = OwnerAsUFunction->GetOwnerClass();
		}

		if (OwnerStruct)
		{
			Field = OwnerStruct;
		}

		const UBlueprint* Blueprint = nullptr;
		if (const UClass* Class = Cast<UClass>(Field))
		{
			Blueprint = UBlueprint::GetBlueprintFromClass(Class);
		}

		if (Blueprint)
		{
			Namespace = GetObjectNamespace(Blueprint);
		}
		else if(const FString * TypeNamespace = Field->FindMetaData(FBlueprintMetadata::MD_Namespace))
		{
			Namespace = *TypeNamespace;
		}
		else
		{
			Namespace = GetObjectNamespace(Field->GetPackage());
		}
	}
	else if (const UBlueprint* Blueprint = Cast<UBlueprint>(InObject))
	{
		if (!Blueprint->BlueprintNamespace.IsEmpty())
		{
			Namespace = Blueprint->BlueprintNamespace;
		}
		else
		{
			Namespace = GetObjectNamespace(Blueprint->GetPackage());
		}
	}
	else if (const UPackage* Package = Cast<UPackage>(InObject))
	{
		using namespace UE::Editor::Kismet::Private;

		if (DefaultBlueprintNamespaceType == EDefaultBlueprintNamespaceType::UsePackagePathAsDefaultNamespace)
		{
			const bool bIsTransientPackage = Package->HasAnyFlags(RF_Transient) || Package == GetTransientPackage();
			if (!bIsTransientPackage)
			{
				ConvertPackagePathToNamespacePath(Package->GetPathName(), Namespace);
			}
		}
	}
	else if (InObject)
	{
		Namespace = GetObjectNamespace(InObject->GetPackage());
	}

	return Namespace;
}

FString FBlueprintNamespaceUtilities::GetObjectNamespace(const FSoftObjectPath& InObjectPath)
{
	const bool bIncludeOnlyOnDiskAssets = false;
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(InObjectPath, bIncludeOnlyOnDiskAssets);
	if (!AssetData.IsValid())
	{
		FString ObjectPathAsString = InObjectPath.ToString();
		if (ObjectPathAsString.RemoveFromEnd(TEXT("_C")))
		{
			AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(ObjectPathAsString), bIncludeOnlyOnDiskAssets);
		}
	}

	return GetAssetNamespace(AssetData);
}

void FBlueprintNamespaceUtilities::GetPropertyValueNamespaces(const FProperty* InProperty, const void* InContainer, TSet<FString>& OutNamespaces)
{
	if (!InProperty || !InContainer)
	{
		return;
	}

	for (int32 ArrayIdx = 0; ArrayIdx < InProperty->ArrayDim; ++ArrayIdx)
	{
		const uint8* ValuePtr = InProperty->ContainerPtrToValuePtr<uint8>(InContainer, ArrayIdx);

		if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
		{
			for (TFieldIterator<FProperty> It(StructProperty->Struct); It; ++It)
			{
				GetPropertyValueNamespaces(*It, ValuePtr, OutNamespaces);
			}
		}
		else if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InProperty))
		{
			FScriptArrayHelper ArrayHelper(ArrayProperty, ValuePtr);
			for (int32 ValueIdx = 0; ValueIdx < ArrayHelper.Num(); ++ValueIdx)
			{
				GetPropertyValueNamespaces(ArrayProperty->Inner, ArrayHelper.GetRawPtr(ValueIdx), OutNamespaces);
			}
		}
		else if (const FSetProperty* SetProperty = CastField<FSetProperty>(InProperty))
		{
			FScriptSetHelper SetHelper(SetProperty, ValuePtr);
			for (FScriptSetHelper::FIterator SetIt = SetHelper.CreateIterator(); SetIt; ++SetIt)
			{
				GetPropertyValueNamespaces(SetProperty->ElementProp, SetHelper.GetElementPtr(SetIt), OutNamespaces);
			}
		}
		else if (const FMapProperty* MapProperty = CastField<FMapProperty>(InProperty))
		{
			FScriptMapHelper MapHelper(MapProperty, ValuePtr);
			for (FScriptMapHelper::FIterator MapIt = MapHelper.CreateIterator(); MapIt; ++MapIt)
			{
				const uint8* MapValuePtr = MapHelper.GetPairPtr(MapIt);
				GetPropertyValueNamespaces(MapProperty->KeyProp, MapValuePtr, OutNamespaces);
				GetPropertyValueNamespaces(MapProperty->ValueProp, MapValuePtr, OutNamespaces);
			}
		}
		else if (const FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(InProperty))
		{
			const FSoftObjectPath& ObjectPath = SoftObjectProperty->GetPropertyValue(ValuePtr).ToSoftObjectPath();
			if (ObjectPath.IsValid())
			{
				FString Namespace = GetObjectNamespace(ObjectPath);
				OutNamespaces.Add(Namespace);
			}
		}
		else if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(InProperty))
		{
			const UObject* ObjectValue = ObjectProperty->GetObjectPropertyValue(ValuePtr);
			if (ObjectValue)
			{
				FString Namespace = GetObjectNamespace(ObjectValue);
				OutNamespaces.Add(Namespace);
			}
		}
	}
}

void FBlueprintNamespaceUtilities::GetSharedGlobalImports(TSet<FString>& OutNamespaces)
{
	// Local editor imports.
	OutNamespaces.Append(GetDefault<UBlueprintEditorSettings>()->NamespacesToAlwaysInclude);

	// Project-wide imports.
	OutNamespaces.Append(GetDefault<UBlueprintEditorProjectSettings>()->NamespacesToAlwaysInclude);

	// Exclude the global namespace (empty) if it was included; this is implied.
	OutNamespaces.Remove(FString());
}

void FBlueprintNamespaceUtilities::GetDefaultImportsForObject(const UObject* InObject, TSet<FString>& OutNamespaces)
{
	if (!InObject)
	{
		return;
	}

	// Get the object's associated namespace (if set).
	FString ObjectNamespace = GetObjectNamespace(InObject);
	if (!ObjectNamespace.IsEmpty())
	{
		OutNamespaces.Add(ObjectNamespace);
	}

	if (const UBlueprint* Blueprint = Cast<UBlueprint>(InObject))
	{
		// Append inherited namespaces from the parent class hierarchy.
		GetDefaultImportsForObject(Blueprint->ParentClass, OutNamespaces);
	}
	else if (const UStruct* StructType = Cast<UStruct>(InObject))
	{
		const bool bAddParentClassImportedNamespaces = GetDefault<UBlueprintEditorSettings>()->bInheritImportedNamespacesFromParentBP;

		UStruct* ParentStruct = StructType->GetSuperStruct();
		while (ParentStruct)
		{
			// Parent type namespace (if set).
			FString ParentTypeNamespace = GetObjectNamespace(ParentStruct);
			if (!ParentTypeNamespace.IsEmpty())
			{
				OutNamespaces.Add(ParentTypeNamespace);
			}

			// If enabled, also include namespaces that are explicitly imported by all ancestor BPs.
			if (bAddParentClassImportedNamespaces)
			{
				if (const UClass* ParentClass = Cast<UClass>(ParentStruct))
				{
					const UBlueprint* ParentBP = UBlueprint::GetBlueprintFromClass(ParentClass);
					if (ParentBP)
					{
						OutNamespaces.Append(ParentBP->ImportedNamespaces);
					}
				}
			}

			ParentStruct = ParentStruct->GetSuperStruct();
		}
	}
}

void FBlueprintNamespaceUtilities::SetDefaultBlueprintNamespaceType(EDefaultBlueprintNamespaceType InType)
{
	using namespace UE::Editor::Kismet::Private;

	if (InType != DefaultBlueprintNamespaceType)
	{
		DefaultBlueprintNamespaceType = InType;

		OnDefaultBlueprintNamespaceTypeChangedDelegate.Broadcast();
	}
}

EDefaultBlueprintNamespaceType FBlueprintNamespaceUtilities::GetDefaultBlueprintNamespaceType()
{
	using namespace UE::Editor::Kismet::Private;
	return DefaultBlueprintNamespaceType;
}

FBlueprintNamespaceUtilities::FOnDefaultBlueprintNamespaceTypeChanged& FBlueprintNamespaceUtilities::OnDefaultBlueprintNamespaceTypeChanged()
{
	using namespace UE::Editor::Kismet::Private;
	return OnDefaultBlueprintNamespaceTypeChangedDelegate;
}

void FBlueprintNamespaceUtilities::RefreshBlueprintEditorFeatures()
{
	if (!GEditor)
	{
		return;
	}

	// Refresh all relevant open Blueprint editor UI elements.
	if (UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
	{
		TArray<UObject*> EditedAssets = AssetEditorSubsystem->GetAllEditedAssets();
		for (UObject* Asset : EditedAssets)
		{
			if (Asset && Asset->IsA<UBlueprint>())
			{
				TSharedPtr<IToolkit> AssetEditorPtr = FToolkitManager::Get().FindEditorForAsset(Asset);
				if (AssetEditorPtr.IsValid() && AssetEditorPtr->IsBlueprintEditor())
				{
					TSharedPtr<IBlueprintEditor> BlueprintEditorPtr = StaticCastSharedPtr<IBlueprintEditor>(AssetEditorPtr);
					BlueprintEditorPtr->RefreshEditors();
					BlueprintEditorPtr->RefreshInspector();
				}
			}
		}
	}
}