// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintPaletteFavorites.h"

#include "BlueprintActionFilter.h"
#include "BlueprintActionMenuItem.h"
#include "BlueprintActionMenuUtils.h"
#include "BlueprintDragDropMenuItem.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "CoreGlobals.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphNode_Comment.h"
#include "EdGraphSchema_K2_Actions.h"
#include "Engine/MemberReference.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "InputCoreTypes.h"
#include "K2Node.h"
#include "K2Node_BaseMCDelegate.h"
#include "K2Node_CallFunction.h"
#include "K2Node_Event.h"
#include "K2Node_IfThenElse.h"
#include "K2Node_InputAction.h"
#include "K2Node_InputAxisEvent.h"
#include "K2Node_InputKey.h"
#include "K2Node_InputTouch.h"
#include "K2Node_MacroInstance.h"
#include "K2Node_MakeArray.h"
#include "K2Node_SpawnActor.h"
#include "K2Node_SpawnActorFromClass.h"
#include "K2Node_Timeline.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Guid.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintPaletteFavorites)

class UEdGraphNode;

/*******************************************************************************
 * Static UBlueprintPaletteFavorites Helpers
 ******************************************************************************/

/** namespace'd to avoid collisions with united builds */
namespace BlueprintPaletteFavoritesImpl
{
	static FString const ConfigSection("BlueprintEditor.Favorites");
	static FString const CustomProfileId("CustomProfile");
	static FString const DefaultProfileConfigKey("DefaultProfile");

	/**
	 * Before we refactored the blueprint menu system, signatures were manually
	 * constructed based off node type, by combining a series of objects and 
	 * names. Here we construct a new FBlueprintNodeSignature from the 
	 * old code (so as to mirror functionality).
	 * 
	 * @param  PaletteAction	The action you want a signature for.
	 * @return A signature object, distinguishing the palette action from others (could also be invalid).
	 */
	static FBlueprintNodeSignature ConstructLegacySignature(const FEdGraphSchemaAction& PaletteAction);
}

//------------------------------------------------------------------------------
static FBlueprintNodeSignature BlueprintPaletteFavoritesImpl::ConstructLegacySignature(const FEdGraphSchemaAction& InPaletteAction)
{
	TSubclassOf<UEdGraphNode> SignatureNodeClass;
	FFieldVariant SignatureSubObject;
	FName SignatureSubObjName;

	FName const ActionId = InPaletteAction.GetTypeId();
	if (ActionId == FEdGraphSchemaAction_K2AddComponent::StaticGetTypeId())
	{
		const FEdGraphSchemaAction_K2AddComponent& AddComponentAction = (const FEdGraphSchemaAction_K2AddComponent&)InPaletteAction;
		checkSlow(AddComponentAction.NodeTemplate != nullptr);

		SignatureNodeClass = AddComponentAction.NodeTemplate->GetClass();
		SignatureSubObject = AddComponentAction.ComponentClass.Get();
	}
	else if (ActionId == FEdGraphSchemaAction_K2AddComment::StaticGetTypeId())
	{
		SignatureNodeClass = UEdGraphNode_Comment::StaticClass();
	}
	else if (ActionId == FEdGraphSchemaAction_K2Delegate::StaticGetTypeId())
	{
		const FEdGraphSchemaAction_K2Delegate& DelegateAction = (const FEdGraphSchemaAction_K2Delegate&)InPaletteAction;

		SignatureNodeClass = UK2Node_BaseMCDelegate::StaticClass();
		SignatureSubObject = DelegateAction.GetDelegateProperty();
	}
	// if we can pull out a node associated with this action
	else if (UK2Node const* NodeTemplate = FBlueprintActionMenuUtils::ExtractNodeTemplateFromAction(InPaletteAction))
	{
		bool bIsSupported = false;
		// now, if we need more info to help identify the node, let's fill 
		// out FieldName/FieldOuter

		// with UK2Node_CallFunction node, we know we can use the function 
		// to discern between them
		if (UK2Node_CallFunction const* CallFuncNode = Cast<UK2Node_CallFunction const>(NodeTemplate))
		{
			SignatureSubObject = CallFuncNode->FunctionReference.ResolveMember<UFunction>(CallFuncNode->GetBlueprintClassFromNode());
			bIsSupported = SignatureSubObject.IsValid();
		}
		else if (UK2Node_InputAxisEvent const* InputAxisEventNode = Cast<UK2Node_InputAxisEvent const>(NodeTemplate))
		{
			SignatureSubObjName = InputAxisEventNode->InputAxisName;
			bIsSupported = (SignatureSubObjName != NAME_None);
		}
		else if (UK2Node_Event const* EventNode = Cast<UK2Node_Event const>(NodeTemplate))
		{
			SignatureSubObject = EventNode->EventReference.ResolveMember<UFunction>(EventNode->GetBlueprintClassFromNode());
			bIsSupported = SignatureSubObject.IsValid();
		}
		else if (UK2Node_MacroInstance const* MacroNode = Cast<UK2Node_MacroInstance const>(NodeTemplate))
		{
			SignatureSubObject = MacroNode->GetMacroGraph();
			bIsSupported = SignatureSubObject.IsValid();
		}
		else if (UK2Node_InputKey const* InputKeyNode = Cast<UK2Node_InputKey const>(NodeTemplate))
		{
			SignatureSubObjName = InputKeyNode->InputKey.GetFName();
			bIsSupported = (SignatureSubObjName != NAME_None);
		}
		else if (UK2Node_InputAction const* InputActionNode = Cast<UK2Node_InputAction const>(NodeTemplate))
		{
			SignatureSubObjName = InputActionNode->InputActionName;
			bIsSupported = (SignatureSubObjName != NAME_None);
		}
		else if (Cast<UK2Node_IfThenElse const>(NodeTemplate) ||
			Cast<UK2Node_MakeArray const>(NodeTemplate) ||
			Cast<UK2Node_SpawnActorFromClass const>(NodeTemplate) ||
			Cast<UK2Node_SpawnActor const>(NodeTemplate) ||
			Cast<UK2Node_Timeline const>(NodeTemplate) ||
			Cast<UK2Node_InputTouch const>(NodeTemplate))
		{
			bIsSupported = true;
		}

		if (bIsSupported)
		{
			SignatureNodeClass = NodeTemplate->GetClass();
		}
	}

	FBlueprintNodeSignature LegacySignatureSet;
	if (SignatureNodeClass != nullptr)
	{
		LegacySignatureSet.SetNodeClass(SignatureNodeClass);
		if (SignatureSubObject.IsValid())
		{
			LegacySignatureSet.AddSubObject(SignatureSubObject);
		}
		else if (SignatureSubObjName != NAME_None)
		{
			LegacySignatureSet.AddKeyValue(SignatureSubObjName.ToString());
		}
	}

	return LegacySignatureSet;
}

/*******************************************************************************
 * FFavoritedBlueprintPaletteItem
 ******************************************************************************/

FFavoritedBlueprintPaletteItem::FFavoritedBlueprintPaletteItem()
{
}

//------------------------------------------------------------------------------
FFavoritedBlueprintPaletteItem::FFavoritedBlueprintPaletteItem(FString const& SerializedAction)
	: ActionSignature(SerializedAction)
{	
}

//------------------------------------------------------------------------------
FFavoritedBlueprintPaletteItem::FFavoritedBlueprintPaletteItem(TSharedPtr<FEdGraphSchemaAction> InPaletteAction)
{
	using namespace BlueprintPaletteFavoritesImpl;

	if (InPaletteAction.IsValid())
	{
		if (InPaletteAction->GetTypeId() == FBlueprintActionMenuItem::StaticGetTypeId())
		{
			FBlueprintActionMenuItem* ActionMenuItem = (FBlueprintActionMenuItem*)InPaletteAction.Get();
			ActionSignature = ActionMenuItem->GetRawAction()->GetSpawnerSignature();
		}
		else if (InPaletteAction->GetTypeId() == FBlueprintDragDropMenuItem::StaticGetTypeId())
		{
			FBlueprintDragDropMenuItem* CollectionMenuItem = (FBlueprintDragDropMenuItem*)InPaletteAction.Get();
			ActionSignature = CollectionMenuItem->GetSampleAction()->GetSpawnerSignature();

			// drag-n-drop menu items represent a collection of actions on the 
			// same field (they spawn a sub-menu for the user to pick from), so
			// they don't have a single node class
			ActionSignature.SetNodeClass(nullptr);

			static const FName CollectionSignatureKey(TEXT("ActionCollection"));
			ActionSignature.AddNamedValue(CollectionSignatureKey, TEXT("true"));
		}
		else
		{
			ActionSignature = BlueprintPaletteFavoritesImpl::ConstructLegacySignature(*InPaletteAction);
		}
	}
}

FFavoritedBlueprintPaletteItem::FFavoritedBlueprintPaletteItem(const FEdGraphSchemaAction& InPaletteAction)
{
	if (InPaletteAction.GetTypeId() == FBlueprintActionMenuItem::StaticGetTypeId())
	{
		const FBlueprintActionMenuItem& ActionMenuItem = (const FBlueprintActionMenuItem&)InPaletteAction;
		ActionSignature = ActionMenuItem.GetRawAction()->GetSpawnerSignature();
	}
	else if (InPaletteAction.GetTypeId() == FBlueprintDragDropMenuItem::StaticGetTypeId())
	{
		const FBlueprintDragDropMenuItem& CollectionMenuItem = (const FBlueprintDragDropMenuItem&)InPaletteAction;
		ActionSignature = CollectionMenuItem.GetSampleAction()->GetSpawnerSignature();

		// drag-n-drop menu items represent a collection of actions on the 
		// same field (they spawn a sub-menu for the user to pick from), so
		// they don't have a single node class
		ActionSignature.SetNodeClass(nullptr);

		static const FName CollectionSignatureKey(TEXT("ActionCollection"));
		ActionSignature.AddNamedValue(CollectionSignatureKey, TEXT("true"));
	}
	else
	{
		ActionSignature = BlueprintPaletteFavoritesImpl::ConstructLegacySignature(InPaletteAction);
	}
}

//------------------------------------------------------------------------------
FFavoritedBlueprintPaletteItem::FFavoritedBlueprintPaletteItem(UBlueprintNodeSpawner const* BlueprintAction)
	: ActionSignature(BlueprintAction->GetSpawnerSignature())
{	
}

//------------------------------------------------------------------------------
bool FFavoritedBlueprintPaletteItem::IsValid() const 
{
	return ActionSignature.IsValid();
}

//------------------------------------------------------------------------------
bool FFavoritedBlueprintPaletteItem::operator==(FFavoritedBlueprintPaletteItem const& Rhs) const
{
	return (ActionSignature.AsGuid() == Rhs.ActionSignature.AsGuid());
}

//------------------------------------------------------------------------------
bool FFavoritedBlueprintPaletteItem::operator==(TSharedPtr<FEdGraphSchemaAction> PaletteAction) const
{
	return PaletteAction.IsValid() && (*this == FFavoritedBlueprintPaletteItem(*PaletteAction));
}

//------------------------------------------------------------------------------
FString const& FFavoritedBlueprintPaletteItem::ToString() const
{
	return ActionSignature.ToString();
}

/*******************************************************************************
 * UBlueprintPaletteFavorites Public Interface
 ******************************************************************************/

//------------------------------------------------------------------------------
UBlueprintPaletteFavorites::UBlueprintPaletteFavorites(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

//------------------------------------------------------------------------------
void UBlueprintPaletteFavorites::PostInitProperties()
{
	Super::PostInitProperties();

	if (CurrentProfile == BlueprintPaletteFavoritesImpl::CustomProfileId)
	{
		LoadCustomFavorites();
	}
	else
	{
		LoadSetProfile();
	}
}

//------------------------------------------------------------------------------
void UBlueprintPaletteFavorites::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	CustomFavorites.Empty();
	if (CurrentProfile == BlueprintPaletteFavoritesImpl::CustomProfileId)
	{
		for (FFavoritedBlueprintPaletteItem const& Favorite : CurrentFavorites) 
		{
			CustomFavorites.Add(Favorite.ToString());
		}
	}

	SaveConfig();
	OnFavoritesUpdated.Broadcast();
}

//------------------------------------------------------------------------------
bool UBlueprintPaletteFavorites::CanBeFavorited(TSharedPtr<FEdGraphSchemaAction> PaletteAction) const
{
	return PaletteAction.IsValid() ? FFavoritedBlueprintPaletteItem(*PaletteAction).IsValid() : false;
}

//------------------------------------------------------------------------------
bool UBlueprintPaletteFavorites::IsFavorited(TSharedPtr<FEdGraphSchemaAction> PaletteAction) const
{
	return PaletteAction.IsValid() ? IsFavorited(*PaletteAction) : false;
}

//------------------------------------------------------------------------------
bool UBlueprintPaletteFavorites::IsFavorited(const FEdGraphSchemaAction& PaletteAction) const
{
	bool bIsFavorited = false;
	if (PaletteAction.GetTypeId() == FBlueprintDragDropMenuItem::StaticGetTypeId())
	{
		const FBlueprintDragDropMenuItem& CollectionMenuItem = (FBlueprintDragDropMenuItem&)PaletteAction;

		bIsFavorited = true;
		for (UBlueprintNodeSpawner const* Action : CollectionMenuItem.GetActionSet())
		{
			if (!IsFavorited(Action))
			{
				bIsFavorited = false;
				break;
			}
		}
	}
	else
	{
		FFavoritedBlueprintPaletteItem ActionAsFavorite(PaletteAction);
		if (ActionAsFavorite.IsValid())
		{
			for (FFavoritedBlueprintPaletteItem const& Favorite : CurrentFavorites)
			{
				if (Favorite == ActionAsFavorite)
				{
					bIsFavorited = true;
					break;
				}
			}
		}
	}
	return bIsFavorited;
}

//------------------------------------------------------------------------------
bool UBlueprintPaletteFavorites::IsFavorited(FBlueprintActionInfo& BlueprintAction) const
{
	return IsFavorited(BlueprintAction.NodeSpawner);
}

//------------------------------------------------------------------------------
bool UBlueprintPaletteFavorites::IsFavorited(UBlueprintNodeSpawner const* BlueprintAction) const
{
	bool bIsFavorited = false;

	FFavoritedBlueprintPaletteItem ActionAsFavorite(BlueprintAction);
	for (FFavoritedBlueprintPaletteItem const& Favorite : CurrentFavorites)
	{
		if (Favorite == ActionAsFavorite)
		{
			bIsFavorited = true;
			break;
		}
	}
	return bIsFavorited;
}

//------------------------------------------------------------------------------
void UBlueprintPaletteFavorites::AddFavorite(TSharedPtr<FEdGraphSchemaAction> PaletteAction)
{
	if (!IsFavorited(PaletteAction) && CanBeFavorited(PaletteAction))
	{
		if (PaletteAction->GetTypeId() == FBlueprintDragDropMenuItem::StaticGetTypeId())
		{
			FBlueprintDragDropMenuItem* CollectionMenuItem = (FBlueprintDragDropMenuItem*)PaletteAction.Get();
			for (UBlueprintNodeSpawner const* Action : CollectionMenuItem->GetActionSet())
			{
				CurrentFavorites.Add(FFavoritedBlueprintPaletteItem(Action));
			}
		}
		else
		{
			CurrentFavorites.Add(FFavoritedBlueprintPaletteItem(*PaletteAction));
		}
		SetProfile(BlueprintPaletteFavoritesImpl::CustomProfileId);
	}
}

//------------------------------------------------------------------------------
void UBlueprintPaletteFavorites::AddFavorites(TArray< TSharedPtr<FEdGraphSchemaAction> > PaletteActions)
{
	for (TSharedPtr<FEdGraphSchemaAction>& NewFave : PaletteActions)
	{
		AddFavorite(NewFave);
	}
}

//------------------------------------------------------------------------------
void UBlueprintPaletteFavorites::RemoveFavorite(TSharedPtr<FEdGraphSchemaAction> PaletteAction)
{
	if (PaletteAction->GetTypeId() == FBlueprintDragDropMenuItem::StaticGetTypeId())
	{
		bool bAnyRemoved = false;

		FBlueprintDragDropMenuItem* CollectionMenuItem = (FBlueprintDragDropMenuItem*)PaletteAction.Get();
		for (UBlueprintNodeSpawner const* Action : CollectionMenuItem->GetActionSet())
		{
			if (IsFavorited(Action))
			{
				CurrentFavorites.Remove(Action);
				bAnyRemoved = true;
			}
		}

		if (bAnyRemoved)
		{
			SetProfile(BlueprintPaletteFavoritesImpl::CustomProfileId);
		}
	}
	else if (IsFavorited(*PaletteAction))
	{
		CurrentFavorites.Remove(*PaletteAction);
		SetProfile(BlueprintPaletteFavoritesImpl::CustomProfileId);
	}
}

//------------------------------------------------------------------------------
void UBlueprintPaletteFavorites::RemoveFavorites(TArray< TSharedPtr<FEdGraphSchemaAction> > PaletteActions)
{
	bool bAnyRemoved = false;
	for (TSharedPtr<FEdGraphSchemaAction>& OldFave : PaletteActions)
	{
		RemoveFavorite(OldFave);
	}
}

//------------------------------------------------------------------------------
void UBlueprintPaletteFavorites::LoadProfile(FString const& ProfileName)
{
	PreEditChange(FindFProperty<FProperty>(GetClass(), TEXT("CurrentProfile")));
	{
		CurrentProfile = ProfileName;
		LoadSetProfile();
	}
	PostEditChange();
}

//------------------------------------------------------------------------------
bool UBlueprintPaletteFavorites::IsUsingCustomProfile() const
{
	return (GetCurrentProfile() == BlueprintPaletteFavoritesImpl::CustomProfileId);
}

//------------------------------------------------------------------------------
FString const& UBlueprintPaletteFavorites::GetCurrentProfile() const
{
	if (CurrentProfile.IsEmpty())
	{
		return GetDefaultProfileId();
	}
	return CurrentProfile;
}

//------------------------------------------------------------------------------
void UBlueprintPaletteFavorites::ClearAllFavorites()
{
	if (CurrentFavorites.Num() > 0) 
	{
		CurrentFavorites.Empty();
		SetProfile(BlueprintPaletteFavoritesImpl::CustomProfileId);
	}
}

/*******************************************************************************
 * UBlueprintPaletteFavorites Private Methods
 ******************************************************************************/

//------------------------------------------------------------------------------
FString const& UBlueprintPaletteFavorites::GetDefaultProfileId() const
{
	static FString DefaultProfileId("DefaultFavorites");
	GConfig->GetString(*BlueprintPaletteFavoritesImpl::ConfigSection, *BlueprintPaletteFavoritesImpl::DefaultProfileConfigKey, DefaultProfileId, GEditorIni);
	return DefaultProfileId;
}

//------------------------------------------------------------------------------
void UBlueprintPaletteFavorites::LoadSetProfile()
{
	CustomFavorites.Empty();
	CurrentFavorites.Empty();

	TArray<FString> ProfileFavorites;
	// if this profile doesn't exist anymore
	if (CurrentProfile.IsEmpty() || (GConfig->GetArray(*BlueprintPaletteFavoritesImpl::ConfigSection, *CurrentProfile, ProfileFavorites, GEditorIni) == 0))
	{
		// @TODO: log warning
		GConfig->GetArray(*BlueprintPaletteFavoritesImpl::ConfigSection, *GetDefaultProfileId(), ProfileFavorites, GEditorIni);
	}

	for (FString const& Favorite : ProfileFavorites)
	{
		FFavoritedBlueprintPaletteItem FavoriteItem(Favorite);
		if (ensure(FavoriteItem.IsValid()))
		{
			CurrentFavorites.Add(FavoriteItem);
		}
	}
}

//------------------------------------------------------------------------------
void UBlueprintPaletteFavorites::LoadCustomFavorites()
{
	CurrentFavorites.Empty();
	check(CurrentProfile == BlueprintPaletteFavoritesImpl::CustomProfileId);

	for (FString const& Favorite : CustomFavorites)
	{
		FFavoritedBlueprintPaletteItem FavoriteItem(Favorite);
		if (ensure(FavoriteItem.IsValid()))
		{
			CurrentFavorites.Add(FavoriteItem);
		}
	}	
}

//------------------------------------------------------------------------------
void UBlueprintPaletteFavorites::SetProfile(FString const& ProfileName)
{
	PreEditChange(FindFProperty<FProperty>(GetClass(), TEXT("CurrentProfile")));
	{
		CurrentProfile = ProfileName;
	}
	PostEditChange();
}
