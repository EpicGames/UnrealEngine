// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintWarningsConfigurationPanel.h"

#include "Blueprint/BlueprintSupport.h"
#include "BlueprintRuntime.h"
#include "BlueprintRuntimeSettings.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Views/ITypedTableView.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "SSettingsEditorCheckoutNotice.h"
#include "Serialization/Archive.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class ITableRow;
class STableViewBase;
class SWidget;
struct FTableRowStyle;

#define LOCTEXT_NAMESPACE "BlueprintWarningConfigurationPanel"

static const TCHAR* ColumnWarningIdentifier = TEXT("WarningIdentifier");
static const TCHAR* ColumnWarningDescription = TEXT("WarningDescription");
static const TCHAR* ColumnWarningBehavior = TEXT("WarningAsError");

typedef SComboBox< TSharedPtr<EBlueprintWarningBehavior> > FBlueprintWarningBehaviorComboBox;

class SBlueprintWarningRow : public SMultiColumnTableRow< FBlueprintWarningListEntry >
{
	SLATE_BEGIN_ARGS(SBlueprintWarningRow) { }
		SLATE_STYLE_ARGUMENT( FTableRowStyle, Style )
	SLATE_END_ARGS()
	
	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, FBlueprintWarningListEntry InWarningInfo, SBlueprintWarningsConfigurationPanel* InParent )
	{
		WarningInfo = InWarningInfo;
		Parent = InParent;

		SMultiColumnTableRow<FBlueprintWarningListEntry>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		if(InColumnName == ColumnWarningIdentifier)
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::FromName(WarningInfo->WarningIdentifier))
				];
		}
		else if (InColumnName == ColumnWarningDescription )
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(WarningInfo->WarningDescription)
				];
		}
		else if (InColumnName == ColumnWarningBehavior )
		{
			const auto& GetWarningText = [this]() -> FText
			{
				UEnum* const BlueprintWarningBehaviorEnum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/BlueprintRuntime.EBlueprintWarningBehavior"));
				EBlueprintWarningBehavior Behavior = EBlueprintWarningBehavior::Warn;
				FName WarningIdentifier = this->WarningInfo->WarningIdentifier;
				if (FBlueprintSupport::ShouldTreatWarningAsError(WarningIdentifier))
				{
					Behavior = EBlueprintWarningBehavior::Error;
				}
				else if (FBlueprintSupport::ShouldSuppressWarning(WarningIdentifier))
				{
					Behavior = EBlueprintWarningBehavior::Suppress;
				}
				return BlueprintWarningBehaviorEnum->GetDisplayNameTextByValue(static_cast<int64>(Behavior));
			};

			return SNew(FBlueprintWarningBehaviorComboBox)
				.Content()
				[
					SNew(STextBlock)
					.Text_Lambda(GetWarningText)
				]
				.OptionsSource(&(Parent->CachedBlueprintWarningBehaviors))
				.OnSelectionChanged(
					FBlueprintWarningBehaviorComboBox::FOnSelectionChanged::CreateLambda(
						[this]( TSharedPtr<EBlueprintWarningBehavior> Behavior, ESelectInfo::Type )
						{
							// apply result to each selected entry:
							this->Parent->UpdateSelectedWarningBehaviors(*Behavior, *(this->WarningInfo));
						}
					)
				)
				.OnGenerateWidget(
					FBlueprintWarningBehaviorComboBox::FOnGenerateWidget::CreateStatic(
						[]( TSharedPtr<EBlueprintWarningBehavior> Behavior )->TSharedRef<SWidget>
						{
							UEnum* const BlueprintWarningBehaviorEnum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/BlueprintRuntime.EBlueprintWarningBehavior"));
							return SNew(STextBlock)
								.Text(BlueprintWarningBehaviorEnum->GetDisplayNameTextByValue(static_cast<int64>(*Behavior)));
						}
					)
				);
		}
		else
		{
			ensure(false);
			return SNew(SBorder);
		}
	}

private:
	FBlueprintWarningListEntry WarningInfo;
	SBlueprintWarningsConfigurationPanel* Parent;
};

void SBlueprintWarningsConfigurationPanel::Construct(const FArguments& InArgs)
{
	IBlueprintRuntime* SettingsModule = FModuleManager::GetModulePtr<IBlueprintRuntime>("BlueprintRuntime");
	if (!SettingsModule)
	{
		return;
	}

	UBlueprintRuntimeSettings* RuntimeSettings = SettingsModule->GetMutableBlueprintRuntimeSettings();

	const TArray<FBlueprintWarningDeclaration>& Warnings = FBlueprintSupport::GetBlueprintWarnings();
	for (const FBlueprintWarningDeclaration& Warning : Warnings )
	{
		CachedBlueprintWarningData.Add(MakeShareable(new FBlueprintWarningDeclaration(Warning)));
	}

	// This is not a great pattern:
	CachedBlueprintWarningBehaviors.Add( MakeShareable( new EBlueprintWarningBehavior( EBlueprintWarningBehavior::Warn ) ) );
	CachedBlueprintWarningBehaviors.Add( MakeShareable( new EBlueprintWarningBehavior( EBlueprintWarningBehavior::Error ) ) );
	CachedBlueprintWarningBehaviors.Add( MakeShareable( new EBlueprintWarningBehavior( EBlueprintWarningBehavior::Suppress ) ) );
	
	FString RelativeConfigFilePath = RuntimeSettings->GetDefaultConfigFilename();
	FString FullSettingsPath = FPaths::ConvertRelativePathToFull(RelativeConfigFilePath);

	TSharedPtr<SSettingsEditorCheckoutNotice> SettingsFile;
	TSharedPtr<SBorder> Label;
	// display a table of all known blueprint warnings, their description, etc:
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
			.AutoHeight()
		[
			SAssignNew(SettingsFile, SSettingsEditorCheckoutNotice)
			.ConfigFilePath(FullSettingsPath)
		]
		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 16.0f, 0.0f, 0.0f)
		[
			SAssignNew(Label, SBorder)
				.Padding(3.0f)
				.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
				.BorderBackgroundColor(FLinearColor(.6f, .6f, .6f, 1.0f))
			[
				SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
				.Text(LOCTEXT("BlueprintWarningSettings", "Warning Behavior"))
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SAssignNew(ListView, FBlueprintWarningListView)
					.SelectionMode(ESelectionMode::Multi)
					.ListItemsSource(&CachedBlueprintWarningData)
					.OnGenerateRow(
						FBlueprintWarningListView::FOnGenerateRow::CreateLambda(
							[this](FBlueprintWarningListEntry Warning, const TSharedRef<STableViewBase>& Owner) -> TSharedRef< ITableRow >
							{
								return SNew(SBlueprintWarningRow, Owner, Warning, this);
							}
						)
					)
					.HeaderRow
					(
						SNew(SHeaderRow)
						+ SHeaderRow::Column(ColumnWarningIdentifier)
							.DefaultLabel(LOCTEXT("BlueprintWarningIdentifierHeaderLabel", "Identifier"))
							.DefaultTooltip(LOCTEXT("BlueprintWarningIdentifierHeaderTooltip", "Identifier used in game runtime when warning is raised"))
							.FillWidth(0.15f)
						+ SHeaderRow::Column(ColumnWarningDescription)
							.DefaultLabel(LOCTEXT("BlueprintWarningDescriptionHeaderLabel", "Description"))
							.DefaultTooltip(LOCTEXT("BlueprintWarningDescriptionHeaderTooltip", "Description of when the warning is raised"))
							.FillWidth(0.55f)
						+ SHeaderRow::Column(ColumnWarningBehavior)
							.DefaultLabel(LOCTEXT("BlueprintWarningBehaviorHeaderLabel", "Behavior"))
							.DefaultTooltip(LOCTEXT("BlueprintWarningBehaviorHeaderTooltip", "Determines what happens when the warning is raised - warnings can be treated more strictly or suppressed entirely"))
							.FillWidth(0.3f)
					)
			]
		]
		+ SVerticalBox::Slot()
		[
			SNew(SSpacer)
		]
	];

	TAttribute<bool> Enabled = TAttribute<bool>::Create(
		TAttribute<bool>::FGetter::CreateLambda(
			[SettingsFile]()
			{
				return SettingsFile->IsUnlocked();
			}
		)
	);
	ListView->SetEnabled(Enabled);
	Label->SetEnabled(Enabled);
}

void SBlueprintWarningsConfigurationPanel::UpdateSelectedWarningBehaviors(EBlueprintWarningBehavior NewBehavior, const FBlueprintWarningDeclaration& AlteredWarning)
{
	IBlueprintRuntime* SettingsModule = FModuleManager::GetModulePtr<IBlueprintRuntime>("BlueprintRuntime");
	if (!SettingsModule)
	{
		return;
	}

	UBlueprintRuntimeSettings* RuntimeSettings = SettingsModule->GetMutableBlueprintRuntimeSettings();

	// no TArray.ForEach makes this the most convenient pattern:
	for (const TSharedPtr<FBlueprintWarningDeclaration>& Declaration : CachedBlueprintWarningData)
	{
		if (ListView->IsItemSelected(Declaration) || Declaration->WarningIdentifier == AlteredWarning.WarningIdentifier)
		{
			// update config data:
			FName CurrentIdentifer = Declaration->WarningIdentifier;
			int WarningEntryIndex = RuntimeSettings->WarningSettings.IndexOfByPredicate(
				[CurrentIdentifer](const FBlueprintWarningSettings& Entry) -> bool
				{
					return Entry.WarningIdentifier == CurrentIdentifer;
				}
			);

			switch(NewBehavior)
			{
			case EBlueprintWarningBehavior::Warn:
				if (WarningEntryIndex >= 0)
				{
					RuntimeSettings->WarningSettings.RemoveAtSwap(WarningEntryIndex);
				}
				break;
			case EBlueprintWarningBehavior::Error:
			case EBlueprintWarningBehavior::Suppress:
				if (WarningEntryIndex >= 0)
				{
					FBlueprintWarningSettings& WarningEntry = RuntimeSettings->WarningSettings[WarningEntryIndex];
					WarningEntry.WarningBehavior = NewBehavior;
				}
				else
				{
					FBlueprintWarningSettings NewEntry;
					NewEntry.WarningIdentifier = CurrentIdentifer;
					NewEntry.WarningDescription = Declaration->WarningDescription;
					NewEntry.WarningBehavior = NewBehavior;
					RuntimeSettings->WarningSettings.Add(NewEntry);
				}
				break;
			}
		}
	}

	// make sure runtime behavior matches config data:
	SettingsModule->PropagateWarningSettings();
}

#undef LOCTEXT_NAMESPACE
