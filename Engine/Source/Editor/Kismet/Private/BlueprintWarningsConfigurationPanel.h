// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SSettingsEditorCheckoutNotice;
struct FBlueprintWarningDeclaration;

enum class EBlueprintWarningBehavior : uint8;
template <typename ItemType> class SListView;

typedef TSharedPtr<FBlueprintWarningDeclaration> FBlueprintWarningListEntry;
typedef SListView<FBlueprintWarningListEntry> FBlueprintWarningListView;

class SBlueprintWarningsConfigurationPanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SBlueprintWarningsConfigurationPanel){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
private:
	friend class SBlueprintWarningRow;

	void UpdateSelectedWarningBehaviors(EBlueprintWarningBehavior NewBehavior, const FBlueprintWarningDeclaration& AlteredWarning );

	// SListView requires TArray<TSharedRef<FFoo>> so we cache off a list from core:
	TArray<TSharedPtr<FBlueprintWarningDeclaration>> CachedBlueprintWarningData;
	// Again, SListView boilerplate:
	TArray<TSharedPtr<EBlueprintWarningBehavior>> CachedBlueprintWarningBehaviors;
	// Storing the listview, so we can apply updates to all selected entries:
	TSharedPtr< FBlueprintWarningListView > ListView;
	// Stored so that we can only enable controls when the settings files is writable:
	TSharedPtr<SSettingsEditorCheckoutNotice> SettingsEditorCheckoutNotice;
};

