// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprints/BlueprintDependencies.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Cooker/CookDependencyContext.h"
#include "Engine/Blueprint.h"
#include "GameplayTagContainer.h"
#include "GameplayTagsManager.h"
#include "HAL/FileManager.h"
#include "IO/IoHash.h"
#include "JsonObjectConverter.h"
#include "JsonObjectGraph/Stringify.h"
#include "K2Node_CallFunction.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Misc/CommandLine.h"
#include "Misc/Optional.h"
#include "Misc/PackageAccessTracking.h"
#include "Misc/ScopeExit.h"
#include "Misc/StringOutputDevice.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinaryWriter.h"
#include "UObject/StructOnScope.h"

DEFINE_LOG_CATEGORY(LogCookBlueprint);

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintDependencies)

/*
	BLUEPRINT DEPENDENCIES DESIGN

	This file is meant to capture external dependencies of a blueprint so that it can
	be reliably recooked when its (cook time) inputs change. Dependencies of the blueprint
	are categorized as (1) native classes, (2) native structures, or (3) assets. Native classes 
	and native structs are canonized as strings for the purpose of comparison. Asset dependencies 
	have their file contents hashed. There is ample opportunity for optimization here, but
	there is even more ample opportunity for false positives and negatives. The current emphasis
	is on simplicity, durability and correctness, at the possible expense of raw throughput.

	Some notes on engine facilities we cannot (yet?) use:
	::GetSchemaHash(UStruct*) does not include default values (for either scriptstructs or classes)
		nor does it include any detection of versioning logic in user's (C++) serialize(FArchive&) 
		routines
	FAssetPackageData::GetPackageSavedHash could be used for uasset based dependencies, but is hard to access
		and long term is imprecise (e.g. many changes to an asset do not have effects
		on dependents)

	@todo - include detection of custom object versions published by native class dependencies
	@todo - include detection of custom object versions published by native scriptstruct dependencies? The payoff here may be small
*/

#define UE_STORE_DEPENDENCY_SNAPSHOT 0

namespace UE::Private
{
constexpr int32 BlueprintCookDependenciesVersion = 5;

// gathering helpers:
static void GetAllImportedObjects(
	TConstArrayView<const UObject*> Roots, 
	TArray<const UObject*>& OutImports, 
	TSet<FName>& OutSoftImports,
	TSet<FName>& OutTagNames);

static void GetAllImportedObjects(const UBlueprint* ForBP, TArray<const UObject*>& OutImports, TSet<FName>& OutSoftImports, TSet<FName>& OutTagNames);

// helpers for handling transitiveness of dependencies (e.g. if i use a struct, i also depend on its member structs):
static void AddStructDependencyImpl(const UScriptStruct* Struct, TSet<const UScriptStruct*>& ReferencedStructs);
static void AddClassDependencyImpl(const UClass* Class, TSet<const UClass*>& ReferencedClasses);

// returns all the dependencies of a bp, for recording/generating a snapshot of dependencies:
static void GetAllDependencies(
	const UBlueprint* ForBP, 
	TArray<FName>& AssetDependencies, 
	TArray<const UClass*>& NativeClassDependencies,
	TArray<const UScriptStruct*>& NativeStructDependencies,
	TArray<FName>& GameplayTagsArray);

// Native struct and class canonization:
static FString HashNativeStruct(const UScriptStruct* Struct);
static FString HashNativeClass( const UClass* NativeClass);
// Asset canonization:
static FString HashPackageFile(FName PackageName);

// cache of dependency hashes, so they repeatedly recalculated:
struct FBPDependencyCacheEntry
{
	FString Hash;
#if UE_STORE_DEPENDENCY_SNAPSHOT
	FUtf8String Source;
#endif
};

class FBPDependencyCache
{
public:
	static FBPDependencyCacheEntry* LookupCache(const UClass* Class);
	static FBPDependencyCacheEntry* LookupCache(const UScriptStruct* Struct);
	static FBPDependencyCacheEntry* LookupCache(FName PackageName);
private:
	static FBPDependencyCache& GetCache();
	static TUniquePtr<FBPDependencyCache> GBPDependencyCache;

	TMap<const UClass*, FBPDependencyCacheEntry> ClassToHash;
	TMap<const UScriptStruct*, FBPDependencyCacheEntry> StructToHash;
	TMap<FName, FBPDependencyCacheEntry> PackageToHash;
	// when disabled we clear and return the scratch entry so that callers can always populate the cache..
	FBPDependencyCacheEntry ScratchEntry; 
	bool bCacheDisabled = false;
};

// Filesystem helper:
static bool ReadBytesFromFile(const TCHAR* Filename, TArray64<uint8>& OutBytes);
// Fail caching helper:
static void NotCacheable(FCbFieldViewIterator Args, UE::Cook::FCookDependencyContext& Context);
// Main implementation, validate and generate are pairs, as are load and save:
static void ValidateBPCookDependenciesImpl(FCbFieldViewIterator Args, UE::Cook::FCookDependencyContext& Context);
static void GenerateBlueprintDependencies(const UBlueprint* ForBP, FBlueprintDependencies& OutDependencies);
static bool LoadBPCookDependenciesImpl(FCbFieldViewIterator Args, UE::Cook::FCookDependencyContext& Context, FBlueprintDependencies& OutDependencies);
static void SaveBPCookDepenenciesImpl(FCbWriter& Writer, const FBlueprintDependencies& Dependencies);
}

static void UE::Private::GetAllImportedObjects(
	TConstArrayView<const UObject*> Roots, 
	TArray<const UObject*>& OutImports, 
	TSet<FName>& OutSoftImports,
	TSet<FName>& OutTagNames)
{
	// This function just gathers all of the UObjects the Root objects (or their inners)
	// depend upon. Inners are not currently tested for reachability. Tautological self reference
	// (e.g. I depend on myself or I depend on my inner) are not reported - only external objects:
	struct FExternalReferenceFinder : public FArchiveUObject
	{
		FExternalReferenceFinder(const UObject* Obj, TSet<const UObject*>& InReferences, const TSet<const UObject*>& InRoots)
			: FArchiveUObject()
			, SearchRoots(InRoots)
			, References(InReferences)
		{
			SetIsPersistent(true); // to avoid transient properties - these often contain unstable caches which cannot be used in key calculation!
			SetIsSaving(true);
			SetShouldSkipCompilingAssets(true);
			SetWantBinaryPropertySerialization(true);
			SetUseUnversionedPropertySerialization(true);
			SetShouldSkipUpdateCustomVersion(true);
			ArIsModifyingWeakAndStrongReferences = true;
			ArIsObjectReferenceCollector = true;
			ArShouldSkipBulkData = true;
			if (Obj->HasAnyFlags(RF_ClassDefaultObject))
			{
				Obj->GetClass()->SerializeDefaultObject((UObject*)Obj, *this);
			}
			else
			{
				((UObject*)Obj)->Serialize(*this);
			}
		}

	private:
		bool ShouldTraverseProperty()
		{
			FProperty* SerializedProperty = GetSerializedProperty();
			if (!SerializedProperty || !SerializedProperty->HasAnyPropertyFlags(CPF_Transient))
			{
				return true;
			}
			return false;
		}

		virtual FArchive& operator<<(FLazyObjectPtr& Value) override { return ShouldTraverseProperty() ? FArchiveUObject::operator<<(Value) : *this; }
		virtual FArchive& operator<<(FObjectPtr& Value) override { return ShouldTraverseProperty() ? FArchiveUObject::operator<<(Value) : *this; }
		virtual FArchive& operator<<(FWeakObjectPtr& Value) override { return ShouldTraverseProperty() ? FArchiveUObject::operator<<(Value) : *this; }

		virtual FArchive& operator<<(UObject*& ObjRef) override
		{
			if (ShouldTraverseProperty() &&
				ObjRef != nullptr &&
				// uht objects are tagged as RF_Transient but they 'are native' 
				// so still referencable:
				(!ObjRef->HasAnyFlags(RF_Transient) || ObjRef->IsNative()) &&
				!ObjRef->IsA<UPackage>() &&
				!IsInRoots(ObjRef))
			{
				References.Add(ObjRef);
			}

			return *this;
		}

		bool IsInRoots(const UObject* Obj)
		{
			const UObject* Iter = Obj;
			while(Iter)
			{
				if(SearchRoots.Contains(Iter))
				{
					return true;
				}
				Iter = Iter->GetOuter();
			}
			return false;
		}

		const TSet<const UObject*>& SearchRoots;
		TSet<const UObject*>& References;
	};

	TSet<const UObject*> RootSet(Roots);
	TSet<const UObject*> Result;
	TArray<const UObject*> Objects;
	for(const UObject* Obj : Roots)
	{
		GetObjectsWithOuter(Obj, (TArray<UObject*>&)Objects, EGetObjectsFlags::IncludeNestedObjects, RF_Transient, EInternalObjectFlags::Garbage);
		Objects.Add(Obj);

		// include any class's super struct chain for reference finding,
		// I'm doing this specifically to gather inherited structure 
		// dependencies..
		if(const UClass* AsClass = Cast<UClass>(Obj))
		{
			const UClass* Iter = AsClass->GetSuperClass();
			while(Iter)
			{
				Objects.Add(Iter);
				Iter = Iter->GetSuperClass();
			}
		}
	}

	for(const UObject* Obj: Objects)
	{
		FExternalReferenceFinder References(Obj, Result, RootSet);

		if (const UK2Node_CallFunction* Node = Cast<UK2Node_CallFunction>(Obj))
		{
			int PinIndex = 0;
			while (const UEdGraphPin* Pin = Node->GetPinAt(PinIndex))
			{
				++PinIndex;

				const FEdGraphPinType& PinType = Pin->PinType;
				const UScriptStruct* PinStruct = Cast<UScriptStruct>(PinType.PinSubCategoryObject);
				
				if(PinStruct != FGameplayTag::StaticStruct())
				{
					continue;
				}

				FString DefaultValue = Pin->GetDefaultAsString();

				if (DefaultValue.IsEmpty())
				{
					continue;
				}
				
				FGameplayTag Tag;
				PinStruct->ImportText(*DefaultValue, &Tag, nullptr, 0, nullptr, PinStruct->GetName(), false);
				OutTagNames.Add(Tag.GetTagName());
			}
		}
	}
	
	OutImports = Result.Array();
}

static void UE::Private::GetAllImportedObjects( const UBlueprint* ForBP, TArray<const UObject*>& OutImports, TSet<FName>& OutSoftImports, TSet<FName>& OutTagNames)
{
	return GetAllImportedObjects({ForBP, ForBP->GeneratedClass, ForBP->GeneratedClass->GetDefaultObject(false) }, OutImports, OutSoftImports, OutTagNames);
}

static void UE::Private::AddStructDependencyImpl(const UScriptStruct* Struct, TSet<const UScriptStruct*>& ReferencedStructs)
{
	bool bWasAlreadyPresent = false;
	ReferencedStructs.Add(Struct, &bWasAlreadyPresent);
	if(bWasAlreadyPresent)
	{
		return;
	}
	
	// add aggregate structs, and their aggregate structs - and their super structs
	for (TFieldIterator<FStructProperty> It(Struct, EFieldIteratorFlags::ExcludeSuper); It; ++It)
	{
		AddStructDependencyImpl(It->Struct, ReferencedStructs);
	}

	const UScriptStruct* Super = Cast<UScriptStruct>(Struct->GetSuperStruct()); // can't be castchecked only because we may have a null super
	if(Super)
	{
		AddStructDependencyImpl(Super, ReferencedStructs);
	}
}

static void UE::Private::AddClassDependencyImpl(const UClass* Class, TSet<const UClass*>& ReferencedClasses)
{
	const UClass* Iter = Class;
	while (Iter)
	{
		bool bWasAlreadyPresent = false;
		ReferencedClasses.Add(Iter, &bWasAlreadyPresent);
		if (bWasAlreadyPresent)
		{
			return;
		}

		Iter = Iter->GetSuperClass();
	}
}

static void UE::Private::GetAllDependencies(
	const UBlueprint* ForBP,
	TArray<FName>& AssetDependencies, 
	TArray<const UClass*>& NativeClassDependencies,
	TArray<const UScriptStruct*>& NativeStructDependencies,
	TArray<FName>& GameplayTagsArray)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(BlueprintDependencies::GetAllDependencies);

	// gather all references in the BP, its class, and the CDO, along with any subobjects:
	TArray<const UObject*> ImportedObjects;
	TSet<FName> ReferencedAssets;
	TSet<FName> TagNames;
	GetAllImportedObjects(ForBP, ImportedObjects, ReferencedAssets, TagNames);

	// honor transitive dependencies - e.g. inherited classes, inherited structs, aggregate structs
	// for now all class and struct references are fully transitive:
	TSet<const UScriptStruct*> ReferencedStructs;
	TSet<const UClass*> ReferencedClasses;
	for(const UObject* Object : ImportedObjects)
	{
		const UPackage* Package = Object->GetPackage();
		if(const UClass* AsClass = Cast<UClass>(Object))
		{
			AddClassDependencyImpl(AsClass, ReferencedClasses);
		}
		else if(const UScriptStruct* AsStruct = Cast<UScriptStruct>(Object))
		{
			AddStructDependencyImpl(AsStruct, ReferencedStructs);
		}
		else if(const UFunction* AsFunction = Cast<UFunction>(Object))
		{
			AddClassDependencyImpl(AsFunction->GetOwnerClass(), ReferencedClasses);
		}
		else if (!Package->HasAnyPackageFlags(PKG_CompiledIn))
		{
			ReferencedAssets.Add(Package->GetFName());
		}
	}

	// process classes and structs, categorize them as native or asset:
	for(const UClass* Class : ReferencedClasses)
	{
		if (Class->HasAnyClassFlags(CLASS_Native))
		{
			NativeClassDependencies.Add(Class);
		}
		else
		{
			ReferencedAssets.Add(Class->GetPackage()->GetFName());
		}
	}

	for(const UScriptStruct* Struct : ReferencedStructs)
	{
		if(Struct->StructFlags & STRUCT_Native)
		{
			NativeStructDependencies.Add(Struct);
		}
		else
		{
			ReferencedAssets.Add(Struct->GetPackage()->GetFName());
		}
	}
	
	AssetDependencies = ReferencedAssets.Array();

	// give the arrays stable order - noisey arrays will be obnoxious for
	// memoization/distribution:
	const auto SortTopLevelObject = [](const UObject& ObjA, const UObject& ObjB)
	{
		ensure(ObjA.GetOuter()->IsA<UPackage>());
		ensure(ObjB.GetOuter()->IsA<UPackage>());
		// sort by packagename/objectname:
		const FName PackageNameA = ObjA.GetOuter()->GetFName();
		const FName PackageNameB = ObjB.GetOuter()->GetFName();
		if(PackageNameA == PackageNameB)
		{
			return ObjA.GetFName().LexicalLess(ObjB.GetFName());
		}
		else
		{
			return PackageNameA.LexicalLess(PackageNameB);
		}
	};
	NativeStructDependencies.Sort(SortTopLevelObject);
	NativeClassDependencies.Sort(SortTopLevelObject);
	AssetDependencies.Sort([](FName A, FName B)
	{
		return A.LexicalLess(B);
	});

	GameplayTagsArray = TagNames.Array();
	GameplayTagsArray.Sort(FNameLexicalLess());
}

static FString UE::Private::HashNativeStruct(const UScriptStruct* Struct)
{
	// The hash calculated in this function is cached. So only the first package asking for it calculates it
	// Other packages will retrieve the cached value. To calculate the hash we might go through pointers
	// and this will be detected by the package tracker and create package dependencies. But only the first package
	// calculating the hash will have those dependencies and since the first package doing the calculation is not always
	// the same we end up with non deterministic dependencies.
	// If accessing the dereferenced object is relevant, the hash should change so tracking those dependencies is useless anyway.
	UE_COOK_DISABLE_PACKAGE_ACCESS_TRACKING_SCOPED();

	TRACE_CPUPROFILER_EVENT_SCOPE(BlueprintDependencies::HashNativeStruct);
	FBPDependencyCacheEntry* CachedValue = FBPDependencyCache::LookupCache(Struct);
	if(CachedValue->Hash.Len() > 0)
	{
		return *CachedValue->Hash;
	}

	check(Struct->StructFlags & STRUCT_Native);
	FUtf8String StringifiedStruct = UE::JsonObjectGraph::Stringify(
		{Struct}, 
		FJsonStringifyOptions(EJsonStringifyFlags::FilterEditorOnlyData));

	// JsonObjectGraph doesn't support const UStruct*/void* pairs,
	// so use another json routine to get identity for default values
	// shortcomings here are that we aren't getting much info about 
	// native serialize overrides (inc. custom version bumps)

	// Filter out IgnoreForMemberInitializationTest as we know those values
	// will change every time the structure is reconstructed - without this
	// then types that have side effects in their ctor will always report
	// as changed:
	FJsonObjectConverter::CustomExportCallback ExportCb = FJsonObjectConverter::CustomExportCallback::CreateLambda(
		[](FProperty* Prop, const void* Data) -> TSharedPtr<FJsonValue>
		{
			static const FName NAME_IgnoreForMemberInitializationTest(TEXT("IgnoreForMemberInitializationTest"));
			if (Prop->HasMetaData(NAME_IgnoreForMemberInitializationTest))
			{
				return MakeShared<FJsonValueString>(TEXT("")); // export unstable members as the empty string
			}

			return {};
		});
	FStructOnScope Defaults(Struct);
	FString DefaultValues;
	FJsonObjectConverter::UStructToJsonObjectString(Struct, Defaults.GetStructMemory(), DefaultValues,
		/*CheckFlags = */ 0, 
		/*SkipFlags =*/ 0, 
		/*int32 Indent =*/ 0, 
		&ExportCb);
	StringifiedStruct.Append(DefaultValues);

	FBlake3 Hash;
	Hash.Update(StringifiedStruct.GetCharArray().GetData(), StringifiedStruct.GetCharArray().Num());
	CachedValue->Hash = LexToString(Hash.Finalize());
	
#if UE_STORE_DEPENDENCY_SNAPSHOT
	CachedValue->Source = StringifiedStruct;
#endif

	return CachedValue->Hash;
}

static FString UE::Private::HashNativeClass( const UClass* NativeClass)
{
	// The hash calculated in this function is cached. So only the first package asking for it calculates it
	// Other packages will retrieve the cached value. To calculate the hash we might go through pointers
	// and this will be detected by the package tracker and create package dependencies. But only the first package
	// calculating the hash will have those dependencies and since the first package doing the calculation is not always
	// the same we end up with non deterministic dependencies.
	// If accessing the dereferenced object is relevant, the hash should change so tracking those dependencies is useless anyway.
	UE_COOK_DISABLE_PACKAGE_ACCESS_TRACKING_SCOPED();

	TRACE_CPUPROFILER_EVENT_SCOPE(BlueprintDependencies::HashNativeClass);
	FBPDependencyCacheEntry* CachedValue = FBPDependencyCache::LookupCache(NativeClass);
	if(CachedValue->Hash.Len() > 0)
	{
		return CachedValue->Hash;
	}

	// We're using UE::JsonObjectGraph because it is robust compared to ExportText and 
	// the other core level facilities. Exporting a class to text using ExportText has
	// always produced an empty object, which is not meaningful. JsonObjectGraph::Stringify
	// will provide identity for the UClass:
	TArray<const UObject*, TInlineAllocator<2>> RootObjects;
	if(NativeClass->HasAnyClassFlags(CLASS_Transient))
	{
		RootObjects = {NativeClass};
	}
	else
	{
		RootObjects = {NativeClass, NativeClass->GetDefaultObject(false)};
	}

	FUtf8String StringifiedClass = UE::JsonObjectGraph::Stringify(
		RootObjects,
		FJsonStringifyOptions(EJsonStringifyFlags::FilterEditorOnlyData));

	static FString ClassToDump;
	static bool bDumpNativeClass = FParse::Value(FCommandLine::Get(), TEXT("-DumpBPNativeClassSerialization="), ClassToDump);
	static FName ClassNameToDump(ClassToDump);

	if (bDumpNativeClass && NativeClass->GetFName() == ClassNameToDump)
	{
		UE_LOGF(LogCookBlueprint, Display, "---Start Blueprint native class serialization---");
		UE_LOGF(LogCookBlueprint, Display, "%ls", *ClassToDump);
		UE_LOGF(LogCookBlueprint, Display, "%s", *StringifiedClass);
		UE_LOGF(LogCookBlueprint, Display, "---End---");
	}

	FBlake3 Hash;
	Hash.Update(StringifiedClass.GetCharArray().GetData(), StringifiedClass.GetCharArray().Num());
	CachedValue->Hash = LexToString(Hash.Finalize());

#if UE_STORE_DEPENDENCY_SNAPSHOT
	CachedValue->Source = StringifiedClass;
#endif

	return CachedValue->Hash;
}

static FString UE::Private::HashPackageFile(FName PackageName)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(BlueprintDependencies::HashPackage);
	FBPDependencyCacheEntry* CachedValue = FBPDependencyCache::LookupCache(PackageName);
	if(CachedValue->Hash.Len() > 0)
	{
		return CachedValue->Hash;
	}

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	TOptional<FAssetPackageData> PackageData = AssetRegistry.GetAssetPackageDataCopy(PackageName);
	if (!PackageData)
	{
		return FString();
	}
	FIoHash PackageHash = PackageData->GetPackageSavedHash();
	CachedValue->Hash = LexToString(PackageHash);
	
#if UE_STORE_DEPENDENCY_SNAPSHOT
	CachedValue->Source = "File hashed on disk";
#endif

	return CachedValue->Hash;
}

UE::Private::FBPDependencyCacheEntry* UE::Private::FBPDependencyCache::LookupCache(const UClass* Class)
{
	FBPDependencyCache& Cache = GetCache();
	if(Cache.bCacheDisabled)
	{
		Cache.ScratchEntry = {};
		return &Cache.ScratchEntry;
	}
	FBPDependencyCacheEntry& Value = Cache.ClassToHash.FindOrAdd(Class);
	return &Value;
}

UE::Private::FBPDependencyCacheEntry* UE::Private::FBPDependencyCache::LookupCache(const UScriptStruct* Struct)
{
	FBPDependencyCache& Cache = GetCache();
	if(Cache.bCacheDisabled)
	{
		Cache.ScratchEntry = {};
		return &Cache.ScratchEntry;
	}
	FBPDependencyCacheEntry& Value = Cache.StructToHash.FindOrAdd(Struct);
	return &Value;
}

UE::Private::FBPDependencyCacheEntry* UE::Private::FBPDependencyCache::LookupCache(FName PackageName)
{
	FBPDependencyCache& Cache = GetCache();
	if(Cache.bCacheDisabled)
	{
		Cache.ScratchEntry = {};
		return &Cache.ScratchEntry;
	}
	FBPDependencyCacheEntry& Value = Cache.PackageToHash.FindOrAdd(PackageName);
	return &Value;
}

TUniquePtr<UE::Private::FBPDependencyCache> UE::Private::FBPDependencyCache::GBPDependencyCache;
UE::Private::FBPDependencyCache& UE::Private::FBPDependencyCache::GetCache()
{
	if(!GBPDependencyCache)
	{
		GBPDependencyCache = MakeUnique<FBPDependencyCache>();
		// increase testability - avoid caching native dependencies when not running 
		// the cook commandlet. This allows us to find e.g. unstable native
		// constructors:
		if(!IsRunningCookCommandlet())
		{
			GBPDependencyCache->bCacheDisabled = true;
		}
	}
	return *GBPDependencyCache;
}

static bool UE::Private::ReadBytesFromFile(const TCHAR* Filename, TArray64<uint8>& OutBytes)
{
	OutBytes.Reset();
	TUniquePtr<FArchive> FileArchive(IFileManager::Get().CreateFileReader(Filename));
	if(FileArchive == TUniquePtr<FArchive>())
	{
		return false;
	}
	
	const int64 TotalSize = FileArchive->TotalSize();
	if(TotalSize <= 0)
	{
		return false;
	}
	
	OutBytes.AddUninitialized(TotalSize);
	FileArchive->Serialize(OutBytes.GetData(), TotalSize);
	return true;
}

void UE::Private::NotCacheable(FCbFieldViewIterator Args, UE::Cook::FCookDependencyContext& Context)
{	
	Context.LogInvalidated(
		TEXT("Package is not yet cacheable"));
}
UE_COOK_DEPENDENCY_FUNCTION(NotCacheable, UE::Private::NotCacheable);

// BEGIN PAIR VALIDATE/GENERATE
void UE::Private::ValidateBPCookDependenciesImpl(FCbFieldViewIterator Args, UE::Cook::FCookDependencyContext& Context)
{
	// MUST MATCH UE::Private::GenerateBlueprintDependencies
	FBlueprintDependencies Dependencies;
	if(!LoadBPCookDependenciesImpl(Args, Context, Dependencies))
	{
		return;
	}
	
	TRACE_CPUPROFILER_EVENT_SCOPE(BlueprintDependencies::ValidateBPCookDependenciesImpl);
	// confirm dependencies do/do not match the artifact - must correspond to 
	// GenerateBlueprintDependencies (hashpackage<->hashpackagefile, hashnative class, hashnative struct)
	for(const FBlueprintDependency& BPDep : Dependencies.BlueprintDependencies)
	{
		switch(BPDep.DependencyType)
		{
			case EBPDependencyType::Asset:
			{
				if(!BPDep.Hash.Equals(UE::Private::HashPackageFile(BPDep.PackageName)))
				{
					Context.LogInvalidated(FString::Printf(TEXT("Peer Package Changed: %s"), *BPDep.PackageName.ToString()));
					return;
				}
				break;
			}
			case EBPDependencyType::Struct:
			{
				const UPackage* Package = FindObjectFast<UPackage>(nullptr, BPDep.PackageName);
				if(!Package)
				{
					Context.LogInvalidated(
						FString::Printf(TEXT("Native Package Missing: %s"), *BPDep.PackageName.ToString()));
					return;
				}
				const UScriptStruct* Struct = FindObjectFast<UScriptStruct>(
					const_cast<UPackage*>(Package), BPDep.NativeObjectName);
				if(!Struct)
				{
					Context.LogInvalidated(
						FString::Printf(TEXT("Native Struct Missing: %s"), *BPDep.NativeObjectName.ToString()));
					return;
				}
				if(!BPDep.Hash.Equals(UE::Private::HashNativeStruct(Struct)))
				{
					Context.LogInvalidated(FString::Printf(TEXT("Native Struct Changed: %s"), *Struct->GetPathName()));
					return;
				}
				break;
			}
			case EBPDependencyType::Class:
			{
				const UPackage* Package = FindObjectFast<UPackage>(nullptr, BPDep.PackageName);
				if(!Package)
				{
					Context.LogInvalidated(
						FString::Printf(TEXT("Native Package Missing: %s"), *BPDep.PackageName.ToString()));
					return;
				}
				const UClass* Class = FindObjectFast<UClass>(
					const_cast<UPackage*>(Package), BPDep.NativeObjectName);
				if(!Class)
				{
					Context.LogInvalidated(
						FString::Printf(TEXT("Native Class Missing: %s"), *BPDep.NativeObjectName.ToString()));
					return;
				}
				if(!BPDep.Hash.Equals(UE::Private::HashNativeClass(Class)))
				{
					Context.LogInvalidated(FString::Printf(TEXT("Native Class Changed: %s"), *Class->GetPathName()));
					return;
				}
				break;
			}
			case EBPDependencyType::GameplayTag:
			{
				FBlake3 Hasher;
				UGameplayTagsManager::Get().AppendDependencyHash(BPDep.NativeObjectName, Hasher);
				FBlake3Hash NewHash = Hasher.Finalize();

				FBlake3Hash OldHash(BPDep.Hash);
				if (NewHash != OldHash)
				{
					Context.LogInvalidated(FString::Printf(TEXT("Gameplay tag changed : %s"), *BPDep.NativeObjectName.ToString()));
					return;
				}
				break;
			}
		}
	}
}
UE_COOK_DEPENDENCY_FUNCTION(ValidateBPCookDependenciesImpl, UE::Private::ValidateBPCookDependenciesImpl);

void UE::Private::GenerateBlueprintDependencies(const UBlueprint* ForBP, FBlueprintDependencies& OutDependencies)
{
	TArray<FName> AssetDependencies;
	TArray<const UClass*> NativeClassDependencies;
	TArray<const UScriptStruct*> NativeStructDependencies;
	TArray<FName> GameplayTagsArray;
	UE::Private::GetAllDependencies(ForBP, AssetDependencies, NativeClassDependencies, NativeStructDependencies, GameplayTagsArray);

	TRACE_CPUPROFILER_EVENT_SCOPE(BlueprintDependencies::GenerateBlueprintDependencies);
	// MUST MATCH UE::Private::ValidateBPCookDependenciesImpl
	// gather dependencies on functions/structs/classes/objects:
	// treat class as transitive, in case we're using them for type comparisons:
	// treat struct dependencies as aggregate, in case inner structs change:
	for(FName PackageName : AssetDependencies)
	{
		FBlueprintDependency Dependency = {
			.DependencyType = EBPDependencyType::Asset,
			.PackageName = PackageName,
			.NativeObjectName = FName(),
			.Hash = UE::Private::HashPackageFile(PackageName)
		};
		
		OutDependencies.BlueprintDependencies.Emplace(
			MoveTemp(Dependency)
		);
	}
	for(const UClass* Class : NativeClassDependencies)
	{
		FBlueprintDependency Dependency = {
			.DependencyType = EBPDependencyType::Class,
			.PackageName = Class->GetPackage()->GetFName(),
			.NativeObjectName = Class->GetFName(),
			.Hash = UE::Private::HashNativeClass(Class)
		};
		
		OutDependencies.BlueprintDependencies.Emplace(
			MoveTemp(Dependency)
		);
	}
	for(const UScriptStruct* Struct : NativeStructDependencies)
	{
		FBlueprintDependency Dependency = {
			.DependencyType = EBPDependencyType::Struct,
			.PackageName = Struct->GetPackage()->GetFName(),
			.NativeObjectName = Struct->GetFName(),
			.Hash = UE::Private::HashNativeStruct(Struct)
		};
		
		OutDependencies.BlueprintDependencies.Emplace(
			MoveTemp(Dependency)
		);
	}

	for (const FName& SingleTagName : GameplayTagsArray)
	{
		FBlake3 Hasher;
		UGameplayTagsManager::Get().AppendDependencyHash(SingleTagName, Hasher);
		FBlake3Hash Hash = Hasher.Finalize();

		FBlueprintDependency Dependency = {
			.DependencyType = EBPDependencyType::GameplayTag,
			.PackageName = FName(),
			.NativeObjectName = SingleTagName,
			.Hash = LexToString(Hash)
		};

		OutDependencies.BlueprintDependencies.Emplace(MoveTemp(Dependency));
	}
}
// END PAIR VALIDATE/GENERATE

// BEGIN PAIR LOAD/SAVE
bool UE::Private::LoadBPCookDependenciesImpl(FCbFieldViewIterator Args, UE::Cook::FCookDependencyContext& Context, FBlueprintDependencies& OutDependencies)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(BlueprintDependencies::LoadBPCookDependenciesImpl);
	// MUST MATCH UE::Private::SaveBPCookDepenenciesImpl:
	FCbFieldViewIterator ArgField(Args);
	const int32 ArgsVersion = (ArgField++).AsInt32(); // Writer << BlueprintCookDependenciesVersion;
	if (ArgsVersion != BlueprintCookDependenciesVersion)
	{
		Context.LogInvalidated(TEXT("Blueprint Cook Dependency Version Changed"));
		return false;
	}
	
	FUtf8StringView DependencyData = (ArgField++).AsString(); // Writer << DependenciesSerialized;
	FString AsFString(DependencyData);
	FStringOutputDevice Errors;
	FBlueprintDependencies::StaticStruct()->ImportText(
		*AsFString, &OutDependencies, nullptr, 0, &Errors, []() {return FString(); });
	if (!Errors.IsEmpty())
	{
		Context.LogError(FString::Printf(TEXT("Could not load Blueprint dependencies: %s"), *Errors));
		return false;
	}

	return true;
}

void UE::Private::SaveBPCookDepenenciesImpl(FCbWriter& Writer, const FBlueprintDependencies& Dependencies)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(BlueprintDependencies::SaveBPCookDepenenciesImpl);
	// MUST MATCH UE::Private::LoadBPCookDependenciesImpl
	FString DependenciesSerialized;
	FBlueprintDependencies::StaticStruct()->ExportText(
		DependenciesSerialized, &Dependencies, nullptr, nullptr, 0, nullptr);

	Writer << BlueprintCookDependenciesVersion;
	Writer << DependenciesSerialized;
}
// END PAIR LOAD/SAVE

UE::Cook::FCookDependency UE::Private::BlueprintDependencies::RecordCookDependencies(const UBlueprint* BP)
{
	if(!BP->ParentClass || 
		!BP->GeneratedClass ||
		!BP->GeneratedClass->IsChildOf(UObject::StaticClass()) ||
		(BP->Status == BS_Error || BP->Status == BS_Unknown))
	{
		// blueprints without classes - malformed or some
		// kind of utility blueprint or in the error state, ignore them
		// blueprints in error state may not be able to report all of their
		// dependencies (e.g. because they've gone missing)
		return UE::Cook::FCookDependency::Function(
			UE_COOK_DEPENDENCY_FUNCTION_CALL(NotCacheable), FCbFieldIterator());
	}

	FBlueprintDependencies Deps;
	UE::Private::GenerateBlueprintDependencies(BP, Deps);

	FCbWriter Writer;
	UE::Private::SaveBPCookDepenenciesImpl(Writer, Deps);

	return UE::Cook::FCookDependency::Function(
		UE_COOK_DEPENDENCY_FUNCTION_CALL(ValidateBPCookDependenciesImpl), Writer.Save());
}

#undef UE_STORE_DEPENDENCY_SNAPSHOT
