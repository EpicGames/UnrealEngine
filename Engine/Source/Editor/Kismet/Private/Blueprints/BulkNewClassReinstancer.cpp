// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprints/BulkNewClassReinstancer.h"
#include "BlueprintCompilationManager.h"
#include "HAL/IConsoleManager.h"
#include "Kismet2/KismetReinstanceUtilities.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "UObject/Package.h"
#include "JsonObjectGraph/Stringify.h"
#include "Logging/StructuredLog.h"

namespace UE::Private
{

// Use this to debug unexpectedly expensive ReparentHierarchies calls due to high numbers of class changes:
bool bPrintFastReinstancingMismatches = false;
static FAutoConsoleVariableRef CVarPrintFastReinstancingMismatches(
	TEXT("BP.bPrintFastReinstancingMismatches"), bPrintFastReinstancingMismatches,
	TEXT("If true we log out the before and after stable representations of UClasses in a reinstancer operation"),
	ECVF_Default);

FBulkNewClassReinstancer::FBulkNewClassReinstancer(TArray<UClass*>& InOutClasses)
{
	StoreIdentities(InOutClasses);
}

FBulkNewClassReinstancer::~FBulkNewClassReinstancer()
{
}

void FBulkNewClassReinstancer::DoReinstance(const TMap<UClass*, UClass*>& OldClassToNewClass)
{
	if(OldClassToNewClass.Num() == 0)
	{
		return;
	}
	
	TMap<UClass*, UClass*> OldToNewFastReinstancing;
	MapIdenticalClasses(OldClassToNewClass, OldToNewFastReinstancing);
	ReparentHierarchiesFast(OldToNewFastReinstancing);
	FBlueprintSupport::ReparentHierarchies(OldClassToNewClass);
}

void FBulkNewClassReinstancer::ReparentHierarchiesFast(const TMap<UClass*, UClass*>& OldClassToNewClass)
{
	const bool bIncludeDerivedClasses = false;

	for(TPair<UClass*, UClass*> OldToNew : OldClassToNewClass)
	{
		TArray< UObject* > ObjectsToReplace;
		GetObjectsOfClass(OldToNew.Key, ObjectsToReplace, bIncludeDerivedClasses, RF_ClassDefaultObject | RF_MirroredGarbage);
		for (UObject* Object : ObjectsToReplace)
		{
			// class layouts match, so we can just set the object back to the new class - the draw back here
			// is that instances may have cached information about the UClass or related objects and will be
			// distressed when/if it changes..
			Object->SetClass(OldToNew.Value); 
		}
	}
}

void FBulkNewClassReinstancer::MapIdenticalClasses(const TMap<UClass*, UClass*>& OldClassToNewClass, TMap<UClass*, UClass*>& OutOldToNewFastReinstancing)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MapIdenticalClasses);

	// performance note: both large sets of class (inputs) and modest performance of CalculateClassIdentity are problems
	for(const TPair<UClass*, UClass*>& Pair : OldClassToNewClass)
	{
		const UClass* NewClass = Pair.Value;
		if(!NewClass)
		{
			continue;
		}
		
		FUtf8String OriginalId;
		if(UE::Private::bPrintFastReinstancingMismatches)
		{
			// Assume NewClass is now using the original path, use it to look up the original id:
			if(FUtf8String* OriginalEntry = DebugClassIdentity.Find(NewClass->GetPathName()))
			{
				OriginalId = *OriginalEntry;
			}
			else
			{
				OriginalId = "Class has been unexpected mapped to a new class with a different name";
			}
		}

		FBlake3Hash NewId = FBulkNewClassReinstancer::CalculateClassIdentity(NewClass);
		const FBlake3Hash* OldHash = OriginalClassIdentities.Find(Pair.Key); 
		if(OldHash)
		{
			if( *OldHash == NewId)
			{
				OutOldToNewFastReinstancing.Add(Pair.Key, Pair.Value);
			}
			else if(UE::Private::bPrintFastReinstancingMismatches)
			{
				UE_LOGFMT(LogTemp, Warning, "original: {OriginalId} new: {NewId}", OriginalId, DebugClassIdentity.FindChecked(NewClass->GetPathName()));
			}
		}
	}
}

void FBulkNewClassReinstancer::StoreIdentities(TArrayView<UClass*> InClasses)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(StoreIdentities);

	// performance note: class identity calculation could be optimized, but reducing the input set will be step function improvement
	for(UClass* Class : InClasses)
	{
		OriginalClassIdentities.Add(Class, FBulkNewClassReinstancer::CalculateClassIdentity(Class));
	}
}

FBlake3Hash FBulkNewClassReinstancer::CalculateClassIdentity(const UClass* ForClass)
{
	// Stringify is a bit slow, but it is stable and quite complete..
	FUtf8String ClassIdString = UE::JsonObjectGraph::Stringify(
		{ForClass, ForClass->GetDefaultObject(false)}, EJsonStringifyFlags::DisableDeltaEncoding);

	FBlake3 Hash;
	Hash.Update(ClassIdString.GetCharArray().GetData(), ClassIdString.GetCharArray().Num());
	FBlake3Hash Result = Hash.Finalize();
	
	if(UE::Private::bPrintFastReinstancingMismatches)
	{
		DebugClassIdentity.Add(ForClass->GetPathName(), ClassIdString);
	}

	return Result;
}

}
