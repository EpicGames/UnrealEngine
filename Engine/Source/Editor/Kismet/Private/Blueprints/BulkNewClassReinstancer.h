// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/IClassReinstancer.h"
#include "Containers/ArrayView.h"
#include "Containers/Utf8String.h"
#include "Hash/Blake3.h"
#include "Templates/SharedPointer.h"

class FBlueprintCompileReinstancer;
class FKismetCompilerContext;

namespace UE::Private
{

struct FBulkNewClassReinstancer : public IClassReinstancer
{
	FBulkNewClassReinstancer(TArray<UClass*>& InOutClasses);
	virtual ~FBulkNewClassReinstancer();
	virtual void DoReinstance(const TMap<UClass*, UClass*>& OldClassToNewClass) override;
private:
	static void ReparentHierarchiesFast(const TMap<UClass*, UClass*>& OldClassToNewClass);
	void MapIdenticalClasses(const TMap<UClass*, UClass*>& OldClassToNewClass, TMap<UClass*, UClass*>& OutOldToNewFastReinstancing);
	void StoreIdentities(TArrayView<UClass*> InClasses);
	FBlake3Hash CalculateClassIdentity(const UClass* ForClass);

	TMap<UClass*, FBlake3Hash> OriginalClassIdentities;
	// This is typically not populated, and is only used when BP.bPrintFastReinstancingMismatches
	// is set:
	TMap<FString, FUtf8String> DebugClassIdentity;
};

}