// Copyright Epic Games, Inc. All Rights Reserved.

#include "JsonObjectGraphFunctionLibrary.h"

#include "HAL/FileManager.h"
#include "Engine/Blueprint.h"
#include "UnrealEngine.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(JsonObjectGraphFunctionLibrary)

namespace UE::Private 
{
FString GetIntermediateAssetName(const UObject* RootObject, const TCHAR* Prefix);

static const TCHAR* SnapshotBlueprintsHelp = TEXT("Usage: snapshotblueprints label - label is a name for the folder where snapshots are saved");

static void SnapshotBlueprints(const TArray<FString>& Args)
{
	if (Args.Num() == 0)
	{
		UE_LOGF(LogEngine, Display, "%ls", SnapshotBlueprintsHelp);
		return;
	}

	const FString& Label = Args[0];
	FString ScratchFilenameWritten;
	for (TObjectIterator<UBlueprint> BPIt; BPIt; ++BPIt)
	{
		const UBlueprint* BP = *BPIt;
		UJsonObjectGraphFunctionLibrary::WritePackageToTempFile(BP, Label, FJsonStringifyOptions(), ScratchFilenameWritten);
	}
}

FAutoConsoleCommand SnapshotBlueprintsCommand(
	TEXT("snapshotblueprints"),
	*FString::Format(TEXT("Write out a snapshot to the Saved directory of currently loaded blueprints.\n{0}"), { SnapshotBlueprintsHelp }),
	FConsoleCommandWithArgsDelegate::CreateStatic(&SnapshotBlueprints),
	ECVF_Default
);

static const TCHAR* SnapshotBlueprintClassesHelp = TEXT("Usage: snapshotblueprintclasses label - label is a name for the folder where snapshots are saved");

static void SnapshotBlueprintClasses(const TArray<FString>& Args)
{
	if (Args.Num() == 0)
	{
		UE_LOGF(LogEngine, Display, "%ls", SnapshotBlueprintClassesHelp);
		return;
	}

	const FString& Label = Args[0];
	FString ScratchFilenameWritten;
	for (TObjectIterator<UBlueprint> BPIt; BPIt; ++BPIt)
	{
		const UBlueprint* BP = *BPIt;
		// skip editor only blueprint classes, they may have unstable
		// data because they are never cooked:
		if (!BP->GeneratedClass || IsEditorOnlyObject(BP->GeneratedClass))
		{
			continue;
		}

		UJsonObjectGraphFunctionLibrary::WriteBlueprintClassToTempFile(BP, Label, FJsonStringifyOptions(), ScratchFilenameWritten);
	}
}

FAutoConsoleCommand SnapshotBlueprintClassesCommand(
	TEXT("snapshotblueprintclasses"),
	*FString::Format(TEXT("Write out a snapshot to the Saved directory of currently loaded blueprint classes - the principle outputs of blueprint compilation.\n{0}"), { SnapshotBlueprintClassesHelp }),
	FConsoleCommandWithArgsDelegate::CreateStatic(&SnapshotBlueprintClasses),
	ECVF_Default
);

static const TCHAR* SnapshotClassesHelp = TEXT("Usage: snapshotclasses <label> - where <label> will be used as the root for snapshot location within the Saved/Temp directory");

void SnapshotClasses(const TArray<FString>& Args)
{
	if (Args.Num() == 0)
	{
		UE_LOGF(LogEngine, Display, "%ls", SnapshotClassesHelp);
		return;
	}

	const FString& Label = Args[0];
	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		const UClass* Class = *ClassIt;

		if (!IsValid(Class) ||
			!Class->GetDefaultObject(false) || 
			Class->GetAuthoritativeClass() != Class ||
			Class->HasAnyClassFlags(CLASS_Transient) // transient classes often have huge amounts of data, or are even incapable of being serialized
			)
		{
			continue;
		}

		FJsonStringifyOptions Options;
		Options.Flags |= EJsonStringifyFlags::FilterEditorOnlyData;
		FUtf8String Result = UE::JsonObjectGraph::Stringify({ Class, Class->GetDefaultObject(false) }, Options);
		FString Filename = UE::Private::GetIntermediateAssetName(Class, *Label);
		TUniquePtr<FArchive> FileArchive(IFileManager::Get().CreateDebugFileWriter(*Filename));
		if(FileArchive) // CreateDebugFileWriter fails if the filename is too long, we may have to truncate
		{
			FileArchive->Serialize((void*)Result.GetCharArray().GetData(), Result.GetCharArray().Num() - 1);
		}
	}
}	

FAutoConsoleCommand SnapshotClassesCommand(
	TEXT("snapshotclasses"),
	*FString::Format(TEXT("Write out a snapshot to the Saved directory of currently loaded classes.\n{0}"), { SnapshotClassesHelp }),
	FConsoleCommandWithArgsDelegate::CreateStatic(&SnapshotClasses),
	ECVF_Default
);

FString GetIntermediateAssetName(const UObject* RootObject, const TCHAR* Prefix)
{
	return
		FPaths::ProjectSavedDir() +
		FString(TEXT("Temp/")) + Prefix +
		RootObject->GetPathName() +
		FString(TEXT("_snap.json"));
}
}

void UJsonObjectGraphFunctionLibrary::Stringify(const TArray<UObject*>& Objects, FJsonStringifyOptions Options, FString& ResultString)
{
	FUtf8String Result = UE::JsonObjectGraph::Stringify(Objects, Options);
	ResultString = FString(Result);
}

void UJsonObjectGraphFunctionLibrary::WritePackageToTempFile(const UObject* Object, const FString& Label, FJsonStringifyOptions Options, FString& OutFilename)
{
	OutFilename = FString();
	if (Object == nullptr)
	{
		return;
	}

	const UPackage* Package = Object->GetPackage();
	if (Package == GetTransientPackage())
	{
		UE_LOGF(LogBlueprint, Warning, "Attempted to snapshot the transient package");
		return;
	}

	FUtf8String Result = UE::JsonObjectGraph::Stringify({Package}, Options);
	FString Filename = UE::Private::GetIntermediateAssetName(Package, *Label);
	if (ensure(!Result.IsEmpty()))
	{
		TUniquePtr<FArchive> FileArchive(IFileManager::Get().CreateDebugFileWriter(*Filename));
		FileArchive->Serialize((void*)Result.GetCharArray().GetData(), Result.GetCharArray().Num() - 1);
		OutFilename = MoveTemp(Filename);
	}
}

void UJsonObjectGraphFunctionLibrary::WriteBlueprintClassToTempFile(const UBlueprint* BP, const FString& Label, FJsonStringifyOptions Options, FString& OutFilename)
{
	OutFilename = FString();
	if (BP == nullptr)
	{
		return;
	}

	UClass* BPGC = BP->GeneratedClass;
	if (BP->BlueprintType == BPTYPE_MacroLibrary ||
		!BPGC)
	{
		return;
	}

	if (!BPGC->GetDefaultObject(false))
	{
		UE_LOGF(LogBlueprint, Warning, "Attempted to serialize class with no CDO: %ls", *BPGC->GetPathName());
		return;
	}

	// When writing a class we should always exclude editor only data:
	Options.Flags |= EJsonStringifyFlags::FilterEditorOnlyData;
	FUtf8String Result = UE::JsonObjectGraph::Stringify({ BPGC, BPGC->GetDefaultObject(false) }, Options);
	FString Filename = UE::Private::GetIntermediateAssetName(BP->GetPackage(), *Label);
	TUniquePtr<FArchive> FileArchive(IFileManager::Get().CreateDebugFileWriter(*Filename));
	FileArchive->Serialize((void*)Result.GetCharArray().GetData(), Result.GetCharArray().Num() - 1);
	OutFilename = MoveTemp(Filename);
}
