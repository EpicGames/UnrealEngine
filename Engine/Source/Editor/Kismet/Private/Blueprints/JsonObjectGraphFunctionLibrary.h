// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "JsonObjectGraph/Stringify.h"

#include "JsonObjectGraphFunctionLibrary.generated.h"

UCLASS()
class UJsonObjectGraphFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * ! EXPERIMENTAL ! 
	 * 
	 * Writes the provided objects to a string output, using the JsonObjectGraph format. Reachable
	 * nested objects will be included automatically. Objects not within a root should be included in 
	 * RootObjects if they want to be deeply represented in the result string
	 * 
	 * Examples of invocation from python:
	 *  Print an object:
	 *	print( unreal.JsonObjectGraphFunctionLibrary.stringify([object], tuple()) )
	 *  Print a list objects:
	 *	print( unreal.JsonObjectGraphFunctionLibrary.stringify(objects, tuple()) )
	 *  Print an object's entire package:
	 *	print( unreal.JsonObjectGraphFunctionLibrary.stringify([unreal.EditorAssetLibrary.get_package_for_object(object)], tuple()) )
	 * 
	 * @param	RootObjects		The objects to write at the root level
	 * @param	Options			Options controlling the written format
	 * @param	ResultString	The objects stringified
	 */
	UFUNCTION(BlueprintCallable, Category = "Experimental|Json")
	static void Stringify(const TArray<UObject*>& RootObjects, FJsonStringifyOptions Options, FString& ResultString);

	/** 
	 * ! EXPERIMENTAL ! 
	 * 
	 * Writes all objects in the provided object's package to a temporary file
	 * using the JsonObjectGraph format.
	 * 
	 * @param	Object			The object whose package will be written to the file
	 * @param	Label			A label to disambiguate the temporary file
	 * @param	Options			Options controlling the written format
	 * @param	OutFilename		The filename written, empty if no file written
	 */
	UFUNCTION(BlueprintCallable, Category = "Experimental|Json")
	static void WritePackageToTempFile(const UObject* Object, const FString& Label, FJsonStringifyOptions Options, FString& OutFilename);
	
	/** 
	 * ! EXPERIMENTAL ! 
	 * 
	 * Writes only the provided blueprint's Class and CDO to a temporary file
	 * using the JsonObjectGraph format. Always excludes editor only data.
	 * 
	 * @param	BP				The blueprint to write to a file
	 * @param	Label			A label to disambiguate the temporary file
	 * @param	OutFilename		The filename written, empty if no file written
	 */
	UFUNCTION(BlueprintCallable, Category = "Experimental|Json")
	static void WriteBlueprintClassToTempFile(const UBlueprint* BP, const FString& Label, FJsonStringifyOptions Options, FString& OutFilename);
};

