// Copyright Epic Games, Inc. All Rights Reserved.

#include "CallStackViewer.h"

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
// So that we can poll the running state:
#include "Editor/UnrealEdEngine.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/World.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/SlateDelegates.h"
#include "Framework/Text/TextLayout.h"
#include "HAL/Platform.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/DebuggerCommands.h"
#include "Kismet2/KismetDebugUtilities.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Textures/SlateIcon.h"
#include "ToolMenuContext.h"
#include "ToolMenus.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Stack.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

class ITableRow;
class SWidget;
struct FGeometry;
struct FKeyEvent;
struct FPointerEvent;

extern UNREALED_API UUnrealEdEngine* GUnrealEd;

#define LOCTEXT_NAMESPACE "CallStackViewer"

enum class ECallstackLanguages : uint8
{
	Blueprints,
	NativeCPP,
};

struct FCallStackRow
{
	FCallStackRow(
		UObject* InContextObject,
		const UFunction* InFunction,
		const FName& InScopeName, 
		const FName& InFunctionName, 
		int32 InScriptOffset, 
		ECallstackLanguages InLanguage, 
		const FText& InScopeDisplayName,
		const FText& InFunctionDisplayName
	)
		: ContextObject(InContextObject)
		, Function(InFunction)
		, ScopeName(InScopeName)
		, FunctionName(InFunctionName)
		, ScriptOffset(InScriptOffset)
		, Language(InLanguage)
		, ScopeDisplayName(InScopeDisplayName)
		, FunctionDisplayName(InFunctionDisplayName)
	{
	}

	TWeakObjectPtr<UObject> ContextObject;
	TWeakObjectPtr<const UFunction> Function;

	FName ScopeName;
	FName FunctionName;
	int32 ScriptOffset;

	ECallstackLanguages Language;

	FText ScopeDisplayName;
	FText FunctionDisplayName;

	FText GetTextForEntry() const
	{
		switch(Language)
		{
		case ECallstackLanguages::Blueprints:
			return FText::Format(
				LOCTEXT("CallStackEntry", "{0}: {1}"), 
				ScopeDisplayName, 
				FunctionDisplayName
			);
		case ECallstackLanguages::NativeCPP:
			return ScopeDisplayName;
		}

		return FText();
	}
};

DECLARE_MULTICAST_DELEGATE_OneParam(FOnDisplayedCallstackChanged, TArray<TSharedRef<FCallStackRow>>*);
FOnDisplayedCallstackChanged ListSubscribers;

typedef STreeView<TSharedRef<FCallStackRow>> SCallStackTree;

class SCallStackViewer : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SCallStackViewer ){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TArray<TSharedRef<FCallStackRow>>* CallStackSource);
	void UpdateCallStack(TArray<TSharedRef<FCallStackRow>>* ScriptStack);
	void CopySelectedRows() const;
	void JumpToEntry(TSharedRef< FCallStackRow > Entry);
	void JumpToSelectedEntry();

	/** SWidget interface */
	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent );

	TSharedPtr<SCallStackTree> CallStackTreeWidget;
	TArray<TSharedRef<FCallStackRow>>* CallStackSource;
	TWeakPtr<FCallStackRow> LastFrameNavigatedTo;
	TWeakPtr<FCallStackRow> LastFrameClickedOn;

	TSharedPtr< FUICommandList > CommandList;
};


class SCallStackViewerTableRow : public SMultiColumnTableRow< TSharedRef<FCallStackRow> >
{
public:
	SLATE_BEGIN_ARGS(SCallStackViewerTableRow) { }
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, TWeakPtr<FCallStackRow> InEntry, TWeakPtr<SCallStackViewer> InOwner, const TSharedRef<STableViewBase>& InOwnerTableView )
	{
		CallStackEntry = InEntry;
		Owner = InOwner;
		SMultiColumnTableRow< TSharedRef<FCallStackRow> >::Construct(FSuperRowType::FArguments(), InOwnerTableView);

		TSharedPtr<FCallStackRow> EntryPinned = InEntry.Pin();
		if(EntryPinned.IsValid())
		{
			switch(EntryPinned->Language)
			{
			case ECallstackLanguages::Blueprints:
				SetEnabled(true);
				break;
			case ECallstackLanguages::NativeCPP:
				SetEnabled(false);
				break;
			}
		}
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& InColumnName )
	{
		TSharedPtr<FCallStackRow> CallStackEntryPinned = CallStackEntry.Pin();

		if(InColumnName == TEXT("ProgramCounter") || !CallStackEntryPinned.IsValid())
		{
			TSharedPtr<SCallStackViewer> OwnerPinned = Owner.Pin();
			if(CallStackEntryPinned.IsValid() && OwnerPinned.IsValid())
			{
				TSharedPtr<SImage> Icon;
				if(OwnerPinned->CallStackSource->Num() > 0 && (*OwnerPinned->CallStackSource)[0] == CallStackEntryPinned)
				{
					Icon =  SNew(SImage)
						.Image(FAppStyle::GetBrush("Kismet.CallStackViewer.CurrentStackFrame"))
						.ColorAndOpacity( FAppStyle::GetColor("Kismet.CallStackViewer.CurrentStackFrameColor") );
				}
				else
				{
					const auto NavigationTrackerVisibility = [](TWeakPtr<SCallStackViewer> InOwner, TWeakPtr<FCallStackRow> InCallStackEntry) -> EVisibility
					{
						TSharedPtr<SCallStackViewer> InOwnerPinned = InOwner.Pin();
						TSharedPtr<FCallStackRow> InCallStackEntryPinned = InCallStackEntry.Pin();
						if(InOwnerPinned.IsValid() && InCallStackEntryPinned.IsValid() && InOwnerPinned->LastFrameNavigatedTo == InCallStackEntryPinned)
						{
							return EVisibility::Visible;
						}

						return EVisibility::Hidden;
					};

					Icon = SNew(SImage)
						.Image(FAppStyle::GetBrush("Kismet.CallStackViewer.CurrentStackFrame"))
						.ColorAndOpacity( FAppStyle::GetColor("Kismet.CallStackViewer.LastStackFrameNavigatedToColor") )
						.Visibility(
							TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateStatic(NavigationTrackerVisibility, Owner, CallStackEntry))
						);
				}

				if(Icon.IsValid())
				{
					return SNew( SHorizontalBox )
						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(2, 2, 2, 2)
						[
							Icon.ToSharedRef()
						];
				}
			}
			return SNew(SBox);
		}
		else if( InColumnName == TEXT("FunctionName"))
		{
			return 
				SNew( SHorizontalBox )
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2, 2, 2, 2)
				[
					SNew(STextBlock)
					.Text(
						CallStackEntryPinned->GetTextForEntry()
					)
				];
			}
		else if (InColumnName == TEXT("Language"))
		{
			FText Language;
			switch( CallStackEntryPinned->Language)
			{
				case ECallstackLanguages::Blueprints:
					Language = LOCTEXT("BlueprintsLanguageName", "Blueprints");
					break;
				case ECallstackLanguages::NativeCPP:
					Language = LOCTEXT("CPPLanguageName", "C++");
					break;
			}

			return SNew( SHorizontalBox )
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2, 2, 2, 2)
			[
				SNew(STextBlock)
				.Text(Language)
			];
		}
		else
		{
			ensure(false);
			return SNew(STextBlock)
				.Text(LOCTEXT("UnexpectedColumn", "Unexpected Column"));
		}
	}

	virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		// Our owner needs to know which row was right clicked in order to provide reliable navigation
		// from the context menu:
		TSharedPtr<SCallStackViewer> OwnerPinned = Owner.Pin();
		if(OwnerPinned.IsValid())
		{
			OwnerPinned->LastFrameClickedOn = CallStackEntry;
		}

		return SMultiColumnTableRow< TSharedRef<FCallStackRow> >::OnMouseButtonUp(MyGeometry, MouseEvent);
	}

private:
	TWeakPtr<FCallStackRow> CallStackEntry;
	TWeakPtr<SCallStackViewer> Owner;
};

void SCallStackViewer::Construct(const FArguments& InArgs, TArray<TSharedRef<FCallStackRow>>* InCallStackSource)
{
	CommandList = MakeShareable( new FUICommandList );
	CommandList->MapAction( 
		FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP( this, &SCallStackViewer::CopySelectedRows ),
		// we need to override the default 'can execute' because we want to be available during debugging:
		FCanExecuteAction::CreateStatic( [](){ return true; } )
	);

	CallStackSource = InCallStackSource;

	// The table view 'owns' the row, but it's too inflexible to do anything useful, so we pass in a pointer to SCallStackViewer,
	// this is only necessary because we have multiple columns and SMultiColumnTableRow requires deriving:
	const auto RowGenerator = [](TSharedRef< FCallStackRow > Entry, const TSharedRef<STableViewBase>& TableOwner, TWeakPtr<SCallStackViewer> ControlOwner) -> TSharedRef< ITableRow >
	{
		return SNew(SCallStackViewerTableRow, Entry, ControlOwner, TableOwner);
	};

	const auto ContextMenuOpened = [](TWeakPtr<FUICommandList> InCommandList, TWeakPtr<SCallStackViewer> ControlOwnerWeak) -> TSharedPtr<SWidget>
	{
		const bool CloseAfterSelection = true;
		FMenuBuilder MenuBuilder( CloseAfterSelection, InCommandList.Pin() );
		TSharedPtr<SCallStackViewer> ControlOwner = ControlOwnerWeak.Pin();
		if(ControlOwner.IsValid())
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("GoToDefinition", "Go to Function Definition"), 
				LOCTEXT("GoToDefinitionTooltip", "Opens the Blueprint that declares the function"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(ControlOwner.ToSharedRef(), &SCallStackViewer::JumpToSelectedEntry),
					FCanExecuteAction::CreateStatic( [](){ return true; } )
				)
		);
		}
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
		return MenuBuilder.MakeWidget();
	};

	// there is no nesting in this list view:
	const auto ChildrenAccessor = [](TSharedRef<FCallStackRow> InTreeItem, TArray< TSharedRef< FCallStackRow > >& OutChildren)
	{
	};

	const auto EmptyWarningVisibility = [](TWeakPtr<SCallStackViewer> ControlOwnerWeak) -> EVisibility
	{
		TSharedPtr<SCallStackViewer> ControlOwner = ControlOwnerWeak.Pin();
		if( ControlOwner.IsValid() &&
			ControlOwner->CallStackSource &&
			ControlOwner->CallStackSource->Num() > 0)
		{
			return EVisibility::Hidden;
		}
		return EVisibility::Visible;
	};

	const auto StaleCallStackWarning = [](TWeakPtr<SCallStackViewer> ControlOwnerWeak)->FText
	{
		TSharedPtr<SCallStackViewer> ControlOwner = ControlOwnerWeak.Pin();
		if (ControlOwner.IsValid() &&
			ControlOwner->CallStackSource &&
			ControlOwner->CallStackSource->Num() > 0)
		{
			if (!GUnrealEd->PlayWorld)
			{
				return LOCTEXT("SessionOver", "Warning: The session has ended and therefore the displayed call stack is out of date");
			}
			if (!GUnrealEd->PlayWorld->bDebugPauseExecution)
			{
				return LOCTEXT("CurrentlyRunning", "Warning: The game is currently running - the callstack will be updated when excution is paused");
			}
		}
		return FText();
	};

	const auto CallStackViewIsEnabled = [](TWeakPtr<SCallStackViewer> ControlOwnerWeak) -> bool
	{
		TSharedPtr<SCallStackViewer> ControlOwner = ControlOwnerWeak.Pin();
		if(ControlOwner.IsValid() &&
			ControlOwner->CallStackSource &&
			ControlOwner->CallStackSource->Num() > 0)
		{
			return true;
		}
		return false;
	};

	// Cast due to TSharedFromThis inheritance issues:
	TSharedRef<SCallStackViewer> SelfTyped = StaticCastSharedRef<SCallStackViewer>(AsShared());
	TWeakPtr<SCallStackViewer> SelfWeak = SelfTyped;
	TWeakPtr<FUICommandList> CommandListWeak = CommandList;

	ChildSlot
	[
		SNew(SBorder)
		.Padding(4.0f)
		.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
		[
			SNew(SOverlay)
			+SOverlay::Slot()
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(CallStackTreeWidget, SCallStackTree)
					.TreeItemsSource(CallStackSource)
					.OnGenerateRow(SCallStackTree::FOnGenerateRow::CreateStatic(RowGenerator, SelfWeak))
					.OnGetChildren(SCallStackTree::FOnGetChildren::CreateStatic(ChildrenAccessor))
					.OnMouseButtonDoubleClick(SCallStackTree::FOnMouseButtonClick::CreateSP(SelfTyped, &SCallStackViewer::JumpToEntry))
					.OnContextMenuOpening(FOnContextMenuOpening::CreateStatic(ContextMenuOpened, CommandListWeak, SelfWeak))
					.IsEnabled(
						TAttribute<bool>::Create(
							TAttribute<bool>::FGetter::CreateStatic(CallStackViewIsEnabled, SelfWeak)
						)
					)
					.HeaderRow
					(
						SNew(SHeaderRow)
						+SHeaderRow::Column(TEXT("ProgramCounter"))
						.DefaultLabel(FText::GetEmpty())
						.FixedWidth(16.f)
						+SHeaderRow::Column(TEXT("FunctionName"))
						.FillWidth(.8f)
						.DefaultLabel(LOCTEXT("FunctionName", "Function Name"))
						+SHeaderRow::Column(TEXT("Language"))
						.DefaultLabel(LOCTEXT("Language", "Language"))
						.FillWidth(.15f)
					)
				]
				+ SVerticalBox::Slot()
				[
					SNew(STextBlock)
					.Text(
						TAttribute<FText>::Create(
							TAttribute<FText>::FGetter::CreateStatic(StaleCallStackWarning, SelfWeak)
						)
					)
					.ColorAndOpacity(FLinearColor::Yellow)
					.Justification(ETextJustify::Center)
				]
			]
			+SOverlay::Slot()
			.Padding(32.f)
			[
				SNew(STextBlock)
				.Text(
					LOCTEXT("NoCallStack", "No call stack to display - set a breakpoint and Play in Editor")
				)
				.Justification(ETextJustify::Center)
				.Visibility(
					TAttribute<EVisibility>::Create(
						TAttribute<EVisibility>::FGetter::CreateStatic(EmptyWarningVisibility, SelfWeak)
					)
				)
			]
		]
	];

	ListSubscribers.AddSP(StaticCastSharedRef<SCallStackViewer>(AsShared()), &SCallStackViewer::UpdateCallStack);
}

void SCallStackViewer::UpdateCallStack(TArray<TSharedRef<FCallStackRow>>* ScriptStack)
{
	CallStackTreeWidget->RequestTreeRefresh();
}

void SCallStackViewer::CopySelectedRows() const
{
	FString StringToCopy;

	// We want to copy in the order displayed, not the order selected, so iterate the list and build up the string:
	if(CallStackSource)
	{
		for(const TSharedRef<FCallStackRow>& Item : *CallStackSource)
		{
			if(CallStackTreeWidget->IsItemSelected(Item))
			{
				StringToCopy.Append(Item->GetTextForEntry().ToString());
				StringToCopy.Append(TEXT("\r\n"));
			}
		}
	}

	if( !StringToCopy.IsEmpty() )
	{
		FPlatformApplicationMisc::ClipboardCopy(*StringToCopy);
	}
}

void SCallStackViewer::JumpToEntry(TSharedRef< FCallStackRow > Entry)
{
	LastFrameNavigatedTo = Entry;

	const UFunction* Function = Entry->Function.Get();
	if (Function)
	{
		UEdGraphNode* Node = FKismetDebugUtilities::FindSourceNodeForCodeLocation(Entry->ContextObject.Get(), Function, Entry->ScriptOffset, true);
		if (Node)
		{
			FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(Node);
		}
		else
		{
			FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(Function);
		}
	}
}

void SCallStackViewer::JumpToSelectedEntry()
{
	TSharedPtr<FCallStackRow> LastFrameClickedOnPinned = LastFrameClickedOn.Pin();
	if(LastFrameClickedOnPinned.IsValid())
	{
		JumpToEntry(LastFrameClickedOnPinned.ToSharedRef());
	}
	else if(CallStackSource)
	{
		for(const TSharedRef<FCallStackRow>& Item : *CallStackSource)
		{
			if(CallStackTreeWidget->IsItemSelected(Item))
			{
				JumpToEntry(Item);
			}
		}
	}
}

FReply SCallStackViewer::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

// Proxy array of the call stack. This allows us to manually refresh UI state when changes are made:
TArray<TSharedRef<FCallStackRow>> Private_CallStackSource;

void CallStackViewer::UpdateDisplayedCallstack(TArrayView<const FFrame* const> ScriptStack)
{
	Private_CallStackSource.Empty();
	if (ScriptStack.Num() > 0)
	{
		for (int32 I = ScriptStack.Num() - 1; I >= 0; --I)
		{
			const FFrame* StackNode = ScriptStack[I];
			
			bool bExactClass = false;
			bool bAnyPackage = true;
			UClass* SourceClass = Cast<UClass>(StackNode->Node->GetOuter());
			// We're using GetClassNameWithoutSuffix so that we can display the BP name, which will
			// be the most useful when debugging:
			FText ScopeDisplayName = SourceClass ? FText::FromString(FBlueprintEditorUtils::GetClassNameWithoutSuffix(SourceClass)) : FText::FromName(StackNode->Node->GetOuter()->GetFName());
			FText FunctionDisplayName = FText::FromName(StackNode->Node->GetFName());
			if(SourceClass)
			{
				if(const UBlueprint* SourceBP = Cast<UBlueprint>(SourceClass->ClassGeneratedBy))
				{
					if( StackNode->Node->GetFName() == FBlueprintEditorUtils::GetUbergraphFunctionName(SourceBP))
					{
						FunctionDisplayName = LOCTEXT("EventGraphCallStackName", "Event Graph");
					}
					else
					{
						UEdGraphNode* Node = FKismetDebugUtilities::FindSourceNodeForCodeLocation(StackNode->Object, StackNode->Node, 0, true);
						if(Node)
						{
							FunctionDisplayName = Node->GetNodeTitle(ENodeTitleType::ListView);
						}
					}
				}
			}

			int32 ScriptOffset = UE_PTRDIFF_TO_INT32(StackNode->Code - StackNode->Node->Script.GetData() - 1);
			Private_CallStackSource.Add(
				MakeShared<FCallStackRow>(
					StackNode->Object,
					StackNode->Node,
					StackNode->Node->GetOuter()->GetFName(),
					StackNode->Node->GetFName(),
					ScriptOffset,
					ECallstackLanguages::Blueprints,
					ScopeDisplayName,
					FunctionDisplayName
				)
			);

			// If the next frame in the call stack does did not call this frame, insert a dummy entry informing
			// the user that there was a native block:
			if(StackNode->PreviousFrame == nullptr)
			{
				Private_CallStackSource.Add(
					MakeShared<FCallStackRow>(
						nullptr,
						nullptr,
						FName(),
						FName(),
						0,
						ECallstackLanguages::NativeCPP,
						LOCTEXT("NativeCodeLabel", "Native Code"),
						FText()
					)
				);
			}
		}
	}

	// Notify subscribers:
	ListSubscribers.Broadcast(&Private_CallStackSource);
}

FName CallStackViewer::GetTabName()
{
	const FName TabName = TEXT("CallStackViewer");
	return TabName;
}

void CallStackViewer::RegisterTabSpawner(FTabManager& TabManager)
{
	const auto SpawnCallStackViewTab = []( const FSpawnTabArgs& Args )
	{
		static const FName ToolbarName = TEXT("Kismet.DebuggingViewToolBar");
		FToolMenuContext MenuContext(FPlayWorldCommands::GlobalPlayWorldActions);
		TSharedRef<SWidget> ToolbarWidget = UToolMenus::Get()->GenerateWidget(ToolbarName, MenuContext);
		return SNew(SDockTab)
			.TabRole( ETabRole::PanelTab )
			.Label( LOCTEXT("TabTitle", "Call Stack") )
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SBorder)
					.BorderImage( FAppStyle::GetBrush( TEXT("NoBorder") ) )
					[
						ToolbarWidget
					]
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SBorder)
					.BorderImage( FAppStyle::GetBrush("Docking.Tab.ContentAreaBrush") )
					[
						SNew(SCallStackViewer, &Private_CallStackSource)
					]
				]
			];
	};
	
	TabManager.RegisterTabSpawner(CallStackViewer::GetTabName(), FOnSpawnTab::CreateStatic(SpawnCallStackViewTab) )
		.SetDisplayName( LOCTEXT("TabTitle", "Call Stack") )
		.SetTooltipText( LOCTEXT("TooltipText", "Open the Call Stack tab") );
}

#undef LOCTEXT_NAMESPACE
