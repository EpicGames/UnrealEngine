// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/BatchProcessCommandlet.h"

#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "BatchProcess/BatchProcessLog.h"
#include "BatchProcess/BatchProcessRunner.h"

static void PrintBatchProcessCommandletUsage()
{
	UE_LOGF(LogBatchProcess, Display,
		"Repro command: UnrealEditor ProjectName -run=BatchProcessCommandlet Jobs.json [-resultsfile=Path/To/results.txt] [-numworkers=N]");
}

UBatchProcessCommandlet::UBatchProcessCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UBatchProcessCommandlet::Main(const FString& Params)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamsMap;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamsMap);

	if (Tokens.Num() != 1)
	{
		PrintBatchProcessCommandletUsage();
		return -1;
	}

	FString JobJson;
	if (!FFileHelper::LoadFileToString(JobJson, *Tokens[0]))
	{
		UE_LOGF(LogBatchProcess, Error, "Failed to read job file: %ls", *Tokens[0]);
		return -1;
	}

	UE::Private::FBatchRunOptions Options;
	Options.ClientAddress = ParamsMap.Find(TEXT("launchforclient"));
	Options.NumWorkers    = ParamsMap.Contains(TEXT("numworkers"))
		? FMath::Max(1, FCString::Atoi(*ParamsMap[TEXT("numworkers")]))
		: 0; // 0 = auto

	const UE::Private::FBatchRunResult RunResult = UE::Private::RunBatchFromJson(JobJson, Options);

	if (RunResult.bHasCoordinatorErrors)
	{
		UE_LOGF(LogBatchProcess, Error, "%ls", *RunResult.CoordinatorErrorString);
		return -1;
	}

	// Write a structured results file — failures first so they're easy to spot.
	FString ResultsFile;
	if (const FString* Override = ParamsMap.Find(TEXT("resultsfile")))
	{
		ResultsFile = *Override;
	}
	else
	{
		ResultsFile = FPaths::ProjectSavedDir() / TEXT("MultiprocessResults") / TEXT("results.txt");
	}
	IFileManager::Get().MakeDirectory(*FPaths::GetPath(ResultsFile), /*Tree=*/true);

	int32 NumFailed = 0;
	TArray<FString> FailLines;
	TArray<FString> PassLines;
	for (int32 Idx = 0; Idx < RunResult.Jobs.Num(); ++Idx)
	{
		const FBatchJobResult& Job = RunResult.Jobs[Idx];
		const FString Output = !Job.Output.IsEmpty()
			? Job.Output
			: FString::Printf(TEXT("(no output) arg: %s"), *Job.ArgumentSummary);
		FString Line = FString::Printf(TEXT("[%s job %4d]  %s"),
			Job.bEncounteredError ? TEXT("FAIL") : TEXT("PASS"), Idx, *Output);
		if (Job.bEncounteredError)
		{
			++NumFailed;
			FailLines.Add(MoveTemp(Line));
		}
		else
		{
			PassLines.Add(MoveTemp(Line));
		}
	}

	TArray<FString> AllLines;
	AllLines.Append(MoveTemp(FailLines));
	AllLines.Append(MoveTemp(PassLines));
	AllLines.Add(FString::Printf(TEXT("\n--- %d total | %d passed | %d failed ---"),
		RunResult.Jobs.Num(), RunResult.Jobs.Num() - NumFailed, NumFailed));
	FFileHelper::SaveStringArrayToFile(AllLines, *ResultsFile);

	UE_LOGF(LogBatchProcess, Display,
		"Results: %d/%d passed  —  report: %ls",
		RunResult.Jobs.Num() - NumFailed, RunResult.Jobs.Num(), *ResultsFile);

	return NumFailed > 0 ? -1 : 0;
}
