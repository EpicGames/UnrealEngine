// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Commandlets/Commandlet.h"

#include "BatchProcessCommandlet.generated.h"

UCLASS()
class UBatchProcessCommandlet : public UCommandlet
{
	GENERATED_BODY()

	UBatchProcessCommandlet(const FObjectInitializer& ObjectInitializer);
private:
	// Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	// End UCommandlet Interface
};
