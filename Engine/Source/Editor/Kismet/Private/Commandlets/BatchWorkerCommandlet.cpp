// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/BatchWorkerCommandlet.h"

#include "Logging/LogMacros.h"
#include "BatchProcess/BatchProcessLog.h"
#include "BatchProcess/BatchProcessWorker.h"


namespace UE::Private
{
static void PrintBatchWorkerCommandletUsage();
}

void UE::Private::PrintBatchWorkerCommandletUsage()
{
	UE_LOGF(LogBatchProcess, Display, "UnrealEditor ProjectName -run=BatchWorkerCommandlet <CoordinatorIP> <WorkerId>");
	UE_LOGF(LogBatchProcess, Display, "  or: -run=BatchWorkerCommandlet -singlecommand='{\"function\":\"...\",\"arguments\":[{...}]}'");
}

UBatchWorkerCommandlet::UBatchWorkerCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UBatchWorkerCommandlet::Main(const FString& Params)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamsMap;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamsMap);

	// -singlecommand mode: run one job inline without a coordinator, for debugging failures.
	if (const FString* SingleCommand = ParamsMap.Find(TEXT("singlecommand")))
	{
		const bool bOk = FBatchProcessWorker::RunSingleCommand(*SingleCommand);
		return bOk ? 0 : -1;
	}

	if(Tokens.Num() > 2 || Tokens.Num() == 0)
	{
		UE::Private::PrintBatchWorkerCommandletUsage();
		return -1;
	}

	FBatchProcessWorker Worker;

	FString CoordinatorAddrStr;
	int32 WorkerId = 0;
	if(Tokens.Num() == 1)
	{
		// Single-argument mode: launch a local coordinator for debugging.
		const bool bLocalCoordinatorLaunched = Worker.LaunchLocalCoordinator(Tokens[0], CoordinatorAddrStr);
		if(!bLocalCoordinatorLaunched)
		{
			return -1;
		}
	}
	else
	{
		CoordinatorAddrStr = Tokens[0];
		WorkerId           = FCString::Atoi(*Tokens[1]);
	}

	uint32 CoordinatorPid = 0;
	if (const FString* PidStr = ParamsMap.Find(TEXT("coordinatorpid")))
	{
		CoordinatorPid = (uint32)FCString::Atoi64(**PidStr);
	}

	const bool bSucceeded = Worker.PerformWork(CoordinatorAddrStr, WorkerId, CoordinatorPid);
	if(!bSucceeded)
	{
		UE_LOGF(LogBatchProcess, Error, "Worker failed");
	}
	return bSucceeded ? 0 : -1;
}
