// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Commandlets/Commandlet.h"

#include "BatchWorkerCommandlet.generated.h"

UCLASS()
class UBatchWorkerCommandlet : public UCommandlet
{
	GENERATED_BODY()

	UBatchWorkerCommandlet(const FObjectInitializer& ObjectInitializer);
private:
	// Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	// End UCommandlet Interface
};
