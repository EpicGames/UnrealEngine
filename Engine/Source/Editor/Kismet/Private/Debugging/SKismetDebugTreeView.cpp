// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debugging/SKismetDebugTreeView.h"

#include "Algo/Reverse.h"
#include "AssetThumbnail.h"
#include "BlueprintDebugger.h"
#include "BlueprintEditor.h"
#include "BlueprintEditorModule.h"
#include "BlueprintEditorTabs.h"
#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "Debugging/SKismetDebuggingView.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Editor/EditorEngine.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Engine.h"
#include "Engine/LatentActionManager.h"
#include "Engine/World.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "GraphEditorSettings.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Input/Reply.h"
#include "Internationalization/Internationalization.h"
#include "K2Node.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/Breakpoint.h"
#include "Kismet2/KismetDebugUtilities.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/WatchedPin.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/Color.h"
#include "Math/NumericLimits.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Modules/ModuleManager.h"
#include "PropertyInfoViewStyle.h"
#include "SlotBase.h"
#include "SourceCodeNavigation.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateIconFinder.h"
#include "Styling/StyleColors.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTypeTraits.h"
#include "Textures/SlateIcon.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Trace/Detail/Channel.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/FieldPath.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UnrealEngine.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"

class ITableRow;
struct FGeometry;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "DebugViewUI"

DEFINE_LOG_CATEGORY_STATIC(LogBlueprintDebugTreeView, Log, All);

//////////////////////////////////////////////////////////////////////////

/** The editor object */
extern UNREALED_API class UEditorEngine* GEditor;

static const FText ViewInDebuggerText = LOCTEXT("ViewInDebugger", "View in Blueprint Debugger");
static const FText ViewInDebuggerTooltipText = LOCTEXT("ViewInDebugger_Tooltip", "Opens the Blueprint Debugger and starts watching this variable if it isn't already watched");
static constexpr float ThumbnailIconSize = 16.0f;
static constexpr uint32 ThumbnailIconResolution = 16;

static int DebuggerMaxSearchDepth = 50;
static FAutoConsoleVariableRef CVarDebuggerMaxDepth(TEXT("bp.DebuggerMaxSearchDepth"), DebuggerMaxSearchDepth, TEXT("The maximum search depth of Blueprint Debugger TreeView widgets (set to <= 0 for infinite depth)"), ECVF_Default);

static bool bDebuggerEnableExternalSearch = false;
static FAutoConsoleVariableRef CVarDebuggerEnableExternalSearch(TEXT("bp.DebuggerEnableExternalSearch"), bDebuggerEnableExternalSearch, TEXT("Allows the Blueprint Debugger TreeView widget to search external objects"), ECVF_Default);

//////////////////////////////////////////////////////////////////////////

const FName SKismetDebugTreeView::ColumnId_Name("Name");
const FName SKismetDebugTreeView::ColumnId_Value("Value");

//////////////////////////////////////////////////////////////////////////
// SKismetDebugTreePropertyValueWidget

namespace
{
	class SKismetDebugTreePropertyValueWidget : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SKismetDebugTreePropertyValueWidget)
			: _PropertyInfo(nullptr)
			, _TreeItem(nullptr)
		{}

		SLATE_ATTRIBUTE(TSharedPtr<FPropertyInstanceInfo>, PropertyInfo)
			SLATE_ARGUMENT(FDebugTreeItemPtr, TreeItem)

			SLATE_END_ARGS()

	public:

		void Construct(const FArguments& InArgs, TSharedPtr<FString> InSearchString)
		{
			PropertyInfo = InArgs._PropertyInfo;
			TreeItem = InArgs._TreeItem;
			check(TreeItem.IsValid());

			TSharedPtr<FPropertyInstanceInfo> Data = PropertyInfo.Get();
			if (Data.IsValid())
			{
				if (Data->Property->IsA<FObjectProperty>() || Data->Property->IsA<FInterfaceProperty>())
				{
					ChildSlot
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(PropertyInfoViewStyle::STextHighlightOverlay)
							.FullText(this, &SKismetDebugTreePropertyValueWidget::GetObjectValueText)
						.HighlightText(this, &SKismetDebugTreePropertyValueWidget::GetHighlightText, InSearchString)
						[
							SNew(STextBlock)
							.ToolTipText(this, &SKismetDebugTreePropertyValueWidget::GetValueTooltipText)
						.Text(this, &SKismetDebugTreePropertyValueWidget::GetObjectValueText)
						]
						]
					+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(SSpacer)
							.Size(FVector2D(2.0f, 1.0f))
						]
					+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(SHyperlink)
							.ToolTipText(this, &SKismetDebugTreePropertyValueWidget::GetClassLinkTooltipText)
						.Text(this, &SKismetDebugTreePropertyValueWidget::GetObjectClassText)
						.OnNavigate(this, &SKismetDebugTreePropertyValueWidget::OnNavigateToClass)
						]
					+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(SSpacer)
							.Size(FVector2D(2.0f, 1.0f))
						]
					+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ObjectValueEnd", ")"))
						]
						];
				}
				else
				{
					ChildSlot
						[
							SNew(PropertyInfoViewStyle::STextHighlightOverlay)
							.FullText(this, &SKismetDebugTreePropertyValueWidget::GetDescription)
						.HighlightText(this, &SKismetDebugTreePropertyValueWidget::GetHighlightText, InSearchString)
						[
							SNew(STextBlock)
							.ToolTipText(this, &SKismetDebugTreePropertyValueWidget::GetDescription)
						.Text(this, &SKismetDebugTreePropertyValueWidget::GetDescription)
						]
						];
				}
			}
		}

	private:

		FText GetDescription() const
		{
			return TreeItem->GetDescription();
		}

		FText GetHighlightText(TSharedPtr<FString> InSearchString) const
		{
			return TreeItem->GetHighlightText(InSearchString);
		}

		FText GetObjectValueText() const
		{
			TSharedPtr<FPropertyInstanceInfo> Data = PropertyInfo.Get();
			if (Data.IsValid())
			{
				if (const UObject* Object = Data->Object.Get())
				{
					return FText::Format(LOCTEXT("ObjectValueBegin", "{0} (Class: "), FText::FromString(Object->GetName()));
				}
			}

			return LOCTEXT("UnknownObjectValueBegin", "[Unknown] (Class: ");
		}

		FText GetValueTooltipText() const
		{
			TSharedPtr<FPropertyInstanceInfo> Data = PropertyInfo.Get();
			if (Data.IsValid())
			{
				// if this is an Object property, tooltip text should include its full name
				if (const UObject* Object = Data->Object.Get())
				{
					return FText::Format(LOCTEXT("ObjectValueTooltip", "{0}\nClass: {1}"),
						FText::FromString(Object->GetFullName()),
						FText::FromString(Object->GetClass()->GetFullName()));
				}
			}

			return GetDescription();
		}

		FText GetClassLinkTooltipText() const
		{
			TSharedPtr<FPropertyInstanceInfo> Data = PropertyInfo.Get();
			if (Data.IsValid())
			{
				if (const UObject* Object = Data->Object.Get())
				{
					if (UClass* Class = Object->GetClass())
					{
						if (UBlueprint* Blueprint = Cast<UBlueprint>(Class->ClassGeneratedBy))
						{
							return LOCTEXT("OpenBlueprintClass", "Opens this Class in the Blueprint Editor");
						}
						else
						{
							// this is a native class
							return LOCTEXT("OpenNativeClass", "Navigates to this class' source file");
						}
					}
				}
			}

			return LOCTEXT("UnknownClassName", "[Unknown]");
		}

		FText GetObjectClassText() const
		{
			TSharedPtr<FPropertyInstanceInfo> Data = PropertyInfo.Get();
			if (Data.IsValid())
			{
				if (const UObject* Object = Data->Object.Get())
				{
					return FText::FromString(Object->GetClass()->GetName());
				}
			}

			return LOCTEXT("UnknownClassName", "[Unknown]");
		}

		void OnNavigateToClass() const
		{
			TSharedPtr<FPropertyInstanceInfo> Data = PropertyInfo.Get();
			if (Data.IsValid())
			{
				if (const UObject* Object = Data->Object.Get())
				{
					if (UClass* Class = Object->GetClass())
					{
						if (UBlueprint* Blueprint = Cast<UBlueprint>(Class->ClassGeneratedBy))
						{
							GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Blueprint);
						}
						else
						{
							// this is a native class
							FSourceCodeNavigation::NavigateToClass(Class);
						}
					}
				}
			}
		}

		FDebugTreeItemPtr TreeItem;
		TAttribute<TSharedPtr<FPropertyInstanceInfo>> PropertyInfo;
	};
}

//////////////////////////////////////////////////////////////////////////
// FDebugLineItem

uint16 FDebugLineItem::ActiveTypeBitset = TNumericLimits<uint16>::Max(); // set all to active by default

FText FDebugLineItem::GetName() const
{
	return FText::GetEmpty();
}

FText FDebugLineItem::GetDisplayName() const
{
	return FText::GetEmpty();
}

FText FDebugLineItem::GetDescription() const
{
	return FText::GetEmpty();
}

bool FDebugLineItem::HasName() const
{
	return !GetDisplayName().IsEmpty();
}

bool FDebugLineItem::HasValue() const
{
	return !GetDescription().IsEmpty();
}

void FDebugLineItem::CopyNameToClipboard() const
{
	FPlatformApplicationMisc::ClipboardCopy(ToCStr(GetDisplayName().ToString()));
}

void FDebugLineItem::CopyValueToClipboard() const
{
	FPlatformApplicationMisc::ClipboardCopy(ToCStr(GetDescription().ToString()));
}

TSharedRef<SWidget> FDebugLineItem::GenerateNameWidget(TSharedPtr<FString> InSearchString)
{
	return SNew(PropertyInfoViewStyle::STextHighlightOverlay)
		.FullText(this, &FDebugLineItem::GetDisplayName)
		.HighlightText(this, &FDebugLineItem::GetHighlightText, InSearchString)
		[
			SNew(STextBlock)
				.ToolTipText(this, &FDebugLineItem::GetDisplayName)
				.Text(this, &FDebugLineItem::GetDisplayName)
		];
}

TSharedRef<SWidget> FDebugLineItem::GenerateValueWidget(TSharedPtr<FString> InSearchString)
{
	return SNew(PropertyInfoViewStyle::STextHighlightOverlay)
		.FullText(this, &FDebugLineItem::GetDescription)
		.HighlightText(this, &FDebugLineItem::GetHighlightText, InSearchString)
		[
			SNew(STextBlock)
				.ToolTipText(this, &FDebugLineItem::GetDescription)
				.Text(this, &FDebugLineItem::GetDescription)
		];
}

void FDebugLineItem::MakeMenu(FMenuBuilder& MenuBuilder, bool bInDebuggerTab)
{
	if (HasName())
	{
		const FUIAction CopyName(
			FExecuteAction::CreateSP(this, &FDebugLineItem::CopyNameToClipboard),
			FCanExecuteAction::CreateSP(this, &FDebugLineItem::HasName)
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("CopyName", "Copy Name"),
			LOCTEXT("CopyName_ToolTip", "Copy name to clipboard"),
			FSlateIcon(),
			CopyName
		);
	}

	if (HasValue())
	{
		const FUIAction CopyValue(
			FExecuteAction::CreateSP(this, &FDebugLineItem::CopyValueToClipboard),
			FCanExecuteAction::CreateSP(this, &FDebugLineItem::HasValue)
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("CopyValue", "Copy Value"),
			LOCTEXT("CopyValue_ToolTip", "Copy value to clipboard"),
			FSlateIcon(),
			CopyValue
		);
	}

	ExtendContextMenu(MenuBuilder, bInDebuggerTab);
}

void FDebugLineItem::ExtendContextMenu(class FMenuBuilder& MenuBuilder, bool bInDebuggerTab)
{
}

void FDebugLineItem::UpdateSearch(const FString& InSearchString, FDebugLineItem::ESearchFlags SearchFlags)
{
	const bool bIsRootNode = SearchFlags & SF_RootNode;
	const bool bIsContainerElement = SearchFlags & SF_ContainerElement;

	// Container elements share their parent's property name, so we shouldn't search them by name
	bVisible = (!bIsContainerElement && GetName().ToString().Contains(InSearchString)) ||
		GetDisplayName().ToString().Contains(InSearchString) ||
		GetDescription().ToString().Contains(InSearchString);

	// for root nodes, bParentsMatchSearch always matches bVisible
	if (bVisible || bIsRootNode)
	{
		bParentsMatchSearch = bVisible;
	}
}

bool FDebugLineItem::IsVisible()
{
	return bVisible;
}

bool FDebugLineItem::DoParentsMatchSearch()
{
	return bParentsMatchSearch;
}

bool FDebugLineItem::HasChildren() const
{
	return false;
}

TSharedRef<SWidget> FDebugLineItem::GetNameIcon()
{
	static const FSlateBrush* CachedBrush = FAppStyle::GetBrush(TEXT("NoBrush"));
	return SNew(SImage).Image(CachedBrush);
}

TSharedRef<SWidget> FDebugLineItem::GetValueIcon()
{
	static const FSlateBrush* CachedBrush = FAppStyle::GetBrush(TEXT("NoBrush"));
	return SNew(SImage).Image(CachedBrush);
}

FText FDebugLineItem::GetHighlightText(const TSharedPtr<FString> InSearchString) const
{
	return FText::FromString(*InSearchString);
}

UBlueprint* FDebugLineItem::GetBlueprintForObject(UObject* ParentObject)
{
	if (ParentObject == nullptr)
	{
		return nullptr;
	}

	if (UBlueprint* ParentBlueprint = Cast<UBlueprint>(ParentObject))
	{
		return ParentBlueprint;
	}

	if (UClass* ParentClass = ParentObject->GetClass())
	{
		if (UBlueprint* ParentBlueprint = Cast<UBlueprint>(ParentClass->ClassGeneratedBy))
		{
			return ParentBlueprint;
		}
	}

	// recursively walk up ownership hierarchy until we find the blueprint
	return GetBlueprintForObject(ParentObject->GetOuter());
}

UBlueprintGeneratedClass* FDebugLineItem::GetClassForObject(UObject* ParentObject)
{
	if (ParentObject != nullptr)
	{
		if (UBlueprint* Blueprint = Cast<UBlueprint>(ParentObject))
		{
			return Cast<UBlueprintGeneratedClass>(*Blueprint->GeneratedClass);
		}
		else if (UBlueprintGeneratedClass* Result = Cast<UBlueprintGeneratedClass>(ParentObject))
		{
			return Result;
		}
		else
		{
			return Cast<UBlueprintGeneratedClass>(ParentObject->GetClass());
		}
	}

	return nullptr;
}

bool FDebugLineItem::IsDebugLineTypeActive(EDebugLineType Type)
{
	const uint16 Mask = 1 << Type;
	return ActiveTypeBitset & Mask;
}

void FDebugLineItem::OnDebugLineTypeActiveChanged(ECheckBoxState CheckState, EDebugLineType Type)
{
	const uint16 Mask = 1 << Type;
	switch (CheckState)
	{
	case ECheckBoxState::Checked:
		ActiveTypeBitset |= Mask;
		break;
	default:
		ActiveTypeBitset &= ~Mask;
		break;
	}
}

//////////////////////////////////////////////////////////////////////////
// ILineItemWithChildren

class FLineItemWithChildren : public FDebugLineItem
{
public:
	FLineItemWithChildren(EDebugLineType InType) :
		FDebugLineItem(InType)
	{}

	virtual ~FLineItemWithChildren() override = default;

	virtual bool HasChildren() const override
	{
		return !ChildrenMirrors.IsEmpty();
	}

	virtual bool CanHaveChildren() override { return true; }

	/** Pilot for Recursive Search */
	bool SearchRecursive(const FString& InSearchString, TSharedPtr<STreeView<FDebugTreeItemPtr>> DebugTreeView)
	{
		TArray<FLineItemWithChildren*> Parents;
		return SearchRecursive(InSearchString, DebugTreeView, Parents);
	}

	// ensures that ChildrenMirrors are set up for calls to EnsureChildIsAdded
	virtual void GatherChildrenBase(TArray<FDebugTreeItemPtr>& OutChildren, const FString& InSearchString, bool bRespectSearch) override
	{
		Swap(PrevChildrenMirrors, ChildrenMirrors);
		ChildrenMirrors.Empty();
		GatherChildren(OutChildren, InSearchString, bRespectSearch);
	}

	// allows FDebugTreeItemPtr to be stored in TSets 
	class FDebugTreeItemKeyFuncs
	{
	public:
		typedef FDebugTreeItemPtr ElementType;
		typedef TTypeTraits<ElementType>::ConstPointerType KeyInitType;
		typedef TCallTraits<ElementType>::ParamType ElementInitType;
		enum { bAllowDuplicateKeys = false };

		/**
		* @return The key used to index the given element.
		*/
		static FORCEINLINE KeyInitType GetSetKey(ElementInitType Element)
		{
			return Element;
		}

		/**
		* @return True if the keys match.
		*/
		static FORCEINLINE bool Matches(KeyInitType A, KeyInitType B)
		{
			const FDebugLineItem* APtr = A.Get();
			const FDebugLineItem* BPtr = B.Get();
			if (APtr && BPtr)
			{
				return (APtr->Type == BPtr->Type) && APtr->Compare(BPtr);
			}
			return APtr == BPtr;
		}

		/** Calculates a hash index for a key. */
		static FORCEINLINE uint32 GetKeyHash(KeyInitType Key)
		{
			if (const FDebugLineItem* KeyPtr = Key.Get())
			{
				return KeyPtr->GetHash();
			}
			return GetTypeHash(Key);
		}
	};
protected:
	// Last frames cached children
	TSet<FDebugTreeItemPtr, FDebugTreeItemKeyFuncs> PrevChildrenMirrors;
	// This frames children
	TSet<FDebugTreeItemPtr, FDebugTreeItemKeyFuncs> ChildrenMirrors;

	/** @returns whether this item represents a container property */
	virtual bool IsContainer() const
	{
		return false;
	}

	virtual void GatherChildren(TArray<FDebugTreeItemPtr>& OutChildren, const FString& InSearchString, bool bRespectSearch) {}

	/** returns the object related to this line item if there is one, used to avoid searching external objects */
	virtual const UObject* GetRelatedObject() const { return nullptr; };

	/** returns whether a child is  */
	virtual bool IsExternalTo(const FLineItemWithChildren* Parent) const
	{
		if (const UObject* ChildObj = GetRelatedObject())
		{
			if (const UObject* ParentObj = Parent->GetRelatedObject())
			{
				if (!ChildObj->IsIn(ParentObj))
				{
					return true;
				}
			}
		}

		return false;
	}
	
	/**
	* returns whether this node should be visible according to the users
	* search query
	*
	* O( number of recursive children )
	*/
	bool SearchRecursive(const FString& InSearchString,
		TSharedPtr<STreeView<FDebugTreeItemPtr>> DebugTreeView,
		TArray<FLineItemWithChildren*>& Parents,
		ESearchFlags SearchFlags = SF_None)
	{
		bVisible = false;

		UpdateSearch(InSearchString, SearchFlags);

		bool bChildMatch = false;
		Parents.Push(this);

		ESearchFlags ChildSearchFlags = IsContainer() ? SF_ContainerElement : SF_None;

		TArray<FDebugTreeItemPtr> Children;
		GatherChildrenBase(Children, InSearchString,/*bRespectSearch =*/ false);
		for (const FDebugTreeItemPtr& ChildRef : Children)
		{
			if (ChildRef->CanHaveChildren())
			{
				ChildRef->bParentsMatchSearch = bParentsMatchSearch;
				FLineItemWithChildren* Child = StaticCast<FLineItemWithChildren*>(ChildRef.Get());

				// check if the child has been seen already in parents.
				// if it has, skip it. (avoids stack overflows)
				if (Parents.FindByPredicate(
					[Child](const FLineItemWithChildren* Relative)
					{
						return (Relative->Type == Child->Type) && Relative->Compare(Child);
					}
				))
				{
					continue;
				}

				// stop recursing if we reached the max depth
				if (DebuggerMaxSearchDepth > 0 && Parents.Num() > DebuggerMaxSearchDepth)
				{
					continue;
				}

				if (!bDebuggerEnableExternalSearch)
				{
					FLineItemWithChildren* ObjectParent = nullptr;
					for (FLineItemWithChildren* Parent : Parents)
					{
						if (Parent->GetRelatedObject())
						{
							ObjectParent = Parent;
							break;
						}
					}
					// check if we need to stop searching due to external objects
					if (ObjectParent && Child->IsExternalTo(ObjectParent))
					{
						// update this child, but don't recurse
						Child->UpdateSearch(InSearchString, ChildSearchFlags);

						// if any children need to expand, so should this
						if (ChildRef->IsVisible())
						{
							bVisible = true;
							bChildMatch = true;
						}
						continue;
					}
				}

				// if any children need to expand, so should this
				if (Child->SearchRecursive(InSearchString, DebugTreeView, Parents, ChildSearchFlags))
				{
					bVisible = true;
					bChildMatch = true;
				}
			}
			else
			{
				ChildRef->UpdateSearch(InSearchString, ChildSearchFlags);

				// if any children need to expand, so should this
				if (ChildRef->IsVisible())
				{
					bVisible = true;
					bChildMatch = true;
				}
			}
		}

		Parents.Pop(EAllowShrinking::No);
		if (bChildMatch)
		{
			DebugTreeView->SetItemExpansion(SharedThis(this), true);
		}

		return bVisible;
	}

	/**
	 * Adds either Item or an identical node that was previously
	 * created (present in ChildrenMirrors) as a child to OutChildren.
	 *
	 * O( 1 )
	 */
	void EnsureChildIsAdded(TArray<FDebugTreeItemPtr>& OutChildren, const FDebugLineItem& Item, const FString& InSearchString, bool bRespectSearch)
	{
		const FDebugTreeItemPtr Shareable = MakeShareable(Item.Duplicate());
		if (FDebugTreeItemPtr* Found = PrevChildrenMirrors.Find(Shareable))
		{
			FDebugTreeItemPtr FoundItem = *Found;
			FoundItem->UpdateData(Item);
			ChildrenMirrors.Add(FoundItem);

			// only add item if it matches search
			if (!bRespectSearch || InSearchString.IsEmpty() || FoundItem->IsVisible())
			{
				OutChildren.Add(FoundItem);
			}
		}
		else
		{
			ChildrenMirrors.Add(Shareable);
			OutChildren.Add(Shareable);
		}
	}
};


//////////////////////////////////////////////////////////////////////////
// FMessageLineItem

struct FMessageLineItem : public FDebugLineItem
{
protected:
	FString Message;
public:
	// Message line
	FMessageLineItem(const FString& InMessage)
		: FDebugLineItem(DLT_Message)
		, Message(InMessage)
	{
	}

	virtual FText GetDescription() const override
	{
		return FText::FromString(Message);
	}

protected:
	virtual FDebugLineItem* Duplicate() const override
	{
		return new FMessageLineItem(Message);
	}

	virtual bool Compare(const FDebugLineItem* BaseOther) const override
	{
		FMessageLineItem* Other = (FMessageLineItem*)BaseOther;
		return Message == Other->Message;
	}

	virtual uint32 GetHash() const override
	{
		return GetTypeHash(Message);
	}
};

//////////////////////////////////////////////////////////////////////////
// FLatentActionLineItem

struct FLatentActionLineItem : public FDebugLineItem
{
protected:
	int32 UUID;
	TWeakObjectPtr< UObject > ParentObjectRef;

public:
	FLatentActionLineItem(int32 InUUID, UObject* ParentObject)
		: FDebugLineItem(DLT_LatentAction)
	{
		UUID = InUUID;
		check(UUID != INDEX_NONE);
		ParentObjectRef = ParentObject;
	}

	virtual TSharedRef<SWidget> GenerateNameWidget(TSharedPtr<FString> InSearchString) override;
	virtual TSharedRef<SWidget> GetNameIcon() override;
	virtual FText GetDescription() const override;

protected:
	virtual FDebugLineItem* Duplicate() const override
	{
		return new FLatentActionLineItem(UUID, ParentObjectRef.Get());
	}

	virtual bool Compare(const FDebugLineItem* BaseOther) const override
	{
		FLatentActionLineItem* Other = (FLatentActionLineItem*)BaseOther;
		return (ParentObjectRef.Get() == Other->ParentObjectRef.Get()) &&
			(UUID == Other->UUID);
	}

	virtual uint32 GetHash() const override
	{
		return HashCombine(GetTypeHash(UUID), GetTypeHash(ParentObjectRef));
	}

protected:
	virtual FText GetDisplayName() const override;
	void OnNavigateToLatentNode();

	class UEdGraphNode* FindAssociatedNode() const;
};

FText FLatentActionLineItem::GetDescription() const
{
	if (UObject* ParentObject = ParentObjectRef.Get())
	{
		if (UWorld* World = GEngine->GetWorldFromContextObject(ParentObject, EGetWorldErrorMode::ReturnNull))
		{
			FLatentActionManager& LatentActionManager = World->GetLatentActionManager();
			return FText::FromString(LatentActionManager.GetDescription(ParentObject, UUID));
		}
	}

	return LOCTEXT("nullptrObject", "Object has been destroyed");
}

TSharedRef<SWidget> FLatentActionLineItem::GenerateNameWidget(TSharedPtr<FString> InSearchString)
{
	return SNew(PropertyInfoViewStyle::STextHighlightOverlay)
		.FullText(this, &FLatentActionLineItem::GetDisplayName)
		.HighlightText(this, &FLatentActionLineItem::GetHighlightText, InSearchString)
		[
			SNew(SHyperlink)
				.Style(FAppStyle::Get(), "HoverOnlyHyperlink")
				.OnNavigate(this, &FLatentActionLineItem::OnNavigateToLatentNode)
				.Text(this, &FLatentActionLineItem::GetDisplayName)
				.ToolTipText(LOCTEXT("NavLatentActionLoc_Tooltip", "Navigate to the latent action location"))
		];
}

TSharedRef<SWidget> FLatentActionLineItem::GetNameIcon()
{
	return SNew(SImage)
		.Image(FAppStyle::GetBrush(TEXT("Kismet.LatentActionIcon")));
}

UEdGraphNode* FLatentActionLineItem::FindAssociatedNode() const
{
	if (UBlueprintGeneratedClass* Class = GetClassForObject(ParentObjectRef.Get()))
	{
		return Class->GetDebugData().FindNodeFromUUID(UUID);
	}

	return nullptr;
}

FText FLatentActionLineItem::GetDisplayName() const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("ID"), UUID);
	if (UK2Node* Node = Cast<UK2Node>(FindAssociatedNode()))
	{
		Args.Add(TEXT("Title"), Node->GetCompactNodeTitle());

		return FText::Format(LOCTEXT("ID", "{Title} (ID: {ID})"), Args);
	}
	else
	{
		return FText::Format(LOCTEXT("LatentAction", "Latent action # {ID}"), Args);
	}
}

void FLatentActionLineItem::OnNavigateToLatentNode()
{
	if (UEdGraphNode* Node = FindAssociatedNode())
	{
		FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(Node);
	}
}

//////////////////////////////////////////////////////////////////////////
// FWatchChildLineItem

struct FWatchChildLineItem : public FLineItemWithChildren
{
protected:
	TSharedRef<FPropertyInstanceInfo> Data;
	TWeakPtr<FDebugLineItem> ParentTreeItem;
private:
	bool bIconHovered = false;
public:
	FWatchChildLineItem(TSharedRef<FPropertyInstanceInfo> Child, const FDebugTreeItemPtr& InParentTreeItem) :
		FLineItemWithChildren(DLT_WatchChild),
		Data(Child),
		ParentTreeItem(InParentTreeItem)
	{}

	virtual TSharedRef<SWidget> GenerateValueWidget(TSharedPtr<FString> InSearchString) override
	{
		return SNew(SKismetDebugTreePropertyValueWidget, InSearchString)
			.PropertyInfo(this, &FWatchChildLineItem::GetPropertyInfo)
			.TreeItem(AsShared());
	}

	virtual void ExtendContextMenu(FMenuBuilder& MenuBuilder, bool bInDebuggerTab) override
	{
		//Navigate to Class source

		// Only add watch options if this has a pin in it's parent chain
		// (ok to discard the path, this only runs when a context menu is constructed)
		TArray<FName> PathToProperty;
		if (UEdGraphPin* PinParent = BuildPathToProperty(PathToProperty))
		{
			//Add Watch
			FUIAction AddThisWatch(
				FExecuteAction::CreateSP(this, &FWatchChildLineItem::AddWatch),
				FCanExecuteAction::CreateSP(this, &FWatchChildLineItem::CanAddWatch)
			);

				MenuBuilder.AddMenuEntry(
					LOCTEXT("AddPropertyWatch", "Start Watching"),
					LOCTEXT("AddPropertyWatchTooltip", "Start Watching This Variable"),
					FSlateIcon(),
					AddThisWatch
				);

			//Add Watch
			FUIAction ClearThisWatch(
				FExecuteAction::CreateSP(this, &FWatchChildLineItem::ClearWatch),
				FCanExecuteAction::CreateSP(this, &FWatchChildLineItem::CanClearWatch)
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("ClearPropertyWatch", "Stop Watching"),
				LOCTEXT("ClearPropertyWatchTooltip", "Stop Watching This Variable"),
				FSlateIcon(),
				ClearThisWatch
			);
		}

		if (!bInDebuggerTab)
		{
			//View in Debugger
			FUIAction ViewInDebugger(
				FExecuteAction::CreateSP(this, &FWatchChildLineItem::ViewInDebugger),
				FCanExecuteAction::CreateSP(this, &FWatchChildLineItem::CanViewInDebugger)
			);

			MenuBuilder.AddMenuEntry(
				ViewInDebuggerText,
				ViewInDebuggerTooltipText,
				FSlateIcon(),
				ViewInDebugger
			);
		}

	}

	// uses the icon and color associated with the property type
	virtual TSharedRef<SWidget> GetNameIcon() override
	{
		FSlateColor BaseColor;
		FSlateColor SecondaryColor;
		FSlateBrush const* SecondaryIcon = nullptr;
		const FSlateBrush* Icon = FBlueprintEditor::GetVarIconAndColorFromProperty(
			Data->Property.Get(),
			BaseColor,
			SecondaryIcon,
			SecondaryColor
		);

		TSharedPtr<SLayeredImage> LayeredImage;

		// make the icon a button so the user can open the asset in editor if there is one
		TSharedRef<SWidget> NameIcon = SNew(SButton)
			.OnClicked(this, &FWatchChildLineItem::OnFocusAsset)
			.ButtonStyle(FAppStyle::Get(), "NoBorder")
			.ContentPadding(0.0f)
			.OnHovered_Lambda(
				[&bIconHovered = bIconHovered]() {bIconHovered = true; }
			)
			.OnUnhovered_Lambda(
				[&bIconHovered = bIconHovered]() {bIconHovered = false; }
			)
			[
				SNew(SOverlay)
				.ToolTipText(this, &FWatchChildLineItem::IconTooltipText)
				+ SOverlay::Slot()
				.Padding(FMargin(10.f, 0.f, 0.f, 0.f))
				[
					SAssignNew(LayeredImage, SLayeredImage)
						.Image(Icon)
						.ColorAndOpacity(this, &FWatchChildLineItem::ModifiedIconColor, BaseColor)
				]
				+ SOverlay::Slot()
				.HAlign(HAlign_Left)
				[
					SNew(SImage)
						.Image(FAppStyle::GetBrush(TEXT("Kismet.WatchIcon")))
						.Visibility(this, &FWatchChildLineItem::GetWatchIconVisibility)
				]
			];

		LayeredImage->AddLayer(
			SecondaryIcon,
			TAttribute<FSlateColor>::CreateSP(this, &FWatchChildLineItem::ModifiedIconColor, SecondaryColor)
		);

		return NameIcon;
	}

	virtual TSharedRef<SWidget> GetValueIcon() override
	{
		if (UObject* Object = Data->Object.Get())
		{
			if (Object->IsAsset())
			{
				FAssetThumbnailConfig ThumbnailConfig;

				if (FSlateApplication::Get().InKismetDebuggingMode())
				{
					ThumbnailConfig.bForceGenericThumbnail = true;
				}

				TSharedPtr<FAssetThumbnail> Thumb = MakeShared<FAssetThumbnail>(Object, ThumbnailIconResolution, ThumbnailIconResolution, UThumbnailManager::Get().GetSharedThumbnailPool());
				return SNew(SButton)
					.OnClicked(this, &FWatchChildLineItem::OnFocusAsset)
					.ToolTipText(this, &FWatchChildLineItem::IconTooltipText)
					.ButtonStyle(FAppStyle::Get(), "NoBorder")
					[
						SNew(SBox)
						.MaxDesiredHeight(ThumbnailIconSize)
						.MaxDesiredWidth(ThumbnailIconSize)
						[
							Thumb->MakeThumbnailWidget(ThumbnailConfig)
						]
					];
			}
		}

		return FDebugLineItem::GetValueIcon();
	}

	virtual FText GetDescription() const override
	{
		const FString ValStr = Data->Value.ToString();
		return FText::FromString(ValStr.Replace(TEXT("\n"), TEXT(" ")));
	}

protected:
	virtual FDebugLineItem* Duplicate() const override
	{
		return new FWatchChildLineItem(Data, ParentTreeItem.Pin());
	}

	virtual bool Compare(const FDebugLineItem* BaseOther) const override
	{
		FWatchChildLineItem* Other = (FWatchChildLineItem*)BaseOther;

		return Data->Property == Other->Data->Property &&
			Data->DisplayName.CompareTo(Other->Data->DisplayName) == 0;
	}

	virtual uint32 GetHash() const override
	{
		return HashCombine(GetTypeHash(Data->Property), GetTypeHash(Data->DisplayName.ToString()));
	}

	virtual void UpdateData(const FDebugLineItem& NewerData) override
	{
		// Compare returns true even if the value or children of this node
		// is different. use this function to update the data without completely
		// replacing the node
		FWatchChildLineItem& Other = (FWatchChildLineItem&)NewerData;
		Data = Other.Data;
	}

	virtual FText GetName() const override
	{
		return Data->Name;
	}

	virtual FText GetDisplayName() const override
	{
		return Data->DisplayName;
	}

	// if data is pointing to an asset, get it's UPackage
	const UPackage* GetDataPackage() const
	{
		if (Data->Object.IsValid())
		{
			if (const UBlueprintGeneratedClass* GeneratedClass = Cast<UBlueprintGeneratedClass>(Data->Object->GetClass()))
			{
				if (const UPackage* Package = GeneratedClass->GetPackage())
				{
					return Package;
				}
			}
			if (const UPackage* Package = Data->Object->GetPackage())
			{
				return Package;
			}
		}
		return {};
	}

	const UObject* GetRelatedObject() const override
	{
		return Data->Object.Get();
	}

	bool CanOpenAsset() const
	{
		if (FSlateApplication::Get().InKismetDebuggingMode())
		{
			if (const UPackage* Package = GetDataPackage())
			{

				UObject* ReferencedAsset = Package->FindAssetInPackage();

				// It is not safe to open asset editors for non-blueprint assets while stopped at a breakpoint
				return Cast<UBlueprint>(ReferencedAsset) != nullptr;
			}
		}

		return true;
	}

	// opens result of GetDataPackage in editor
	FReply OnFocusAsset() const
	{
		if (!CanOpenAsset())
		{
			return FReply::Unhandled();
		}

		const UPackage* Package = GetDataPackage();
		if (!Package)
		{
			return FReply::Unhandled();
		}

		const FString Path = Package->GetPathName();
		if (Path.IsEmpty())
		{
			return FReply::Unhandled();
		}

		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Path);
		return FReply::Handled();
	}

	// returns the icon color given a precalculated color associated with this datatype.
	// the color changes slightly based on whether it's null or a hovered button
	FSlateColor ModifiedIconColor(FSlateColor BaseColor) const
	{
		FLinearColor LinearRGB = BaseColor.GetSpecifiedColor();

		// check if Data is a UObject
		if (CastField<FObjectPropertyBase>(Data->Property.Get()))
		{
			FLinearColor LinearHSV = LinearRGB.LinearRGBToHSV();

			// if it's a null object, darken the icon so it's clear that it's not a button
			if (Data->Object == nullptr)
			{
				LinearHSV.B *= 0.5f; // decrease value
				LinearHSV.A *= 0.5f; // decrease alpha
				LinearRGB = LinearHSV.HSVToLinearRGB();
			}
			// if the icon is hovered, lighten the icon
			else if (bIconHovered)
			{
				LinearHSV.B *= 2.f;  // increase value
				LinearHSV.G *= 0.8f; // decrease Saturation
				LinearRGB = LinearHSV.HSVToLinearRGB();
			}
		}

		bool bPinWatched = false;
		TArray<FName> PathToProperty;
		if (UEdGraphPin* PinToWatch = BuildPathToProperty(PathToProperty))
		{
			bPinWatched = FKismetDebugUtilities::IsPinBeingWatched(FBlueprintEditorUtils::FindBlueprintForNode(PinToWatch->GetOwningNode()), PinToWatch, PathToProperty);
		}

		if (bPinWatched)
		{
			LinearRGB.A *= 0.3f;
		}

		return LinearRGB;
	}

	FText IconTooltipText() const
	{
		const UPackage* Package = GetDataPackage();
		if (Package)
		{
			if (CanOpenAsset())
			{
				return FText::Format(LOCTEXT("OpenPackage", "Open: {0}"), FText::FromString(Package->GetName()));
			}
		}
		return Data->Type;
	}

	virtual void GatherChildren(TArray<FDebugTreeItemPtr>& OutChildren, const FString& InSearchString, bool bRespectSearch) override
	{
		for (const TSharedPtr<FPropertyInstanceInfo>& ChildData : Data->GetChildren())
		{
			EnsureChildIsAdded(OutChildren, FWatchChildLineItem(ChildData.ToSharedRef(), AsShared()), InSearchString, bRespectSearch);
		}
	}

	virtual bool IsContainer() const override
	{
		return Data->Property->IsA<FSetProperty>() || Data->Property->IsA<FArrayProperty>() || Data->Property->IsA<FMapProperty>();
	}

	EVisibility GetWatchIconVisibility() const
	{
		bool bPinWatched = false;
		TArray<FName> PathToProperty;
		if (UEdGraphPin* PinToWatch = BuildPathToProperty(PathToProperty))
		{
			bPinWatched = FKismetDebugUtilities::IsPinBeingWatched(FBlueprintEditorUtils::FindBlueprintForNode(PinToWatch->GetOwningNode()), PinToWatch, PathToProperty);
		}

		return bPinWatched ? EVisibility::Visible : EVisibility::Collapsed;
	}

	TSharedPtr<FPropertyInstanceInfo> GetPropertyInfo() const
	{
		return Data;
	}

	void AddWatch();
	bool CanAddWatch() const;
	void ClearWatch();
	bool CanClearWatch() const;
	void ViewInDebugger();
	bool CanViewInDebugger() const;
	UEdGraphPin* BuildPathToProperty(TArray<FName>& OutPathToProperty) const;
};

//////////////////////////////////////////////////////////////////////////\
// FSelfWatchLineItem

struct FSelfWatchLineItem : public FLineItemWithChildren
{
protected:
	// watches a UObject instead of a pin
	TWeakObjectPtr<UObject> ObjectToWatch;
public:
	FSelfWatchLineItem(UObject* Object) :
		FLineItemWithChildren(DLT_SelfWatch),
		ObjectToWatch(Object)
	{}

	virtual TSharedRef<SWidget> GetNameIcon() override
	{
		return SNew(SImage)
			.Image(FAppStyle::GetBrush(TEXT("Kismet.WatchIcon")));
	}

protected:
	virtual FDebugLineItem* Duplicate() const override
	{
		return new FSelfWatchLineItem(ObjectToWatch.Get());
	}

	virtual bool Compare(const FDebugLineItem* BaseOther) const override
	{
		FSelfWatchLineItem* Other = (FSelfWatchLineItem*)BaseOther;
		return (ObjectToWatch.Get() == Other->ObjectToWatch.Get());
	}

	virtual uint32 GetHash() const override
	{
		return GetTypeHash(ObjectToWatch);
	}

	virtual FText GetDisplayName() const override
	{
		return LOCTEXT("SelfName", "Self");
	}

	virtual void GatherChildren(TArray<FDebugTreeItemPtr>& OutChildren, const FString& InSearchString, bool bRespectSearch) override
	{
		if (UObject* Object = ObjectToWatch.Get())
		{
			for (TFieldIterator<FProperty> It(Object->GetClass()); It; ++It)
			{
				TSharedPtr<FPropertyInstanceInfo> DebugInfo;
				FProperty* Property = *It;
				if (Property->HasAllPropertyFlags(CPF_BlueprintVisible))
				{
					void* Value = Property->ContainerPtrToValuePtr<void*>(Object);
					FKismetDebugUtilities::GetDebugInfoInternal(DebugInfo, Property, Value);

					EnsureChildIsAdded(OutChildren, FWatchChildLineItem(DebugInfo.ToSharedRef(), AsShared()), InSearchString, bRespectSearch);
				}
			}
		}
	}

};

//////////////////////////////////////////////////////////////////////////
// FWatchLineItem 

struct FWatchLineItem : public FLineItemWithChildren
{
protected:
	TWeakObjectPtr< UObject > ParentObjectRef;
	const FEdGraphPinReference ObjectRef;
	mutable TSharedPtr<FPropertyInstanceInfo> CachedPropertyInfo = nullptr;
	mutable FText CachedErrorString;
	const TArray<FName> PathToProperty;

public:
	FWatchLineItem(const UEdGraphPin* PinToWatch, UObject* ParentObject)
		: FLineItemWithChildren(DLT_Watch)
		, ObjectRef(PinToWatch)
		, PathToProperty()
	{
		ParentObjectRef = ParentObject;
	}

	FWatchLineItem(const UEdGraphPin* PinToWatch, UObject* ParentObject, const TArray<FName>& InPathToProperty)
		: FLineItemWithChildren(DLT_Watch)
		, ObjectRef(PinToWatch)
		, PathToProperty(InPathToProperty)
	{
		ParentObjectRef = ParentObject;
	}

	virtual void ExtendContextMenu(class FMenuBuilder& MenuBuilder, bool bInDebuggerTab) override
	{
		if (UEdGraphPin* WatchedPin = ObjectRef.Get())
		{
			FUIAction AddThisWatch(
				FExecuteAction::CreateSP(this, &FWatchLineItem::AddWatch),
				FCanExecuteAction::CreateSP(this, &FWatchLineItem::CanAddWatch)
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("AddWatch", "Start Watching"),
				LOCTEXT("AddWatch_ToolTip", "Start watching this variable"),
				FSlateIcon(),
				AddThisWatch);

			FUIAction ClearThisWatch(
				FExecuteAction::CreateSP(this, &FWatchLineItem::ClearWatch),
				FCanExecuteAction::CreateSP(this, &FWatchLineItem::CanClearWatch)
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("ClearWatch", "Stop watching"),
				LOCTEXT("ClearWatch_ToolTip", "Stop watching this variable"),
				FSlateIcon(),
				ClearThisWatch);

			if (bInDebuggerTab)
			{
				FUIAction ViewInDebugger(
					FExecuteAction::CreateSP(this, &FWatchLineItem::ViewInDebugger),
					FCanExecuteAction::CreateSP(this, &FWatchLineItem::CanViewInDebugger)
				);

				MenuBuilder.AddMenuEntry(
					ViewInDebuggerText,
					ViewInDebuggerTooltipText,
					FSlateIcon(),
					ViewInDebugger
				);
			}
		}
	}


	const FEdGraphPinReference& GetPin() const
	{
		return ObjectRef;
	}

	const TArray<FName>& GetPathToProperty() const
	{
		return PathToProperty;
	}

	virtual FText GetDescription() const override;
	virtual TSharedRef<SWidget> GenerateNameWidget(TSharedPtr<FString> InSearchString) override;
	virtual TSharedRef<SWidget> GenerateValueWidget(TSharedPtr<FString> InSearchString) override;
	virtual TSharedRef<SWidget> GetValueIcon() override;
	virtual TSharedRef<SWidget> GetNameIcon() override;

	/** Returns a SharedPtr to the debug info for the property being represented by this TreeItem */
	TSharedPtr<FPropertyInstanceInfo> GetPropertyInfo() const;

protected:
	virtual FDebugLineItem* Duplicate() const override
	{
		return new FWatchLineItem(ObjectRef.Get(), ParentObjectRef.Get(), PathToProperty);
	}

	virtual bool Compare(const FDebugLineItem* BaseOther) const override
	{
		FWatchLineItem* Other = (FWatchLineItem*)BaseOther;
		return (ParentObjectRef == Other->ParentObjectRef) &&
			(ObjectRef == Other->ObjectRef) && 
			(PathToProperty == Other->PathToProperty);
	}

	virtual uint32 GetHash() const override
	{
		return HashCombine(GetTypeHash(ParentObjectRef), GetTypeHash(ObjectRef));
	}

	virtual void GatherChildren(TArray<FDebugTreeItemPtr>& OutChildren, const FString& InSearchString, bool bRespectSearch) override
	{
		TSharedPtr<FPropertyInstanceInfo> ThisDebugInfo = GetPropertyInfo();
		if (ThisDebugInfo.IsValid())
		{
			for (const TSharedPtr<FPropertyInstanceInfo>& ChildData : ThisDebugInfo->GetChildren())
			{
				EnsureChildIsAdded(OutChildren, FWatchChildLineItem(ChildData.ToSharedRef(), AsShared()), InSearchString, bRespectSearch);
			}
		}
	}
	
	virtual const UObject* GetRelatedObject() const override
	{
		if (CachedPropertyInfo.Get())
		{
			return CachedPropertyInfo->Object.Get();
		}
		return nullptr;
	}

	virtual FText GetDisplayName() const override;
	const FSlateBrush* GetPinIcon() const;
	const FSlateBrush* GetSecondaryPinIcon() const;
	FSlateColor GetPinIconColor() const;
	FSlateColor GetSecondaryPinIconColor() const;
	EVisibility GetWatchIconVisibility() const;
	FText GetTypename() const;

	void OnNavigateToWatchLocation();

private:
	void AddWatch() const
	{
		if (UEdGraphPin* PinToWatch = ObjectRef.Get())
		{
			UBlueprint* ParentBlueprint = GetBlueprintForObject(ParentObjectRef.Get());
			FKismetDebugUtilities::AddPinWatch(ParentBlueprint, FBlueprintWatchedPin(PinToWatch, TArray<FName>(PathToProperty)));
		}
	}

	bool CanAddWatch() const
	{
		if (UEdGraphPin* PinToWatch = ObjectRef.Get())
		{
			UBlueprint* ParentBlueprint = GetBlueprintForObject(ParentObjectRef.Get());
			return FKismetDebugUtilities::CanWatchPin(ParentBlueprint, PinToWatch, PathToProperty);
		}

		return false;
	}

	void ClearWatch() const
	{
		if (UEdGraphPin* PinToWatch = ObjectRef.Get())
		{
			UBlueprint* ParentBlueprint = GetBlueprintForObject(ParentObjectRef.Get());
			TArray<FName> PathToPropertyCopy = PathToProperty;
			FKismetDebugUtilities::RemovePinWatch(ParentBlueprint, PinToWatch, MoveTemp(PathToPropertyCopy));
		}
	}

	bool CanClearWatch() const
	{
		if (UEdGraphPin* PinToWatch = ObjectRef.Get())
		{
			UBlueprint* ParentBlueprint = GetBlueprintForObject(ParentObjectRef.Get());
			return FKismetDebugUtilities::IsPinBeingWatched(ParentBlueprint, PinToWatch, PathToProperty);
		}

		return false;
	}

	void ViewInDebugger() const
	{
		// If this isn't already watched, add it as a watch
		if (CanAddWatch())
		{
			AddWatch();
		}

		FGlobalTabmanager::Get()->TryInvokeTab(FBlueprintEditorTabs::BlueprintDebuggerID);
		FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::Get().LoadModuleChecked<FBlueprintEditorModule>(TEXT("Kismet"));
		BlueprintEditorModule.GetBlueprintDebugger()->SetDebuggedBlueprint(GetBlueprintForObject(ParentObjectRef.Get()));
	}

	bool CanViewInDebugger() const
	{
		if (UEdGraphPin* PinToWatch = ObjectRef.Get())
		{
			UBlueprint* ParentBlueprint = GetBlueprintForObject(ParentObjectRef.Get());
			return FKismetDebugUtilities::IsPinBeingWatched(ParentBlueprint, PinToWatch, PathToProperty) ||
				FKismetDebugUtilities::CanWatchPin(ParentBlueprint, PinToWatch, PathToProperty);
		}

		return false;
	}

	bool CanOpenAsset(UObject* AssetObject) const
	{
		if (FSlateApplication::Get().InKismetDebuggingMode())
		{
			// opening non-blueprint assets while stopped at a breakpoint is unsafe
			return Cast<UBlueprint>(AssetObject) != nullptr;
		}

		return false;
	}

	FReply OpenEditorForAsset() const
	{
		TSharedPtr<FPropertyInstanceInfo> PropertyInfo = GetPropertyInfo();
		if (PropertyInfo.IsValid())
		{
			if (UObject* Object = PropertyInfo->Object.Get())
			{
				if (Object->IsAsset() && CanOpenAsset(Object))
				{
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Object);
					return FReply::Handled();
				}
			}
		}

		return FReply::Unhandled();
	}

	FReply OpenEditorForType() const
	{
		TSharedPtr<FPropertyInstanceInfo> PropertyInfo = GetPropertyInfo();
		if (PropertyInfo.IsValid())
		{
			if (UObject* Object = PropertyInfo->Object.Get())
			{
				if (UBlueprint* Blueprint = Cast<UBlueprint>(PropertyInfo->Object->GetClass()->ClassGeneratedBy))
				{
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Blueprint);
					return FReply::Handled();
				}
			}
		}

		return FReply::Unhandled();
	}

	FText GetAssetThumbnailTooltip() const
	{
		TSharedPtr<FPropertyInstanceInfo> PropertyInfo = GetPropertyInfo();
		if (PropertyInfo.IsValid())
		{
			if (UObject* Object = PropertyInfo->Object.Get())
			{
				if (CanOpenAsset(Object))
				{
					return FText::Format(LOCTEXT("OpenPackage", "Open: {0}"), FText::FromString(Object->GetFullName()));
				}
			}
			return PropertyInfo->Type;
		}

		return FText::GetEmpty();
	}

	FText GetIconTooltipText() const
	{
		TSharedPtr<FPropertyInstanceInfo> PropertyInfo = GetPropertyInfo();
		if (PropertyInfo.IsValid())
		{
			if (UObject* Object = PropertyInfo->Object.Get())
			{
				if (UBlueprint* Blueprint = Cast<UBlueprint>(Object->GetClass()->ClassGeneratedBy))
				{
					return FText::Format(LOCTEXT("OpenPackage", "Open: {0}"), FText::FromString(Blueprint->GetFullName()));
				}
			}
			return PropertyInfo->Type;
		}

		return FText::GetEmpty();
	}
};

FText FWatchLineItem::GetDisplayName() const
{
	if (UEdGraphPin* PinToWatch = ObjectRef.Get())
	{
		if (UBlueprint* Blueprint = GetBlueprintForObject(ParentObjectRef.Get()))
		{
			if (FProperty* Property = FKismetDebugUtilities::FindClassPropertyForPin(Blueprint, PinToWatch))
			{
				FString PathString;
				for (FName PropertyName : PathToProperty)
				{
					PathString.Append(TEXT("/") + PropertyName.ToString());
				}

				return FText::FromString(UEditorEngine::GetFriendlyName(Property) + PathString);
			}
		}

		FFormatNamedArguments Args;
		Args.Add(TEXT("PinWatchName"), PinToWatch->GetDisplayName());
		return FText::Format(LOCTEXT("DisplayNameNoProperty", "{PinWatchName} (no prop)"), Args);
	}
	else
	{
		return FText::GetEmpty();
	}
}

FText FWatchLineItem::GetDescription() const
{
	if (UEdGraphPin* PinToWatch = ObjectRef.Get())
	{
		// Try to determine the blueprint that generated the watch
		UBlueprint* ParentBlueprint = GetBlueprintForObject(ParentObjectRef.Get());

		// Find a valid property mapping and display the current value
		UObject* ParentObject = ParentObjectRef.Get();
		if ((ParentBlueprint != ParentObject) && (ParentBlueprint != nullptr))
		{
			TSharedPtr<FPropertyInstanceInfo> DebugInfo = GetPropertyInfo();
			if (DebugInfo.IsValid())
			{
				const FString ValStr = DebugInfo->Value.ToString();
				return FText::FromString(ValStr.Replace(TEXT("\n"), TEXT(" ")));
			}
			return CachedErrorString;
		}
	}

	return FText::GetEmpty();
}

TSharedRef<SWidget> FWatchLineItem::GenerateNameWidget(TSharedPtr<FString> InSearchString)
{
	FText NodeName = LOCTEXT("UnknownNode", "[Unknown]");
	FText GraphName = LOCTEXT("UnknownGraph", "[Unknown]");
	if (const UEdGraphPin* Pin = ObjectRef.Get())
	{
		if (const UEdGraphNode* Node = Pin->GetOwningNode())
		{
			NodeName = FText::FromString(Node->GetName());
			if (const UEdGraph* Graph = Node->GetGraph())
			{
				GraphName = FText::FromString(Graph->GetName());
			}
		}
	}

	const FText ToolTipText = FText::FormatNamed(LOCTEXT("NavWatchLoc", "Navigate to the watch location\nGraph: {GraphName}\nNode: {NodeName}"),
		TEXT("GraphName"), GraphName,
		TEXT("NodeName"), NodeName
	);

	return SNew(PropertyInfoViewStyle::STextHighlightOverlay)
		.FullText(this, &FWatchLineItem::GetDisplayName)
		.HighlightText(this, &FWatchLineItem::GetHighlightText, InSearchString)
		[
			SNew(SHyperlink)
				.Style(FAppStyle::Get(), "HoverOnlyHyperlink")
				.OnNavigate(this, &FWatchLineItem::OnNavigateToWatchLocation)
				.Text(this, &FWatchLineItem::GetDisplayName)
				.ToolTipText(ToolTipText)
		];
}

TSharedRef<SWidget> FWatchLineItem::GenerateValueWidget(TSharedPtr<FString> InSearchString)
{
	return SNew(SKismetDebugTreePropertyValueWidget, InSearchString)
		.PropertyInfo(this, &FWatchLineItem::GetPropertyInfo)
		.TreeItem(AsShared());
}

TSharedRef<SWidget> FWatchLineItem::GetValueIcon()
{
	TSharedPtr<FPropertyInstanceInfo> PropertyInfo = GetPropertyInfo();
	if (PropertyInfo.IsValid())
	{
		if (UObject* Object = PropertyInfo->Object.Get())
		{
			if (Object->IsAsset())
			{
				FAssetThumbnailConfig ThumbnailConfig;

				if (FSlateApplication::Get().InKismetDebuggingMode())
				{
					ThumbnailConfig.bForceGenericThumbnail = true;
				}

				TSharedPtr<FAssetThumbnail> Thumb = MakeShared<FAssetThumbnail>(Object, ThumbnailIconResolution, ThumbnailIconResolution, UThumbnailManager::Get().GetSharedThumbnailPool());
				return SNew(SButton)
					.OnClicked(this, &FWatchLineItem::OpenEditorForAsset)
					.ToolTipText(this, &FWatchLineItem::GetAssetThumbnailTooltip)
					.ButtonStyle(FAppStyle::Get(), "NoBorder")
					[
						SNew(SBox)
						.MaxDesiredHeight(ThumbnailIconSize)
						.MaxDesiredWidth(ThumbnailIconSize)
						[
							Thumb->MakeThumbnailWidget(ThumbnailConfig)
						]
					];
			}
		}
	}

	return FDebugLineItem::GetValueIcon();
}

// overlays the watch icon on top of a faded icon associated with the pin type
TSharedRef<SWidget> FWatchLineItem::GetNameIcon()
{
	TSharedPtr<SLayeredImage> LayeredImage;
	TSharedRef<SWidget> NameIcon = SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "NoBorder")
		.ToolTipText(this, &FWatchLineItem::GetIconTooltipText)
		.OnClicked(this, &FWatchLineItem::OpenEditorForType)
		[
			SNew(SOverlay)
			.ToolTipText(this, &FWatchLineItem::GetTypename)
			+ SOverlay::Slot()
			.Padding(FMargin(10.f, 0.f, 0.f, 0.f))
			[
				SAssignNew(LayeredImage, SLayeredImage)
				.Image(this, &FWatchLineItem::GetPinIcon)
				.ColorAndOpacity(this, &FWatchLineItem::GetPinIconColor)
			]
			+ SOverlay::Slot()
			.HAlign(HAlign_Left)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush(TEXT("Kismet.WatchIcon")))
				.Visibility(this, &FWatchLineItem::GetWatchIconVisibility)
			]
		];

	LayeredImage->AddLayer(
		TAttribute<const FSlateBrush*>(this, &FWatchLineItem::GetSecondaryPinIcon),
		TAttribute<FSlateColor>(this, &FWatchLineItem::GetSecondaryPinIconColor)
	);

	return NameIcon;
}

const FSlateBrush* FWatchLineItem::GetPinIcon() const
{
	TSharedPtr<FPropertyInstanceInfo> ThisDebugInfo = GetPropertyInfo();
	if (ThisDebugInfo.IsValid())
	{
		FSlateColor BaseColor;
		FSlateColor SecondaryColor;
		FSlateBrush const* SecondaryIcon;
		const FSlateBrush* Icon = FBlueprintEditor::GetVarIconAndColorFromProperty(
			ThisDebugInfo->Property.Get(),
			BaseColor,
			SecondaryIcon,
			SecondaryColor
		);

		return Icon;
	}

	return FAppStyle::GetBrush(TEXT("NoBrush"));
}

const FSlateBrush* FWatchLineItem::GetSecondaryPinIcon() const
{
	TSharedPtr<FPropertyInstanceInfo> ThisDebugInfo = GetPropertyInfo();
	if (ThisDebugInfo.IsValid())
	{
		FSlateColor BaseColor;
		FSlateColor SecondaryColor;
		FSlateBrush const* SecondaryIcon;
		const FSlateBrush* Icon = FBlueprintEditor::GetVarIconAndColorFromProperty(
			ThisDebugInfo->Property.Get(),
			BaseColor,
			SecondaryIcon,
			SecondaryColor
		);

		return SecondaryIcon;
	}

	return FAppStyle::GetBrush(TEXT("NoBrush"));
}

FSlateColor FWatchLineItem::GetPinIconColor() const
{
	FSlateColor PinIconColor = FLinearColor::White;

	if (UEdGraphPin* ObjectToFocus = ObjectRef.Get())
	{
		TSharedPtr<FPropertyInstanceInfo> ThisDebugInfo = GetPropertyInfo();
		if (ThisDebugInfo.IsValid())
		{
			if (const UEdGraphSchema* Schema = ObjectToFocus->GetSchema())
			{
				PinIconColor = Schema->GetPinTypeColor(ObjectToFocus->PinType);
			}
		}	

		if (FKismetDebugUtilities::IsPinBeingWatched(FBlueprintEditorUtils::FindBlueprintForNode(ObjectToFocus->GetOwningNode()), ObjectToFocus, PathToProperty))
		{
			FLinearColor Color = PinIconColor.GetSpecifiedColor();
			Color.A = 0.3f;
			PinIconColor = Color;
		}
	}

	return PinIconColor;
}

FSlateColor FWatchLineItem::GetSecondaryPinIconColor() const
{
	FSlateColor PinIconColor = FLinearColor::White;

	if (UEdGraphPin* ObjectToFocus = ObjectRef.Get())
	{
		TSharedPtr<FPropertyInstanceInfo> ThisDebugInfo = GetPropertyInfo();
		if (ThisDebugInfo.IsValid())
		{
			if (const UEdGraphSchema* Schema = ObjectToFocus->GetSchema())
			{
				PinIconColor = Schema->GetSecondaryPinTypeColor(ObjectToFocus->PinType);
			}
		}

		if (FKismetDebugUtilities::IsPinBeingWatched(FBlueprintEditorUtils::FindBlueprintForNode(ObjectToFocus->GetOwningNode()), ObjectToFocus, PathToProperty))
		{
			FLinearColor Color = PinIconColor.GetSpecifiedColor();
			Color.A = 0.3f;
			PinIconColor = Color;
		}
	}


	return PinIconColor;
}

EVisibility FWatchLineItem::GetWatchIconVisibility() const
{
	if (const UEdGraphPin* Pin = ObjectRef.Get())
	{
		return FKismetDebugUtilities::IsPinBeingWatched(FBlueprintEditorUtils::FindBlueprintForNode(Pin->GetOwningNode()), Pin, PathToProperty) ?
			EVisibility::Visible :
			EVisibility::Collapsed;
	}

	return EVisibility::Collapsed;
}

FText FWatchLineItem::GetTypename() const
{
	TSharedPtr<FPropertyInstanceInfo> ThisDebugInfo = GetPropertyInfo();
	if (ThisDebugInfo.IsValid())
	{
		return UEdGraphSchema_K2::TypeToText(ThisDebugInfo->Property.Get());
	}

	return FText::GetEmpty();
}

void FWatchLineItem::OnNavigateToWatchLocation()
{
	if (UEdGraphPin* ObjectToFocus = ObjectRef.Get())
	{
		FKismetEditorUtilities::BringKismetToFocusAttentionOnPin(ObjectToFocus);
	}
}

TSharedPtr<FPropertyInstanceInfo> FWatchLineItem::GetPropertyInfo() const
{
	if (CachedPropertyInfo.IsValid())
	{
		return CachedPropertyInfo;
	}

	if (UEdGraphPin* ObjectToFocus = ObjectRef.Get())
	{
		// Try to determine the blueprint that generated the watch
		UBlueprint* ParentBlueprint = GetBlueprintForObject(ParentObjectRef.Get());

		// Find a valid property mapping and display the current value
		UObject* ParentObject = ParentObjectRef.Get();
		if ((ParentBlueprint != ParentObject) && (ParentBlueprint != nullptr))
		{
			TSharedPtr<FPropertyInstanceInfo> DebugInfo;
			const FKismetDebugUtilities::EWatchTextResult WatchStatus = FKismetDebugUtilities::GetDebugInfo(DebugInfo, ParentBlueprint, ParentObject, ObjectToFocus);
			switch (WatchStatus)
			{
			case FKismetDebugUtilities::EWTR_Valid:
			{
					check(DebugInfo);
					if (PathToProperty.IsEmpty())
					{
						CachedPropertyInfo = DebugInfo;
					}
					else
					{
						CachedPropertyInfo = DebugInfo->ResolvePathToProperty(PathToProperty);
					}
					return CachedPropertyInfo;
			}

			case FKismetDebugUtilities::EWTR_NotInScope:
				CachedErrorString = LOCTEXT("NotInScope", "Not in scope");

			case FKismetDebugUtilities::EWTR_NoProperty:
				CachedErrorString = LOCTEXT("UnknownProperty", "No debug data");

			default:
			case FKismetDebugUtilities::EWTR_NoDebugObject:
				CachedErrorString = LOCTEXT("NoDebugObject", "No debug object");
			}
		}
	}
	
	return nullptr;
}

//////////////////////////////////////////////////////////////////////////
// FWatchChildLineItem (some functions that need the definition of FWatchLineItem)

void FWatchChildLineItem::AddWatch()
{
	TArray<FName> PathToProperty;
	if (UEdGraphPin* PinToWatch = BuildPathToProperty(PathToProperty))
	{
		FKismetDebugUtilities::AddPinWatch(FBlueprintEditorUtils::FindBlueprintForNode(PinToWatch->GetOwningNode()), FBlueprintWatchedPin(PinToWatch, MoveTemp(PathToProperty)));
	}
}

bool FWatchChildLineItem::CanAddWatch() const
{
	TArray<FName> PathToProperty;
	if (UEdGraphPin* PinToWatch = BuildPathToProperty(PathToProperty))
	{
		return FKismetDebugUtilities::CanWatchPin(FBlueprintEditorUtils::FindBlueprintForNode(PinToWatch->GetOwningNode()), PinToWatch, PathToProperty);
	}

	return false;
}

void FWatchChildLineItem::ClearWatch()
{
	TArray<FName> PathToProperty;
	if (UEdGraphPin* PinToWatch = BuildPathToProperty(PathToProperty))
	{
		FKismetDebugUtilities::RemovePinWatch(FBlueprintEditorUtils::FindBlueprintForNode(PinToWatch->GetOwningNode()), PinToWatch, PathToProperty);
	}
}

bool FWatchChildLineItem::CanClearWatch() const
{
	TArray<FName> PathToProperty;
	if (UEdGraphPin* PinToWatch = BuildPathToProperty(PathToProperty))
	{
		return FKismetDebugUtilities::IsPinBeingWatched(FBlueprintEditorUtils::FindBlueprintForNode(PinToWatch->GetOwningNode()), PinToWatch, PathToProperty);
	}

	return false;
}

void FWatchChildLineItem::ViewInDebugger()
{
	// If this isn't already watched, add it as a watch
	if (CanAddWatch())
	{
		AddWatch();
	}

	TArray<FName> PathToProperty;
	if (UEdGraphPin* PinToWatch = BuildPathToProperty(PathToProperty))
	{
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(PinToWatch->GetOwningNode());
		FGlobalTabmanager::Get()->TryInvokeTab(FBlueprintEditorTabs::BlueprintDebuggerID);
		FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::Get().LoadModuleChecked<FBlueprintEditorModule>(TEXT("Kismet"));
		BlueprintEditorModule.GetBlueprintDebugger()->SetDebuggedBlueprint(Blueprint);
	}
}

bool FWatchChildLineItem::CanViewInDebugger() const
{
	TArray<FName> PathToProperty;
	if (UEdGraphPin* PinToWatch = BuildPathToProperty(PathToProperty))
	{
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(PinToWatch->GetOwningNode());
		return FKismetDebugUtilities::IsPinBeingWatched(Blueprint, PinToWatch, PathToProperty) || FKismetDebugUtilities::CanWatchPin(Blueprint, PinToWatch, PathToProperty);
	}

	return false;
}

UEdGraphPin* FWatchChildLineItem::BuildPathToProperty(TArray<FName>& OutPathToProperty) const
{
	UEdGraphPin* PinToWatch = nullptr;
	OutPathToProperty.Reset();

	auto AddPropertyToPath = [&OutPathToProperty](const TSharedRef<FPropertyInstanceInfo>& InPropertyInfo)
	{
		if (InPropertyInfo->bIsInContainer)
		{
			// display name of map elements is their key exported to text
			OutPathToProperty.Emplace(InPropertyInfo->DisplayName.ToString());
		}
		else
		{
			OutPathToProperty.Emplace(InPropertyInfo->Property->GetAuthoredName());
		}
	};

	AddPropertyToPath(Data);

	FDebugTreeItemPtr Parent = ParentTreeItem.Pin();
	while (Parent.IsValid())
	{
		if (Parent->GetType() == DLT_Watch)
		{
			// this is a FWatchLineItem
			TSharedPtr<FWatchLineItem> ParentAsWatch = StaticCastSharedPtr<FWatchLineItem>(Parent);
			PinToWatch = ParentAsWatch->GetPin().Get();

			// Add the original path to the front of ours (we're still constructing it backwards) 
			const TArray<FName>& ParentPath = ParentAsWatch->GetPathToProperty();
			for (int32 PathIndex = ParentPath.Num() - 1; PathIndex >= 0; --PathIndex)
			{
				OutPathToProperty.Emplace(ParentPath[PathIndex]);
			}

			break;
		}

		if (Parent->GetType() == DLT_WatchChild)
		{
			// This is a FWatchChildLineItem
			TSharedPtr<FWatchChildLineItem> ParentAsChild = StaticCastSharedPtr<FWatchChildLineItem>(Parent);

			AddPropertyToPath(ParentAsChild->Data);

			Parent = ParentAsChild->ParentTreeItem.Pin();
		}
		else
		{
			Parent.Reset();
		}
	}

	// Reverse the array because we built it backwards
	Algo::Reverse(OutPathToProperty);

	return PinToWatch;
}

//////////////////////////////////////////////////////////////////////////
// FBreakpointLineItem

struct FBreakpointLineItem : public FDebugLineItem
{
protected:
	TWeakObjectPtr<UObject> ParentObjectRef;
	TSoftObjectPtr<UEdGraphNode> BreakpointNode;
public:
	FBreakpointLineItem(TSoftObjectPtr<UEdGraphNode> BreakpointToWatch, UObject* ParentObject)
		: FDebugLineItem(DLT_Breakpoint)
	{
		BreakpointNode = BreakpointToWatch;
		ParentObjectRef = ParentObject;
	}

	virtual TSharedRef<SWidget> GenerateNameWidget(TSharedPtr<FString> InSearchString) override
	{
		return SNew(PropertyInfoViewStyle::STextHighlightOverlay)
			.FullText(this, &FBreakpointLineItem::GetDisplayName)
			.HighlightText(this, &FBreakpointLineItem::GetHighlightText, InSearchString)
			[
				SNew(SHyperlink)
					.Style(FAppStyle::Get(), "HoverOnlyHyperlink")
					.Text(this, &FBreakpointLineItem::GetDisplayName)
					.ToolTipText(LOCTEXT("NavBreakpointLoc", "Navigate to the breakpoint location"))
					.OnNavigate(this, &FBreakpointLineItem::OnNavigateToBreakpointLocation)
			];
	}

	virtual void ExtendContextMenu(class FMenuBuilder& MenuBuilder, bool bInDebuggerTab) override
	{
		FBlueprintBreakpoint* Breakpoint = GetBreakpoint();
		const UBlueprint* ParentBlueprint = GetBlueprintForObject(ParentObjectRef.Get());

		// By default, we don't allow actions to execute when in debug mode.
		// Create an empty action to always allow execution for these commands (they are allowed in debug mode)
		FCanExecuteAction AlwaysAllowExecute;

		if (Breakpoint != nullptr)
		{
			const bool bNewEnabledState = !Breakpoint->IsEnabledByUser();

			FUIAction ToggleThisBreakpoint(
				FExecuteAction::CreateSP(this, &FBreakpointLineItem::ToggleBreakpoint),
				AlwaysAllowExecute
			);

			if (bNewEnabledState)
			{
				// Enable
				MenuBuilder.AddMenuEntry(
					LOCTEXT("EnableBreakpoint", "Enable breakpoint"),
					LOCTEXT("EnableBreakpoint_ToolTip", "Enable this breakpoint; the debugger will appear when this node is about to be executed."),
					FSlateIcon(),
					ToggleThisBreakpoint);
			}
			else
			{
				// Disable
				MenuBuilder.AddMenuEntry(
					LOCTEXT("DisableBreakpoint", "Disable breakpoint"),
					LOCTEXT("DisableBreakpoint_ToolTip", "Disable this breakpoint."),
					FSlateIcon(),
					ToggleThisBreakpoint);
			}
		}

		if ((Breakpoint != nullptr) && (ParentBlueprint != nullptr))
		{
			FUIAction ClearThisBreakpoint(
				FExecuteAction::CreateSP(this, &FBreakpointLineItem::ClearBreakpoint),
				AlwaysAllowExecute
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("ClearBreakpoint", "Remove breakpoint"),
				LOCTEXT("ClearBreakpoint_ToolTip", "Remove the breakpoint from this node."),
				FSlateIcon(),
				ClearThisBreakpoint);
		}
	}

	virtual TSharedRef<SWidget> GetNameIcon() override
	{
		return SNew(SButton)
			.OnClicked(this, &FBreakpointLineItem::OnUserToggledEnabled)
			.ToolTipText(LOCTEXT("ToggleBreakpointButton_ToolTip", "Toggle this breakpoint"))
			.ButtonStyle(FAppStyle::Get(), "NoBorder")
			.ContentPadding(0.0f)
			[
				SNew(SImage)
					.Image(this, &FBreakpointLineItem::GetStatusImage)
					.ToolTipText(this, &FBreakpointLineItem::GetStatusTooltip)
			];
	}

protected:
	FBlueprintBreakpoint* GetBreakpoint() const
	{
		if (UEdGraphNode* Node = BreakpointNode.Get())
		{
			if (const UBlueprint* Blueprint = GetBlueprintForObject(Node))
			{
				return FKismetDebugUtilities::FindBreakpointForNode(Node, Blueprint);
			}
		}
		return nullptr;
	}

	virtual FDebugLineItem* Duplicate() const override
	{
		return new FBreakpointLineItem(BreakpointNode, ParentObjectRef.Get());
	}

	virtual bool Compare(const FDebugLineItem* BaseOther) const override
	{
		FBreakpointLineItem* Other = (FBreakpointLineItem*)BaseOther;
		return (ParentObjectRef.Get() == Other->ParentObjectRef.Get()) &&
			(BreakpointNode == Other->BreakpointNode);
	}

	virtual uint32 GetHash() const override
	{
		return HashCombine(GetTypeHash(ParentObjectRef), GetTypeHash(BreakpointNode));
	}

	virtual FText GetDisplayName() const override;

private:
	void ToggleBreakpoint() const
	{
		if (FBlueprintBreakpoint* Breakpoint = GetBreakpoint())
		{
			FKismetDebugUtilities::SetBreakpointEnabled(*Breakpoint, !Breakpoint->IsEnabled());
		}
	}

	void ClearBreakpoint() const
	{
		if (UEdGraphNode* Node = BreakpointNode.Get())
		{
			if (UBlueprint* Blueprint = GetBlueprintForObject(Node))
			{
				FKismetDebugUtilities::RemoveBreakpointFromNode(Node, Blueprint);
			}
		}
	}

	FReply OnUserToggledEnabled();
	void OnNavigateToBreakpointLocation();
	const FSlateBrush* GetStatusImage() const;
	FText GetStatusTooltip() const;
};

FText FBreakpointLineItem::GetDisplayName() const
{
	if (FBlueprintBreakpoint* MyBreakpoint = GetBreakpoint())
	{
		return MyBreakpoint->GetLocationDescription();
	}
	return FText::GetEmpty();
}

FReply FBreakpointLineItem::OnUserToggledEnabled()
{
	if (FBlueprintBreakpoint* MyBreakpoint = GetBreakpoint())
	{
		FKismetDebugUtilities::SetBreakpointEnabled(*MyBreakpoint, !MyBreakpoint->IsEnabledByUser());
	}
	return FReply::Handled();
}

void FBreakpointLineItem::OnNavigateToBreakpointLocation()
{
	if (FBlueprintBreakpoint* MyBreakpoint = GetBreakpoint())
	{
		FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(MyBreakpoint->GetLocation());
	}
}

const FSlateBrush* FBreakpointLineItem::GetStatusImage() const
{
	if (FBlueprintBreakpoint* MyBreakpoint = GetBreakpoint())
	{
		if (MyBreakpoint->IsEnabledByUser())
		{
			return FAppStyle::GetBrush(FKismetDebugUtilities::IsBreakpointValid(*MyBreakpoint) ? TEXT("Kismet.Breakpoint.EnabledAndValid") : TEXT("Kismet.Breakpoint.EnabledAndInvalid"));
		}
		else
		{
			return FAppStyle::GetBrush(TEXT("Kismet.Breakpoint.Disabled"));
		}
	}

	return FAppStyle::GetDefaultBrush();
}

FText FBreakpointLineItem::GetStatusTooltip() const
{
	if (FBlueprintBreakpoint* MyBreakpoint = GetBreakpoint())
	{
		if (!FKismetDebugUtilities::IsBreakpointValid(*MyBreakpoint))
		{
			return LOCTEXT("Breakpoint_NoHit", "This breakpoint will not be hit because its node generated no code");
		}
		else
		{
			return MyBreakpoint->IsEnabledByUser() ? LOCTEXT("ActiveBreakpoint", "Active breakpoint") : LOCTEXT("InactiveBreakpoint", "Inactive breakpoint");
		}
	}
	else
	{
		return LOCTEXT("NoBreakpoint", "No Breakpoint");
	}
}

//////////////////////////////////////////////////////////////////////////
// FTraceStackParentItem

class FBreakpointParentItem : public FLineItemWithChildren
{
public:
	// The parent object
	TWeakObjectPtr<UBlueprint> Blueprint;

	FBreakpointParentItem(TWeakObjectPtr<UBlueprint> InBlueprint)
		: FLineItemWithChildren(DLT_BreakpointParent)
		, Blueprint(InBlueprint)
	{
	}

	virtual void ExtendContextMenu(FMenuBuilder& MenuBuilder, bool bInDebuggerTab) override
	{
		if (FKismetDebugUtilities::BlueprintHasBreakpoints(Blueprint.Get()))
		{
			const FUIAction ClearAllBreakpoints(
				FExecuteAction::CreateSP(this, &FBreakpointParentItem::ClearAllBreakpoints),
				FCanExecuteAction() // always allow
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("ClearBreakpoints", "Remove all breakpoints"),
				LOCTEXT("ClearBreakpoints_ToolTip", "Clear all breakpoints in this blueprint"),
				FSlateIcon(),
				ClearAllBreakpoints);

			const bool bEnabledBreakpointExists = FKismetDebugUtilities::FindBreakpointByPredicate(
				Blueprint.Get(),
				[](const FBlueprintBreakpoint& Breakpoint)->bool
				{
					return Breakpoint.IsEnabled();
				}
			) != nullptr;

			if (bEnabledBreakpointExists)
			{
				const FUIAction DisableAllBreakpoints(
					FExecuteAction::CreateSP(this, &FBreakpointParentItem::DisableAllBreakpoints),
					FCanExecuteAction() // always allow
				);

				MenuBuilder.AddMenuEntry(
					LOCTEXT("DisableBreakpoints", "Disable all breakpoints"),
					LOCTEXT("DisableBreakpoints_ToolTip", "Disable all breakpoints in this blueprint"),
					FSlateIcon(),
					DisableAllBreakpoints);
			}

			const bool bDisabledBreakpointExists = FKismetDebugUtilities::FindBreakpointByPredicate(
				Blueprint.Get(),
				[](const FBlueprintBreakpoint& Breakpoint)->bool
				{
					return !Breakpoint.IsEnabled();
				}
			) != nullptr;

			if (bDisabledBreakpointExists)
			{
				const FUIAction EnableAllBreakpoints(
					FExecuteAction::CreateSP(this, &FBreakpointParentItem::EnableAllBreakpoints),
					FCanExecuteAction() // always allow
				);

				MenuBuilder.AddMenuEntry(
					LOCTEXT("EnableBreakpoints", "Enable all breakpoints"),
					LOCTEXT("EnableBreakpoints_ToolTip", "Enable all breakpoints in this blueprint"),
					FSlateIcon(),
					EnableAllBreakpoints);
			}
		}
	}

protected:
	virtual void GatherChildren(TArray<FDebugTreeItemPtr>& OutChildren, const FString& InSearchString, bool bRespectSearch) override
	{
		// update search flags to match that of a root node
		UpdateSearch(InSearchString, FDebugLineItem::SF_RootNode);

		if (!Blueprint.IsValid())
		{
			return;
		}

		// Create children for each breakpoint
		FKismetDebugUtilities::ForeachBreakpoint(
			Blueprint.Get(),
			[this, &OutChildren, &InSearchString, bRespectSearch](FBlueprintBreakpoint& Breakpoint)
			{
				EnsureChildIsAdded(OutChildren,
					FBreakpointLineItem(Breakpoint.GetLocation(), Blueprint.Get()), InSearchString, bRespectSearch);
			}
		);

		// Make sure there is something there, to let the user know if there is nothing
		if (OutChildren.Num() == 0)
		{
			EnsureChildIsAdded(OutChildren,
				FMessageLineItem(LOCTEXT("NoBreakpoints", "No breakpoints").ToString()), InSearchString, bRespectSearch);
		}
	}

	virtual FText GetDisplayName() const override
	{
		return LOCTEXT("Breakpoints", "Breakpoints");
	}

	virtual FDebugLineItem* Duplicate() const override
	{
		check(false);
		return nullptr;
	}

	virtual bool Compare(const FDebugLineItem* BaseOther) const override
	{
		check(false);
		return false;
	}

	virtual uint32 GetHash() const override
	{
		check(false);
		return 0;
	}

private:

	void DisableAllBreakpoints() const
	{
		FKismetDebugUtilities::ForeachBreakpoint(Blueprint.Get(),
			[](FBlueprintBreakpoint& Breakpoint)
			{
				FKismetDebugUtilities::SetBreakpointEnabled(Breakpoint, false);
			}
		);
	}

	void EnableAllBreakpoints() const
	{
		FKismetDebugUtilities::ForeachBreakpoint(Blueprint.Get(),
			[](FBlueprintBreakpoint& Breakpoint)
			{
				FKismetDebugUtilities::SetBreakpointEnabled(Breakpoint, true);
			}
		);
	}

	void ClearAllBreakpoints() const
	{
		FKismetDebugUtilities::ClearBreakpoints(Blueprint.Get());
	}
};

void FDebugLineItem::SetBreakpointParentItemBlueprint(FDebugTreeItemPtr InBreakpointParentItem, TWeakObjectPtr<UBlueprint> InBlueprint)
{
	if (ensureMsgf(InBreakpointParentItem.IsValid() && InBreakpointParentItem->Type == DLT_BreakpointParent, TEXT("TreeItem is not Valid!")))
	{
		TSharedPtr<FBreakpointParentItem> BreakpointItem = StaticCastSharedPtr<FBreakpointParentItem>(InBreakpointParentItem);
		BreakpointItem->Blueprint = InBlueprint;
	}
}

//////////////////////////////////////////////////////////////////////////
// FParentLineItem

class FParentLineItem : public FLineItemWithChildren
{
protected:
	// The parent object
	TWeakObjectPtr<UObject> ObjectRef;
public:
	FParentLineItem(UObject* Object)
		: FLineItemWithChildren(DLT_Parent)
	{
		ObjectRef = Object;
	}

	virtual TSharedRef<SWidget> GenerateNameWidget(TSharedPtr<FString> InSearchString) override
	{
		return SNew(PropertyInfoViewStyle::STextHighlightOverlay)
			.FullText(this, &FParentLineItem::GetDisplayName)
			.HighlightText(this, &FParentLineItem::GetHighlightText, InSearchString)
			[
				SNew(STextBlock)
					.ToolTipText(this, &FParentLineItem::GetTooltipText)
					.Text(this, &FParentLineItem::GetDisplayName)
			];
	}

	virtual UObject* GetParentObject() const override
	{
		return ObjectRef.Get();
	}

	virtual void GatherChildren(TArray<FDebugTreeItemPtr>& OutChildren, const FString& InSearchString, bool bRespectSearch) override
	{
		// update search flags to match that of a root node
		UpdateSearch(InSearchString, SF_RootNode);

		if (UObject* ParentObject = ObjectRef.Get())
		{
			// every instance should have an automatic watch for 'self'
			EnsureChildIsAdded(OutChildren, FSelfWatchLineItem(ParentObject), InSearchString, bRespectSearch);

			UBlueprint* ParentBP = FDebugLineItem::GetBlueprintForObject(ParentObject);
			if (ParentBP != nullptr)
			{
				// Create children for each watch
				if (IsDebugLineTypeActive(DLT_Watch))
				{
					FKismetDebugUtilities::ForeachPinPropertyWatch(ParentBP,
						[this, &OutChildren, ParentObject, &InSearchString, bRespectSearch] (FBlueprintWatchedPin& WatchedPin)
						{
							EnsureChildIsAdded(OutChildren,
								FWatchLineItem(WatchedPin.Get(), ParentObject, WatchedPin.GetPathToProperty()), InSearchString, bRespectSearch);
						}
					);
				}

				// It could also have active latent behaviors
				if (IsDebugLineTypeActive(DLT_LatentAction))
				{
					if (UWorld* World = GEngine->GetWorldFromContextObject(ParentObject, EGetWorldErrorMode::ReturnNull))
					{
						FLatentActionManager& LatentActionManager = World->GetLatentActionManager();

						// Get the current list of action UUIDs
						TSet<int32> UUIDSet;
						LatentActionManager.GetActiveUUIDs(ParentObject, /*inout*/ UUIDSet);

						// Add the new ones
						for (TSet<int32>::TConstIterator RemainingIt(UUIDSet); RemainingIt; ++RemainingIt)
						{
							const int32 UUID = *RemainingIt;
							EnsureChildIsAdded(OutChildren,
								FLatentActionLineItem(UUID, ParentObject), InSearchString, bRespectSearch);
						}
					}
				}

				// Make sure there is something there, to let the user know if there is nothing
				if (OutChildren.Num() == 0)
				{
					EnsureChildIsAdded(OutChildren,
						FMessageLineItem(LOCTEXT("NoDebugInfo", "No debugging info").ToString()), InSearchString, bRespectSearch);
				}
			}
			//@TODO: try to get at TArray<struct FDebugDisplayProperty> DebugProperties in UGameViewportClient, if available
		}
	}

	virtual TSharedRef<SWidget> GetNameIcon() override
	{
		return SNew(SImage)
			.Image(this, &FParentLineItem::GetStatusImage)
			.ColorAndOpacity_Raw(this, &FParentLineItem::GetStatusColor)
			.ToolTipText(this, &FParentLineItem::GetStatusTooltip);
	}

	const FSlateBrush* GetStatusImage() const
	{
		if (SKismetDebuggingView::CurrentActiveObject == ObjectRef)
		{
			return FAppStyle::GetBrush(TEXT("Kismet.Trace.CurrentIndex"));
		}
		if (ObjectRef.IsValid())
		{
			return FSlateIconFinder::FindIconBrushForClass(ObjectRef->GetClass());
		}
		return FAppStyle::GetBrush(TEXT("None"));
	}

	FSlateColor GetStatusColor() const
	{
		if (SKismetDebuggingView::CurrentActiveObject == ObjectRef)
		{
			return FSlateColor(EStyleColor::AccentYellow);
		}
		const UGraphEditorSettings* Settings = GetDefault<UGraphEditorSettings>();
		return Settings->ObjectPinTypeColor;
	}

	FText GetStatusTooltip() const
	{
		if (SKismetDebuggingView::CurrentActiveObject == ObjectRef)
		{
			return LOCTEXT("BreakpointHIt", "Breakpoint Hit");
		}
		return FText::GetEmpty();
	}

	virtual void ExtendContextMenu(class FMenuBuilder& MenuBuilder, bool bInDebuggerTab) override
	{
		if (UBlueprint* BP = Cast<UBlueprint>(ObjectRef.Get()))
		{
			if (FKismetDebugUtilities::BlueprintHasPinWatches(BP))
			{
				FUIAction ClearAllWatches(
					FExecuteAction::CreateSP(this, &FParentLineItem::ClearAllWatches),
					FCanExecuteAction() // always allow
				);

				MenuBuilder.AddMenuEntry(
					LOCTEXT("ClearWatches", "Clear all watches"),
					LOCTEXT("ClearWatches_ToolTip", "Clear all watches in this blueprint"),
					FSlateIcon(),
					ClearAllWatches);
			}
		}
	}
protected:
	virtual FDebugLineItem* Duplicate() const override
	{
		return new FParentLineItem(ObjectRef.Get());
	}

	virtual bool Compare(const FDebugLineItem* BaseOther) const override
	{
		FParentLineItem* Other = (FParentLineItem*)BaseOther;
		return ObjectRef.Get() == Other->ObjectRef.Get();
	}

	virtual uint32 GetHash() const override
	{
		return GetTypeHash(ObjectRef);
	}

	virtual FText GetDisplayName() const override
	{
		UObject* Object = ObjectRef.Get();
		AActor* Actor = Cast<AActor>(Object);

		if (Actor != nullptr)
		{
			return FText::FromString(Actor->GetActorLabel());
		}
		else
		{
			return (Object != nullptr) ? FText::FromString(Object->GetName()) : LOCTEXT("nullptr", "(nullptr)");
		}
	}

private:

	void ClearAllWatches() const
	{
		if (UBlueprint* Blueprint = Cast<UBlueprint>(ObjectRef.Get()))
		{
			FKismetDebugUtilities::ClearPinWatches(Blueprint);
		}
	}

	FText GetTooltipText() const
	{
		if (const UObject* Object = ObjectRef.Get())
		{
			if (UWorld* World = Object->GetTypedOuter<UWorld>())
			{
				FText WorldName = FText::FromString(GetDebugStringForWorld(World));

				return FText::FormatNamed(LOCTEXT("ParentLineTooltip", "{ObjectFullPath}\nWorld: {WorldFullPath}\nWorld Type: {WorldType}"),
					TEXT("ObjectFullPath"), FText::FromString(Object->GetPathName()),
					TEXT("WorldFullPath"), FText::FromString(World->GetPathName()),
					TEXT("WorldType"), WorldName);
			}
		}

		return GetDisplayName();
	}
};

//////////////////////////////////////////////////////////////////////////
// FTraceStackChildItem

class FTraceStackChildItem : public FDebugLineItem
{
protected:
	int32 StackIndex;
public:
	FTraceStackChildItem(int32 InStackIndex)
		: FDebugLineItem(DLT_TraceStackChild)
	{
		StackIndex = InStackIndex;
	}
	virtual TSharedRef<SWidget> GenerateNameWidget(TSharedPtr<FString> InSearchString) override
	{
		return SNew(PropertyInfoViewStyle::STextHighlightOverlay)
			.FullText(this, &FTraceStackChildItem::GetDisplayName)
			.HighlightText(this, &FTraceStackChildItem::GetHighlightText, InSearchString)
			[
				SNew(SHyperlink)
					.Text(this, &FTraceStackChildItem::GetDisplayName)
					.Style(FAppStyle::Get(), "HoverOnlyHyperlink")
					.ToolTipText(LOCTEXT("NavigateToDebugTraceLocationHyperlink_ToolTip", "Navigate to the trace location"))
					.OnNavigate(this, &FTraceStackChildItem::OnNavigateToNode)
			];
	}

	// Visit time and actor name
	virtual TSharedRef<SWidget> GenerateValueWidget(TSharedPtr<FString> InSearchString) override
	{
		return SNew(PropertyInfoViewStyle::STextHighlightOverlay)
			.FullText(this, &FTraceStackChildItem::GetDescription)
			.HighlightText(this, &FTraceStackChildItem::GetHighlightText, InSearchString)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SHyperlink)
							.Text(this, &FTraceStackChildItem::GetContextObjectName)
							.Style(FAppStyle::Get(), "HoverOnlyHyperlink")
							.ToolTipText(LOCTEXT("SelectActor_Tooltip", "Select this actor"))
							.OnNavigate(this, &FTraceStackChildItem::OnSelectContextObject)
					]
				+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(this, &FTraceStackChildItem::GetVisitTime)
					]
			];
	}

	virtual TSharedRef<SWidget> GetNameIcon() override
	{
		return SNew(SImage)
			.Image(FAppStyle::GetBrush(
				(StackIndex > 0) ?
				TEXT("Kismet.Trace.PreviousIndex") :
				TEXT("Kismet.Trace.CurrentIndex"))
			);
	}

	virtual FText GetDescription() const override
	{
		return FText::FromString(GetContextObjectName().ToString() + GetVisitTime().ToString());
	}

protected:
	virtual FDebugLineItem* Duplicate() const override
	{
		check(false);
		return nullptr;
	}

	virtual bool Compare(const FDebugLineItem* BaseOther) const override
	{
		check(false);
		return false;
	}

	virtual uint32 GetHash() const override
	{
		check(false);
		return 0;
	}

	virtual FText GetDisplayName() const override
	{
		UEdGraphNode* Node = GetNode();
		if (Node != nullptr)
		{
			return Node->GetNodeTitle(ENodeTitleType::ListView);
		}
		else
		{
			return LOCTEXT("Unknown", "(unknown)");
		}
	}

	UEdGraphNode* GetNode() const
	{
		const TSimpleRingBuffer<FKismetTraceSample>& TraceStack = FKismetDebugUtilities::GetTraceStack();
		if (StackIndex < TraceStack.Num())
		{
			const FKismetTraceSample& Sample = TraceStack(StackIndex);
			UObject* ObjectContext = Sample.Context.Get();

			FString ContextName = (ObjectContext != nullptr) ? ObjectContext->GetName() : LOCTEXT("ObjectDoesNotExist", "(object no longer exists)").ToString();
			FString NodeName = TEXT(" ");

			if (ObjectContext != nullptr)
			{
				// Try to find the node that got executed
				UEdGraphNode* Node = FKismetDebugUtilities::FindSourceNodeForCodeLocation(ObjectContext, Sample.Function.Get(), Sample.Offset);
				return Node;
			}
		}

		return nullptr;
	}

	FText GetVisitTime() const
	{
		const TSimpleRingBuffer<FKismetTraceSample>& TraceStack = FKismetDebugUtilities::GetTraceStack();
		if (StackIndex < TraceStack.Num())
		{
			static const FNumberFormattingOptions TimeFormatOptions = FNumberFormattingOptions()
				.SetMinimumFractionalDigits(2)
				.SetMaximumFractionalDigits(2);
			return FText::Format(LOCTEXT("VisitTimeFmt", " @ {0} s"), FText::AsNumber(TraceStack(StackIndex).ObservationTime - GStartTime, &TimeFormatOptions));
		}

		return FText::GetEmpty();
	}

	FText GetContextObjectName() const
	{
		const TSimpleRingBuffer<FKismetTraceSample>& TraceStack = FKismetDebugUtilities::GetTraceStack();

		UObject* ObjectContext = (StackIndex < TraceStack.Num()) ? TraceStack(StackIndex).Context.Get() : nullptr;

		return (ObjectContext != nullptr) ? FText::FromString(ObjectContext->GetName()) : LOCTEXT("ObjectDoesNotExist", "(object no longer exists)");
	}

	void OnNavigateToNode()
	{
		if (UEdGraphNode* Node = GetNode())
		{
			FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(Node);
		}
	}

	void OnSelectContextObject()
	{
		const TSimpleRingBuffer<FKismetTraceSample>& TraceStack = FKismetDebugUtilities::GetTraceStack();

		UObject* ObjectContext = (StackIndex < TraceStack.Num()) ? TraceStack(StackIndex).Context.Get() : nullptr;

		// Add the object to the selection set
		if (AActor* Actor = Cast<AActor>(ObjectContext))
		{
			GEditor->SelectActor(Actor, true, true, true);
		}
		else
		{
			UE_LOGF(LogBlueprintDebugTreeView, Warning, "Cannot select the non-actor object '%ls'", (ObjectContext != nullptr) ? *ObjectContext->GetName() : TEXT("(nullptr)"));
		}
	}
};

//////////////////////////////////////////////////////////////////////////
// FTraceStackParentItem

class FTraceStackParentItem : public FLineItemWithChildren
{
public:
	FTraceStackParentItem()
		: FLineItemWithChildren(DLT_TraceStackParent)
	{
	}

	virtual bool HasChildren() const override
	{
		return !ChildrenMirrorsArr.IsEmpty();
	}

protected:
	virtual FDebugLineItem* Duplicate() const override
	{
		check(false);
		return nullptr;
	}

	virtual bool Compare(const FDebugLineItem* BaseOther) const override
	{
		check(false);
		return false;
	}

	virtual uint32 GetHash() const override
	{
		check(false);
		return 0;
	}

	virtual FText GetDisplayName() const override
	{
		return LOCTEXT("ExecutionTrace", "Execution Trace");
	}

	virtual void GatherChildren(TArray<FDebugTreeItemPtr>& OutChildren, const FString& InSearchString, bool bRespectSearch) override
	{
		// update search flags to match that of a root node
		UpdateSearch(InSearchString, SF_RootNode);

		const TSimpleRingBuffer<FKismetTraceSample>& TraceStack = FKismetDebugUtilities::GetTraceStack();
		const int32 NumVisible = TraceStack.Num();

		// Create any new stack entries that are needed
		for (int32 i = ChildrenMirrorsArr.Num(); i < NumVisible; ++i)
		{
			ChildrenMirrorsArr.Add(MakeShareable(new FTraceStackChildItem(i)));
		}

		// Add the visible stack entries as children
		for (int32 i = 0; i < NumVisible; ++i)
		{
			OutChildren.Add(ChildrenMirrorsArr[i]);
		}
	}

	// use an array to store children mirrors instead of a set so it's ordered
	TArray<FDebugTreeItemPtr> ChildrenMirrorsArr;
};

//////////////////////////////////////////////////////////////////////////
// SDebugLineItem

class SDebugLineItem : public SMultiColumnTableRow< FDebugTreeItemPtr >
{
protected:
	FDebugTreeItemPtr ItemToEdit;
	TSharedPtr<FString> SearchString;
public:
	SLATE_BEGIN_ARGS(SDebugLineItem) {}
	SLATE_END_ARGS()

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		TSharedPtr<SWidget> ColumnContent = nullptr;
		if (ColumnName == SKismetDebugTreeView::ColumnId_Name)
		{
			SAssignNew(ColumnContent, SHorizontalBox)
			+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Fill)
				.AutoWidth()
				[
					SNew(PropertyInfoViewStyle::SIndent, SharedThis(this))
				]
			+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(PropertyInfoViewStyle::SExpanderArrow, SharedThis(this))
					.HasChildren_Lambda([ItemToEdit = ItemToEdit]()
						{
							const bool HasChildren = ItemToEdit->HasChildren();
							return HasChildren;
						})
				]
			+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					ItemToEdit->GetNameIcon()
				]
			+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(5.f, 0.f, 0.f, 0.f)
				[
					ItemToEdit->GenerateNameWidget(SearchString)
				];
		}
		else if (ColumnName == SKismetDebugTreeView::ColumnId_Value)
		{
			SAssignNew(ColumnContent, SHorizontalBox)
			+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					ItemToEdit->GetValueIcon()
				]
			+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.Padding(.5f, 1.f)
				[
					ItemToEdit->GenerateValueWidget(SearchString)
				];
		}
		else
		{
			SAssignNew(ColumnContent, STextBlock)
				.Text(LOCTEXT("Error", "Error"));
		}

		return SNew(SBox)
			.Padding(FMargin(0.5f, 0.5f))
			[
				SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle"))
					.BorderBackgroundColor_Static(
						PropertyInfoViewStyle::GetRowBackgroundColor,
						static_cast<ITableRow*>(this)
					)
					[
						ColumnContent.ToSharedRef()
					]
			];
	}

	void Construct(const FArguments& InArgs, TSharedRef<STableViewBase> OwnerTableView, FDebugTreeItemPtr InItemToEdit, TSharedPtr<FString> InSearchString)
	{
		ItemToEdit = InItemToEdit;
		SearchString = InSearchString;
		SMultiColumnTableRow<FDebugTreeItemPtr>::Construct(FSuperRowType::FArguments(), OwnerTableView);
	}

protected:

	virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override
	{
		const TSharedRef<SWidget>* NameWidget = GetWidgetFromColumnId(SKismetDebugTreeView::ColumnId_Name);
		const TSharedRef<SWidget>* ValWidget = GetWidgetFromColumnId(SKismetDebugTreeView::ColumnId_Value);

		if (NameWidget && ValWidget)
		{
			return FVector2D::Max((*NameWidget)->GetDesiredSize(), (*ValWidget)->GetDesiredSize()) * FVector2D(2.0f, 1.0f);
		}

		return STableRow<FDebugTreeItemPtr>::ComputeDesiredSize(LayoutScaleMultiplier);
	}
};

//////////////////////////////////////////////////////////////////////////
// SKismetDebugTreeView

void SKismetDebugTreeView::Construct(const FArguments& InArgs)
{
	bFilteredItemsDirty = false;
	bInDebuggerTab = InArgs._InDebuggerTab;
	SearchString = MakeShared<FString>();
	SearchMessageItem = MakeMessageItem(LOCTEXT("NoItemsMatchSearch", "No entries match the search text").ToString());

	ChildSlot
	[
		SAssignNew(TreeView, STreeView< FDebugTreeItemPtr >)
			.TreeItemsSource(&FilteredTreeRoots)
			.SelectionMode(InArgs._SelectionMode)
			.OnGetChildren(this, &SKismetDebugTreeView::OnGetChildren)
			.OnGenerateRow(this, &SKismetDebugTreeView::OnGenerateRow)
			.OnExpansionChanged(InArgs._OnExpansionChanged)
			.OnContextMenuOpening(this, &SKismetDebugTreeView::OnMakeContextMenu)
			.TreeViewStyle(&FAppStyle::Get().GetWidgetStyle<FTableViewStyle>("PropertyTable.InViewport.ListView"))
			.HeaderRow(InArgs._HeaderRow)
	];
}

void SKismetDebugTreeView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bFilteredItemsDirty)
	{
		UpdateFilteredItems();
		bFilteredItemsDirty = false;
	}
}

void SKismetDebugTreeView::AddTreeItemUnique(const FDebugTreeItemPtr& Item)
{
	RootTreeItems.AddUnique(Item);
	RequestUpdateFilteredItems();
}

bool SKismetDebugTreeView::RemoveTreeItem(const FDebugTreeItemPtr& Item)
{
	if (RootTreeItems.Remove(Item) != 0)
	{
		RequestUpdateFilteredItems();
		return true;
	}

	return false;
}

void SKismetDebugTreeView::ClearTreeItems()
{
	if (!RootTreeItems.IsEmpty())
	{
		RootTreeItems.Empty();
		RequestUpdateFilteredItems();
	}
}

void SKismetDebugTreeView::SetSearchText(const FText& InSearchText)
{
	*SearchString = InSearchText.ToString();
	RequestUpdateFilteredItems();
}

void SKismetDebugTreeView::RequestUpdateFilteredItems()
{
	bFilteredItemsDirty = true;
}

const TArray<FDebugTreeItemPtr>& SKismetDebugTreeView::GetRootTreeItems() const
{
	return RootTreeItems;
}

int32 SKismetDebugTreeView::GetSelectedItems(TArray<FDebugTreeItemPtr>& OutItems)
{
	return TreeView->GetSelectedItems(OutItems);
}

void SKismetDebugTreeView::ClearExpandedItems()
{
	TreeView->ClearExpandedItems();
}

bool SKismetDebugTreeView::IsScrolling() const
{
	return TreeView->IsScrolling();
}

void SKismetDebugTreeView::SetItemExpansion(FDebugTreeItemPtr InItem, bool bInShouldExpandItem)
{
	TreeView->SetItemExpansion(InItem, bInShouldExpandItem);
}

void SKismetDebugTreeView::UpdateFilteredItems()
{
	FilteredTreeRoots.Empty();
	for (FDebugTreeItemPtr Item : RootTreeItems)
	{
		if (Item.IsValid())
		{
			if (Item->CanHaveChildren())
			{
				FLineItemWithChildren* ItemWithChildren = StaticCast<FLineItemWithChildren*>(Item.Get());
				if (SearchString->IsEmpty() || ItemWithChildren->SearchRecursive(*SearchString, TreeView))
				{
					FilteredTreeRoots.Add(Item);
				}
			}
			else
			{
				Item->UpdateSearch(*SearchString, FDebugLineItem::SF_RootNode);
				if (SearchString->IsEmpty() || Item->IsVisible())
				{
					FilteredTreeRoots.Add(Item);
				}
			}
		}
	}

	if (FilteredTreeRoots.IsEmpty())
	{
		FilteredTreeRoots.Add(SearchMessageItem);
	}

	TreeView->RequestTreeRefresh();
}

TSharedRef<ITableRow> SKismetDebugTreeView::OnGenerateRow(FDebugTreeItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SDebugLineItem, OwnerTable, InItem, SearchString);
}

void SKismetDebugTreeView::OnGetChildren(FDebugTreeItemPtr InParent, TArray<FDebugTreeItemPtr>& OutChildren)
{
	InParent->GatherChildrenBase(OutChildren, *SearchString);
}

TSharedPtr<SWidget> SKismetDebugTreeView::OnMakeContextMenu() const
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.BeginSection("DebugActions", LOCTEXT("DebugActionsMenuHeading", "Debug Actions"));
	{
		TArray<FDebugTreeItemPtr> SelectedItems;
		TreeView->GetSelectedItems(SelectedItems);

		for (FDebugTreeItemPtr& Item : SelectedItems)
		{
			Item->MakeMenu(MenuBuilder, bInDebuggerTab);
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

FDebugTreeItemPtr SKismetDebugTreeView::MakeTraceStackParentItem()
{
	return MakeShared<FTraceStackParentItem>();
}

FDebugTreeItemPtr SKismetDebugTreeView::MakeBreakpointParentItem(TWeakObjectPtr<UBlueprint> InBlueprint)
{
	return MakeShared<FBreakpointParentItem>(InBlueprint);
}

FDebugTreeItemPtr SKismetDebugTreeView::MakeMessageItem(const FString& InMessage)
{
	return MakeShared<FMessageLineItem>(InMessage);
}

FDebugTreeItemPtr SKismetDebugTreeView::MakeParentItem(UObject* InObject)
{
	return MakeShared<FParentLineItem>(InObject);
}

FDebugTreeItemPtr SKismetDebugTreeView::MakeWatchLineItem(const UEdGraphPin* InPinRef, UObject* InDebugObject)
{
	return MakeShared<FWatchLineItem>(InPinRef, InDebugObject);
}

#undef LOCTEXT_NAMESPACE
