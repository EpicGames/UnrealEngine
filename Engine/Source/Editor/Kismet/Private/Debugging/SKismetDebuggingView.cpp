// Copyright Epic Games, Inc. All Rights Reserved.


#include "Debugging/SKismetDebuggingView.h"

#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Set.h"
#include "Debugging/SKismetDebugTreeView.h"
#include "Delegates/Delegate.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Blueprint.h"
#include "Engine/EngineTypes.h"
#include "Engine/World.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/Breakpoint.h"
#include "Kismet2/DebuggerCommands.h"
#include "Kismet2/KismetDebugUtilities.h"
#include "Layout/Children.h"
#include "Logging/LogMacros.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateTypes.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "ToolMenu.h"
#include "ToolMenuContext.h"
#include "ToolMenus.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Script.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"

class SWidget;
struct FGeometry;
struct FToolMenuSection;

#define LOCTEXT_NAMESPACE "DebugViewUI"

DEFINE_LOG_CATEGORY_STATIC(LogBlueprintDebuggingView, Log, All);

//////////////////////////////////////////////////////////////////////////

namespace KismetDebugViewConstants
{
	const FText ColumnText_Name( NSLOCTEXT("DebugViewUI", "Name", "Name") );
	const FText ColumnText_Value( NSLOCTEXT("DebugViewUI", "Value", "Value") );
	const FText ColumnText_DebugKey( FText::GetEmpty() );
	const FText ColumnText_Info( NSLOCTEXT("DebugViewUI", "Info", "Info") );
}


//////////////////////////////////////////////////////////////////////////
// SKismetDebuggingView

TWeakObjectPtr<const UObject> SKismetDebuggingView::CurrentActiveObject = nullptr;

TSharedRef<SHorizontalBox> SKismetDebuggingView::GetDebugLineTypeToggle(FDebugLineItem::EDebugLineType Type, const FText& Text)
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SCheckBox)
				.IsChecked(ECheckBoxState::Checked)
				.OnCheckStateChanged_Static(&FDebugLineItem::OnDebugLineTypeActiveChanged, Type)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4.0f, 0.0f, 10.0f, 0.0f)
		.VAlign(VAlign_Center)
		[
			SNew( STextBlock )
				.Text(Text)
		];
}


void SKismetDebuggingView::OnSearchTextChanged(const FText& Text)
{
	DebugTreeView->ClearExpandedItems();
	OtherTreeView->ClearExpandedItems();
	DebugTreeView->SetSearchText(Text);
	OtherTreeView->SetSearchText(Text);
}

FText SKismetDebuggingView::GetTabLabel() const
{
	return BlueprintToWatchPtr.IsValid() ?
		FText::FromString(BlueprintToWatchPtr->GetName()) :
		NSLOCTEXT("BlueprintExecutionFlow", "TabTitle", "Data Flow");
}

void SKismetDebuggingView::TryRegisterDebugToolbar()
{
	static const FName ToolbarName = "Kismet.DebuggingViewToolBar";
	if (!UToolMenus::Get()->IsMenuRegistered(ToolbarName))
	{
		UToolMenu* ToolBar = UToolMenus::Get()->RegisterMenu(ToolbarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
		FToolMenuSection& Section = ToolBar->AddSection("Debug");
		FPlayWorldCommands::BuildToolbar(Section);
	}
}

FText SKismetDebuggingView::GetTopText() const
{
	return LOCTEXT("ShowDebugForActors", "Showing debug info for instances of the blueprint:");
}

FText SKismetDebuggingView::GetToggleAllBreakpointsText() const
{
	const FBlueprintBreakpoint* EnabledBreakpoint = FKismetDebugUtilities::FindBreakpointByPredicate(BlueprintToWatchPtr.Get(), [](const FBlueprintBreakpoint& Breakpoint) 
		{
			return Breakpoint.IsEnabled();
		});
	
	if (EnabledBreakpoint)
	{
		return LOCTEXT("DisableAllBreakPoints", "Disable All Breakpoints");
	}
	else
	{
		return LOCTEXT("EnableAllBreakPoints", "Enable All Breakpoints");
	}
}

bool SKismetDebuggingView::CanToggleAllBreakpoints() const
{
	if(BlueprintToWatchPtr.IsValid())
	{
		return FKismetDebugUtilities::BlueprintHasBreakpoints(BlueprintToWatchPtr.Get());
	}
	return false;
}

FReply SKismetDebuggingView::OnToggleAllBreakpointsClicked()
{
	if (UBlueprint* Blueprint = BlueprintToWatchPtr.Get())
	{
		const FBlueprintBreakpoint* EnabledBreakpoint = FKismetDebugUtilities::FindBreakpointByPredicate(BlueprintToWatchPtr.Get(), [](const FBlueprintBreakpoint& Breakpoint)
			{
				return Breakpoint.IsEnabled();
			});

		bool bHasAnyEnabledBreakpoint = EnabledBreakpoint != nullptr;
		if (BlueprintToWatchPtr.IsValid())
		{
			FKismetDebugUtilities::ForeachBreakpoint(BlueprintToWatchPtr.Get(),
				[bHasAnyEnabledBreakpoint](FBlueprintBreakpoint& Breakpoint)
				{
					FKismetDebugUtilities::SetBreakpointEnabled(Breakpoint, !bHasAnyEnabledBreakpoint);
				}
			);
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

class FBlueprintFilter : public IClassViewerFilter
{
public:
	FBlueprintFilter() = default;

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs ) override
	{
		return InClass && !InClass->HasAnyClassFlags(CLASS_Deprecated) &&
				InClass->HasAllClassFlags(CLASS_CompiledFromBlueprint);
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		return !InUnloadedClassData->HasAnyClassFlags(CLASS_Deprecated) &&
				InUnloadedClassData->HasAllClassFlags(CLASS_CompiledFromBlueprint);
	}
};

void SKismetDebuggingView::OnBlueprintClassPicked(UClass* PickedClass)
{
	if (PickedClass)
	{
		BlueprintToWatchPtr = Cast<UBlueprint>(PickedClass->ClassGeneratedBy);
	}
	else
	{
		// User selected None Option
		BlueprintToWatchPtr.Reset();
	}

	FDebugLineItem::SetBreakpointParentItemBlueprint(BreakpointParentItem, BlueprintToWatchPtr);
	DebugClassComboButton->SetIsOpen(false);
}

TSharedRef<SWidget> SKismetDebuggingView::ConstructBlueprintClassPicker()
{
	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;
	Options.bShowBackgroundBorder = false;
	Options.ClassFilters.Add(MakeShared<FBlueprintFilter>());
	Options.bIsBlueprintBaseOnly = true;
	Options.bShowUnloadedBlueprints = false;
	Options.bShowNoneOption = true;
	
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	FOnClassPicked OnClassPicked;
	OnClassPicked.BindRaw(this, &SKismetDebuggingView::OnBlueprintClassPicked);
	return SNew(SBox)
		.HeightOverride(500.f)
		[
			ClassViewerModule.CreateClassViewer(Options, OnClassPicked)
		];
}

void SKismetDebuggingView::Construct(const FArguments& InArgs)
{	
	BlueprintToWatchPtr = InArgs._BlueprintToWatch;

	// Build the debug toolbar
	static const FName ToolbarName = "Kismet.DebuggingViewToolBar";
	TryRegisterDebugToolbar();
	FToolMenuContext MenuContext(FPlayWorldCommands::GlobalPlayWorldActions);
	TSharedRef<SWidget> ToolbarWidget = UToolMenus::Get()->GenerateWidget(ToolbarName, MenuContext);
	
	DebugClassComboButton =
		SNew(SComboButton)
			.OnGetMenuContent_Raw(this, &SKismetDebuggingView::ConstructBlueprintClassPicker)
			.ButtonContent()
			[
				SNew(STextBlock)
					.Text_Lambda([&BlueprintToWatchPtr = BlueprintToWatchPtr]()
					{
						return BlueprintToWatchPtr.IsValid()?
							FText::FromString(BlueprintToWatchPtr->GetName()) :
							LOCTEXT("SelectBlueprint", "Select Blueprint");
					})
			];

	FBlueprintContextTracker::OnEnterScriptContext.AddLambda(
		[](const FBlueprintContextTracker& ContextTracker, const UObject* ContextObject, const UFunction* ContextFunction)
		{
			CurrentActiveObject = ContextObject;
		}
	);
	
	FBlueprintContextTracker::OnExitScriptContext.AddLambda(
		[](const FBlueprintContextTracker& ContextTracker)
		{
			CurrentActiveObject = nullptr;
		}
	);

	
	this->ChildSlot
	[
		SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
					.BorderImage( FAppStyle::GetBrush( TEXT("NoBorder") ) )
					[
						ToolbarWidget
					]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				
				SNew( SVerticalBox )
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew( STextBlock )
							.Text( this, &SKismetDebuggingView::GetTopText )
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew( SHorizontalBox )
							+ SHorizontalBox::Slot()
							.HAlign( HAlign_Left )
							[
								SNew(SBox)
									.WidthOverride(400.f)
									[
										DebugClassComboButton.ToSharedRef()
									]
							]
							+ SHorizontalBox::Slot()
							.HAlign( HAlign_Right )
							[
								SNew( SButton )
					                .IsEnabled( this, &SKismetDebuggingView::CanToggleAllBreakpoints )
					                .Text( this, &SKismetDebuggingView::GetToggleAllBreakpointsText )
					                .OnClicked( this, &SKismetDebuggingView::OnToggleAllBreakpointsClicked )
							]
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SAssignNew(SearchBox, SSearchBox)
							.OnTextChanged(this, &SKismetDebuggingView::OnSearchTextChanged)
					]
			]
			+SVerticalBox::Slot()
			[
				SNew(SSplitter)
					.Orientation(Orient_Vertical)
					+SSplitter::Slot()
					[
						SAssignNew( DebugTreeView, SKismetDebugTreeView )
							.InDebuggerTab(true)
							.HeaderRow
							(
								SNew(SHeaderRow)
									+ SHeaderRow::Column(SKismetDebugTreeView::ColumnId_Name)
									.DefaultLabel(KismetDebugViewConstants::ColumnText_Name)
									+ SHeaderRow::Column(SKismetDebugTreeView::ColumnId_Value)
									.DefaultLabel(KismetDebugViewConstants::ColumnText_Value)
							)
					]
					+SSplitter::Slot()
					[
						SAssignNew( OtherTreeView, SKismetDebugTreeView )
							.InDebuggerTab(true)
							.HeaderRow
							(
								SNew(SHeaderRow)
									+ SHeaderRow::Column(SKismetDebugTreeView::ColumnId_Name)
									.DefaultLabel(KismetDebugViewConstants::ColumnText_DebugKey)
									+ SHeaderRow::Column(SKismetDebugTreeView::ColumnId_Value)
									.DefaultLabel(KismetDebugViewConstants::ColumnText_Info)
							)
					]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						GetDebugLineTypeToggle(FDebugLineItem::DLT_Watch, LOCTEXT("Watchpoints", "Watchpoints"))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						GetDebugLineTypeToggle(FDebugLineItem::DLT_LatentAction, LOCTEXT("LatentActions", "Latent Actions"))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						GetDebugLineTypeToggle(FDebugLineItem::DLT_BreakpointParent, LOCTEXT("Breakpoints", "Breakpoints"))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						GetDebugLineTypeToggle(FDebugLineItem::DLT_TraceStackParent, LOCTEXT("ExecutionTrace", "Execution Trace"))
					]
			]
	];

	TraceStackItem = SKismetDebugTreeView::MakeTraceStackParentItem();
	BreakpointParentItem = SKismetDebugTreeView::MakeBreakpointParentItem(BlueprintToWatchPtr);
}

void SKismetDebuggingView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// don't update during scroll. this will help make the scroll smoother
	if (DebugTreeView->IsScrolling() || OtherTreeView->IsScrolling())
	{
		return;
	}

	// If there is no play world, immediately clear the list. this avoids showing a phantom 'nullptr' item when ending PIE
	const bool bIsDebugging = GEditor->PlayWorld != nullptr;
	if (!bIsDebugging && DebugTreeView->GetRootTreeItems().Num() > 0)
	{
		DebugTreeView->ClearTreeItems();
		return;
	}

	// update less often to avoid lag
	TreeUpdateTimer += InDeltaTime;
	if (TreeUpdateTimer < UpdateInterval)
	{
		return;
	}
	TreeUpdateTimer = 0.f;

	// Gather the old root set
	TSet<UObject*> OldRootSet;
	for (const FDebugTreeItemPtr& Item : DebugTreeView->GetRootTreeItems())
	{
		if (UObject* OldObject = Item->GetParentObject())
		{
			OldRootSet.Add(OldObject);
		}
	}

	// Gather what we'd like to be the new root set
	TSet<UObject*> NewRootSet;

	const auto TryAddBlueprintToNewRootSet = [&NewRootSet](UBlueprint* InBlueprint)
	{
		for (FThreadSafeObjectIterator Iter(InBlueprint->GeneratedClass, /*bOnlyGCedObjects =*/ false, /*AdditionalExclusionFlags =*/ RF_ArchetypeObject | RF_ClassDefaultObject); Iter; ++Iter)
		{
			UObject* Instance = *Iter;
			if (!Instance)
			{
				continue;
			}

			// only include instances of objects in a PIE world
			if (UWorld* World = Instance->GetTypedOuter<UWorld>())
			{
				if (!World || World->WorldType != EWorldType::PIE)
				{
					continue;
				}
			}

			NewRootSet.Add(Instance);
		}
	};

	if(bIsDebugging)
	{
		if (BlueprintToWatchPtr.IsValid())
		{
			// Show blueprint objects of the selected class
			TryAddBlueprintToNewRootSet(BlueprintToWatchPtr.Get());
		}
		else
		{
			// Show all blueprint objects with watches
			for (FThreadSafeObjectIterator BlueprintIter(UBlueprint::StaticClass(), /*bOnlyGCedObjects =*/ false, /*AdditionalExclusionFlags =*/ RF_ArchetypeObject | RF_ClassDefaultObject); BlueprintIter; ++BlueprintIter)
			{
				UBlueprint* Blueprint = Cast<UBlueprint>(*BlueprintIter);
				if (Blueprint && FKismetDebugUtilities::BlueprintHasPinWatches(Blueprint))
				{
					TryAddBlueprintToNewRootSet(Blueprint);
				}
			}
		}
	}

	// This will pull anything out of Old that is also New (sticking around), so afterwards Old is a list of things to remove
	DebugTreeView->ClearTreeItems();
	for (UObject* ObjectToAdd : NewRootSet)
	{
		TWeakObjectPtr<UObject> WeakObject = ObjectToAdd;

		// destroyed objects can still appear if they haven't ben GCed yet.
		// weak object pointers will detect it and return nullptr
		if(!WeakObject.Get())
		{
			continue;
		}

		if (OldRootSet.Contains(ObjectToAdd))
		{
			OldRootSet.Remove(ObjectToAdd);
			
			const TSharedPtr<FDebugLineItem>& Item = ObjectToTreeItemMap.FindChecked(ObjectToAdd);
			DebugTreeView->AddTreeItemUnique(Item);
		}
		else
		{
			FDebugTreeItemPtr NewPtr = SKismetDebugTreeView::MakeParentItem(ObjectToAdd);
			ObjectToTreeItemMap.Add(ObjectToAdd, NewPtr);
			DebugTreeView->AddTreeItemUnique(NewPtr);
		}
	}

	// Remove the old root set items that didn't get used again
	for (UObject* ObjectToRemove : OldRootSet)
	{
		ObjectToTreeItemMap.Remove(ObjectToRemove);
	}

	// Add a message if there are no active instances of DebugClass
	if (DebugTreeView->GetRootTreeItems().Num() == 0)
	{
		DebugTreeView->AddTreeItemUnique(SKismetDebugTreeView::MakeMessageItem(
			bIsDebugging ?
				LOCTEXT("NoInstances", "No instances of this blueprint in existence").ToString() :
				LOCTEXT("NoPIEorSIE", "run PIE or SIE to see instance debug info").ToString()
		));
	}
	
	// Show Breakpoints
	if(FDebugLineItem::IsDebugLineTypeActive(FDebugLineItem::DLT_BreakpointParent))
	{
		OtherTreeView->AddTreeItemUnique(BreakpointParentItem);
	}
	else
	{
		OtherTreeView->RemoveTreeItem(BreakpointParentItem);
	}

	// Show the trace stack when debugging
	if (bIsDebugging && FDebugLineItem::IsDebugLineTypeActive(FDebugLineItem::DLT_TraceStackParent))
	{
		OtherTreeView->AddTreeItemUnique(TraceStackItem);
	}
	else
	{
		OtherTreeView->RemoveTreeItem(TraceStackItem);
	}

	OtherTreeView->RequestUpdateFilteredItems();
}

void SKismetDebuggingView::SetBlueprintToWatch(TWeakObjectPtr<UBlueprint> InBlueprintToWatch)
{
	BlueprintToWatchPtr = InBlueprintToWatch;
	FDebugLineItem::SetBreakpointParentItemBlueprint(BreakpointParentItem, BlueprintToWatchPtr);
}

#undef LOCTEXT_NAMESPACE
