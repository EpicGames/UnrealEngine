// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Debugging/SKismetDebugTreeView.h"
#include "HAL/Platform.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SComboButton;
class SHorizontalBox;
class SSearchBox;
class SWidget;
class UBlueprint;
class UClass;
class UObject;
struct FGeometry;

//////////////////////////////////////////////////////////////////////////
// SKismetDebuggingView

class SKismetDebuggingView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SKismetDebuggingView )
		: _BlueprintToWatch()
		{}

		SLATE_ARGUMENT( TWeakObjectPtr<UBlueprint>, BlueprintToWatch )
	SLATE_END_ARGS()
public:
	void Construct( const FArguments& InArgs );

	// SWidget interface
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	// End of SWidget interface

	/* set to an object that's paused at a breakpoint and null otherwise */
	static TWeakObjectPtr<const UObject> CurrentActiveObject;
	
	FText GetTabLabel() const;

	/** Registers the Kismet.DebuggingViewToolBar if it hasn't already been registered */
	static void TryRegisterDebugToolbar();

	void SetBlueprintToWatch(TWeakObjectPtr<UBlueprint> InBlueprintToWatch);
protected:
	FText GetTopText() const;
	bool CanToggleAllBreakpoints() const;
	FText GetToggleAllBreakpointsText() const;
	FReply OnToggleAllBreakpointsClicked();

	void OnBlueprintClassPicked(UClass* PickedClass);
	TSharedRef<SWidget> ConstructBlueprintClassPicker();

	static TSharedRef<SHorizontalBox> GetDebugLineTypeToggle(FDebugLineItem::EDebugLineType Type, const FText& Text);
	
	// called when SearchBox query is changed by user
	void OnSearchTextChanged(const FText& Text);
protected:
	TSharedPtr<SKismetDebugTreeView> DebugTreeView;
	TMap<UObject*, FDebugTreeItemPtr> ObjectToTreeItemMap;

	// includes items such as breakpoints and Exectution trace
	TSharedPtr< SKismetDebugTreeView > OtherTreeView;

	// UI tree entries for stack trace and breakpoints
	FDebugTreeItemPtr TraceStackItem;
	FDebugTreeItemPtr BreakpointParentItem;

	// Combo button for selecting which blueprint is being watched
	TSharedPtr<SComboButton> DebugClassComboButton;
	TWeakObjectPtr<UBlueprint> BlueprintToWatchPtr;

	// Search Box for tree
	TSharedPtr<SSearchBox> SearchBox;

	// updating the tree every tick is slow. use this to
	// update less frequently
	static constexpr uint8 TreeUpdatesPerSecond = 2;
	static constexpr float UpdateInterval = 1.f / TreeUpdatesPerSecond;
	float TreeUpdateTimer = UpdateInterval;
};
