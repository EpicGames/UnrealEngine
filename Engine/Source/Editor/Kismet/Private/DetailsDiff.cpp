// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsDiff.h"

#include "BlueprintDetailsCustomization.h"
#include "BlueprintEditorLibrary.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Algo/Copy.h"
#include "Algo/Transform.h"
#include "Containers/Set.h"
#include "DetailsViewArgs.h"
#include "DetailWidgetRow.h"
#include "HAL/PlatformCrt.h"
#include "IDetailsView.h"
#include "Misc/AssertionMacros.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "PropertyPath.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Engine/MemberReference.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Widgets/Layout/LinkableScrollBar.h"
#include "Widgets/Input/SComboButton.h"
#include "Engine/BlueprintGeneratedClass.h"

class UObject;

#define LOCTEXT_NAMESPACE "DetailsDif"

FDetailsDiff::FDetailsDiff(const UObject* InObject, bool bScrollbarOnLeft )
	: DisplayedObject( InObject )
{
	ScrollBar = SNew(SLinkableScrollBar);
	DetailsView = CreateDetailsView(InObject, ScrollBar, bScrollbarOnLeft);
	DisplayedProperties = DetailsView->GetPropertiesInOrderDisplayed();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FDetailsDiff::FDetailsDiff( const UObject* InObject, FDetailsDiff::FOnDisplayedPropertiesChanged InOnDisplayedPropertiesChanged, bool bScrollbarOnLeft )
	: FDetailsDiff(InObject, bScrollbarOnLeft)
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

class DOBPDetailsCustomization : public IDetailCustomization
{
public:
	DOBPDetailsCustomization(UBlueprint* InBlueprint) : Blueprint(InBlueprint)
	{}
	
	static TSharedRef<IDetailCustomization> Make(UBlueprint* Blueprint)
	{
		return MakeShared<DOBPDetailsCustomization>(Blueprint);
	}
	
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override
	{
		if (!Blueprint.IsValid())
		{
			return;
		}
		
		IDetailCategoryBuilder& Category = DetailLayout.EditCategory("ClassOptions", LOCTEXT("ClassOptions", "Class Options"));

		// put the ClassOptions first
		Category.SetSortOrder(0);
		
		// ParentClass is a hidden property so we have to add it to the property map manually to use it
		const TSharedPtr<IPropertyHandle> ParentClassProperty = DetailLayout.AddObjectPropertyData({Blueprint.Get()}, TEXT("ParentClass"));

		Category.AddCustomRow( LOCTEXT("ClassOptions", "Class Options") )
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BlueprintDetails_ParentClass", "Parent Class"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SAssignNew(ParentClassComboButton, SComboButton)
				.IsEnabled(this, &DOBPDetailsCustomization::CanReparent)
				.OnGetMenuContent(this, &DOBPDetailsCustomization::GetParentClassMenuContent)
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text(this, &DOBPDetailsCustomization::GetParentClassName)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			]
		]
		.PropertyHandleList({ParentClassProperty});
	}

	bool CanReparent() const
	{
		// Don't show the reparent option if it's an Interface or we're not in editing mode
		return Blueprint.IsValid() && !FBlueprintEditorUtils::IsInterfaceBlueprint(Blueprint.Get()) && (BPTYPE_FunctionLibrary != Blueprint->BlueprintType);
	}
	
	TSharedRef<SWidget> GetParentClassMenuContent() const
	{
		if (!Blueprint.IsValid())
		{
			return SNew(SBox);
		}
		
		TArray<UBlueprint*> Blueprints;
		Blueprints.Add(Blueprint.Get());
		const TSharedRef<SWidget> ClassPicker = FBlueprintEditorUtils::ConstructBlueprintParentClassPicker(Blueprints, FOnClassPicked::CreateSP(this, &DOBPDetailsCustomization::OnClassPicked));

		// Achieving fixed width by nesting items within a fixed width box.
		return SNew(SBox)
			.WidthOverride(350.0f)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.MaxHeight(400.0f)
				.AutoHeight()
				[
					ClassPicker
				]
			];
	}
	
	FText GetParentClassName() const
	{
		const UClass* ParentClass = Blueprint.IsValid() ? Blueprint->ParentClass : nullptr;
		return ParentClass ? ParentClass->GetDisplayNameText() : FText::FromName(NAME_None);
	}

	void OnClassPicked(UClass* PickedClass) const
	{
		ParentClassComboButton->SetIsOpen(false);
		UBlueprintEditorLibrary::ReparentBlueprint(Blueprint.Get(), PickedClass);
	}
private:
	TWeakObjectPtr<UBlueprint> Blueprint;

	/** Combo button used to choose a parent class */
	TSharedPtr<SComboButton> ParentClassComboButton;
};

TSharedRef<IDetailsView> FDetailsDiff::CreateDetailsView(const UObject* InObject, TSharedPtr<SScrollBar> ExternalScrollbar, bool bScrollbarOnLeft)
{
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bShowDifferingPropertiesOption = true;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.ExternalScrollbar = ExternalScrollbar;
	DetailsViewArgs.ScrollbarAlignment = bScrollbarOnLeft ? HAlign_Left : HAlign_Right;

	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	TSharedRef<IDetailsView> DetailsView = EditModule.CreateDetailView(DetailsViewArgs);
	DetailsView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateStatic([]{return false; }));
	if (InObject && InObject->IsA<UBlueprint>())
	{
		TWeakObjectPtr<UBlueprint> ForCapture = const_cast<UBlueprint*>(Cast<UBlueprint>(InObject));
		// create a custom property layout so that sections like interfaces will be included in the diff view
		const FOnGetDetailCustomizationInstance LayoutOptionDetails = FOnGetDetailCustomizationInstance::CreateStatic(
			&FBlueprintGlobalOptionsDetails::MakeInstanceForDiff,
			ForCapture
		);
		DetailsView->RegisterInstancedCustomPropertyLayout(UBlueprint::StaticClass(), LayoutOptionDetails);
	}
	// if InObject is a BP-CDO, add a "Parent Class" Combo button
	if (InObject && Cast<UObject>(InObject->GetClass())->IsA<UBlueprintGeneratedClass>() )
	{
		UBlueprintGeneratedClass* Class = Cast<UBlueprintGeneratedClass>(InObject->GetClass());
		if (Class->GetDefaultObject() == InObject)
		{
			const FOnGetDetailCustomizationInstance LayoutOptionDetails = FOnGetDetailCustomizationInstance::CreateStatic(
				&DOBPDetailsCustomization::Make,
				Cast<UBlueprint>(Class->ClassGeneratedBy)
			);
			DetailsView->RegisterInstancedCustomPropertyLayout(Class, LayoutOptionDetails);
		}
	}
	// Forcing all advanced properties to be displayed for now, the logic to show changes made to advance properties
	// conditionally is fragile and low priority for now:
	DetailsView->ShowAllAdvancedProperties();
	// This is a read only details view (see the property editing delegate), but const correctness at the details view
	// level would stress the type system a little:
	DetailsView->SetObject(const_cast<UObject*>(InObject));

	return DetailsView;
}

void FDetailsDiff::HighlightProperty(const FPropertySoftPath& PropertyName)
{
	// resolve the property soft path:
	const FPropertyPath ResolvedProperty = PropertyName.ResolvePath(DisplayedObject);
	DetailsView->HighlightProperty(ResolvedProperty);
}

TSharedRef< IDetailsView > FDetailsDiff::DetailsWidget() const
{
	return DetailsView.ToSharedRef();
}

TArray<FPropertySoftPath> FDetailsDiff::GetDisplayedProperties() const
{
	TArray<FPropertySoftPath> Ret;
	Algo::Copy(DisplayedProperties, Ret);
	return Ret;
}

void FDetailsDiff::DiffAgainst(const FDetailsDiff& Newer, TArray< FSingleObjectDiffEntry > &OutDifferences, bool bSortByDisplayOrder) const
{
	if (!DisplayedObject || !Newer.DisplayedObject)
	{
		return;
	}

	TSharedPtr< class IDetailsView > OldDetailsView = DetailsView;
	TSharedPtr< class IDetailsView > NewDetailsView = Newer.DetailsView;

	const TArray<TWeakObjectPtr<UObject>>& OldSelectedObjects = OldDetailsView->GetSelectedObjects();
	const TArray<TWeakObjectPtr<UObject>>& NewSelectedObjects = NewDetailsView->GetSelectedObjects();

	const TArray<FPropertyPath> OldProperties = OldDetailsView->GetPropertiesInOrderDisplayed();
	const TArray<FPropertyPath> NewProperties = NewDetailsView->GetPropertiesInOrderDisplayed();

	TSet<FPropertySoftPath> OldPropertiesSet;
	TSet<FPropertySoftPath> NewPropertiesSet;

	Algo::Transform(OldProperties, OldPropertiesSet, [](const FPropertyPath& Entry) { return FPropertySoftPath(Entry); });
	Algo::Transform(NewProperties, NewPropertiesSet, [](const FPropertyPath& Entry) { return FPropertySoftPath(Entry); });

	// detect removed properties:
	const TSet<FPropertySoftPath> RemovedProperties = OldPropertiesSet.Difference(NewPropertiesSet);
	for (const FPropertySoftPath& RemovedProperty : RemovedProperties)
	{
		// @todo: (doc) label these as removed, rather than added to a
		FSingleObjectDiffEntry Entry(RemovedProperty, EPropertyDiffType::PropertyAddedToA);
		OutDifferences.Push(Entry);
	}

	// detect added properties:
	const TSet<FPropertySoftPath> AddedProperties = NewPropertiesSet.Difference(OldPropertiesSet);
	for (const FPropertySoftPath& AddedProperty : AddedProperties)
	{
		FSingleObjectDiffEntry Entry(AddedProperty, EPropertyDiffType::PropertyAddedToB);
		OutDifferences.Push(Entry);
	}

	// check for changed properties
	const TSet<FPropertySoftPath> CommonProperties = NewPropertiesSet.Intersect(OldPropertiesSet);
	for (const FPropertySoftPath& CommonProperty : CommonProperties)
	{
		// get value, diff:
		check(NewSelectedObjects.Num() == 1);
		FResolvedProperty OldProperty = CommonProperty.Resolve(OldSelectedObjects[0].Get());
		FResolvedProperty NewProperty = CommonProperty.Resolve(NewSelectedObjects[0].Get());
		const UPackage* OldPackage = OldSelectedObjects[0]->GetPackage();
		const UPackage* NewPackage = NewSelectedObjects[0]->GetPackage();

		TArray<FPropertySoftPath> DifferingSubProperties;

		if (!DiffUtils::Identical(OldProperty, NewProperty, OldPackage, NewPackage, DiffUtils::FDiffParameters(CommonProperty), DifferingSubProperties))
		{
			for (const FPropertySoftPath& DifferingSubProperty : DifferingSubProperties)
			{
				OutDifferences.Push(FSingleObjectDiffEntry(DifferingSubProperty, EPropertyDiffType::PropertyValueChanged));
			}
		}
	}

	if (bSortByDisplayOrder)
	{
		TArray<FSingleObjectDiffEntry> AllDifferingProperties = OutDifferences;
		OutDifferences.Reset();

		// OrderedProperties will contain differences in the order they are displayed:
		TArray< const FSingleObjectDiffEntry* > OrderedProperties;

		// create differing properties list based on what is displayed by the old properties..
		TArray<FPropertySoftPath> SoftOldProperties = GetDisplayedProperties();
		TArray<FPropertySoftPath> SoftNewProperties = Newer.GetDisplayedProperties();

		const auto FindAndPushDiff = [&OrderedProperties, &AllDifferingProperties](const FPropertySoftPath& PropertyIdentifier) -> bool
		{
			bool bDiffers = false;
			for (const auto& Difference : AllDifferingProperties)
			{
				if (Difference.Identifier == PropertyIdentifier)
				{
					bDiffers = true;
					// if there are any nested differences associated with PropertyIdentifier, add those
					// as well:
					OrderedProperties.AddUnique(&Difference);
				}
				else if (Difference.Identifier.IsSubPropertyMatch(PropertyIdentifier))
				{
					bDiffers = true;
					OrderedProperties.AddUnique(&Difference);
				}
			}
			return bDiffers;
		};

		// zip the two sets of properties, zip iterators are not trivial to write in c++,
		// so this procedural stuff will have to do:
		int IterOld = 0;
		int IterNew = 0;
		while (IterOld < SoftOldProperties.Num() || IterNew < SoftNewProperties.Num())
		{
			const bool bOldIterValid = IterOld < SoftOldProperties.Num();
			const bool bNewIterValid = IterNew < SoftNewProperties.Num();

			// We've reached the end of the new list, but still have properties in the old list.
			// Continue over the old list to catch any remaining diffs.
			if (bOldIterValid && !bNewIterValid)
			{
				FindAndPushDiff(SoftOldProperties[IterOld]);
				++IterOld;
			}
			// We've reached the end of the old list, but still have properties in the new list.
			// Continue over the new list to catch any remaining diffs.
			else if (!bOldIterValid && bNewIterValid)
			{
				FindAndPushDiff(SoftNewProperties[IterNew]);
				++IterNew;
			}
			else
			{
				// If both properties have the same path, check to ensure the property hasn't changed.
				if (SoftOldProperties[IterOld] == SoftNewProperties[IterNew])
				{
					FindAndPushDiff(SoftOldProperties[IterOld]);
					++IterNew;
					++IterOld;
				}
				else
				{
					// If the old property is different, add it to the list and increment the old iter.
					// This indicates the property was removed.
					if (FindAndPushDiff(SoftOldProperties[IterOld]))
					{
						++IterOld;
					}
					// If the new property is different, add it to the list and increment the new iter.
					// This indicates the property was added.
					else if (FindAndPushDiff(SoftNewProperties[IterNew]))
					{
						++IterNew;
					}
					// Neither property was different.
					// This indicates the iterators were just out of step from a previous addition or removal.
					else
					{
						++IterOld;
						++IterNew;
					}
				}
			}
		}

		// Readd to OutDifferences
		for (const FSingleObjectDiffEntry* Difference : OrderedProperties)
		{
			OutDifferences.Add(*Difference);
		}
	}
}

void FDetailsDiff::LinkScrolling(FDetailsDiff& LeftPanel, FDetailsDiff& RightPanel,
	const TAttribute<TArray<FVector2f>>& ScrollRate)
{
	SLinkableScrollBar::LinkScrollBars(LeftPanel.ScrollBar.ToSharedRef(), RightPanel.ScrollBar.ToSharedRef(), ScrollRate);
}

#undef LOCTEXT_NAMESPACE
