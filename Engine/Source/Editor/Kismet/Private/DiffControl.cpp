// Copyright Epic Games, Inc. All Rights Reserved.

#include "DiffControl.h"
#include "Algo/Transform.h"
#include "DiffResults.h"
#include "GraphDiffControl.h"
#include "SBlueprintDiff.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "IDetailsView.h"
#include "ReviewComments.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Framework/Application/SlateApplication.h"
#include "AsyncTreeDifferences.h"
#include "DetailTreeNode.h"
#include "AsyncDetailViewDiff.h"

#define LOCTEXT_NAMESPACE "SBlueprintDif"

namespace UE::DiffControl
{
	KISMET_API const TArray<FReviewComment>*(*GGetReviewCommentsForFile)(const FString&) = nullptr;
	KISMET_API void(*GPostReviewComment)(FReviewComment&) = nullptr;
	KISMET_API void(*GEditReviewComment)(FReviewComment&) = nullptr;
	KISMET_API FString (*GGetReviewerUsername)() = nullptr;
	KISMET_API bool (*GIsFileInReview)(const FString& File) = nullptr;
	
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnCommentPosted, const FReviewComment&)
	KISMET_API FOnCommentPosted GOnCommentPosted;
}

// well known algorithm for computing the Longest Common Subsequence table of two ordered lists
template <typename RangeType, typename ComparePredicate>
static TArray<TArray<int32>> CalculateLCSTable(const RangeType& Range1, const RangeType& Range2, ComparePredicate Comparison)
{
	TArray<TArray<int32>> LCS;
	LCS.SetNum(Range1.Num() + 1);
	for (int32 I = 0; I <= Range1.Num(); I++)
	{
		LCS[I].SetNum(Range2.Num() + 1);
		if (I == 0)
		{
			continue;
		}
	
		for (int32 J = 1; J <= Range2.Num(); J++)
		{
			if (Comparison(Range1[I - 1], Range2[J - 1]))
			{
				LCS[I][J] = LCS[I - 1][J - 1] + 1;
			}
			else
			{
				LCS[I][J] = FMath::Max(LCS[I - 1][J], LCS[I][J - 1]);
			}
		}
	}
	return LCS;
}

/////////////////////////////////////////////////////////////////////////////
/// IDiffControl

FText IDiffControl::RightRevision = LOCTEXT("OlderRevisionIdentifier", "Right Revision");

TSharedRef<SWidget> IDiffControl::GenerateSimpleDiffWidget(FText DiffText)
{
	return SNew(STextBlock)
		.Text(DiffText)
		.ToolTipText(DiffText)
		.ColorAndOpacity(DiffViewUtils::Differs());
}

TSharedRef<SWidget> IDiffControl::GenerateObjectDiffWidget(FSingleObjectDiffEntry DiffEntry, FText ObjectName)
{
	return SNew(STextBlock)
		.Text(DiffViewUtils::PropertyDiffMessage(DiffEntry, ObjectName))
		.ToolTipText(DiffViewUtils::PropertyDiffMessage(DiffEntry, ObjectName))
		.ColorAndOpacity(DiffViewUtils::Differs());
}

void IDiffControl::EnableComments(TWeakPtr<STreeView<TSharedPtr<FBlueprintDifferenceTreeEntry>>> TreeView,
	const UObject* OldObject, const UObject* NewObject)
{
	const UPackage* Package = OldObject ? OldObject->GetPackage() : NewObject->GetPackage();
	const FPackagePath PackagePath = FPackagePath::FromPackageNameChecked(Package->GetName());
	ReviewCommentsDiffControl = MakeShared<FReviewCommentsDiffControl>(PackagePath.GetLocalFullPath(), TreeView);
}

void IDiffControl::EnableComments(TWeakPtr<STreeView<TSharedPtr<FBlueprintDifferenceTreeEntry>>> TreeView, const TArray<const UObject*>& Objects)
{
	for (const UObject* Object : Objects)
	{
		if (!ensure(Object))
		{
			continue;
		}
		if (const UPackage* Package = Object->GetPackage())
		{
			const FPackagePath PackagePath = FPackagePath::FromPackageNameChecked(Package->GetName());
			ReviewCommentsDiffControl = MakeShared<FReviewCommentsDiffControl>(PackagePath.GetLocalFullPath(), TreeView);
			return;
		}
	}
}

void IDiffControl::GenerateCategoryCommentTreeEntries(TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& OutChildrenEntries,
                                                      TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& OutRealDifferences, FString CategoryKey)
{
	if (ReviewCommentsDiffControl)
	{
		ReviewCommentsDiffControl->SetCategory(CategoryKey);
		ReviewCommentsDiffControl->GenerateTreeEntries(OutChildrenEntries, OutRealDifferences);
	}
}


/////////////////////////////////////////////////////////////////////////////
/// FMyBlueprintDiffControl

FMyBlueprintDiffControl::FMyBlueprintDiffControl(const UBlueprint* InOldBlueprint, const UBlueprint* InNewBlueprint, FOnDiffEntryFocused InSelectionCallback)
	: SelectionCallback(MoveTemp(InSelectionCallback))
	, OldBlueprint(InOldBlueprint)
	, NewBlueprint(InNewBlueprint)
{}

void FMyBlueprintDiffControl::GenerateTreeEntries(TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& OutTreeEntries, TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& OutRealDifferences)
{
	TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> > Children;

	if (OldBlueprint && OldBlueprint->SkeletonGeneratedClass && NewBlueprint && NewBlueprint->SkeletonGeneratedClass)
	{
		for (TFieldIterator<FProperty> PropertyIt(OldBlueprint->SkeletonGeneratedClass); PropertyIt; ++PropertyIt)
		{
			FProperty* OldProperty = *PropertyIt;
			FProperty* NewProperty = NewBlueprint->SkeletonGeneratedClass->FindPropertyByName(OldProperty->GetFName());

			FText PropertyText = FText::FromString(OldProperty->GetAuthoredName());

			if (NewProperty)
			{
				const int32 OldVarIndex = FBlueprintEditorUtils::FindNewVariableIndex(OldBlueprint, OldProperty->GetFName());
				const int32 NewVarIndex = FBlueprintEditorUtils::FindNewVariableIndex(NewBlueprint, OldProperty->GetFName());

				if (OldVarIndex != INDEX_NONE && NewVarIndex != INDEX_NONE)
				{
					TArray<FSingleObjectDiffEntry> DifferingProperties;
					static const UStruct* Struct = FBPVariableDescription::StaticStruct();
					const void* OldVal = &OldBlueprint->NewVariables[OldVarIndex];
					const void* NewVal = &NewBlueprint->NewVariables[NewVarIndex];
					const UPackage* OldPackage = OldBlueprint->GetPackage();
					const UPackage* NewPackage = NewBlueprint->GetPackage();
					DiffUtils::CompareUnrelatedStructs(Struct, OldVal, OldPackage, Struct, NewVal, NewPackage, DifferingProperties);
					for (const FSingleObjectDiffEntry& Difference : DifferingProperties)
					{
						TSharedPtr<FBlueprintDifferenceTreeEntry> Entry = MakeShared<FBlueprintDifferenceTreeEntry>(
							SelectionCallback,
							FGenerateDiffEntryWidget::CreateStatic(&GenerateObjectDiffWidget, Difference, PropertyText));
						Children.Push(Entry);
						OutRealDifferences.Push(Entry);
					}
				}	
			}
			else
			{
				FText DiffText = FText::Format(LOCTEXT("VariableRemoved", "Removed Variable {0}"), PropertyText);

				TSharedPtr<FBlueprintDifferenceTreeEntry> Entry = MakeShared<FBlueprintDifferenceTreeEntry>(
					SelectionCallback,
					FGenerateDiffEntryWidget::CreateStatic(&GenerateSimpleDiffWidget, DiffText));

				Children.Push(Entry);
				OutRealDifferences.Push(Entry);
			}
		}

		for (TFieldIterator<FProperty> PropertyIt(NewBlueprint->SkeletonGeneratedClass); PropertyIt; ++PropertyIt)
		{
			FProperty* NewProperty = *PropertyIt;
			FProperty* OldProperty = OldBlueprint->SkeletonGeneratedClass->FindPropertyByName(NewProperty->GetFName());

			if (!OldProperty)
			{
				FText DiffText = FText::Format(LOCTEXT("VariableAdded", "Added Variable {0}"), FText::FromString(NewProperty->GetAuthoredName()));

				TSharedPtr<FBlueprintDifferenceTreeEntry> Entry = MakeShared<FBlueprintDifferenceTreeEntry>(
					SelectionCallback,
					FGenerateDiffEntryWidget::CreateStatic(&GenerateSimpleDiffWidget, DiffText));

				Children.Push(Entry);
				OutRealDifferences.Push(Entry);
			}
		}
	}
	const bool bHasDifferences = Children.Num() != 0;
	if (!bHasDifferences)
	{
		// make one child informing the user that there are no differences:
		Children.Push(FBlueprintDifferenceTreeEntry::NoDifferencesEntry());
	}

	static const FText MyBlueprintLabel = NSLOCTEXT("FBlueprintDifferenceTreeEntry", "MyBlueprintLabel", "My Blueprint");
	static const FText MyBlueprintTooltip = NSLOCTEXT("FBlueprintDifferenceTreeEntry", "MyBlueprintTooltip", "The list of changes made to blueprint structure in the My Blueprint panel");
	OutTreeEntries.Push(FBlueprintDifferenceTreeEntry::CreateCategoryEntry(
		MyBlueprintLabel,
		MyBlueprintTooltip,
		SelectionCallback,
		Children,
		bHasDifferences
	));

	// add comments as children to this category
	GenerateCategoryCommentTreeEntries(OutTreeEntries.Last()->Children, OutRealDifferences, TEXT("My Blueprint"));
	
}

void FMyBlueprintDiffControl::EnableComments(TWeakPtr<STreeView<TSharedPtr<FBlueprintDifferenceTreeEntry>>> TreeView)
{
	IDiffControl::EnableComments(TreeView, OldBlueprint, NewBlueprint);
}


/////////////////////////////////////////////////////////////////////////////
/// FSCSDiffControl

FSCSDiffControl::FSCSDiffControl(const UBlueprint* InOldBlueprint, const UBlueprint* InNewBlueprint, FOnDiffEntryFocused InSelectionCallback)
	: SelectionCallback(InSelectionCallback)
	, OldSCS(InOldBlueprint)
	, NewSCS(InNewBlueprint)
{
}

void FSCSDiffControl::GenerateTreeEntries(TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& OutTreeEntries, TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& OutRealDifferences)
{
	TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>> Children;
	if (OldSCS.GetBlueprint() && NewSCS.GetBlueprint())
	{
		const TArray< FSCSResolvedIdentifier > OldHierarchy = OldSCS.GetDisplayedHierarchy();
		const TArray< FSCSResolvedIdentifier > NewHierarchy = NewSCS.GetDisplayedHierarchy();
		DiffUtils::CompareUnrelatedSCS(OldSCS.GetBlueprint(), OldHierarchy, NewSCS.GetBlueprint(), NewHierarchy, DifferingProperties);

		const auto FocusSCSDifferenceEntry = [](FSCSDiffEntry Entry, FOnDiffEntryFocused InSelectionCallback, FSCSDiffControl* Owner)
		{
			InSelectionCallback.ExecuteIfBound();
			if (Entry.TreeIdentifier.Name != NAME_None)
			{
				Owner->OldSCS.HighlightProperty(Entry.TreeIdentifier.Name, FPropertyPath());
				Owner->NewSCS.HighlightProperty(Entry.TreeIdentifier.Name, FPropertyPath());
			}
		};

		const auto CreateSCSDifferenceWidget = [](FSCSDiffEntry Entry, FText ObjectName) -> TSharedRef<SWidget>
		{
			return SNew(STextBlock)
					.Text(DiffViewUtils::SCSDiffMessage(Entry, ObjectName))
					.ColorAndOpacity(Entry.DiffType == ETreeDiffType::NODE_CORRUPTED ?
						DiffViewUtils::Conflicting() :
						DiffViewUtils::Differs()
						);
		};

		for (const FSCSDiffEntry& Difference : DifferingProperties.Entries)
		{
			TSharedPtr<FBlueprintDifferenceTreeEntry> Entry = MakeShared<FBlueprintDifferenceTreeEntry>(
				FOnDiffEntryFocused::CreateStatic(FocusSCSDifferenceEntry, Difference, SelectionCallback, this),
				FGenerateDiffEntryWidget::CreateStatic(CreateSCSDifferenceWidget, Difference, RightRevision));
			Children.Push(Entry);
			OutRealDifferences.Push(Entry);
		}
	}
	const bool bHasDifferences = Children.Num() != 0;
	if (!bHasDifferences)
	{
		// make one child informing the user that there are no differences:
		Children.Push(FBlueprintDifferenceTreeEntry::NoDifferencesEntry());
	}

	static const FText SCSLabel = NSLOCTEXT("FBlueprintDifferenceTreeEntry", "SCSLabel", "Components");
	static const FText SCSTooltip = NSLOCTEXT("FBlueprintDifferenceTreeEntry", "SCSTooltip", "The list of changes made in the Components panel");
	OutTreeEntries.Push(FBlueprintDifferenceTreeEntry::CreateCategoryEntry(
		SCSLabel,
		SCSTooltip,
		SelectionCallback,
		Children,
		bHasDifferences
	));

	// add comments as children to this category
	GenerateCategoryCommentTreeEntries(OutTreeEntries.Last()->Children, OutRealDifferences, TEXT("Components"));
}

void FSCSDiffControl::EnableComments(TWeakPtr<STreeView<TSharedPtr<FBlueprintDifferenceTreeEntry>>> TreeView)
{
	IDiffControl::EnableComments(TreeView, OldSCS.GetBlueprint(), NewSCS.GetBlueprint());
}

FDetailsDiffControl::FDetailsDiffControl(const UObject* InOldObject, const UObject* InNewObject,
	FOnDiffEntryFocused InSelectionCallback, bool bPopulateOutTreeEntries)
	: SelectionCallback(InSelectionCallback)
	, bPopulateOutTreeEntries(bPopulateOutTreeEntries)
{
	if (InOldObject)
	{
		InsertObject(InOldObject, true);
	}
	if (InNewObject)
	{
		InsertObject(InNewObject, false);
	}
}

FDetailsDiffControl::~FDetailsDiffControl()
{
	for(const TPair<const UObject*, FDetailsDiff>& Pair : DetailsDiffs)
	{
		Pair.Value.DetailsWidget()->SetOnDisplayedPropertiesChanged(FOnDisplayedPropertiesChanged());
	}
}

void FDetailsDiffControl::Tick()
{
	for (auto& [Object, PropertyTreeDiffs] : PropertyTreeDifferences)
	{
		if (PropertyTreeDiffs.Left)
		{
			constexpr float MaxTickTimeMs = 0.01f;
			PropertyTreeDiffs.Left->Tick(MaxTickTimeMs);
		}
	}
}

void FDetailsDiffControl::GenerateTreeEntries(TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& OutTreeEntries, TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& OutRealDifferences)
{
	GenerateTreeEntriesWithoutComments(OutTreeEntries, OutRealDifferences);
	
	// add comments
	GenerateCategoryCommentTreeEntries(OutTreeEntries, OutRealDifferences, TEXT("Details"));
}

void FDetailsDiffControl::GenerateTreeEntriesWithoutComments(
	TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& OutTreeEntries,
	TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& OutRealDifferences)
{
	TArray<FSingleObjectDiffEntry> DifferingProperties;
	
	for (int32 LeftIndex = 0; LeftIndex < ObjectDisplayOrder.Num() - 1; ++LeftIndex)
	{
		const UObject* LeftObject = ObjectDisplayOrder[LeftIndex];
		if (!ensure(LeftObject))
		{
			continue;
		}
		const TSharedPtr<FAsyncDetailViewDiff> Diff = PropertyTreeDifferences[LeftObject].Right;
		Diff->FlushQueue(); // make sure differences are fully up to date
		Diff->GetPropertyDifferences(DifferingProperties);
	}

	for (auto&[Object, DetailsDiff] : DetailsDiffs)
	{
		Algo::Transform(DifferingProperties, PropertyAllowList,
		[&InObject = Object](const FSingleObjectDiffEntry& DiffEntry)
		{
			return DiffEntry.Identifier.ResolvePath(InObject);
		});

		DetailsDiff.DetailsWidget()->UpdatePropertyAllowList(PropertyAllowList);
	}

	if (GenerateCustomEntriesCallback.IsBound())
	{
		TArray<FSingleObjectDiffEntry> CustomEntries;
		GenerateCustomEntriesCallback.Execute(CustomEntries);
		DifferingProperties.Append(CustomEntries);
	}

	auto GenerateDiffEntry = [&](const FSingleObjectDiffEntry& Difference)
	{
		return MakeShared<FBlueprintDifferenceTreeEntry>(
			FOnDiffEntryFocused::CreateSP(TSharedFromThis<FDetailsDiffControl>::AsShared(), &FDetailsDiffControl::OnSelectDiffEntry, Difference.Identifier),
			FGenerateDiffEntryWidget::CreateStatic(&GenerateObjectDiffWidget, Difference, RightRevision));
	};

	if (OrganizeEntriesCallback.IsBound())
	{
		auto GenerateCategoryEntry = [&](FText& CategoryName)
		{
			return MakeShared<FBlueprintDifferenceTreeEntry>(
				/*InOnFocus*/nullptr,
				FGenerateDiffEntryWidget::CreateStatic(&GenerateSimpleDiffWidget, CategoryName));
		};

		TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>> Entries;
		OrganizeEntriesCallback.Execute(Entries, DifferingProperties, GenerateDiffEntry, GenerateCategoryEntry);

		for (const TSharedPtr<FBlueprintDifferenceTreeEntry>& Entry : Entries)
		{
			Children.Push(Entry);
			OutRealDifferences.Push(Entry);
			if (bPopulateOutTreeEntries)
			{
				OutTreeEntries.Push(Entry);
			}
		}
	}
	else
	{
		for (const FSingleObjectDiffEntry& Difference : DifferingProperties)
		{
			TSharedPtr<FBlueprintDifferenceTreeEntry> Entry = GenerateDiffEntry(Difference);
			Children.Push(Entry);
			OutRealDifferences.Push(Entry);
			if (bPopulateOutTreeEntries)
			{
				OutTreeEntries.Push(Entry);
			}
		}
	}
}

void FDetailsDiffControl::EnableComments(TWeakPtr<STreeView<TSharedPtr<FBlueprintDifferenceTreeEntry>>> TreeView)
{
	IDiffControl::EnableComments(TreeView, ObjectDisplayOrder);
}

TSharedRef<IDetailsView> FDetailsDiffControl::InsertObject(const UObject* Object, bool bScrollbarOnLeft, int32 Index)
{
	FDetailsDiff DetailsDiff(Object, bScrollbarOnLeft);
	const TSharedRef<IDetailsView> DetailsView = DetailsDiff.DetailsWidget();
	DetailsView->UpdatePropertyAllowList(PropertyAllowList);
	// hook details view so we can recalculate diffs if/when a customization
	// triggers a refresh:
	DetailsView->SetOnDisplayedPropertiesChanged( ::FOnDisplayedPropertiesChanged::CreateRaw(this, &FDetailsDiffControl::HandlePropertiesChanged) );

	if (Index == INDEX_NONE)
	{
		Index = ObjectDisplayOrder.Num();
	}
	
	DetailsDiffs.Add(Object, DetailsDiff);
	ObjectDisplayOrder.Insert(Object, Index);
	PropertyTreeDifferences.Add(Object, {});
	
	// set up interaction with left panel
	if (ObjectDisplayOrder.IsValidIndex(Index - 1))
	{
		const UObject* OtherObject = ObjectDisplayOrder[Index - 1];
		FDetailsDiff& OtherDetailsDiff = DetailsDiffs[OtherObject];
		const TSharedRef<IDetailsView> OtherDetailsView = OtherDetailsDiff.DetailsWidget();
		const auto ScrollRate = GetLinkedScrollRateAttribute(OtherDetailsView, DetailsView);
		FDetailsDiff::LinkScrolling(OtherDetailsDiff, DetailsDiff, ScrollRate);

		PropertyTreeDifferences[OtherObject].Right = MakeShared<FAsyncDetailViewDiff>(OtherDetailsView, DetailsView);
		PropertyTreeDifferences[Object].Left = PropertyTreeDifferences[OtherObject].Right;
	}
	// Set up interaction with right panel
	if (ObjectDisplayOrder.IsValidIndex(Index + 1))
	{
		const UObject* OtherObject = ObjectDisplayOrder[Index + 1];
		FDetailsDiff& OtherDetailsDiff = DetailsDiffs[OtherObject];
		const TSharedRef<IDetailsView> OtherDetailsView = OtherDetailsDiff.DetailsWidget();
		const auto ScrollRate = GetLinkedScrollRateAttribute(DetailsView, OtherDetailsView);
		FDetailsDiff::LinkScrolling(DetailsDiff, OtherDetailsDiff, ScrollRate);

		PropertyTreeDifferences[OtherObject].Left = MakeShared<FAsyncDetailViewDiff>(DetailsView, OtherDetailsView);
		PropertyTreeDifferences[Object].Right = PropertyTreeDifferences[OtherObject].Left;
	}

	return DetailsView;
}

TSharedRef<IDetailsView> FDetailsDiffControl::GetDetailsWidget(const UObject* Object) const
{
	return DetailsDiffs[Object].DetailsWidget();
}

TSharedPtr<IDetailsView> FDetailsDiffControl::TryGetDetailsWidget(const UObject* Object) const
{
	if (const FDetailsDiff* Found = DetailsDiffs.Find(Object))
	{
		return Found->DetailsWidget();
	}
	return nullptr;
}

TSharedPtr<FAsyncDetailViewDiff> FDetailsDiffControl::GetDifferencesWithLeft(const UObject* Object) const
{
	return PropertyTreeDifferences[Object].Left;
}

TSharedPtr<FAsyncDetailViewDiff> FDetailsDiffControl::GetDifferencesWithRight(const UObject* Object) const
{
	return PropertyTreeDifferences[Object].Right;
}

int32 FDetailsDiffControl::IndexOfObject(const UObject* Object) const
{
	return ObjectDisplayOrder.IndexOfByKey(Object);
}

void FDetailsDiffControl::OnSelectDiffEntry(FPropertySoftPath PropertyName)
{
	SelectionCallback.ExecuteIfBound();
	for (auto&[Object, DetailsDiff] : DetailsDiffs)
	{
		DetailsDiff.HighlightProperty(PropertyName);
	}
}

TAttribute<TArray<FVector2f>> FDetailsDiffControl::GetLinkedScrollRateAttribute(const TSharedRef<IDetailsView>& OldDetailsView, const TSharedRef<IDetailsView>& NewDetailsView)
{
	return TAttribute<TArray<FVector2f>>::CreateRaw(this, &FDetailsDiffControl::GetLinkedScrollRate, OldDetailsView, NewDetailsView);
}

TArray<FVector2f> FDetailsDiffControl::GetLinkedScrollRate(TSharedRef<IDetailsView> LeftDetailsView, TSharedRef<IDetailsView> RightDetailsView) const
{
	const UObject* LeftObject = LeftDetailsView->GetSelectedObjects()[0].Get();
	return PropertyTreeDifferences[LeftObject].Right->GenerateScrollSyncRate();
}

void FDetailsDiffControl::HandlePropertiesChanged()
{
	// Set up interaction with left panel
	if(ObjectDisplayOrder.Num() != 2)
	{
		return;
	}

	const UObject* Object = ObjectDisplayOrder[0];
	const UObject* OtherObject = ObjectDisplayOrder[1];

	if(!Object || !OtherObject)
	{
		return; // should have any FAsyncDetailViewDiff to refresh
	}

	const TSharedRef<IDetailsView> DetailsView = GetDetailsWidget(Object);
	const TSharedRef<IDetailsView> OtherDetailsView = GetDetailsWidget(OtherObject);
	PropertyTreeDifferences[OtherObject].Left = MakeShared<FAsyncDetailViewDiff>(DetailsView, OtherDetailsView);
	PropertyTreeDifferences[Object].Right = PropertyTreeDifferences[OtherObject].Left;
}


/////////////////////////////////////////////////////////////////////////////
/// FCDODiffControl

FCDODiffControl::FCDODiffControl(const UObject* InOldObject, const UObject* InNewObject, FOnDiffEntryFocused InSelectionCallback)
	: FDetailsDiffControl(InOldObject, InNewObject, InSelectionCallback, false)
{
}

void FCDODiffControl::GenerateTreeEntries(TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& OutTreeEntries, TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& OutRealDifferences)
{
	FDetailsDiffControl::GenerateTreeEntriesWithoutComments(OutTreeEntries, OutRealDifferences);

	const bool bHasDifferences = Children.Num() != 0;
	if (!bHasDifferences)
	{
		// make one child informing the user that there are no differences:
		Children.Push(FBlueprintDifferenceTreeEntry::NoDifferencesEntry());
	}

	static const FText DefaultsLabel = NSLOCTEXT("FBlueprintDifferenceTreeEntry", "DefaultsLabel", "Defaults");
	static const FText DefaultsTooltip = NSLOCTEXT("FBlueprintDifferenceTreeEntry", "DefaultsTooltip", "The list of changes made in the Defaults panel");
	OutTreeEntries.Push(FBlueprintDifferenceTreeEntry::CreateCategoryEntry(
		DefaultsLabel,
		DefaultsTooltip,
		SelectionCallback,
		Children,
		bHasDifferences
	));

	// add comments as children to this category
	GenerateCategoryCommentTreeEntries(OutTreeEntries.Last()->Children, OutRealDifferences, TEXT("Defaults"));
	Children = OutTreeEntries.Last()->Children;
}

/////////////////////////////////////////////////////////////////////////////
/// FClassSettingsDiffControl

FClassSettingsDiffControl::FClassSettingsDiffControl(const UObject* InOldObject, const UObject* InNewObject, FOnDiffEntryFocused InSelectionCallback)
	: FDetailsDiffControl(InOldObject, InNewObject, InSelectionCallback, false)
{
}

void FClassSettingsDiffControl::GenerateTreeEntries(TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& OutTreeEntries, TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& OutRealDifferences)
{
	FDetailsDiffControl::GenerateTreeEntriesWithoutComments(OutTreeEntries, OutRealDifferences);

	const bool bHasDifferences = Children.Num() != 0;
	if (!bHasDifferences)
	{
		// make one child informing the user that there are no differences:
		Children.Push(FBlueprintDifferenceTreeEntry::NoDifferencesEntry());
	}

	static const FText SettingsLabel = NSLOCTEXT("FBlueprintDifferenceTreeEntry", "SettingsLabel", "Class Settings");
	static const FText SettingsTooltip = NSLOCTEXT("FBlueprintDifferenceTreeEntry", "SettingsTooltip", "The list of changes made in the Class Settings panel");
	OutTreeEntries.Push(FBlueprintDifferenceTreeEntry::CreateCategoryEntry(
		SettingsLabel,
		SettingsTooltip,
		SelectionCallback,
		Children,
		bHasDifferences
		));

	// add comments as children to this category
	GenerateCategoryCommentTreeEntries(OutTreeEntries.Last()->Children, OutRealDifferences, TEXT("Class Settings"));
	Children = OutTreeEntries.Last()->Children;
}


/////////////////////////////////////////////////////////////////////////////
/// FBlueprintTypeDiffControl

FBlueprintTypeDiffControl::FBlueprintTypeDiffControl(const UBlueprint* InBlueprintOld, const UBlueprint* InBlueprintNew, FOnDiffEntryFocused InSelectionCallback)
	: BlueprintOld(InBlueprintOld)
	, BlueprintNew(InBlueprintNew)
	, SelectionCallback(InSelectionCallback)
	, bDiffSucceeded(false)
{
	check(InBlueprintNew || InBlueprintOld);
}

void FBlueprintTypeDiffControl::GenerateTreeEntries(TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutTreeEntries, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutRealDifferences)
{
	BuildDiffSourceArray();

	TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> > Children;
	
	bool bHasRealChange = false;

	// First add manual diffs in main category
	for (const TSharedPtr<FDiffResultItem>& Difference : DiffListSource)
	{
		TSharedPtr<FBlueprintDifferenceTreeEntry> ChildEntry = MakeShared<FBlueprintDifferenceTreeEntry>(
			SelectionCallback,
			FGenerateDiffEntryWidget::CreateSP(Difference.ToSharedRef(), &FDiffResultItem::GenerateWidget));
		Children.Push(ChildEntry);
		OutRealDifferences.Push(ChildEntry);

		if (Difference->Result.IsRealDifference())
		{
			bHasRealChange = true;
		}
	}

	if (Children.Num() == 0)
	{
		// Make one child informing the user that there are no differences, or that it is unknown
		if (bDiffSucceeded)
		{
			Children.Push(FBlueprintDifferenceTreeEntry::NoDifferencesEntry());
		}
		else
		{
			Children.Push(FBlueprintDifferenceTreeEntry::UnknownDifferencesEntry());
		}
	}

	TSharedPtr<FBlueprintDifferenceTreeEntry> CategoryEntry = MakeShared<FBlueprintDifferenceTreeEntry>(
			SelectionCallback,
			FGenerateDiffEntryWidget::CreateSP(AsShared(), &FBlueprintTypeDiffControl::GenerateCategoryWidget, bHasRealChange),
			Children);
	OutTreeEntries.Push(CategoryEntry);

	// add comments as children to this category
	const FString CommentCategoryKey = (BlueprintNew ? BlueprintNew : BlueprintOld)->GetClass()->GetName();
	GenerateCategoryCommentTreeEntries(OutTreeEntries.Last()->Children, OutRealDifferences, CommentCategoryKey);

	// Now add subobject diffs, one category per object
	for (const TSharedPtr<FSubObjectDiff>& SubObjectDiff : SubObjectDiffs)
	{
		Children.Reset();

		Children.Append(SubObjectDiff->Diffs);
		OutRealDifferences.Append(SubObjectDiff->Diffs);

		TSharedPtr<FBlueprintDifferenceTreeEntry> SubObjectEntry = FBlueprintDifferenceTreeEntry::CreateCategoryEntry(
			SubObjectDiff->SourceResult.DisplayString,
			SubObjectDiff->SourceResult.ToolTip,
			FOnDiffEntryFocused::CreateSP(AsShared(), &FBlueprintTypeDiffControl::OnSelectSubobjectDiff, FPropertySoftPath(), SubObjectDiff),
			Children,
			true);
		OutTreeEntries.Push(SubObjectEntry);

		// add comments as children to this category
		GenerateCategoryCommentTreeEntries(OutTreeEntries.Last()->Children, OutRealDifferences, SubObjectDiff->SourceResult.OwningObjectPath);
	}

	// Now add customobject diffs, one category per object
	for (const TSharedPtr<FDiffCustomObject>& DiffCustomObject : DiffCustomObjects)
	{
		Children.Reset();

		Children.Append(DiffCustomObject->Diffs);
		OutRealDifferences.Append(DiffCustomObject->Diffs);

		TSharedPtr<FBlueprintDifferenceTreeEntry> CustomObjectEntry = FBlueprintDifferenceTreeEntry::CreateCategoryEntry(
			DiffCustomObject->SourceResult.DisplayString,
			DiffCustomObject->SourceResult.ToolTip,
			FOnDiffEntryFocused::CreateSP(AsShared(), &FBlueprintTypeDiffControl::OnSelectDiffCustomObject, TSharedPtr<IDiffCustomProperty>(), DiffCustomObject),
			Children,
			true);
		OutTreeEntries.Push(CustomObjectEntry);
	}
}

void FBlueprintTypeDiffControl::EnableComments(TWeakPtr<STreeView<TSharedPtr<FBlueprintDifferenceTreeEntry>>> TreeView)
{
	IDiffControl::EnableComments(TreeView, BlueprintOld, BlueprintNew);
}

TSharedRef<SWidget> FBlueprintTypeDiffControl::GenerateCategoryWidget(bool bHasRealDiffs)
{
	FLinearColor Color = FLinearColor::White;

	if (bHasRealDiffs)
	{
		Color = DiffViewUtils::Differs();
	}

	const FText Label = (BlueprintNew ? BlueprintNew : BlueprintOld)->GetClass()->GetDisplayNameText();

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			SNew(STextBlock)
			.ColorAndOpacity(Color)
		.Text(Label)
		];
}

void FBlueprintTypeDiffControl::BuildDiffSourceArray()
{
	if (!BlueprintNew || !BlueprintOld)
	{
		bDiffSucceeded = true;
		return;
	}
	
	TArray<FDiffSingleResult> BlueprintDiffResults;
	FDiffResults BlueprintDiffs(&BlueprintDiffResults);
	if (BlueprintNew->FindDiffs(BlueprintOld, BlueprintDiffs))
	{
		bDiffSucceeded = true;

		// Add manual diffs
		for (const FDiffSingleResult& CurrentDiff : BlueprintDiffResults)
		{
			if (CurrentDiff.Diff == EDiffType::OBJECT_REQUEST_DIFF)
			{
				// Turn into a subobject diff

				// Invert order, we want old then new
				TSharedPtr<FSubObjectDiff> SubObjectDiff = MakeShared<FSubObjectDiff>(CurrentDiff, CurrentDiff.Object2, CurrentDiff.Object1);

				TArray<FSingleObjectDiffEntry> DifferingProperties;
				SubObjectDiff->OldDetails.DiffAgainst(SubObjectDiff->NewDetails, DifferingProperties, true);

				if (DifferingProperties.Num() > 0)
				{
					// Actual differences, so add to tree
					SubObjectDiffs.Add(SubObjectDiff);

					for (const FSingleObjectDiffEntry& Difference : DifferingProperties)
					{
						TSharedPtr<FBlueprintDifferenceTreeEntry> Entry = MakeShared<FBlueprintDifferenceTreeEntry>(
							FOnDiffEntryFocused::CreateSP(AsShared(), &FBlueprintTypeDiffControl::OnSelectSubobjectDiff, Difference.Identifier, SubObjectDiff),
							FGenerateDiffEntryWidget::CreateStatic(&GenerateObjectDiffWidget, Difference, RightRevision));
						SubObjectDiff->Diffs.Push(Entry);
					}
				}
			}
			else if (CurrentDiff.Diff == EDiffType::CUSTOM_OBJECT)
			{
				// If no custom object, do not add to DiffListSource
				if (!CurrentDiff.CustomObject.IsValid())
				{
					continue;
				}
				
				// Invert order, we want old then new
				TSharedPtr<FDiffCustomObject> DiffCustomObject = MakeShared<FDiffCustomObject>(CurrentDiff, CurrentDiff.CustomObject);

				// Gather associated custom property diffs for the object
				for (const FDiffSingleResult& OtherDiff : BlueprintDiffResults)
				{
					// If the property diff is not associated with the custom object diff, ignore
					if (OtherDiff.Object1 != CurrentDiff.Object1 || OtherDiff.Object2 != CurrentDiff.Object2)
					{
						continue;
					}

					if (OtherDiff.CustomProperty.IsValid())
					{
						FGenerateDiffEntryWidget CreateEntryWidget = FGenerateDiffEntryWidget::CreateSPLambda(this, [this](const FDiffSingleResult& Diff) {
							return SNew(STextBlock)
							.Text(Diff.DisplayString)
							.ToolTipText(Diff.ToolTip)
							.ColorAndOpacity(Diff.GetDisplayColor());
						}, OtherDiff);

						TSharedPtr<FBlueprintDifferenceTreeEntry> Entry = MakeShared<FBlueprintDifferenceTreeEntry>(
							FOnDiffEntryFocused::CreateSP(AsShared(), &FBlueprintTypeDiffControl::OnSelectDiffCustomObject, OtherDiff.CustomProperty, DiffCustomObject),
							CreateEntryWidget);
						DiffCustomObject->Diffs.Push(Entry);
					}
				}

				if(!DiffCustomObject->Diffs.IsEmpty())
				{
					DiffCustomObjects.Add(DiffCustomObject);
				}
			}
			else
			{
				DiffListSource.Add(MakeShared<FDiffResultItem>(CurrentDiff));
			}
		}

		Algo::SortBy(DiffListSource, [](const TSharedPtr<FDiffResultItem>& Data) { return Data->Result.Diff; });
	}
}

void FBlueprintTypeDiffControl::OnSelectSubobjectDiff(FPropertySoftPath Identifier, TSharedPtr<FSubObjectDiff> SubObjectDiff)
{
	// This allows the owning control to focus the correct tab (or do whatever else it likes):
	SelectionCallback.ExecuteIfBound();

	if (SubObjectDiff.IsValid())
	{
		SubObjectDiff->OldDetails.HighlightProperty(Identifier);
		SubObjectDiff->NewDetails.HighlightProperty(Identifier);

		OldDetailsBox->SetContent(SubObjectDiff->OldDetails.DetailsWidget());
		NewDetailsBox->SetContent(SubObjectDiff->NewDetails.DetailsWidget());
	}
}

void FBlueprintTypeDiffControl::OnSelectDiffCustomObject(TSharedPtr<IDiffCustomProperty> CustomProperty, TSharedPtr<FDiffCustomObject> DiffCustomObject)
{
	// This allows the owning control to focus the correct tab (or do whatever else it likes):
	SelectionCallback.ExecuteIfBound();

	if (DiffCustomObject.IsValid())
	{
		if (TSharedPtr<IDiffCustomObject> DiffWidget = DiffCustomObject->DiffWidget)
		{
			if (CustomProperty.IsValid())
			{
				CustomProperty->HighlightProperty(DiffWidget);
			}

			if (DiffWidget->GetOldWidget().IsValid())
			{
				OldDetailsBox->SetContent(DiffWidget->GetOldWidget().ToSharedRef());
			}
			if (DiffWidget->GetNewWidget().IsValid())
			{
				NewDetailsBox->SetContent(DiffWidget->GetNewWidget().ToSharedRef());
			}
		}
		else
		{
			OldDetailsBox->SetContent(SNullWidget::NullWidget);
			NewDetailsBox->SetContent(SNullWidget::NullWidget);
		}
	}
}

/////////////////////////////////////////////////////////////////////////////
/// FGraphToDiff

FGraphToDiff::FGraphToDiff(SBlueprintDiff* InDiffWidget, UEdGraph* InGraphOld, UEdGraph* InGraphNew, const FRevisionInfo& InRevisionOld, const FRevisionInfo& InRevisionNew)
	: FoundDiffs(MakeShared<TArray<FDiffSingleResult>>()), DiffWidget(InDiffWidget), GraphOld(InGraphOld), GraphNew(InGraphNew), RevisionOld(InRevisionOld), RevisionNew(InRevisionNew)
{
	check(InGraphOld || InGraphNew); //one of them needs to exist

	//need to know when it is modified
	if (InGraphNew)
	{
		OnGraphChangedDelegateHandle = InGraphNew->AddOnGraphChangedHandler( FOnGraphChanged::FDelegate::CreateRaw(this, &FGraphToDiff::OnGraphChanged));
	}

	BuildDiffSourceArray();
}

FGraphToDiff::~FGraphToDiff()
{
	if (GraphNew)
	{
		GraphNew->RemoveOnGraphChangedHandler( OnGraphChangedDelegateHandle);
	}
}

void FGraphToDiff::GenerateTreeEntries(TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutTreeEntries, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& OutRealDifferences)
{
	if (!DiffListSource.IsEmpty())
	{
		RealDifferencesStartIndex = OutRealDifferences.Num();
	}
	
	TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> > Children;
	for (const TSharedPtr<FDiffResultItem>& Difference : DiffListSource)
	{
		TSharedPtr<FBlueprintDifferenceTreeEntry> ChildEntry = MakeShared<FBlueprintDifferenceTreeEntry>(
				FOnDiffEntryFocused::CreateRaw(DiffWidget, &SBlueprintDiff::OnDiffListSelectionChanged, Difference),
				FGenerateDiffEntryWidget::CreateSP(Difference.ToSharedRef(), &FDiffResultItem::GenerateWidget));
		Children.Push(ChildEntry);
		OutRealDifferences.Push(ChildEntry);
	}

	if (Children.Num() == 0)
	{
		// make one child informing the user that there are no differences:
		Children.Push(FBlueprintDifferenceTreeEntry::NoDifferencesEntry());
	}

	TSharedPtr<FBlueprintDifferenceTreeEntry> Entry = MakeShared<FBlueprintDifferenceTreeEntry>(
			FOnDiffEntryFocused::CreateRaw(DiffWidget, &SBlueprintDiff::OnGraphSelectionChanged, TSharedPtr<FGraphToDiff>(AsShared()), ESelectInfo::Direct),
			FGenerateDiffEntryWidget::CreateSP(AsShared(), &FGraphToDiff::GenerateCategoryWidget),
			Children);
	OutTreeEntries.Push(Entry);
	
	// add comments as children to this category
	FString GraphName;
	const UEdGraph* Graph = GraphOld ? GraphOld : GraphNew;
	if (const UEdGraphSchema* Schema = Graph->GetSchema())
	{
		FGraphDisplayInfo DisplayInfo;
		Schema->GetGraphDisplayInformation(*Graph, DisplayInfo);

		GraphName = DisplayInfo.DisplayName.ToString();
	}
	else
	{
		GraphName = Graph->GetFName().ToString();
	}
	GenerateCategoryCommentTreeEntries(OutTreeEntries.Last()->Children, OutRealDifferences, GraphName);
}

UEdGraph* FGraphToDiff::GetGraphOld() const
{
	return GraphOld;
}

UEdGraph* FGraphToDiff::GetGraphNew() const
{
	return GraphNew;
}

void FGraphToDiff::EnableComments(TWeakPtr<STreeView<TSharedPtr<FBlueprintDifferenceTreeEntry>>> TreeView)
{
	IDiffControl::EnableComments(TreeView, GetGraphOld(), GetGraphNew());
}

FText FGraphToDiff::GetToolTip()
{
	if (GraphOld && GraphNew)
	{
		if (DiffListSource.Num() > 0)
		{
			return LOCTEXT("ContainsDifferences", "Revisions are different");
		}
		else
		{
			return LOCTEXT("GraphsIdentical", "Revisions appear to be identical");
		}
	}
	else
	{
		UEdGraph* GoodGraph = GraphOld ? GraphOld : GraphNew;
		check(GoodGraph);
		const FRevisionInfo& Revision = GraphNew ? RevisionOld : RevisionNew;
		FText RevisionText = LOCTEXT("CurrentRevision", "Current Revision");

		if (!Revision.Revision.IsEmpty())
		{
			RevisionText = FText::Format(LOCTEXT("Revision Number", "Revision {0}"), FText::FromString(Revision.Revision));
		}

		return FText::Format(LOCTEXT("MissingGraph", "Graph '{0}' missing from {1}"), FText::FromString(GoodGraph->GetName()), RevisionText);
	}
}

TSharedRef<SWidget> FGraphToDiff::GenerateCategoryWidget()
{
	const UEdGraph* Graph = GraphOld ? GraphOld : GraphNew;
	check(Graph);
	
	FLinearColor Color = (GraphOld && GraphNew) ? DiffViewUtils::Identical() : FLinearColor(0.3f,0.3f,1.f);

	const bool bHasDiffs = DiffListSource.Num() > 0;

	if (bHasDiffs)
	{
		Color = DiffViewUtils::Differs();
	}

	FText GraphName;
	if (const UEdGraphSchema* Schema = Graph->GetSchema())
	{
		FGraphDisplayInfo DisplayInfo;
		Schema->GetGraphDisplayInformation(*Graph, DisplayInfo);

		GraphName = DisplayInfo.DisplayName;
	}
	else
	{
		GraphName = FText::FromName(Graph->GetFName());
	}

	return SNew(SHorizontalBox)
	+ SHorizontalBox::Slot()
	[
		SNew(STextBlock)
		.ColorAndOpacity(Color)
		.Text(GraphName)
		.ToolTipText(GetToolTip())
	]
	+ DiffViewUtils::Box( GraphOld != nullptr, Color )
	+ DiffViewUtils::Box( GraphNew != nullptr, Color );
}

void FGraphToDiff::BuildDiffSourceArray()
{
	FoundDiffs->Empty();
	FGraphDiffControl::DiffGraphs(GraphOld, GraphNew, *FoundDiffs);
	
	Algo::SortBy(*FoundDiffs, &FDiffSingleResult::Diff);

	DiffListSource.Empty();
	for (const FDiffSingleResult& Diff : *FoundDiffs)
	{
		DiffListSource.Add(MakeShared<FDiffResultItem>(Diff));
	}
}

void FGraphToDiff::OnGraphChanged( const FEdGraphEditAction& Action )
{
	DiffWidget->OnGraphChanged(this);
}

// returns whether a file being diffed should display comments
static bool AreCommentsEnabled(FString File)
{
	if (UE::DiffControl::GIsFileInReview)
	{
		return UE::DiffControl::GIsFileInReview(File);
	}
	return false;
}

FCommentTreeEntry::FCommentTreeEntry(TWeakPtr<FReviewCommentsDiffControl> InCommentsControl, const FReviewComment& InComment, const TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& InChildren)
	: FBlueprintDifferenceTreeEntry({}, FGenerateDiffEntryWidget::CreateRaw(this, &FCommentTreeEntry::CreateWidget), InChildren)
	, Comment(InComment)
	, CommentsControl(InCommentsControl)
{}

FCommentTreeEntry::~FCommentTreeEntry()
{
	if (OnCommentPostedHandle.IsValid())
	{
		UE::DiffControl::GOnCommentPosted.Remove(OnCommentPostedHandle);
	}
}

TSharedRef<FCommentTreeEntry> FCommentTreeEntry::Make(TWeakPtr<FReviewCommentsDiffControl> CommentsControl, const FReviewComment& Comment,
                                                      const TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& Children)
{
	return MakeShared<FCommentTreeEntry>(CommentsControl, Comment, Children);
}

int32 FCommentTreeEntry::GetCommentIDChecked() const
{
	return *Comment.CommentID;
}

void FCommentTreeEntry::AwaitCommentPost()
{
	OnCommentPostedHandle = UE::DiffControl::GOnCommentPosted.AddSP(this, &FCommentTreeEntry::OnCommentPosted);
}

TSharedRef<SWidget> FCommentTreeEntry::CreateWidget()
{
	// if a widget for this comment is being created that means it's in view. Mark the ReadBy state to reflect that the user is reading it
	if (!Comment.ReadBy.IsSet())
	{
		Comment.ReadBy = TSet<FString>();
	}
	const FString Username = UE::DiffControl::GGetReviewerUsername();
	if (!Comment.ReadBy->Find(Username))
	{
		Comment.ReadBy->Add(Username);
		
		FReviewComment ReadByEdit;
		ReadByEdit.CommentID = Comment.CommentID;
		ReadByEdit.ReadBy = Comment.ReadBy;
		UE::DiffControl::GEditReviewComment(ReadByEdit);
	}
	
	FDateTime DateTime = FDateTime::UtcNow();
	if (Comment.EditedTime.IsSet())
	{
		DateTime = *Comment.EditedTime;
	}
	else if (Comment.CreatedTime.IsSet())
	{
		DateTime = *Comment.CreatedTime;
	}
	FText DateText;
	if (DateTime.GetDay() == FDateTime::UtcNow().GetDay())
	{
		DateText = FText::AsTime(DateTime, EDateTimeStyle::Short);
	}
	else
	{
		DateText = FText::AsDate(DateTime, EDateTimeStyle::Short);
	}

	// constructs a hyperlink-like text button
	auto MakeCommentOptionButton = [](const FText &Text)
	{
		const FLinearColor HoveredColor = FStyleColors::AccentBlue.GetSpecifiedColor();
		const FLinearColor UnhoveredColor = HoveredColor.Desaturate(0.2f);
		TSharedRef<SButton> Button = SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "NoBorder")
			.ForegroundColor(UnhoveredColor)
			.Text(Text);

		Button->SetOnHovered(FSimpleDelegate::CreateLambda([ButtonWeak = Button.ToWeakPtr(), HoveredColor]()
		{
			if (const TSharedPtr<SButton> Button = ButtonWeak.Pin())
			{
				Button->SetForegroundColor(HoveredColor);
			}
		}));
		
		Button->SetOnUnhovered(FSimpleDelegate::CreateLambda([ButtonWeak = Button.ToWeakPtr(), UnhoveredColor]()
		{
			if (const TSharedPtr<SButton> Button = ButtonWeak.Pin())
			{
				Button->SetForegroundColor(UnhoveredColor);
			}
		}));
		return Button;
	};

	const TSharedPtr<SButton> ReplyButton = MakeCommentOptionButton(LOCTEXT("ReplyToComment","Reply"));
	ReplyButton->SetOnClicked(FOnClicked::CreateSP(this, &FCommentTreeEntry::OnClickReply));
	
	const TSharedPtr<SButton> EditButton = MakeCommentOptionButton(LOCTEXT("EditComment","Edit"));
	EditButton->SetOnClicked(FOnClicked::CreateSP(this, &FCommentTreeEntry::OnClickEdit));
	EditButton->SetVisibility(TAttribute<EVisibility>::CreateSP(this, &FCommentTreeEntry::GetEditButtonVisibility));

	FLinearColor BorderColor = FStyleColors::Recessed.GetSpecifiedColor();
	BorderColor.A = 0.95f;

	return SAssignNew(Content, SBorder)
	.IsEnabled_Static(&AreCommentsEnabled, *Comment.Context.File)
	.BorderImage(FCoreStyle::Get().GetBrush("GenericWhiteBox"))
	.BorderBackgroundColor(BorderColor)
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.VAlign(VAlign_Top)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.ColorAndOpacity(this, &FCommentTreeEntry::GetUsernameColor)
				.Text(FText::FromString(Comment.User.Get(TEXT("[Unknown User]"))))
				.Font(FAppStyle::GetFontStyle(TEXT("BoldFont")))
			]
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.FillWidth(1)
			.Padding(10.f,0.f,0.f,0.f)
			[
				SNew(STextBlock)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					.Text(DateText)
			]
		]
		+SVerticalBox::Slot()
		.VAlign(VAlign_Top)
		.AutoHeight()
		[
			SAssignNew(CommentTextBox, SMultiLineEditableTextBox)
			.Text(FText::FromString(GetCommentString()))
			.AutoWrapText(true)
			.IsReadOnly(this, &FCommentTreeEntry::IsCommentTextBoxReadonly)
		]
		+SVerticalBox::Slot()
		.VAlign(VAlign_Top)
		.AutoHeight()
		[
			SAssignNew(EditReplyButtonGroup, SHorizontalBox)
			.Visibility(this, &FCommentTreeEntry::GetEditReplyButtonGroupVisibility)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				ReplyButton.ToSharedRef()
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				EditButton.ToSharedRef()
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
            	.ButtonStyle(FAppStyle::Get(), "SimpleButton")
            	.OnClicked(this, &FCommentTreeEntry::OnLikeToggle)
            	.ToolTipText(this, &FCommentTreeEntry::GetLikeTooltip)
            	.ContentScale(0.8f)
            	.HAlign(HAlign_Center)
            	.VAlign(VAlign_Center)
            	[
					SNew(SImage)
					.Image(this, &FCommentTreeEntry::GetLikeIcon)
					.ColorAndOpacity(this, &FCommentTreeEntry::GetLikeIconColor)
				]
				
			]
		]
		+SVerticalBox::Slot()
		.VAlign(VAlign_Top)
		.AutoHeight()
		[
			SAssignNew(SubmitCancelButtonGroup, SHorizontalBox)
			.Visibility(this, &FCommentTreeEntry::GetSubmitCancelButtonGroupVisibility)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.Text(LOCTEXT("SubmitEdit","Submit"))
				.IsEnabled(this, &FCommentTreeEntry::IsSubmitButtonEnabled)
				.OnClicked(this, &FCommentTreeEntry::OnEditSubmitClicked)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.Text(LOCTEXT("CancelEdit","Cancel"))
				.OnClicked(this, &FCommentTreeEntry::OnEditCancelClicked)
			]
		]
	];
}

bool FCommentTreeEntry::IsCommentTextBoxReadonly() const
{
	return !IsEditMode();
}

FReply FCommentTreeEntry::OnClickReply()
{
	if (const TSharedPtr<FReviewCommentsDiffControl> Control = CommentsControl.Pin())
	{
		Control->DraftReply(AsShared());
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply FCommentTreeEntry::OnClickEdit()
{
	SetEditMode(true);
	return FReply::Handled();
}

FReply FCommentTreeEntry::OnLikeToggle()
{
	if (!Comment.Likes.IsSet())
	{
		Comment.Likes = TSet<FString>{};
	}
	const FString Username = UE::DiffControl::GGetReviewerUsername();
	if (Comment.Likes->Contains(Username))
	{
		Comment.Likes->Remove(Username);
	}
	else
	{
		Comment.Likes->Add(Username);
	}
	
	FReviewComment LikeEdit;
	LikeEdit.CommentID = Comment.CommentID;
	LikeEdit.Likes = Comment.Likes;
    UE::DiffControl::GEditReviewComment(LikeEdit);
	
	return FReply::Handled();
}

// convert an iterable structure to a text based list
// for example,
// {"a"} -> "{Range[0]}" -> "a"
// {"a", "b"} -> "{Range[0]} {Conjunction} {Range[1]}" -> "a and b"
// {"a", "b", "c"} -> "{Range[0]}{Delimiter} {Range[1]}{Delimiter} {Conjunction} {Range[2]}" -> "a, b, and c"
template <typename RangeType, typename PredicateType>
static FText RangeAsText(RangeType&& Range, const FText& Delimiter, const FText& Conjunction, PredicateType Transformer)
{
	TArray<FText> ItemsAsText;
	int32 Index = 0;
	const int32 LastIndex = Range.Num() - 1;
	if (Range.Num() == 1)
	{
		return Invoke(Transformer, *Range.begin());
	}
	for (auto&& Elem : Forward<RangeType>(Range))
	{
		FText ElementText = Invoke(Transformer, Elem);
		if (Index == LastIndex && !Conjunction.IsEmpty())
		{
			// for the last item, prepend a conjunction
			ItemsAsText.Add(FText::FormatNamed(
				LOCTEXT("ListLastElement", "{Conjunction} {Element}"),
				TEXT("Conjunction"), Conjunction,
				TEXT("Element"), Invoke(Transformer, Elem)
			));
		}
		else
		{
			ItemsAsText.Add(Invoke(Transformer, Elem));
		}
		++Index;
	}
	if (Range.Num() == 2)
	{
		// for two elements, don't add a comma
		return FText::Join(LOCTEXT("Space"," "), ItemsAsText);
	}
	return FText::Join(FText::Format(LOCTEXT("DelimeterWithSpace","{0} "), Delimiter), ItemsAsText);
}

FText FCommentTreeEntry::GetLikeTooltip() const
{
	if (Comment.Likes.IsSet() && !Comment.Likes->IsEmpty())
	{
		FText(* const StringToText)(const FString&) = &FText::FromString;
		const FText LikesListText = RangeAsText(*Comment.Likes, LOCTEXT("ListDelimeter",","), LOCTEXT("ConjunctionAnd","and"), StringToText);
		return FText::Format(LOCTEXT("LikedBy", "Liked By: {0}"), LikesListText);
	}
	return LOCTEXT("NoLikes","No Likes");
}

const FSlateBrush* FCommentTreeEntry::GetLikeIcon() const
{
	if (Comment.Likes.IsSet() && !Comment.Likes->IsEmpty())
	{
		return FAppStyle::Get().GetBrush(TEXT("Icons.Heart"));
	}
	return FAppStyle::Get().GetBrush(TEXT("Icons.HollowHeart"));
}

FSlateColor FCommentTreeEntry::GetUsernameColor() const
{
	const FString Username = UE::DiffControl::GGetReviewerUsername();
	if (Comment.User == Username)
	{
		return FStyleColors::AccentBlue;
	}
	return FStyleColors::AccentGreen.GetSpecifiedColor().Desaturate(0.2f);
}

FSlateColor FCommentTreeEntry::GetLikeIconColor() const
{
	const FString Username = UE::DiffControl::GGetReviewerUsername();
	if (Comment.Likes.IsSet())
	{
		if (Comment.Likes->Contains(Username))
		{
			return FSlateColor::UseForeground();
		}
	}
	return FSlateColor::UseSubduedForeground();
}

EVisibility FCommentTreeEntry::GetEditReplyButtonGroupVisibility() const
{
	if (IsEditMode())
	{
		return EVisibility::Collapsed;
	}
	// if the comment is still being submitted, don't allow edits or replies yet
	if (!Comment.CommentID.IsSet())
	{
		return EVisibility::Hidden;
	}
	return Content->IsHovered() ? EVisibility::Visible : EVisibility::Hidden;
}

EVisibility FCommentTreeEntry::GetEditButtonVisibility() const
{
	const FString Reviewer = UE::DiffControl::GGetReviewerUsername();
	if (Reviewer == Comment.User)
	{
		return EVisibility::Visible;
	}
	return EVisibility::Collapsed;
}

EVisibility FCommentTreeEntry::GetSubmitCancelButtonGroupVisibility() const
{
	return IsEditMode() ? EVisibility::Visible : EVisibility::Collapsed;
}

bool FCommentTreeEntry::IsSubmitButtonEnabled() const
{
	// don't allow submission of empty strings
	return !CommentTextBox->GetText().ToString().TrimEnd().IsEmpty();
}

FReply FCommentTreeEntry::OnEditSubmitClicked()
{
	if (HasCommentStringChanged())
	{
		SetCommentString(CommentTextBox->GetText().ToString());
		FReviewComment BodyEdit;
		BodyEdit.CommentID = Comment.CommentID;
		BodyEdit.Body = Comment.Body;
		UE::DiffControl::GEditReviewComment(BodyEdit);
	}
	CommentTextBox->SetText(FText::FromString(GetCommentString()));
	SetEditMode(false);
	return FReply::Handled();
}

FReply FCommentTreeEntry::OnEditCancelClicked()
{
	CommentTextBox->SetText(FText::FromString(GetCommentString()));
	SetEditMode(false);
	return FReply::Handled();
}

void FCommentTreeEntry::OnCommentPosted(const FReviewComment& InComment)
{
	if (InComment.Body != Comment.Body)
	{
		return;
	}
	if (InComment.User == Comment.User)
	{
		Comment = InComment;
		UE::DiffControl::GOnCommentPosted.Remove(OnCommentPostedHandle);
	}
}

FString FCommentTreeEntry::GetCommentString() const
{
	return Comment.Body.Get(TEXT("[Missing Comment Body]")).TrimEnd();
}

void FCommentTreeEntry::SetCommentString(const FString& NewComment)
{
	Comment.Body = NewComment.TrimEnd();
}

bool FCommentTreeEntry::HasCommentStringChanged() const
{
	const FString EditedCommentString = CommentTextBox->GetText().ToString().TrimEnd();
	const FString CommentString = GetCommentString();
	return EditedCommentString != CommentString;
}

bool FCommentTreeEntry::IsEditMode() const
{
	if (bExpectedEditMode)
	{
		// if user de-focused the textbox without changing anything, cancel the edit
		if (!CommentTextBox->HasKeyboardFocus() && !HasCommentStringChanged())
		{
			CommentTextBox->SetText(FText::FromString(GetCommentString()));
			return false;
		}
	}
	return bExpectedEditMode;
}

void FCommentTreeEntry::SetEditMode(bool bIsEditMode)
{
	if (bIsEditMode)
	{
		FSlateApplication::Get().SetKeyboardFocus(CommentTextBox);
	}
	bExpectedEditMode = bIsEditMode;
}

FCommentDraftTreeEntry::FCommentDraftTreeEntry(TWeakPtr<FReviewCommentsDiffControl> InCommentsControl, TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>* InSiblings, int32 InReplyID)
	: FBlueprintDifferenceTreeEntry({}, FGenerateDiffEntryWidget::CreateRaw(this, &FCommentDraftTreeEntry::CreateWidget), {})
	, CommentsControl(InCommentsControl)
	, Siblings(InSiblings)
	, ReplyID(InReplyID)
{}

TSharedRef<FCommentDraftTreeEntry> FCommentDraftTreeEntry::MakeCommentDraft(TWeakPtr<FReviewCommentsDiffControl> CommentsControl, TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>* Siblings)
{
	return MakeShared<FCommentDraftTreeEntry>(CommentsControl, Siblings);
}

TSharedRef<FCommentDraftTreeEntry> FCommentDraftTreeEntry::MakeReplyDraft(TWeakPtr<FReviewCommentsDiffControl> CommentsControl, TSharedPtr<FCommentTreeEntry> InParent)
{
	return MakeShared<FCommentDraftTreeEntry>(CommentsControl, &InParent->Children, InParent->GetCommentIDChecked());
}

void FCommentDraftTreeEntry::ReassignSiblings(TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>* InSiblings, int32 InReplyID)
{
	const int32 MyIndex = Siblings->Find(AsShared());
	if (MyIndex != INDEX_NONE)
	{
		Siblings->RemoveAt(MyIndex);
	}
	Siblings = InSiblings;
	ReplyID = InReplyID;
}

void FCommentDraftTreeEntry::ReassignReplyParent(TSharedPtr<FCommentTreeEntry> InParent)
{
	ReassignSiblings(&InParent->Children, InParent->GetCommentIDChecked());
}

bool FCommentDraftTreeEntry::IsReply() const
{
	return ReplyID != -1;
}

TSharedRef<SWidget> FCommentDraftTreeEntry::CreateWidget()
{
	FLinearColor BorderColor = FStyleColors::Recessed.GetSpecifiedColor();
	BorderColor.A = 0.95f;

	FString FilePath;
	if (const TSharedPtr<FReviewCommentsDiffControl> Control = CommentsControl.Pin())
	{
		FilePath = Control->GetCommentFilePath();
	}

	SAssignNew(Content, SBorder)
	.IsEnabled_Static(&AreCommentsEnabled, FilePath)
	.BorderImage(FCoreStyle::Get().GetBrush("GenericWhiteBox"))
	.BorderBackgroundColor(BorderColor)
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.VAlign(VAlign_Top)
		.AutoHeight()
		[
			SNew(SBox)
			.MinDesiredHeight(50.f)
			[
				SAssignNew(CommentTextBox, SMultiLineEditableTextBox)
				.HintText(IsReply()? LOCTEXT("AddReply", "Reply...") : LOCTEXT("AddAComment", "Add a comment"))
				.ForegroundColor(FStyleColors::White)
				.AutoWrapText(true)
			]
		]
		+SVerticalBox::Slot()
		.VAlign(VAlign_Top)
		.AutoHeight()
		.Padding(0.f, 0.f, 0.f, 20.f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(0.f,0.f,10.f,0.f)
			[
				SNew(SButton)
				.Text(LOCTEXT("PostComment","Post"))
				.IsEnabled(this, &FCommentDraftTreeEntry::IsPostButtonEnabled)
				.OnClicked(this, &FCommentDraftTreeEntry::OnCommentPostClicked)
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(0.f,0.f,10.f,0.f)
			[
				SAssignNew(FlagAsTaskCheckBox, SCheckBox)
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("FlagCommentAsTask","Flag as Task"))
			]
		]
	];
	
	return Content.ToSharedRef();
}

bool FCommentDraftTreeEntry::IsPostButtonEnabled() const
{
	// only allow posting if the comment has non-whitespace characters
	return !CommentTextBox->GetText().ToString().TrimEnd().IsEmpty();
}

FReply FCommentDraftTreeEntry::OnCommentPostClicked()
{
	if (const TSharedPtr<FReviewCommentsDiffControl> Control = CommentsControl.Pin())
	{
		const FString& CommentCategory = Control->GetCommentCategory();
		const FString& CommentFilePath = Control->GetCommentFilePath();
		
		FReviewComment Comment;
		Comment.Body = CommentTextBox->GetText().ToString().TrimEnd();
		Comment.Context.File = CommentFilePath;
		Comment.ReadBy = TSet<FString>{UE::DiffControl::GGetReviewerUsername()};
		
		if (!CommentCategory.IsEmpty())
		{
			Comment.Context.Category = CommentCategory;
		}
		if (IsReply())
		{
			Comment.Context.ReplyTo = ReplyID;
		}
		
		const bool bIsTask = FlagAsTaskCheckBox->GetCheckedState() == ECheckBoxState::Checked;
		Comment.TaskState = bIsTask ? EReviewCommentTaskState::Open : EReviewCommentTaskState::Comment;

		const int32 InsertIndex = Siblings->Find(AsShared());
		check(InsertIndex != INDEX_NONE);

		Control->PostComment(Comment);
		const TSharedRef<FCommentTreeEntry> CommentEntry = FCommentTreeEntry::Make(Control, Comment);
		CommentEntry->AwaitCommentPost(); // tell entry to update once comment is posted
		Siblings->Insert(CommentEntry, InsertIndex);
		
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReviewCommentsDiffControl::FReviewCommentsDiffControl(const FString& InCommentFilePath, TWeakPtr<STreeView<TSharedPtr<FBlueprintDifferenceTreeEntry>>> TreeView)
	: CommentFilePath(InCommentFilePath)
	, CommentsTreeView(TreeView)
{
}

void FReviewCommentsDiffControl::GenerateCommentThreadRecursive(const FReviewComment& Comment,
	const TMap<int32, TArray<const FReviewComment*>>& CommentReplyMap,
	TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& OutTreeEntries)
{
	if (Comment.bIsClosed)
	{
		return;
	}
	
	TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>> Children;
	if (const TArray<const FReviewComment*>* Replies = CommentReplyMap.Find(*Comment.CommentID))
	{
		for (const FReviewComment* Reply : *Replies)
		{
			GenerateCommentThreadRecursive(*Reply, CommentReplyMap, Children);
		}
	}
	
	OutTreeEntries.Push(FCommentTreeEntry::Make(AsShared(), Comment, Children));
}

void FReviewCommentsDiffControl::GenerateTreeEntries(TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& OutTreeEntries,
                                                     TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& OutRealDifferences)
{
	if (!AreCommentsEnabled(GetCommentFilePath()))
	{
		return;
	}


	TMap<int32, TArray<const FReviewComment*>> CommentReplyMap;
	TArray<const FReviewComment*> HeadComments;
	if (const TArray<FReviewComment>* FileComments = UE::DiffControl::GGetReviewCommentsForFile(CommentFilePath))
	{
		for (const FReviewComment& Comment : *FileComments)
		{
			if (Comment.Context.ReplyTo.IsSet())
			{
				CommentReplyMap.FindOrAdd(*Comment.Context.ReplyTo, {}).Add(&Comment);
				continue;
			}
			if (Comment.Context.Category.Get({}) != CommentCategory)
			{
				continue;
			}
			
			HeadComments.Add(&Comment);
		}
	}

	// add an empty entry that visually spaces the comments from the diff entries
	TSharedPtr<FBlueprintDifferenceTreeEntry> PaddingEntry = TSharedPtr<FBlueprintDifferenceTreeEntry>(new FBlueprintDifferenceTreeEntry(
		{}
		, FGenerateDiffEntryWidget::CreateLambda([]()
		{
			return
			SNew(SBox)
			.Padding(0.f, 20.f, 0.f, 10.f)
			.HAlign(HAlign_Fill)
			[
				SNew(SImage)
				.DesiredSizeOverride(FVector2D{100.0, 2.0})
				.Image(FCoreStyle::Get().GetBrush("GenericWhiteBox"))
				.ColorAndOpacity(FStyleColors::White25)
			];
		})
		, {}
	));
	OutTreeEntries.Push(PaddingEntry);
	
	for (const FReviewComment* Comment : HeadComments)
	{
		// add comment as a thread (including replies as it's children)
		GenerateCommentThreadRecursive(*Comment, CommentReplyMap, OutTreeEntries);
	}
	
	OutTreeEntries.Push(FCommentDraftTreeEntry::MakeCommentDraft(AsShared(), &OutTreeEntries));
}

void FReviewCommentsDiffControl::SetCategory(const FString& CategoryKey)
{
	CommentCategory = CategoryKey;
}

void FReviewCommentsDiffControl::PostComment(FReviewComment& Comment)
{
	UE::DiffControl::GPostReviewComment(Comment);
	if (const TSharedPtr<STreeView<TSharedPtr<FBlueprintDifferenceTreeEntry>>> TreeView = CommentsTreeView.Pin())
	{
		TreeView->RebuildList();
	}
}

static void DeferredCallback(TSharedPtr<SWidget> Widget, int32 FramesToSkip, TFunction<void(void)> Callback)
{
	Widget->RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateLambda(
			[Callback, FramesToSkip, FrameCount = 0](double, float) mutable
			{
				if (++FrameCount < FramesToSkip)
				{
					return EActiveTimerReturnType::Continue;
				}
				Callback();
				return EActiveTimerReturnType::Stop;
			}
		)
	);
}

TWeakPtr<FCommentDraftTreeEntry> FReviewCommentsDiffControl::ReplyDraftEntry;

void FReviewCommentsDiffControl::DraftReply(TSharedPtr<FCommentTreeEntry> ParentComment)
{
	if (const TSharedPtr<STreeView<TSharedPtr<FBlueprintDifferenceTreeEntry>>> TreeView = CommentsTreeView.Pin())
	{
		TSharedPtr<FCommentDraftTreeEntry> DraftEntry = ReplyDraftEntry.Pin();
		if (DraftEntry)
		{
			DraftEntry->ReassignReplyParent(ParentComment);
		}
		else
		{
			DraftEntry = FCommentDraftTreeEntry::MakeReplyDraft(AsShared(), ParentComment);
			ReplyDraftEntry = DraftEntry;
		}
		ParentComment->Children.Push(DraftEntry);
		TreeView->SetItemExpansion(ParentComment, true);
		TreeView->RebuildList();
		
		// focus comment text box once it's finished constructing
		DeferredCallback(TreeView, 2, [DraftEntryWeak = TWeakPtr<FCommentDraftTreeEntry>(DraftEntry)]()
		{
			if (const TSharedPtr<FCommentDraftTreeEntry> DraftEntry = DraftEntryWeak.Pin())
			{
				FSlateApplication::Get().SetKeyboardFocus(DraftEntry->GetCommentTextBox());
			}
		});
	}
}

void FReviewCommentsDiffControl::RebuildListView() const
{
	if (const TSharedPtr<STreeView<TSharedPtr<FBlueprintDifferenceTreeEntry>>> TreeView = CommentsTreeView.Pin())
	{
		TreeView->RebuildList();
	}
}

#undef LOCTEXT_NAMESPACE
