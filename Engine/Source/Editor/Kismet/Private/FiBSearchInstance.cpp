// Copyright Epic Games, Inc. All Rights Reserved.
#include "FiBSearchInstance.h"

#include "Containers/SparseArray.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "ImaginaryBlueprintData.h"
#include "Internationalization/Text.h"
#include "Misc/CString.h"
#include "Misc/ExpressionParserTypes.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/ValueOrError.h"
#include "UObject/NameTypes.h"

/** All operators when evaluating FiB searched expressions must return this token, it helps to manage
 *	the results from functions as well as the specific components that were matched, and allows
 *	for combining those results through complex operator combinations that may eliminate entire sections
 *	of search results.
 */
class FFiBToken
{
public:
	FFiBToken(bool bInValue)
		: bValue(bInValue)
	{
	}

	FFiBToken(bool bInValue, TMultiMap< const FImaginaryFiBData*, FComponentUniqueDisplay >& InMatchingSearchComponents)
		: MatchingSearchComponents(MoveTemp(InMatchingSearchComponents))
		, bValue(bInValue)
	{
	}

	FFiBToken(bool bInValue, TArray< const FImaginaryFiBData* >& InMatchesSearchQuery)
		: MatchesSearchQuery(InMatchesSearchQuery)
		, bValue(bInValue)
	{
	}

	FFiBToken(const FFiBToken& Other)
		: MatchesSearchQuery(Other.MatchesSearchQuery)
		, MatchingSearchComponents(Other.MatchingSearchComponents)
		, bValue(Other.bValue)
	{
	}

	FFiBToken(FFiBToken&& Other)
		: MatchesSearchQuery(MoveTemp(Other.MatchesSearchQuery))
		, MatchingSearchComponents(MoveTemp(Other.MatchingSearchComponents))
		, bValue(Other.bValue)
	{
	}

	FFiBToken& operator=(const FFiBToken& Other)
	{
		MatchesSearchQuery = Other.MatchesSearchQuery;
		bValue = Other.bValue;
		return *this;
	}

	FFiBToken& operator=(FFiBToken&& Other)
	{
		MatchesSearchQuery = MoveTemp(Other.MatchesSearchQuery);
		bValue = Other.bValue;
		return *this;
	}

	/** Combines another token into this one, merging all collected data */
	void CombineToken(const FFiBToken& InToken)
	{
		MergeMatchesSearchQuery(InToken.MatchesSearchQuery);
		MergeMatchingSearchComponents(InToken.MatchingSearchComponents);
	}

	/** Helper to only merge the MatchesSearchQuery data with this token */
	void MergeMatchesSearchQuery(const TArray< const FImaginaryFiBData* >& InMatchesSearchQuery)
	{
		for (const FImaginaryFiBData* MatchesItem : InMatchesSearchQuery)
		{
			MatchesSearchQuery.AddUnique(MatchesItem);
		}
	}

	/** Helper to only merge the MatchingSearchComponents data with this token */
	void MergeMatchingSearchComponents(const TMultiMap< const FImaginaryFiBData*, FComponentUniqueDisplay >& InMatchingSearchComponents)
	{
		for (auto& MatchesItem : InMatchingSearchComponents)
		{
			MatchingSearchComponents.AddUnique(MatchesItem.Key, MatchesItem.Value);
		}
	}

public:
	/** A going list of all imaginary items that matched the search query at the time of this result token's creation */
	TArray< const FImaginaryFiBData* > MatchesSearchQuery;

	/** A mapping of items and their components that match the search query at the time of this result token's creation */
	TMultiMap< const FImaginaryFiBData*, FComponentUniqueDisplay > MatchingSearchComponents;

	/** Whether this result token should be considered TRUE or FALSE for purposes of further evaluation */
	bool bValue;
};

DEFINE_EXPRESSION_NODE_TYPE(FFiBToken,	0x03378490, 0x42D14E26, 0x8E95AD2F, 0x74567868)

/////////////////////////////////////
// FFiBContextHelper

/** Helper class to reroute testing of expressions against the context so 
 *	that a mapping of the components in the context can be prepared and returned
 */
class FFiBContextHelper : public ITextFilterExpressionContext
{
public:
	FFiBContextHelper()
	{

	}

	FFiBContextHelper(FImaginaryFiBDataWeakPtr InContext)
		: Context(InContext)
	{
	}

	virtual bool TestBasicStringExpression(const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode) const override
	{
		if (Context.IsValid())
		{
			return Context.Pin()->TestBasicStringExpression(InValue, InTextComparisonMode, MatchingSearchComponents);
		}

		return false;
	}

	virtual bool TestComplexExpression(const FName& InKey, const FTextFilterString& InValue, const ETextFilterComparisonOperation InComparisonOperation, const ETextFilterTextComparisonMode InTextComparisonMode) const override
	{
		if (Context.IsValid())
		{
			return Context.Pin()->TestComplexExpression(InKey, InValue, InComparisonOperation, InTextComparisonMode, MatchingSearchComponents);
		}

		return false;
	}

	/** Context that is actually being tested */
	FImaginaryFiBDataWeakPtr Context;

	/** Modified in a const function callback, this is a going list of all search components that matched the expression */
	mutable TMultiMap< const FImaginaryFiBData*, FComponentUniqueDisplay > MatchingSearchComponents;
};

////////////////////////
// FFiBSearchInstance

FSearchResult FFiBSearchInstance::StartSearchQuery(const FString& InSearchString, FImaginaryFiBDataSharedPtr InImaginaryBlueprintRoot)
{
	PendingSearchables.Add(InImaginaryBlueprintRoot);
	DoSearchQuery(InSearchString);

	return GetSearchResults(InImaginaryBlueprintRoot);
}

void FFiBSearchInstance::MakeSearchQuery(const FString& InSearchString, FImaginaryFiBDataSharedPtr InImaginaryBlueprintRoot)
{
	PendingSearchables.Add(InImaginaryBlueprintRoot);
	DoSearchQuery(InSearchString);
}

FSearchResult FFiBSearchInstance::GetSearchResults(FImaginaryFiBDataSharedPtr InImaginaryBlueprintRoot)
{
	FSearchResult CachedSearchResult = FImaginaryFiBData::CreateSearchTree(nullptr, InImaginaryBlueprintRoot, MatchesSearchQuery, MatchingSearchComponents);
	return MatchesSearchQuery.Num() > 0? CachedSearchResult : nullptr;
}

bool FFiBSearchInstance::DoSearchQuery(const FString& InSearchString, bool bInComplete/* = true*/)
{
	TSharedPtr< FFindInBlueprintExpressionEvaluator> ExpressionEvaluator(new FFindInBlueprintExpressionEvaluator(ETextFilterExpressionEvaluatorMode::Complex, this));

	// Add all the required function bindings
	ExpressionEvaluator->AddFunctionTokenCallback( TEXT("All"), FTokenFunctionHandler::CreateRaw(this, &FFiBSearchInstance::OnFilterFunction, ESearchQueryFilter::AllFilter));
	ExpressionEvaluator->AddFunctionTokenCallback( TEXT("Blueprint"), FTokenFunctionHandler::CreateRaw(this, &FFiBSearchInstance::OnFilterFunction, ESearchQueryFilter::BlueprintFilter));
	ExpressionEvaluator->AddFunctionTokenCallback( TEXT("Graphs"), FTokenFunctionHandler::CreateRaw(this, &FFiBSearchInstance::OnFilterFunction, ESearchQueryFilter::GraphsFilter));
	ExpressionEvaluator->AddFunctionTokenCallback( TEXT("EventGraphs"), FTokenFunctionHandler::CreateRaw(this, &FFiBSearchInstance::OnFilterFunction, ESearchQueryFilter::UberGraphsFilter));
	ExpressionEvaluator->AddFunctionTokenCallback( TEXT("Functions"), FTokenFunctionHandler::CreateRaw(this, &FFiBSearchInstance::OnFilterFunction, ESearchQueryFilter::FunctionsFilter));
	ExpressionEvaluator->AddFunctionTokenCallback( TEXT("Macros"), FTokenFunctionHandler::CreateRaw(this, &FFiBSearchInstance::OnFilterFunction, ESearchQueryFilter::MacrosFilter));
	ExpressionEvaluator->AddFunctionTokenCallback( TEXT("Properties"), FTokenFunctionHandler::CreateRaw(this, &FFiBSearchInstance::OnFilterFunction, ESearchQueryFilter::PropertiesFilter));
	ExpressionEvaluator->AddFunctionTokenCallback( TEXT("Variables"), FTokenFunctionHandler::CreateRaw(this, &FFiBSearchInstance::OnFilterFunction, ESearchQueryFilter::PropertiesFilter));
	ExpressionEvaluator->AddFunctionTokenCallback( TEXT("Components"), FTokenFunctionHandler::CreateRaw(this, &FFiBSearchInstance::OnFilterFunction, ESearchQueryFilter::ComponentsFilter));
	ExpressionEvaluator->AddFunctionTokenCallback( TEXT("Nodes"), FTokenFunctionHandler::CreateRaw(this, &FFiBSearchInstance::OnFilterFunction, ESearchQueryFilter::NodesFilter));
	ExpressionEvaluator->AddFunctionTokenCallback( TEXT("Pins"), FTokenFunctionHandler::CreateRaw(this, &FFiBSearchInstance::OnFilterFunction, ESearchQueryFilter::PinsFilter));
	ExpressionEvaluator->SetDefaultFunctionHandler(FTokenDefaultFunctionHandler::CreateRaw(this, &FFiBSearchInstance::OnFilterDefaultFunction));
	ExpressionEvaluator->SetFilterText(FText::FromString(InSearchString));

	for (int SearchableIdx = 0; SearchableIdx < PendingSearchables.Num(); ++SearchableIdx)
	{
		CurrentSearchable = PendingSearchables[SearchableIdx];
		FImaginaryFiBDataSharedPtr CurrentSearchablePinned = CurrentSearchable.Pin();
		CurrentSearchablePinned->ParseAllChildData();
		if (ExpressionEvaluator->TestTextFilter(*CurrentSearchablePinned.Get()))
		{
			MatchesSearchQuery.AddUnique(CurrentSearchablePinned.Get());
		}

		if (bInComplete || CurrentSearchablePinned->IsCategory())
		{
			// Any children that are not treated as a TagAndValue Category should be added for independent searching
			TArray<FImaginaryFiBDataSharedPtr> Children = CurrentSearchablePinned->GetAllParsedChildData();
			for (FImaginaryFiBDataSharedPtr Child : Children)
			{
				if (!Child->IsTagAndValueCategory())
				{
					PendingSearchables.Add(Child);
				}
			}
		}
	}
	CurrentSearchable.Reset();

	return MatchesSearchQuery.Num() > 0;
}

void FFiBSearchInstance::CreateFilteredResultsListFromTree(ESearchQueryFilter InSearchQueryFilter, TArray<FImaginaryFiBDataSharedPtr>& InOutValidSearchResults)
{
	for (const FImaginaryFiBData* ImaginaryDataPtr : MatchesSearchQuery)
	{
		if (!ImaginaryDataPtr->IsCategory() && ImaginaryDataPtr->IsCompatibleWithFilter(InSearchQueryFilter))
		{
			FImaginaryFiBData* NonCastImaginaryDataPtr = const_cast<FImaginaryFiBData*>(ImaginaryDataPtr);
			InOutValidSearchResults.Add(NonCastImaginaryDataPtr->AsShared());
		}
	}
}

void FFiBSearchInstance::BuildFunctionTargets(FImaginaryFiBDataSharedPtr InRootData, ESearchQueryFilter InSearchQueryFilter, TArray<FImaginaryFiBDataWeakPtr>& OutTargetPendingSearchables)
{
	for (FImaginaryFiBDataSharedPtr ChildData : InRootData->GetAllParsedChildData())
	{
		if (!ChildData->IsCategory() && ChildData->IsCompatibleWithFilter(InSearchQueryFilter))
		{
			OutTargetPendingSearchables.Add(ChildData);
		}
		else if (ChildData->IsCategory() || ChildData->CanCallFilter(InSearchQueryFilter))
		{
			ChildData->ParseAllChildData();
			BuildFunctionTargets(ChildData, InSearchQueryFilter, OutTargetPendingSearchables);
		}
	}
}

void FFiBSearchInstance::BuildFunctionTargetsByName(FImaginaryFiBDataSharedPtr InRootData, FString InTagName, TArray<FImaginaryFiBDataWeakPtr>& OutTargetPendingSearchables)
{
	for (FImaginaryFiBDataSharedPtr ChildData : InRootData->GetAllParsedChildData())
	{
		if (ChildData->IsCategory())
		{
			FCategorySectionHelper* CategoryData = static_cast<FCategorySectionHelper*>(ChildData.Get());
			if (CategoryData->GetCategoryFunctionName().Equals(InTagName, ESearchCase::IgnoreCase))
			{
				OutTargetPendingSearchables.Add(ChildData);
			}
			else if (CategoryData->IsTagAndValueCategory())
			{
				BuildFunctionTargetsByName(ChildData, InTagName, OutTargetPendingSearchables);
			}
		}
	}
}

bool FFiBSearchInstance::OnFilterFunction(const FTextFilterString& A, ESearchQueryFilter InSearchQueryFilter)
{
	CSV_SCOPED_TIMING_STAT(FindInBlueprint, OnFilterFunction);

	if (CurrentSearchable.IsValid())
	{
		TSharedPtr< FFiBSearchInstance > SubSearchInstance(new FFiBSearchInstance);
		bool bSearchSuccess = false;

		FImaginaryFiBDataSharedPtr CurrentSearchablePinned = CurrentSearchable.Pin();
		if (CurrentSearchablePinned->CanCallFilter(InSearchQueryFilter))
		{
			CurrentSearchablePinned->ParseAllChildData();
			BuildFunctionTargets(CurrentSearchablePinned, InSearchQueryFilter, SubSearchInstance->PendingSearchables);
		}
		else if (InSearchQueryFilter == ESearchQueryFilter::BlueprintFilter && CurrentSearchablePinned->IsCompatibleWithFilter(ESearchQueryFilter::BlueprintFilter))
		{
			// We are filtering by Blueprint, since this is a Blueprint just add the CurrentSearchable to the PendingSearchables and do the sub-search on it
			SubSearchInstance->PendingSearchables.Add(CurrentSearchable);
		}

		// Proceed to doing a sub-search
		if (SubSearchInstance->PendingSearchables.Num() > 0)
		{
			// Make a copy of results so far. We'll intersect with sub-search results.
			TSet<FImaginaryFiBData*> InitialResultsCopy;
			for (const FImaginaryFiBDataWeakPtr& ResultCopy : SubSearchInstance->PendingSearchables)
			{
				if (ResultCopy.IsValid())
				{
					InitialResultsCopy.Add(ResultCopy.Pin().Get());
				}
			}

			bSearchSuccess = SubSearchInstance->DoSearchQuery(A.AsString(), InSearchQueryFilter == ESearchQueryFilter::AllFilter);
			if (bSearchSuccess)
			{
				for (const FImaginaryFiBData* MatchesItem : SubSearchInstance->MatchesSearchQuery)
				{
					if (InitialResultsCopy.Contains(MatchesItem))
					{
						LastFunctionResultMatchesSearchQuery.AddUnique(MatchesItem);
					}
				}

				for (const TPair<const FImaginaryFiBData*, FComponentUniqueDisplay>& MatchesItem : SubSearchInstance->MatchingSearchComponents)
				{
					if (InitialResultsCopy.Contains(MatchesItem.Key))
					{
						LastFunctionMatchingSearchComponents.AddUnique(MatchesItem.Key, MatchesItem.Value);
					}
				}
			}
		}
		return bSearchSuccess;
	}

	return false;
}

bool FFiBSearchInstance::OnFilterDefaultFunction(const FTextFilterString& InFunctionName, const FTextFilterString& InFunctionParams)
{
	CSV_SCOPED_TIMING_STAT(FindInBlueprint, OnFilterDefaultFunction);

	if (CurrentSearchable.IsValid())
	{
		TSharedPtr< FFiBSearchInstance > SubSearchInstance(new FFiBSearchInstance);
		bool bSearchSuccess = false;

		FImaginaryFiBDataSharedPtr CurrentSearchablePinned = CurrentSearchable.Pin();
		CurrentSearchablePinned->ParseAllChildData();
		BuildFunctionTargetsByName(CurrentSearchablePinned, InFunctionName.AsString(), SubSearchInstance->PendingSearchables);

		// Proceed to doing a sub-search
		if (SubSearchInstance->PendingSearchables.Num() > 0)
		{
			bSearchSuccess = SubSearchInstance->DoSearchQuery(InFunctionParams.AsString(), true);
			if (bSearchSuccess)
			{
				for (auto& MatchesItem : SubSearchInstance->MatchesSearchQuery)
				{
					LastFunctionResultMatchesSearchQuery.AddUnique(MatchesItem);
				}

				for (auto& MatchesItem : SubSearchInstance->MatchingSearchComponents)
				{
					LastFunctionMatchingSearchComponents.AddUnique(MatchesItem.Key, MatchesItem.Value);
				}
			}
		}
		return bSearchSuccess;
	}

	return false;
}

////////////////////////////////////////
// FFindInBlueprintExpressionEvaluator

bool FFindInBlueprintExpressionEvaluator::EvaluateCompiledExpression(const ExpressionParser::CompileResultType& InCompiledResult, const ITextFilterExpressionContext& InContext, FText* OutErrorText) const
{
	CSV_SCOPED_TIMING_STAT(FindInBlueprint, EvaluateCompiledExpression);

	using namespace TextFilterExpressionParser;
	using TextFilterExpressionParser::FTextToken;

	if (InCompiledResult.IsValid())
	{
		auto EvalResult = ExpressionParser::Evaluate(InCompiledResult.GetValue(), JumpTable, &InContext);
		if (EvalResult.IsValid())
		{
			if (const bool* BoolResult = EvalResult.GetValue().Cast<bool>())
			{
				return *BoolResult;
			}
			else if (const FTextToken* TextResult = EvalResult.GetValue().Cast<FTextToken>())
			{
				FFiBContextHelper ContextHelper(SearchInstance->CurrentSearchable);
				bool bResult = TextResult->EvaluateAsBasicStringExpression(&ContextHelper);
				if (bResult)
				{
					for (auto& MatchesItem : ContextHelper.MatchingSearchComponents)
					{
						SearchInstance->MatchingSearchComponents.AddUnique(MatchesItem.Key, MatchesItem.Value);
					}
				}
				return bResult;
			}
			else if (const FFiBToken* FiBToken = EvalResult.GetValue().Cast<FFiBToken>())
			{
				if (FiBToken->bValue)
				{
					for (auto& MatchesItem : FiBToken->MatchesSearchQuery)
					{
						SearchInstance->MatchesSearchQuery.AddUnique(MatchesItem);
					}

					for (auto& MatchesItem : FiBToken->MatchingSearchComponents)
					{
						SearchInstance->MatchingSearchComponents.AddUnique(MatchesItem.Key, MatchesItem.Value);
					}
				}
				return FiBToken->bValue;
			}
		}
		else if (OutErrorText)
		{
			*OutErrorText = EvalResult.GetError().Text;
		}
	}

	return false;
}

void FFindInBlueprintExpressionEvaluator::MapBasicJumps()
{
	using namespace TextFilterExpressionParser;
	using TextFilterExpressionParser::FTextToken;

	JumpTable.MapBinary<FLessOrEqual>(
		[this](const FTextToken& A, const FTextToken& B, const ITextFilterExpressionContext* InContext)
	{ 
		FFiBContextHelper ContextHelper(SearchInstance->CurrentSearchable);
		bool bResult = B.EvaluateAsComplexExpression(&ContextHelper, A.GetString(), ETextFilterComparisonOperation::LessOrEqual);
		return FFiBToken(bResult, ContextHelper.MatchingSearchComponents);
	});

	JumpTable.MapBinary<FLess>(
		[this](const FTextToken& A, const FTextToken& B, const ITextFilterExpressionContext* InContext)				
	{ 
		FFiBContextHelper ContextHelper(SearchInstance->CurrentSearchable);
		bool bResult = B.EvaluateAsComplexExpression(&ContextHelper, A.GetString(), ETextFilterComparisonOperation::Less); 
		return FFiBToken(bResult, ContextHelper.MatchingSearchComponents);
	});

	JumpTable.MapBinary<FGreaterOrEqual>(
		[this](const FTextToken& A, const FTextToken& B, const ITextFilterExpressionContext* InContext)	
	{ 
		FFiBContextHelper ContextHelper(SearchInstance->CurrentSearchable);
		bool bResult = B.EvaluateAsComplexExpression(&ContextHelper, A.GetString(), ETextFilterComparisonOperation::GreaterOrEqual); 
		return FFiBToken(bResult, ContextHelper.MatchingSearchComponents);
	});

	JumpTable.MapBinary<FGreater>(
		[this](const FTextToken& A, const FTextToken& B, const ITextFilterExpressionContext* InContext)			
	{ 
		FFiBContextHelper ContextHelper(SearchInstance->CurrentSearchable);
		bool bResult = B.EvaluateAsComplexExpression(&ContextHelper, A.GetString(), ETextFilterComparisonOperation::Greater); 
		return FFiBToken(bResult, ContextHelper.MatchingSearchComponents);
	});

	JumpTable.MapBinary<FNotEqual>(
		[this](const FTextToken& A, const FTextToken& B, const ITextFilterExpressionContext* InContext)			
	{ 
		FFiBContextHelper ContextHelper(SearchInstance->CurrentSearchable);
		bool bResult = B.EvaluateAsComplexExpression(&ContextHelper, A.GetString(), ETextFilterComparisonOperation::NotEqual); 
		return FFiBToken(bResult, ContextHelper.MatchingSearchComponents);
	});

	JumpTable.MapBinary<FEqual>(
		[this](const FTextToken& A, const FTextToken& B, const ITextFilterExpressionContext* InContext)				
	{ 
		FFiBContextHelper ContextHelper(SearchInstance->CurrentSearchable);
		bool bResult = B.EvaluateAsComplexExpression(&ContextHelper, A.GetString(), ETextFilterComparisonOperation::Equal); 
		return FFiBToken(bResult, ContextHelper.MatchingSearchComponents);
	});

	JumpTable.MapPreUnary<FNot>(
		[this](const FTextToken& V, const ITextFilterExpressionContext* InContext)									
	{ 
		FFiBContextHelper ContextHelper(SearchInstance->CurrentSearchable);
		bool bResult = !V.EvaluateAsBasicStringExpression(&ContextHelper); 
		return FFiBToken(bResult, ContextHelper.MatchingSearchComponents);
	});

	JumpTable.MapPreUnary<FNot>(
		[](bool V, const ITextFilterExpressionContext* InContext)
	{
		return !V;
	});
}

void FFindInBlueprintExpressionEvaluator::MapOrBinaryJumps()
{
	using namespace TextFilterExpressionParser;

	// Core Updated
	JumpTable.MapBinary<FOr>(
		[this](const TextFilterExpressionParser::FTextToken& A, const TextFilterExpressionParser::FTextToken& B, const ITextFilterExpressionContext* InContext)				
	{ 
		FFiBContextHelper ContextHelperA(SearchInstance->CurrentSearchable);
		bool bAResult = A.EvaluateAsBasicStringExpression(&ContextHelperA);

		FFiBContextHelper ContextHelperB(SearchInstance->CurrentSearchable);
		bool bBResult = B.EvaluateAsBasicStringExpression(&ContextHelperB); 
		bool bResult = bAResult || bBResult;

		FFiBToken ResultToken(bResult);
		if (bResult)
		{
			ResultToken.MatchesSearchQuery.AddUnique(static_cast<const FImaginaryFiBData*>(InContext));

			if (bAResult)
			{
				ResultToken.MergeMatchingSearchComponents(ContextHelperA.MatchingSearchComponents);
			}

			if (bBResult)
			{
				ResultToken.MergeMatchingSearchComponents(ContextHelperB.MatchingSearchComponents);
			}
		}
		return ResultToken;
	});

	JumpTable.MapBinary<FOr>(
		[this](const TextFilterExpressionParser::FTextToken& A, bool B, const ITextFilterExpressionContext* InContext)							
	{ 
		FFiBContextHelper ContextHelper(SearchInstance->CurrentSearchable);
		bool bAResult = A.EvaluateAsBasicStringExpression(&ContextHelper);
		bool bResult = bAResult || B;

		FFiBToken ResultToken(bResult);
		if (bResult)
		{
			ResultToken.MatchesSearchQuery.AddUnique(static_cast<const FImaginaryFiBData*>(InContext));

			if (bAResult)
			{
				ResultToken.MergeMatchingSearchComponents(ContextHelper.MatchingSearchComponents);
			}
		}
		return ResultToken;
	});

	JumpTable.MapBinary<FOr>(
		[this](bool A, const TextFilterExpressionParser::FTextToken& B, const ITextFilterExpressionContext* InContext)							
	{ 
		FFiBContextHelper ContextHelper(SearchInstance->CurrentSearchable);
		bool bBResult = B.EvaluateAsBasicStringExpression(&ContextHelper);
		bool bResult = A || bBResult;

		FFiBToken ResultToken(bResult);
		if (bResult)
		{
			ResultToken.MatchesSearchQuery.AddUnique(static_cast<const FImaginaryFiBData*>(InContext));

			if (bBResult)
			{
				ResultToken.MergeMatchingSearchComponents(ContextHelper.MatchingSearchComponents);
			}
		}
		return ResultToken;
	});

	JumpTable.MapBinary<FOr>(
		[this](bool A, bool B, const ITextFilterExpressionContext* InContext)											
	{ 
		return A || B; 
	});

	// FiB Specific
	JumpTable.MapBinary<FOr>(
		[](const FFiBToken& A, const FFiBToken& B, const ITextFilterExpressionContext* InContext)
	{
		bool bResult = A.bValue || B.bValue;
		FFiBToken ResultToken(bResult);

		if (A.bValue)
		{
			ResultToken.CombineToken(A);
		}

		if (B.bValue)
		{
			ResultToken.CombineToken(B);
		}
		return ResultToken;
	});
	JumpTable.MapBinary<FOr>(
		[](const FFiBToken& A, bool B, const ITextFilterExpressionContext* InContext)								
	{ 
		bool bResult = A.bValue || B;

		FFiBToken ResultToken(bResult);
		if (A.bValue)
		{
			ResultToken.CombineToken(A);
		}
		if (B)
		{
			ResultToken.MatchesSearchQuery.AddUnique(static_cast<const FImaginaryFiBData*>(InContext));
		}
		return ResultToken;
	});
	JumpTable.MapBinary<FOr>(
		[](bool A, const FFiBToken& B, const ITextFilterExpressionContext* InContext)								
	{
		bool bResult = A || B.bValue;

		FFiBToken ResultToken(bResult);

		if (A)
		{
			ResultToken.MatchesSearchQuery.AddUnique(static_cast<const FImaginaryFiBData*>(InContext));
		}
		if (B.bValue)
		{
			ResultToken.CombineToken(B);
		}
		return ResultToken;
	});
	JumpTable.MapBinary<FOr>(
		[this](const FFiBToken& A, TextFilterExpressionParser::FTextToken B, const ITextFilterExpressionContext* InContext)								
	{
		FFiBContextHelper ContextHelper(SearchInstance->CurrentSearchable);
		bool bBResult = B.EvaluateAsBasicStringExpression(&ContextHelper);
		bool bResult = A.bValue || bBResult;
		FFiBToken ResultToken(bResult);
		if (A.bValue)
		{
			ResultToken.CombineToken(A);
		}
		if (bBResult)
		{
			ResultToken.MatchesSearchQuery.AddUnique(static_cast<const FImaginaryFiBData*>(InContext));
			ResultToken.MergeMatchingSearchComponents(ContextHelper.MatchingSearchComponents);
		}
		return ResultToken;
	});
	JumpTable.MapBinary<FOr>(
		[this](TextFilterExpressionParser::FTextToken A, const FFiBToken& B, const ITextFilterExpressionContext* InContext)
	{
		FFiBContextHelper ContextHelper(SearchInstance->CurrentSearchable);
		bool bAResult = A.EvaluateAsBasicStringExpression(&ContextHelper);
		bool bResult = bAResult || B.bValue;
		FFiBToken ResultToken(bResult);

		if (bAResult)
		{
			ResultToken.MatchesSearchQuery.AddUnique(static_cast<const FImaginaryFiBData*>(InContext));
			ResultToken.MergeMatchingSearchComponents(ContextHelper.MatchingSearchComponents);
		}
		if (B.bValue)
		{
			ResultToken.CombineToken(B);
		}
		return ResultToken;
	});
}

void FFindInBlueprintExpressionEvaluator::MapAndBinaryJumps()
{
	using namespace TextFilterExpressionParser;

	// Core Updated
	JumpTable.MapBinary<FAnd>(
		[this](const TextFilterExpressionParser::FTextToken& A, const TextFilterExpressionParser::FTextToken& B, const ITextFilterExpressionContext* InContext)				
	{ 
		FFiBContextHelper ContextHelper(SearchInstance->CurrentSearchable);
		bool bResult = A.EvaluateAsBasicStringExpression(&ContextHelper) && B.EvaluateAsBasicStringExpression(&ContextHelper); 

		FFiBToken ResultToken(bResult);
		if (bResult)
		{
			ResultToken.MatchesSearchQuery.AddUnique(static_cast<const FImaginaryFiBData*>(InContext));
			ResultToken.MergeMatchingSearchComponents(ContextHelper.MatchingSearchComponents);
		}
		return ResultToken;
	});

	JumpTable.MapBinary<FAnd>(
		[this](const TextFilterExpressionParser::FTextToken& A, bool B, const ITextFilterExpressionContext* InContext)							
	{ 
		FFiBContextHelper ContextHelper(SearchInstance->CurrentSearchable);
		bool bResult = A.EvaluateAsBasicStringExpression(&ContextHelper) && B;

		FFiBToken ResultToken(bResult);
		if (bResult)
		{
			ResultToken.MatchesSearchQuery.AddUnique(static_cast<const FImaginaryFiBData*>(InContext));
			ResultToken.MergeMatchingSearchComponents(ContextHelper.MatchingSearchComponents);
		}
		return ResultToken;
	});

	JumpTable.MapBinary<FAnd>(
		[this](bool A, const TextFilterExpressionParser::FTextToken& B, const ITextFilterExpressionContext* InContext)							
	{ 
		FFiBContextHelper ContextHelper(SearchInstance->CurrentSearchable);
		bool bResult = A && B.EvaluateAsBasicStringExpression(&ContextHelper);

		FFiBToken ResultToken(bResult);
		if (bResult)
		{
			ResultToken.MatchesSearchQuery.AddUnique(static_cast<const FImaginaryFiBData*>(InContext));
			ResultToken.MergeMatchingSearchComponents(ContextHelper.MatchingSearchComponents);
		}
		return ResultToken;
	});

	JumpTable.MapBinary<FAnd>(
		[](bool A, bool B, const ITextFilterExpressionContext* InContext)											
	{ 
		return A && B; 
	});

	// FiB Specific
	JumpTable.MapBinary<FAnd>(
		[](const FFiBToken& A, const FFiBToken& B, const ITextFilterExpressionContext* InContext)
	{
		bool bResult = A.bValue && B.bValue;
		FFiBToken ResultToken(bResult);
		if (bResult)
		{
			ResultToken.CombineToken(A);
			ResultToken.CombineToken(B);
		}
		return ResultToken;
	});
	JumpTable.MapBinary<FAnd>(
		[](const FFiBToken& A, bool B, const ITextFilterExpressionContext* InContext)							
	{ 
		bool bResult = A.bValue && B;
		FFiBToken ResultToken(bResult);
		if (bResult)
		{
			ResultToken.CombineToken(A);
		}
		if (B)
		{
			ResultToken.MatchesSearchQuery.AddUnique(static_cast<const FImaginaryFiBData*>(InContext));
		}
		return ResultToken;
	});
	JumpTable.MapBinary<FAnd>(
		[](bool A, const FFiBToken& B, const ITextFilterExpressionContext* InContext)							
	{
		bool bResult = A && B.bValue;
		FFiBToken ResultToken(bResult);
		if (A)
		{
			ResultToken.MatchesSearchQuery.AddUnique(static_cast<const FImaginaryFiBData*>(InContext));
		}
		if (bResult)
		{
			ResultToken.CombineToken(B);
		}
		return ResultToken;
	});
	JumpTable.MapBinary<FAnd>(
		[this](const FFiBToken& A, TextFilterExpressionParser::FTextToken B, const ITextFilterExpressionContext* InContext)							
	{ 
		FFiBContextHelper ContextHelper(SearchInstance->CurrentSearchable);
		bool bBResult = B.EvaluateAsBasicStringExpression(&ContextHelper);
		bool bResult = A.bValue && bBResult;
		FFiBToken ResultToken(bResult);
		if (bResult)
		{
			ResultToken.CombineToken(A);
		}
		if (bBResult)
		{
			ResultToken.MatchesSearchQuery.AddUnique(static_cast<const FImaginaryFiBData*>(InContext));
			ResultToken.MergeMatchingSearchComponents(ContextHelper.MatchingSearchComponents);
		}
		return ResultToken;
	});
	JumpTable.MapBinary<FAnd>(
		[this](TextFilterExpressionParser::FTextToken A, const FFiBToken& B, const ITextFilterExpressionContext* InContext)							
	{ 
		FFiBContextHelper ContextHelper(SearchInstance->CurrentSearchable);
		bool bAResult = A.EvaluateAsBasicStringExpression(&ContextHelper);
		bool bResult = bAResult && B.bValue;
		FFiBToken ResultToken(bResult);
		if (bAResult)
		{
			ResultToken.MatchesSearchQuery.AddUnique(static_cast<const FImaginaryFiBData*>(InContext));
			ResultToken.MergeMatchingSearchComponents(ContextHelper.MatchingSearchComponents);
		}
		if (bResult)
		{
			ResultToken.CombineToken(B);
		}
		return ResultToken;
	});
}

void FFindInBlueprintExpressionEvaluator::ConstructExpressionParser()
{
	SetupGrammar();
	MapBasicJumps();
	MapOrBinaryJumps();
	MapAndBinaryJumps();

	using namespace TextFilterExpressionParser;

	JumpTable.MapBinary<TextFilterExpressionParser::FFunction>(
		[this](const TextFilterExpressionParser::FTextToken& A, const TextFilterExpressionParser::FTextToken& B, const ITextFilterExpressionContext* InContext)	
	{
		bool bResult = false;
		if (FTokenFunctionHandler* FunctionCallback = TokenFunctionHandlers.Find(A.GetString().AsString()))
		{
			bResult = FunctionCallback->Execute(B.GetString());
		}
		else
		{
			bResult = DefaultFunctionHandler.Execute(A.GetString(), B.GetString());
		}
		FFiBToken Result(bResult, SearchInstance->LastFunctionResultMatchesSearchQuery);
		Result.MatchingSearchComponents = SearchInstance->LastFunctionMatchingSearchComponents;
		return Result;
	});
}
