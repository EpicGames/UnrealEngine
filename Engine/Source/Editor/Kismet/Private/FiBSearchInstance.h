// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "FindInBlueprintManager.h"
#include "ImaginaryBlueprintData.h"
#include "Misc/ExpressionParser.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "Misc/TextFilterUtils.h"

class FFiBSearchInstance;
class FImaginaryFiBData;
class FText;
struct FComponentUniqueDisplay;

DECLARE_DELEGATE_RetVal_TwoParams(bool, FTokenDefaultFunctionHandler, const FTextFilterString&, const FTextFilterString&);

/** Evaluates the expression the user submitted to be searched for */
class FFindInBlueprintExpressionEvaluator : public FTextFilterExpressionEvaluator
{
public:
	/** Construction and assignment */
	FFindInBlueprintExpressionEvaluator(const ETextFilterExpressionEvaluatorMode InMode, FFiBSearchInstance* InSearchInstance)
		: FTextFilterExpressionEvaluator(InMode)
		, SearchInstance(InSearchInstance)
	{
		ConstructExpressionParser();
	}

	/** Sets the default function handler, which supports generic functions which are categories or other object based on FImaginaryFiBData */
	void SetDefaultFunctionHandler(FTokenDefaultFunctionHandler InFunctionHandler)
	{
		DefaultFunctionHandler = InFunctionHandler;
	}

protected:
	/** FTextFilterExpressionEvaluator Interface */
	virtual void ConstructExpressionParser() override final;
	virtual bool EvaluateCompiledExpression(const ExpressionParser::CompileResultType& InCompiledResult, const ITextFilterExpressionContext& InContext, FText* OutErrorText) const override;
	/** End FTextFilterExpressionEvaluator Interface */

	/** Helper function to make a mapping of all basic jump operations */
	void MapBasicJumps();

	/** Helper function to make a mapping of all "Or" binary jump operations */
	void MapOrBinaryJumps();

	/** Helper function to make a mapping of all "And" binary jump operations */
	void MapAndBinaryJumps();

protected:
	/** Referenced SearchInstance that is powering this search */
	FFiBSearchInstance* SearchInstance;

	/** Fallback for all functions, Find-in-Blueprints filters into any sub-data using functions */
	FTokenDefaultFunctionHandler DefaultFunctionHandler;
};

/** Used to manage searches through imaginary Blueprints */
class FFiBSearchInstance
{
public:
	/**
	 * Starts a search query given a string and an imaginary Blueprint
	 *
	 * @param InSearchString				The string to search using
	 * @param InImaginaryBlueprintRoot		The imaginary Blueprint to search through
	 * @return								Search result shared pointer, can be used for display in the search results window
	 */
	FSearchResult StartSearchQuery(const FString& InSearchString, FImaginaryFiBDataSharedPtr InImaginaryBlueprintRoot);

	/**
	 * Starts a search query given a string and an imaginary Blueprint
	 *
	 * @param InSearchString				The string to search using
	 * @param InImaginaryBlueprintRoot		The imaginary Blueprint to search through
	 */
	void MakeSearchQuery(const FString& InSearchString, FImaginaryFiBDataSharedPtr InImaginaryBlueprintRoot);
	
	/**
	 * Runs a search query on any pending imaginary data
	 *
	 * @param InSearchString		The string to search using
	 * @param bInComplete			TRUE if a complete search of all child items should also be done, this should be FALSE when you only want to compare the string against the immediate items and not their children
	 * @return						TRUE if the search was successful.
	 */
	bool DoSearchQuery(const FString& InSearchString, bool bInComplete = true);

	/**
	 * Builds a list of imaginary items that can be targeted by a function
	 *
	 * @param InRootData						Item to find the child imaginary data that can have be sub-searched for the function
	 * @param InSearchQueryFilter				A filter to decide if the imaginary item is compatible with the function
	 * @param OutTargetPendingSearchables		List of pending imaginary items that will be sub-searched, this gets filled out by the function
	 */
	void BuildFunctionTargets(FImaginaryFiBDataSharedPtr InRootData, ESearchQueryFilter InSearchQueryFilter, TArray<FImaginaryFiBDataWeakPtr>& OutTargetPendingSearchables);

	/**
	 * Builds a list of imaginary items, using their names, that can be targeted by a function
	 *
	 * @param InRootData						Item to find the child imaginary data that can have be sub-searched for the function
	 * @param InTagName							The name of objects to find
	 * @param OutTargetPendingSearchables		List of pending imaginary items that will be sub-searched, this gets filled out by the function
	 */
	void BuildFunctionTargetsByName(FImaginaryFiBDataSharedPtr InRootData, FString InTagName, TArray<FImaginaryFiBDataWeakPtr>& OutTargetPendingSearchables);

	/**
	 * Callback when a function is called by the evaluator
	 *
	 * @param InParameterString		The parameter string to sub-search using
	 * @param InSearchQueryFilter	Effectively this is the function being called, the filter is used to determine if the current item can have the function called on it and whether it's sub items can have it called on them
	 * @return						TRUE if the function was successful at finding valid results
	 */
	bool OnFilterFunction(const FTextFilterString& A, ESearchQueryFilter InSearchQueryFilter);

	/**
	 * Callback when a default/generic function is called by the evaluator
	 *
	 * @param InFunctionName		The function name to call, will query for sub-objects with this name to do a sub-search on
	 * @param InFunctionParams		The parameter string to sub-search using
	 * @return						TRUE if the function was successful at finding valid results
	 */
	bool OnFilterDefaultFunction(const FTextFilterString& InFunctionName, const FTextFilterString& InFunctionParams);

	/** Builds a list of search results in their imaginary data form, filtered by an object type */
	void CreateFilteredResultsListFromTree(ESearchQueryFilter InSearchQueryFilter, TArray<FImaginaryFiBDataSharedPtr>& InOutValidSearchResults);

	/** Helper function to return search results given an imaginary Blueprint root */
	FSearchResult GetSearchResults(FImaginaryFiBDataSharedPtr InImaginaryBlueprintRoot);
public:
	/** Current item being searched in the Imaginary Blueprint. Must be declared as thread-safe because this is a shared object. */
	FImaginaryFiBDataWeakPtr CurrentSearchable;

	/** A going list of all imaginary items that match the search query */
	TArray< const FImaginaryFiBData* > MatchesSearchQuery;

	/** A mapping of items and their components that match the search query */
	TMultiMap< const FImaginaryFiBData*, FComponentUniqueDisplay > MatchingSearchComponents;

	/** A list of imaginary items that still need to be searched. Must be declared as thread-safe because it points to a shared object. */
	TArray<FImaginaryFiBDataWeakPtr> PendingSearchables;

	/** When a function returns on an item, this is the list of items that matched the sub-search query */
	TArray< const FImaginaryFiBData* > LastFunctionResultMatchesSearchQuery;

	/** When a function returns on an item, this is the mapping of imaginary items to components that matched the sub-search query */
	TMultiMap< const FImaginaryFiBData*, FComponentUniqueDisplay > LastFunctionMatchingSearchComponents;
};
