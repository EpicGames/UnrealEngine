// Copyright Epic Games, Inc. All Rights Reserved.

#include "FieldNotifyToggle.h"

#include "BlueprintEditor.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphSchema_K2_Actions.h"
#include "Editor/BlueprintGraph/Classes/K2Node_EditablePinBase.h"
#include "Engine/Blueprint.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/SlateDelegates.h"
#include "INotifyFieldValueChanged.h"
#include "K2Node_CallFunction.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_FunctionResult.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Layout/Visibility.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateBrush.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FieldNotifyToggle"


/*******************************************************************************
* SPaletteItemFunctionFieldNotifyToggle
*******************************************************************************/

void SPaletteItemFunctionFieldNotifyToggle::Construct(const FArguments& InArgs, TWeakPtr<FEdGraphSchemaAction> ActionPtrIn, TWeakPtr<FBlueprintEditor> InBlueprintEditor, UBlueprint* InBlueprint)
{
	ActionPtr = ActionPtrIn;
	BlueprintEditorPtr = InBlueprintEditor;
	BlueprintObjPtr = InBlueprint;
	TSharedPtr<FEdGraphSchemaAction> PaletteAction = ActionPtrIn.Pin();
	TSharedPtr<SWidget> ChildContent;
	UBlueprint* BlueprintObj = BlueprintObjPtr.Get();

	FieldNotifyOnIcon = FAppStyle::GetBrush("Kismet.VariableList.FieldNotify");
	FieldNotifyOffIcon = FAppStyle::GetBrush("Kismet.VariableList.NotFieldNotify");

	if (BlueprintEditorPtr.IsValid() && PaletteAction.IsValid())
	{
		if (PaletteAction->GetTypeId() == FEdGraphSchemaAction_K2Graph::StaticGetTypeId() && BlueprintObj)
		{
			if (UFunction* Function = StaticCastSharedPtr<FEdGraphSchemaAction_K2Graph>(PaletteAction)->GetFunction())
			{
				FunctionPtr = Function;
				if (UEdGraph* FunctionGraphPtr = StaticCastSharedPtr<FEdGraphSchemaAction_K2Graph>(PaletteAction)->EdGraph)
				{
					TArray<UK2Node_FunctionEntry*> EntryNodes;
					FunctionGraphPtr->GetNodesOfClass(EntryNodes);
					if (EntryNodes.Num())
					{
						FunctionEntryNodePtr = EntryNodes[0];
					}

					TArray<UK2Node_FunctionResult*> ResultNodes;
					FunctionGraphPtr->GetNodesOfClass(ResultNodes);
					if (ResultNodes.Num())
					{
						FunctionResultNodePtr = ResultNodes[0];
					}
				}
			}
		}
	}

	this->ChildSlot
	[
		SNew(SBorder)
		.Padding(0.0f)
		.BorderImage(FStyleDefaults::GetNoBrush())
		.Visibility(FunctionEntryNodePtr.IsValid() && FunctionPtr.IsValid() ? EVisibility::Visible : EVisibility::Collapsed)
		.IsEnabled(this, &SPaletteItemFunctionFieldNotifyToggle::GetFieldNotifyToggleEnabled)
		[
			SNew(SCheckBox)
			.ToolTipText(this, &SPaletteItemFunctionFieldNotifyToggle::GetFieldNotifyToggleToolTip)
			.OnCheckStateChanged(this, &SPaletteItemFunctionFieldNotifyToggle::OnFieldNotifyToggleFlipped)
			.IsChecked(this, &SPaletteItemFunctionFieldNotifyToggle::GetFieldNotifyToggleState)
			.Style(FAppStyle::Get(), "TransparentCheckBox")
			[
				SNew(SImage)
				.Image(this, &SPaletteItemFunctionFieldNotifyToggle::GetFieldNotifyIcon)
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]
	];
}

bool SPaletteItemFunctionFieldNotifyToggle::GetFieldNotifyToggleEnabled() const
{
	UK2Node_FunctionEntry* FunctionEntry = FunctionEntryNodePtr.Get();
	UK2Node_FunctionResult* FunctionResult = FunctionResultNodePtr.Get();
	UFunction* Function = FunctionPtr.Get();

	if (FunctionEntry != nullptr && FunctionResult != nullptr)
	{
		return Function != nullptr && IsConstFunction() && IsPureFunction() && FunctionEntry->GetAllPins().Num() == 1 && FunctionResult->GetAllPins().Num() == 2 ? true : false;
	}
	return false;
}

ECheckBoxState SPaletteItemFunctionFieldNotifyToggle::GetFieldNotifyToggleState() const
{
	UFunction* Function = FunctionPtr.Get();
	UBlueprint* BlueprintObj = BlueprintObjPtr.Get();
	if (Function == nullptr || BlueprintObj == nullptr)
	{
		return ECheckBoxState::Undetermined;
	}

	const FName FuncName = Function->GetFName();

	if (BlueprintObj && GetFieldNotifyToggleEnabled())
	{
		if (!FuncName.IsNone())
		{
			return GetMetadataBlock()->HasMetaData(FBlueprintMetadata::MD_FieldNotify) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
		else if (BlueprintObj->GeneratedClass && BlueprintObj->GeneratedClass->GetDefaultObject())
		{
			TScriptInterface<INotifyFieldValueChanged> DefaultObject = BlueprintObj->GeneratedClass->GetDefaultObject();
			return DefaultObject->GetFieldNotificationDescriptor().GetField(BlueprintObj->GeneratedClass, FuncName).IsValid() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
	}

	FBlueprintEditorUtils::RemoveFieldNotifyFromAllMetadata(BlueprintObj, FuncName);
	GetMetadataBlock()->RemoveMetaData(FBlueprintMetadata::MD_FieldNotify);
	return ECheckBoxState::Unchecked;
}

void SPaletteItemFunctionFieldNotifyToggle::OnFieldNotifyToggleFlipped(ECheckBoxState InNewState)
{
	UFunction* Function = FunctionPtr.Get();
	UBlueprint* BlueprintObj = BlueprintObjPtr.Get();
	if (!BlueprintEditorPtr.IsValid() || Function == nullptr || BlueprintObj == nullptr)
	{
		return;
	}

	FName FuncName = Function->GetFName();
	const bool bFuncIsFieldNotify = InNewState == ECheckBoxState::Checked;

	if (bFuncIsFieldNotify)
	{
		GetMetadataBlock()->SetMetaData(FBlueprintMetadata::MD_FieldNotify, FString());
	}
	else
	{
		FBlueprintEditorUtils::RemoveFieldNotifyFromAllMetadata(BlueprintObj, FuncName);
		GetMetadataBlock()->RemoveMetaData(FBlueprintMetadata::MD_FieldNotify);
	}
}

const FSlateBrush* SPaletteItemFunctionFieldNotifyToggle::GetFieldNotifyIcon() const
{
	return GetFieldNotifyToggleState() == ECheckBoxState::Checked ?
		FieldNotifyOnIcon : FieldNotifyOffIcon;
}

FText SPaletteItemFunctionFieldNotifyToggle::GetFieldNotifyToggleToolTip() const
{
	return (GetFieldNotifyToggleState() != ECheckBoxState::Checked) ? LOCTEXT("Function_not_fieldnotify_Tooltip", "Function is not Field Notify. Enable this to broadcast changes in its value.")
		: LOCTEXT("Function_is_fieldnotify_Tooltip", "Function is Field Notify. Any changes to its value will be broadcasted to listeners.");
}

FKismetUserDeclaredFunctionMetadata* SPaletteItemFunctionFieldNotifyToggle::GetMetadataBlock() const
{
	UK2Node_FunctionEntry* FunctionEntry = FunctionEntryNodePtr.Get();
	return (FunctionEntry != nullptr) ? &(FunctionEntry->MetaData) : nullptr;
}

bool SPaletteItemFunctionFieldNotifyToggle::IsConstFunction() const
{
	UK2Node_FunctionEntry* FunctionEntry = FunctionEntryNodePtr.Get();
	return (FunctionEntry != nullptr) ? (FunctionEntry->GetFunctionFlags() & FUNC_Const) != 0 : false;
}

bool SPaletteItemFunctionFieldNotifyToggle::IsPureFunction() const
{
	UK2Node_FunctionEntry* FunctionEntry = FunctionEntryNodePtr.Get();
	return (FunctionEntry != nullptr) ? (FunctionEntry->GetFunctionFlags() & FUNC_BlueprintPure) != 0 : false;
}


/*******************************************************************************
* SPaletteItemVariableFieldNotifyToggle
*******************************************************************************/

void SPaletteItemVarFieldNotifyToggle::Construct(const FArguments& InArgs, TWeakPtr<FEdGraphSchemaAction> ActionPtrIn, TWeakPtr<FBlueprintEditor> InBlueprintEditor, UBlueprint* InBlueprint)
{
	ActionPtr = ActionPtrIn;
	BlueprintEditorPtr = InBlueprintEditor;
	BlueprintObjPtr = InBlueprint;
	TSharedPtr<FEdGraphSchemaAction> PaletteAction = ActionPtrIn.Pin();
	UBlueprint* BlueprintObj = BlueprintObjPtr.Get();

	FieldNotifyOnIcon = FAppStyle::GetBrush("Kismet.VariableList.FieldNotify");
	FieldNotifyOffIcon = FAppStyle::GetBrush("Kismet.VariableList.NotFieldNotify");
	bool bShouldHaveAFieldNotifyToggle = false;
	if (BlueprintEditorPtr.IsValid() && PaletteAction.IsValid())
	{
		if (PaletteAction->GetTypeId() == FEdGraphSchemaAction_K2Var::StaticGetTypeId() && BlueprintObj)
		{
			FProperty* VariableProp = StaticCastSharedPtr<FEdGraphSchemaAction_K2Var>(PaletteAction)->GetProperty();
			FObjectProperty* VariableObjProp = CastField<FObjectProperty>(VariableProp);

			UStruct* VarSourceScope = (VariableProp ? CastChecked<UStruct>(VariableProp->GetOwner<UObject>()) : nullptr);
			const bool bIsBlueprintVariable = (VarSourceScope == BlueprintObj->SkeletonGeneratedClass);
			bShouldHaveAFieldNotifyToggle = bIsBlueprintVariable && VariableProp && FBlueprintEditorUtils::FindNewVariableIndex(BlueprintObj, VariableProp->GetFName()) != INDEX_NONE;
		}
	}

	this->ChildSlot
	[
		SNew(SBorder)
		.Padding(0.0f)
		.BorderImage(FStyleDefaults::GetNoBrush())
		.IsEnabled(bShouldHaveAFieldNotifyToggle ? true : false)
		[
			SNew(SCheckBox)
			.ToolTipText(this, &SPaletteItemVarFieldNotifyToggle::GetFieldNotifyToggleToolTip)
			.OnCheckStateChanged(this, &SPaletteItemVarFieldNotifyToggle::OnFieldNotifyToggleFlipped)
			.IsChecked(this, &SPaletteItemVarFieldNotifyToggle::GetFieldNotifyToggleState)
			.Style(FAppStyle::Get(), "TransparentCheckBox")
			[
				SNew(SImage)
				.Image(this, &SPaletteItemVarFieldNotifyToggle::GetFieldNotifyIcon)
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]
	];
}

ECheckBoxState SPaletteItemVarFieldNotifyToggle::GetFieldNotifyToggleState() const
{
	UBlueprint* BlueprintObj = BlueprintObjPtr.Get();
	TSharedPtr<FEdGraphSchemaAction> PaletteAction = ActionPtr.Pin();
	if (PaletteAction->GetTypeId() == FEdGraphSchemaAction_K2Var::StaticGetTypeId() && BlueprintObj)
	{
		TSharedPtr<FEdGraphSchemaAction_K2Var> VarAction = StaticCastSharedPtr<FEdGraphSchemaAction_K2Var>(PaletteAction);
		if (FProperty* VariableProperty = VarAction->GetProperty())
		{
			const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(BlueprintObj, VarAction->GetVariableName());
			if (VarIndex != INDEX_NONE)
			{
				return VariableProperty->HasMetaData(FBlueprintMetadata::MD_FieldNotify) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
			else if (BlueprintObj->GeneratedClass && BlueprintObj->GeneratedClass->ImplementsInterface(UNotifyFieldValueChanged::StaticClass()) && BlueprintObj->GeneratedClass->GetDefaultObject())
			{
				TScriptInterface<INotifyFieldValueChanged> DefaultObject = BlueprintObj->GeneratedClass->GetDefaultObject();
				return DefaultObject->GetFieldNotificationDescriptor().GetField(BlueprintObj->GeneratedClass, VarAction->GetVariableName()).IsValid() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}

		}
	}

	return ECheckBoxState::Unchecked;
}

void SPaletteItemVarFieldNotifyToggle::OnFieldNotifyToggleFlipped(ECheckBoxState InNewState)
{
	if (!BlueprintEditorPtr.IsValid())
	{
		return;
	}

	UBlueprint* BlueprintObj = BlueprintObjPtr.Get();
	TSharedPtr<FEdGraphSchemaAction> PaletteAction = ActionPtr.Pin();
	if (PaletteAction->GetTypeId() == FEdGraphSchemaAction_K2Var::StaticGetTypeId() && BlueprintObj)
	{
		TSharedPtr<FEdGraphSchemaAction_K2Var> VarAction = StaticCastSharedPtr<FEdGraphSchemaAction_K2Var>(PaletteAction);
		const bool bVariableIsFieldNotify = InNewState == ECheckBoxState::Checked;

		bool IsLocalVariable = VarAction->GetProperty() && (VarAction->GetProperty()->GetOwner<UFunction>() != NULL);
		ensure(!IsLocalVariable);

		if (bVariableIsFieldNotify)
		{
			FBlueprintEditorUtils::SetBlueprintVariableMetaData(BlueprintObj, VarAction->GetVariableName(), NULL, FBlueprintMetadata::MD_FieldNotify, TEXT(""));
		}
		else
		{
			FBlueprintEditorUtils::RemoveFieldNotifyFromAllMetadata(BlueprintObj, VarAction->GetVariableName());
			FBlueprintEditorUtils::RemoveBlueprintVariableMetaData(BlueprintObj, VarAction->GetVariableName(), NULL, FBlueprintMetadata::MD_FieldNotify);
		}
	}
}

const FSlateBrush* SPaletteItemVarFieldNotifyToggle::GetFieldNotifyIcon() const
{
	return GetFieldNotifyToggleState() == ECheckBoxState::Checked ?
		FieldNotifyOnIcon : FieldNotifyOffIcon;
}

FText SPaletteItemVarFieldNotifyToggle::GetFieldNotifyToggleToolTip() const
{
	return (GetFieldNotifyToggleState() != ECheckBoxState::Checked) ?
		LOCTEXT("Variable_not_fieldnotify_Tooltip", "Variable is not Field Notify. Enable this to broadcast changes in its value.")
		: LOCTEXT("Variable_is_fieldnotify_Tooltip", "Variable is Field Notify. Any changes to its value will be broadcasted to listeners.");
}

#undef LOCTEXT_NAMESPACE
