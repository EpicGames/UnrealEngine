// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Attribute.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FBlueprintEditor;
struct FEdGraphSchemaAction;
struct FKismetUserDeclaredFunctionMetadata;
struct FSlateBrush;
class SWidget;
class UBlueprint;
class UFunction;
class UK2Node_FunctionEntry;
class UK2Node_FunctionResult;
enum class ECheckBoxState : uint8;

class SPaletteItemFunctionFieldNotifyToggle : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPaletteItemFunctionFieldNotifyToggle) {}
	SLATE_END_ARGS()
	/**
	 * Constructs a field notify toggle widget
	 *
	 * @param  InArgs			A set of slate arguments, defined above.
	 * @param  ActionPtrIn		The FEdGraphSchemaAction that the parent item represents.
	 * @param  InBlueprintEdPtr	A pointer to the blueprint editor that the palette belongs to.
	 * @param  InBlueprint	A pointer to the blueprint being edited.
	 */
	void Construct(const FArguments& InArgs, TWeakPtr<FEdGraphSchemaAction> ActionPtrIn, TWeakPtr<FBlueprintEditor> InBlueprintEditor, UBlueprint* InBlueprint);

private:
	bool GetFieldNotifyToggleEnabled() const;
	ECheckBoxState GetFieldNotifyToggleState() const;
	void OnFieldNotifyToggleFlipped(ECheckBoxState InNewState);
	const FSlateBrush* GetFieldNotifyIcon() const;
	FText GetFieldNotifyToggleToolTip() const;

	FKismetUserDeclaredFunctionMetadata* GetMetadataBlock() const;
	bool IsConstFunction() const;
	bool IsPureFunction() const;

private:
	/** The action that the owning palette entry represents */
	TWeakPtr<FEdGraphSchemaAction> ActionPtr;

	/** Pointer back to the blueprint editor that owns this, optional because of diff and merge views: */
	TWeakPtr<FBlueprintEditor>     BlueprintEditorPtr;

	/** Pointer back to the blueprint that is being displayed: */
	TWeakObjectPtr<UBlueprint> BlueprintObjPtr;

	/** Pointer to the entry node of function */
	TWeakObjectPtr<UK2Node_FunctionEntry> FunctionEntryNodePtr;

	/** Pointer to a result node of function */
	TWeakObjectPtr<UK2Node_FunctionResult> FunctionResultNodePtr;

	/** Pointer to the function item */
	TWeakObjectPtr<UFunction> FunctionPtr;

	/** Pointer to the icon representing the active field notify state. */
	const FSlateBrush* FieldNotifyOnIcon;

	/** Pointer to the icon representing the inactive field notify state. */
	const FSlateBrush* FieldNotifyOffIcon;
};


/*******************************************************************************
* SPaletteItemVariableFieldNotifyToggle
*******************************************************************************/

class SPaletteItemVarFieldNotifyToggle : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPaletteItemVarFieldNotifyToggle) {}
	SLATE_END_ARGS()

	/**
	 * Constructs a field notify toggle widget
	 *
	 * @param  InArgs			A set of slate arguments, defined above.
	 * @param  ActionPtrIn		The FEdGraphSchemaAction that the parent item represents.
	 * @param  BlueprintEdPtrIn	A pointer to the blueprint editor that the palette belongs to.
	 * @param  InBlueprint	A pointer to the blueprint being edited.
	 */
	void Construct(const FArguments& InArgs, TWeakPtr<FEdGraphSchemaAction> ActionPtrIn, TWeakPtr<FBlueprintEditor> InBlueprintEditor, UBlueprint* InBlueprint);

private:
	ECheckBoxState GetFieldNotifyToggleState() const;
	void OnFieldNotifyToggleFlipped(ECheckBoxState InNewState);
	const FSlateBrush* GetFieldNotifyIcon() const;
	FText GetFieldNotifyToggleToolTip() const;

private:
	/** The action that the owning palette entry represents */
	TWeakPtr<FEdGraphSchemaAction> ActionPtr;

	/** Pointer back to the blueprint editor that owns this, optional because of diff and merge views: */
	TWeakPtr<FBlueprintEditor>     BlueprintEditorPtr;

	/** Pointer back to the blueprint that is being displayed: */
	TWeakObjectPtr<UBlueprint> BlueprintObjPtr;

	/** Pointer to the icon representing the active field notify state. */
	const FSlateBrush* FieldNotifyOnIcon;

	/** Pointer to the icon representing the inactive field notify state. */
	const FSlateBrush* FieldNotifyOffIcon;
};