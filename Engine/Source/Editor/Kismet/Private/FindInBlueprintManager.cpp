// Copyright Epic Games, Inc. All Rights Reserved.

#include "FindInBlueprintManager.h"
#include "Misc/CoreMisc.h"
#include "Misc/MessageDialog.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "Misc/Paths.h"
#include "Async/ParallelFor.h"
#include "Async/TaskGraphInterfaces.h"
#include "Misc/ScopeLock.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Misc/FeedbackContext.h"
#include "Modules/ModuleManager.h"
#include "UObject/SavePackage.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealType.h"
#include "Misc/PackageName.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonReader.h"
#include "Policies/PrettyJsonPrintPolicy.h"
#include "Serialization/JsonSerializer.h"
#include "Types/SlateEnums.h"
#include "Settings/EditorStyleSettings.h"
#include "Engine/Level.h"
#include "Components/ActorComponent.h"
#include "AssetRegistry/AssetData.h"
#include "EdGraph/EdGraphSchema.h"
#include "ISourceControlModule.h"
#include "Editor.h"
#include "Misc/FileHelper.h"
#include "FileHelpers.h"
#include "EdGraphSchema_K2.h"
#include "K2Node_FunctionEntry.h"
#include "Styling/AppStyle.h"
#include "Framework/Docking/TabManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#include "BlueprintEditorSettings.h"
#include "Blueprint/BlueprintExtension.h"
#include "Engine/SimpleConstructionScript.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ImaginaryBlueprintData.h"
#include "FiBSearchInstance.h"
#include "Misc/ConfigCacheIni.h"
#include "BlueprintAssetHandler.h"

#include "JsonObjectConverter.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Application/ThrottleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FindInBlueprintManager)

DEFINE_LOG_CATEGORY(LogFindInBlueprint);
CSV_DEFINE_CATEGORY(FindInBlueprint, false);

#define LOCTEXT_NAMESPACE "FindInBlueprintManager"

FFindInBlueprintSearchManager* FFindInBlueprintSearchManager::Instance = NULL;

const FText FFindInBlueprintSearchTags::FiB_Properties = LOCTEXT("Properties", "Properties");

const FText FFindInBlueprintSearchTags::FiB_Components = LOCTEXT("Components", "Components");
const FText FFindInBlueprintSearchTags::FiB_IsSCSComponent = LOCTEXT("IsSCSComponent", "IsSCSComponent");

const FText FFindInBlueprintSearchTags::FiB_Nodes = LOCTEXT("Nodes", "Nodes");

const FText FFindInBlueprintSearchTags::FiB_SchemaName = LOCTEXT("SchemaName", "SchemaName");

const FText FFindInBlueprintSearchTags::FiB_UberGraphs = LOCTEXT("Uber", "Uber");
const FText FFindInBlueprintSearchTags::FiB_Functions = LOCTEXT("Functions", "Functions");
const FText FFindInBlueprintSearchTags::FiB_Macros = LOCTEXT("Macros", "Macros");
const FText FFindInBlueprintSearchTags::FiB_SubGraphs = LOCTEXT("Sub", "Sub");
const FText FFindInBlueprintSearchTags::FiB_ExtensionGraphs = LOCTEXT("ExtGraphs", "ExtGraphs");
const FText FFindInBlueprintSearchTags::FiB_Extensions = LOCTEXT("Extensions", "Extensions");

const FText FFindInBlueprintSearchTags::FiB_Name = LOCTEXT("Name", "Name");
const FText FFindInBlueprintSearchTags::FiB_NativeName = LOCTEXT("NativeName", "Native Name");
const FText FFindInBlueprintSearchTags::FiB_ClassName = LOCTEXT("ClassName", "ClassName");
const FText FFindInBlueprintSearchTags::FiB_NodeGuid = LOCTEXT("NodeGuid", "NodeGuid");
const FText FFindInBlueprintSearchTags::FiB_Tooltip = LOCTEXT("Tooltip", "Tooltip");
const FText FFindInBlueprintSearchTags::FiB_DefaultValue = LOCTEXT("DefaultValue", "DefaultValue");
const FText FFindInBlueprintSearchTags::FiB_Description = LOCTEXT("Description", "Description");
const FText FFindInBlueprintSearchTags::FiB_Comment = LOCTEXT("Comment", "Comment");
const FText FFindInBlueprintSearchTags::FiB_Path = LOCTEXT("Path", "Path");
const FText FFindInBlueprintSearchTags::FiB_ParentClass = LOCTEXT("ParentClass", "ParentClass");
const FText FFindInBlueprintSearchTags::FiB_Interfaces = LOCTEXT("Interfaces", "Interfaces");
const FText FFindInBlueprintSearchTags::FiB_FuncOriginClass = LOCTEXT("FuncOriginClass", "FuncOriginClass");

const FText FFindInBlueprintSearchTags::FiB_Pins = LOCTEXT("Pins", "Pins");
const FText FFindInBlueprintSearchTags::FiB_PinCategory = LOCTEXT("PinCategory", "PinCategory");
const FText FFindInBlueprintSearchTags::FiB_PinSubCategory = LOCTEXT("SubCategory", "SubCategory");
const FText FFindInBlueprintSearchTags::FiB_ObjectClass = LOCTEXT("ObjectClass", "ObjectClass");
const FText FFindInBlueprintSearchTags::FiB_IsArray = LOCTEXT("IsArray", "IsArray");
const FText FFindInBlueprintSearchTags::FiB_IsReference = LOCTEXT("IsReference", "IsReference");
const FText FFindInBlueprintSearchTags::FiB_Glyph = LOCTEXT("Glyph", "Glyph");
const FText FFindInBlueprintSearchTags::FiB_GlyphStyleSet = LOCTEXT("GlyphStyleSet", "GlyphStyleSet");
const FText FFindInBlueprintSearchTags::FiB_GlyphColor = LOCTEXT("GlyphColor", "GlyphColor");

const FText FFindInBlueprintSearchTags::FiBMetaDataTag = LOCTEXT("FiBMetaDataTag", "!!FiBMD");

const FString FFiBMD::FiBSearchableMD = TEXT("BlueprintSearchable");
const FString FFiBMD::FiBSearchableShallowMD = TEXT("BlueprintSearchableShallow");
const FString FFiBMD::FiBSearchableExplicitMD = TEXT("BlueprintSearchableExplicit");
const FString FFiBMD::FiBSearchableHiddenExplicitMD = TEXT("BlueprintSearchableHiddenExplicit");
const FString FFiBMD::FiBSearchableFormatVersionMD = TEXT("BlueprintSearchableFormatVersion");

/** Attempts to initiate a rebuild of a random subset of the index cache each tick. */
static TAutoConsoleVariable<bool> CVarRandomSearchIndexCacheRebuildEachTick(
	TEXT("bp.RandomSearchIndexCacheRebuildEachTick"),
	false,
	TEXT("Attempts to initiate a random index cache update each tick (for debugging purposes)"),
	ECVF_Default
);

/* Return the outer of the specified object that is a direct child of a package */
inline UObject* GetAssetObject(UObject* InObject)
{
	UObject* AssetObject = InObject;
	while (AssetObject && !AssetObject->GetOuter()->IsA<UPackage>())
	{
		AssetObject = AssetObject->GetOuter();
	}
	return AssetObject;
}

////////////////////////////////////
// FSearchDataVersionInfo
FSearchDataVersionInfo FSearchDataVersionInfo::Current()
{
	return
	{
		.FiBDataVersion = EFiBVersion::FIB_VER_LATEST,
		.CustomVersions = FCurrentCustomVersions::GetAll()
	};
}

////////////////////////////////////
// FStreamSearch

FStreamSearch::FStreamSearch(const FString& InSearchValue, const FStreamSearchOptions& InSearchOptions)
	: SearchValue(InSearchValue)
	, SearchOptions(InSearchOptions)
	, BlueprintCountBelowVersion(0)
	, bThreadCompleted(false)
	, StopTaskCounter(0)
{
	// Unique identifier for this search, used to generate a unique label for profiling and debugging.
	static int32 GlobalSearchCounter = 0;
	SearchId = GlobalSearchCounter++;

	// Create a uniquely-named thread to ensure disambiguation in the thread view and assist with debugging multiple instances running in parallel.
	Thread = TUniquePtr<FRunnableThread>(FRunnableThread::Create( this, *FString::Printf(TEXT("FStreamSearch_%d"), SearchId), 0, TPri_BelowNormal ));
}

bool FStreamSearch::Init()
{
	return true;
}

uint32 FStreamSearch::Run()
{
	const double StartTime = FPlatformTime::Seconds();
	CSV_EVENT(FindInBlueprint, TEXT("FStreamSearch_%d START"), SearchId);

	FFindInBlueprintSearchManager::Get().BeginSearchQuery(this);

	TFunction<void(const FSearchResult&)> OnResultReady = [this](const FSearchResult& Result) {
		FScopeLock ScopeLock(&SearchCriticalSection);
		ItemsFound.Add(Result);
	};

	// Searching comes to an end if it is requested using the StopTaskCounter or continuing the search query yields no results
	FSearchData QueryResult;
	while (FFindInBlueprintSearchManager::Get().ContinueSearchQuery(this, QueryResult))
	{
		if (QueryResult.ImaginaryBlueprint.IsValid())
		{
			// If the Blueprint is below the version, add it to a list. The search will still proceed on this Blueprint
			if (QueryResult.VersionInfo.FiBDataVersion < SearchOptions.MinimiumVersionRequirement)
			{
				++BlueprintCountBelowVersion;
			}

			TSharedPtr< FFiBSearchInstance > SearchInstance(new FFiBSearchInstance);
			FSearchResult SearchResult;
			if (SearchOptions.ImaginaryDataFilter != ESearchQueryFilter::AllFilter)
			{
				SearchInstance->MakeSearchQuery(*SearchValue, QueryResult.ImaginaryBlueprint);
				SearchInstance->CreateFilteredResultsListFromTree(SearchOptions.ImaginaryDataFilter, FilteredImaginaryResults);
				SearchResult = SearchInstance->GetSearchResults(QueryResult.ImaginaryBlueprint);
			}
			else
			{
				SearchResult = SearchInstance->StartSearchQuery(*SearchValue, QueryResult.ImaginaryBlueprint);
			}

			// If there are children, add the item to the search results
			if(SearchResult.IsValid() && SearchResult->Children.Num() != 0)
			{
				OnResultReady(SearchResult);
			}
		}
	}

	// Ensure that the FiB Manager knows that we are done searching
	FFindInBlueprintSearchManager::Get().EnsureSearchQueryEnds(this);

	bThreadCompleted = true;

	CSV_EVENT(FindInBlueprint, TEXT("FStreamSearch_%d END"), SearchId);
	UE_LOGF(LogFindInBlueprint, Log, "Search completed in %0.2f seconds.", FPlatformTime::Seconds() - StartTime);

	return 0;
}

void FStreamSearch::Stop()
{
	StopTaskCounter.Increment();
}

void FStreamSearch::Exit()
{

}

void FStreamSearch::EnsureCompletion()
{
	{
		FScopeLock CritSectionLock(&SearchCriticalSection);
		ItemsFound.Empty();
	}

	// Signal to the search thread to stop its operation.
	Stop();

	// Wait until the search thread has signaled its completion.
	while (!IsComplete())
	{
		// Async tasks may have been registered to this thread (e.g. FSearchableValueInfo::GetDisplayText),
		// so make sure we process those to unblock the stream search thread. Otherwise we can deadlock below.
		FTaskGraphInterface::Get().ProcessThreadUntilIdle(FTaskGraphInterface::Get().GetCurrentThreadIfKnown());

		// Yield time to other threads, including the search thread.
		FPlatformProcess::Sleep(0.1f);
	}

	// Wait for the search thread to terminate.
	Thread->WaitForCompletion();
}

bool FStreamSearch::IsComplete() const
{
	return bThreadCompleted;
}

bool FStreamSearch::WasStopped() const
{
	return StopTaskCounter.GetValue() > 0;
}

void FStreamSearch::GetFilteredItems(TArray<FSearchResult>& OutItemsFound)
{
	FScopeLock ScopeLock(&SearchCriticalSection);
	OutItemsFound.Append(ItemsFound);
	ItemsFound.Empty();
}

float FStreamSearch::GetPercentComplete() const
{
	return FFindInBlueprintSearchManager::Get().GetPercentComplete(this);
}

void FStreamSearch::GetFilteredImaginaryResults(TArray<FImaginaryFiBDataSharedPtr>& OutFilteredImaginaryResults)
{
	OutFilteredImaginaryResults = MoveTemp(FilteredImaginaryResults);
}

/** Temporarily forces all nodes and pins to use non-friendly names, forces all schema to have nodes clear their cached values so they will re-cache, and then reverts at the end */
struct FTemporarilyUseFriendlyNodeTitles
{
	FTemporarilyUseFriendlyNodeTitles()
	{
		UEditorStyleSettings* EditorSettings = GetMutableDefault<UEditorStyleSettings>();

		// Cache the value of bShowFriendlyNames, we will force it to true for gathering BP search data and then restore it
		bCacheShowFriendlyNames = EditorSettings->bShowFriendlyNames;

		EditorSettings->bShowFriendlyNames = true;
		ForceVisualizationCacheClear();
	}

	~FTemporarilyUseFriendlyNodeTitles()
	{
		UEditorStyleSettings* EditorSettings = GetMutableDefault<UEditorStyleSettings>();
		EditorSettings->bShowFriendlyNames = bCacheShowFriendlyNames;
		ForceVisualizationCacheClear();
	}

	/** Go through all Schemas and force a visualization cache clear, forcing nodes to refresh their titles */
	void ForceVisualizationCacheClear()
	{
		// Only do the purge if the state was changed
		if (!bCacheShowFriendlyNames)
		{
			// Find all Schemas and force a visualization cache clear
			for ( TObjectIterator<UEdGraphSchema> SchemaIt(RF_NoFlags); SchemaIt; ++SchemaIt)
			{
				SchemaIt->ForceVisualizationCacheClear();
			}
		}
	}

private:
	/** Cached state of ShowFriendlyNames in EditorSettings */
	bool bCacheShowFriendlyNames;
};

/** Helper functions for serialization of types to and from an FString */
namespace FiBSerializationHelpers
{
	/**
	* Helper function to handle properly encoding and serialization of a type into an FString
	*
	* @param InValue				Value to serialize
	* @param bInIncludeSize		If true, include the size of the type. This will place an int32
									before the value in the FString. This is needed for non-basic types
									because everything is stored in an FString and is impossible to distinguish
	*/
	template<class Type>
	const FString Serialize(Type& InValue, bool bInIncludeSize)
	{
		TArray<uint8> SerializedData;
		FMemoryWriter Ar(SerializedData);

		Ar << InValue;
		Ar.Close();
		FString Result = BytesToString(SerializedData.GetData(), SerializedData.Num());

		// If the size is included, prepend it onto the Result string.
		if(bInIncludeSize)
		{
			SerializedData.Empty();
			FMemoryWriter ArWithLength(SerializedData);
			int32 Length = Result.Len();
			ArWithLength << Length;

			Result = BytesToString(SerializedData.GetData(), SerializedData.Num()) + Result;
		}
		return Result;
	}

	/** Helper function to handle properly decoding of uint8 arrays so they can be deserialized as their respective types */
	void DecodeFromStream(FBufferReader& InStream, int32 InBytes, TArray<uint8>& OutDerivedData)
	{
		// Read, as a byte string, the number of characters composing the Lookup Table for the Json.
		FString SizeOfDataAsHex;
		SizeOfDataAsHex.GetCharArray().AddUninitialized(InBytes + 1);
		SizeOfDataAsHex.GetCharArray()[InBytes] = TEXT('\0');
		InStream.Serialize((char*)SizeOfDataAsHex.GetCharArray().GetData(), sizeof(TCHAR) * InBytes);

		// Convert the number (which is stored in 1 serialized byte per TChar) into an int32
		OutDerivedData.Empty();
		OutDerivedData.AddUninitialized(InBytes);
		StringToBytes(SizeOfDataAsHex, OutDerivedData.GetData(), InBytes);
	}

	/** Helper function to deserialize from a Stream the sizeof the templated type */
	template<class Type>
	Type Deserialize(FBufferReader& InStream)
	{
		TArray<uint8> DerivedData;
		DecodeFromStream(InStream, sizeof(Type), DerivedData);

		FMemoryReader SizeOfDataAr(DerivedData);
		SizeOfDataAr.SetCustomVersions(InStream.GetCustomVersions());

		Type ReturnValue;
		SizeOfDataAr << ReturnValue;
		return ReturnValue;
	}

	/** Helper function to deserialize from a Stream a certain number of bytes */
	template<class Type>
	Type Deserialize(FBufferReader& InStream, int32 InBytes)
	{
		TArray<uint8> DerivedData;
		DecodeFromStream(InStream, InBytes, DerivedData);

		FMemoryReader SizeOfDataAr(DerivedData);
		SizeOfDataAr.SetCustomVersions(InStream.GetCustomVersions());

		Type ReturnValue;
		SizeOfDataAr << ReturnValue;
		return ReturnValue;
	}

	/** Helper function to validate search data version info */
	bool ValidateSearchDataVersionInfo(const FSearchDataVersionInfo& InVersionInfo)
	{
		return InVersionInfo.FiBDataVersion != EFiBVersion::FIB_VER_NONE && InVersionInfo.CustomVersions.GetAllVersions().Num() > 0;
	}

	/** Helper function to validate and/or deserialize version info if necessary */
	bool ValidateSearchDataVersionInfo(const FString& InAssetPath, const FString& InFiBData, FSearchDataVersionInfo& InOutVersionInfo)
	{
		CSV_SCOPED_TIMING_STAT(FindInBlueprint, ValidateSearchDataVersionInfo);

		if (InOutVersionInfo.FiBDataVersion == EFiBVersion::FIB_VER_NONE)
		{
			// Deserialize the FiB data version
			const int32 FiBDataLength = InFiBData.Len();
			if (ensureMsgf(FiBDataLength > 0, TEXT("Versioned search data was zero length!")))
			{
				FBufferReader ReaderStream((void*)*InFiBData, FiBDataLength * sizeof(TCHAR), false);
				InOutVersionInfo.FiBDataVersion = FiBSerializationHelpers::Deserialize<int32>(ReaderStream);
			}
		}

		if (InOutVersionInfo.CustomVersions.GetAllVersions().Num() == 0)
		{
			// Determine the custom versions that the asset package was last serialized with
			FString PackageFilename;
			const FString PackageName = FPackageName::ObjectPathToPackageName(InAssetPath);
			if (ensureMsgf(FPackageName::DoesPackageExist(PackageName, &PackageFilename), TEXT("FiB: Failed to map package to filename.")))
			{
				// Open a new file archive for reading
				FArchive* PackageFile = IFileManager::Get().CreateFileReader(*PackageFilename);
				if (ensureMsgf(PackageFile != nullptr, TEXT("FiB: Unable to open package to read file summary.")))
				{
					// Read the package file summary
					FPackageFileSummary PackageFileSummary;
					*PackageFile << PackageFileSummary;

					// Close the file
					delete PackageFile;

					// Record all custom versions from the package file summary
					InOutVersionInfo.CustomVersions = PackageFileSummary.GetCustomVersionContainer();
				}
			}
		}

		return ValidateSearchDataVersionInfo(InOutVersionInfo);
	}
}

namespace BlueprintSearchMetaDataHelpers
{
	/** Cache structure of searchable metadata and sub-properties relating to a Property */
	struct FSearchableProperty
	{
		FProperty* TargetProperty;
		bool bIsSearchableMD;
		bool bIsShallowSearchableMD;
		bool bIsMarkedNotSearchableMD;
		EFiBVersion MinDataFormatVersion;
		TArray<FSearchableProperty> ChildProperties;
	};

	/** Json Writer used for serializing FText's in the correct format for Find-in-Blueprints */
	template<class PrintPolicy>
	class TFindInBlueprintJsonStringWriter : public TJsonStringWriter<PrintPolicy>
	{
	public:
		typedef PrintPolicy InnerPrintPolicy;

		TFindInBlueprintJsonStringWriter(FString* const InOutString, int32 InFormatVersion)
			: TJsonStringWriter<PrintPolicy>(InOutString, 0)
			, CachedFormatVersion(InFormatVersion)
		{
		}

		using TJsonStringWriter<PrintPolicy>::WriteObjectStart;

		const int32& GetFormatVersion() const
		{
			return CachedFormatVersion;
		}

		void WriteObjectStart( const FText& Identifier )
		{
			check( this->Stack.Top() == EJson::Object );
			WriteIdentifier( Identifier );

			PrintPolicy::WriteLineTerminator(this->Stream);
			PrintPolicy::WriteTabs(this->Stream, this->IndentLevel);
			PrintPolicy::WriteChar(this->Stream, TCHAR('{'));
			++(this->IndentLevel);
			this->Stack.Push( EJson::Object );
			this->PreviousTokenWritten = EJsonToken::CurlyOpen;
		}

		void WriteArrayStart( const FText& Identifier )
		{
			check( this->Stack.Top() == EJson::Object );
			WriteIdentifier( Identifier );

			PrintPolicy::WriteSpace( this->Stream );
			PrintPolicy::WriteChar(this->Stream, TCHAR('['));
			++(this->IndentLevel);
			this->Stack.Push( EJson::Array );
			this->PreviousTokenWritten = EJsonToken::SquareOpen;
		}

		using TJsonStringWriter<PrintPolicy>::WriteValueOnly;

		EJsonToken WriteValueOnly(const FText& Value)
		{
			WriteTextValue(Value);
			return EJsonToken::String;
		}

		template <class FValue>
		void WriteValue( const FText& Identifier, FValue Value )
		{
			check( this->Stack.Top() == EJson::Object );
			WriteIdentifier( Identifier );

			PrintPolicy::WriteSpace(this->Stream);
			this->PreviousTokenWritten = this->WriteValueOnly( Value );
		}

	protected:
		virtual void WriteTextValue( const FText& Text )
		{
			TJsonStringWriter<PrintPolicy>::WriteStringValue(Text.ToString());
		}

		FORCEINLINE void WriteIdentifier( const FText& Identifier )
		{
			this->WriteCommaIfNeeded();
			PrintPolicy::WriteLineTerminator(this->Stream);

			PrintPolicy::WriteTabs(this->Stream, this->IndentLevel);

			WriteTextValue( Identifier );
			PrintPolicy::WriteChar(this->Stream, TCHAR(':'));
		}

	public:
		/** Cached mapping of all searchable properties that have been discovered while gathering searchable data for the current Blueprint */
		TMap<UStruct*, TArray<FSearchableProperty>> CachedPropertyMapping;

	private:
		/** Cached version used to determine the data serialization format */
		int32 CachedFormatVersion;
	};

	/** Json Writer used for serializing FText's in the correct format for Find-in-Blueprints */
	class FFindInBlueprintJsonWriter : public TFindInBlueprintJsonStringWriter<TCondensedJsonPrintPolicy<TCHAR>>
	{
	public:
		FFindInBlueprintJsonWriter(FString* const InOutString, int32 InFormatVersion)
			:TFindInBlueprintJsonStringWriter<TCondensedJsonPrintPolicy<TCHAR>>(InOutString, InFormatVersion)
			,JsonOutput(InOutString)
		{
		}

		virtual bool Close() override
		{
			// This will copy the JSON output to the string given as input (must do this first)
			bool bResult = TFindInBlueprintJsonStringWriter<TCondensedJsonPrintPolicy<TCHAR>>::Close();

			// Build the search metadata string for the asset tag (version + LUT + JSON)
			int32 DataVersion = GetFormatVersion();
			*JsonOutput = FiBSerializationHelpers::Serialize(DataVersion, false)
				+ FiBSerializationHelpers::Serialize(LookupTable, true)
				+ MoveTemp(*JsonOutput);

			return bResult;
		}

	protected:
		virtual void WriteStringValue(const FString& String) override
		{
			// We just want to make sure all strings are converted into FText hex strings, used by the FiB system
			WriteTextValue(FText::FromString(String));
		}

		virtual void WriteStringValue(FStringView String) override
		{
			// We just want to make sure all strings are converted into FText hex strings, used by the FiB system
			WriteTextValue(FText::FromStringView(String));
		}
		
		virtual void WriteTextValue(const FText& Text) override
		{
			// Check to see if the value has already been added.
			if (int32* TableLookupValuePtr = ReverseLookupTable.Find(Text))
			{
				TFindInBlueprintJsonStringWriter<TCondensedJsonPrintPolicy<TCHAR>>::WriteStringValue(FString::FromInt(*TableLookupValuePtr));
			}
			else
			{
				// Add the FText to the table and write to the Json the ID to look the item up using
				int32 TableLookupValue = LookupTable.Num();
				{
					LookupTable.Add(TableLookupValue, Text);
					ReverseLookupTable.Add(Text, TableLookupValue);
				}
				TFindInBlueprintJsonStringWriter<TCondensedJsonPrintPolicy<TCHAR>>::WriteStringValue(FString::FromInt(TableLookupValue));
			}
		}

	private:
		// Output stream
		FString* JsonOutput;

		// This gets serialized
		TMap< int32, FText > LookupTable;

		// This is just locally needed for the write, to lookup the integer value by using the string of the FText
		/** Case sensitive hashing function for TMap */
		struct FTextKeyMapFuncs : BaseKeyFuncs<int32, FText, /*bInAllowDuplicateKeys*/false>
		{
			static const FString& GetSourceString(const FText& InText)
			{
				const FString* SourceString = FTextInspector::GetSourceString(InText);
				check(SourceString);
				return *SourceString;
			}
			static const FText& GetSetKey(const TPair<FText, int32>& Element)
			{
				return Element.Key;
			}
			static bool Matches(const FText& A, const FText& B)
			{
				return GetSourceString(A).Equals(GetSourceString(B), ESearchCase::CaseSensitive);
			}
			static uint32 GetKeyHash(const FText& Key)
			{
				return FLocKey::ProduceHash(GetSourceString(Key));
			}
		};
		TMap< FText, int32, FDefaultSetAllocator, FTextKeyMapFuncs > ReverseLookupTable;
	};

	/**
	 * Checks if Json value is searchable, eliminating data that not considered useful to search for
	 *
	 * @param InJsonValue		The Json value object to examine for searchability
	 * @return					TRUE if the value should be searchable
	 */
	bool CheckIfJsonValueIsSearchable( TSharedPtr< FJsonValue > InJsonValue, const FBPVariableDescription& InVariableDescription)
	{
		/** Check for interesting values
		 *  booleans are not interesting, there are a lot of them
		 *  strings are not interesting if they are empty
		 *  numbers are not interesting if they are 0
		 *  arrays are not interesting if they are empty or if they are filled with un-interesting types
		 *  objects may not have interesting values when dug into
		 */
		bool bValidPropertyValue = true;
		if(InJsonValue->Type == EJson::Boolean || InJsonValue->Type == EJson::None || InJsonValue->Type == EJson::Null)
		{
			bValidPropertyValue = false;
		}
		else if(InJsonValue->Type == EJson::String)
		{
			FString temp = InJsonValue->AsString();
			if(InJsonValue->AsString().IsEmpty())
			{
				bValidPropertyValue = false;
			}
		}
		else if(InJsonValue->Type == EJson::Number)
		{
			if(InJsonValue->AsNumber() == 0.0)
			{
				bValidPropertyValue = false;
			}
		}
		else if(InJsonValue->Type == EJson::Array)
		{
			const TArray<TSharedPtr<FJsonValue>>& JsonArray = InJsonValue->AsArray();
			if(JsonArray.Num() > 0)
			{
				// Some types are never interesting and the contents of the array should be ignored. Other types can be interesting, the contents of the array should be stored (even if
				// the values may not be interesting, so that index values can be obtained)
				if(JsonArray[0]->Type != EJson::Array && JsonArray[0]->Type != EJson::String && JsonArray[0]->Type != EJson::Number && JsonArray[0]->Type != EJson::Object)
				{
					bValidPropertyValue = false;
				}
			}
		}
		else if(InJsonValue->Type == EJson::Object)
		{
			// Start it out as not being valid, if we find any sub-items that are searchable, it will be marked to TRUE
			bValidPropertyValue = false;

			// Go through all value/key pairs to see if any of them are searchable, remove the ones that are not
			const TSharedPtr<FJsonObject>& JsonObject = InJsonValue->AsObject();
			for(TMap<FJsonObject::FStringType, TSharedPtr<FJsonValue>>::TIterator Iter = JsonObject->Values.CreateIterator(); Iter; ++Iter)
			{
				// Map variables should always be searchable. (Iter->Key, Iter->Value) represents a user entered key-value pair.
				// Even if the value is uninteresting (empty string, 0, etc) then the key could be interesting.
				const bool bIsMapVariable = InVariableDescription.VarType.ContainerType == EPinContainerType::Map;

				// If not a map variable, check per entry if the value is also 'interesting' (see comment at the top of this function).
				if (!bIsMapVariable && !CheckIfJsonValueIsSearchable(Iter->Value, InVariableDescription))
				{
					// Note: It's safe to keep incrementing after this; the underlying logic maps to TSparseArray/TConstSetBitIterator::RemoveCurrent().
					Iter.RemoveCurrent();
				}
				else
				{
					bValidPropertyValue = true;
				}
			}
			
		}

		return bValidPropertyValue;
	}

	/**
	 * Saves a graph pin type to a Json object
	 *
	 * @param InWriter				Writer used for saving the Json
	 * @param InPinType				The pin type to save
	 */
	template<class PrintPolicy>
	void SavePinTypeToJson(const TSharedRef<TFindInBlueprintJsonStringWriter<PrintPolicy>>& InWriter, const FEdGraphPinType& InPinType)
	{
		// Only save strings that are not empty

		if(!InPinType.PinCategory.IsNone())
		{
			InWriter->WriteValue(FFindInBlueprintSearchTags::FiB_PinCategory, InPinType.PinCategory.ToString());
		}

		if(!InPinType.PinSubCategory.IsNone())
		{
			InWriter->WriteValue(FFindInBlueprintSearchTags::FiB_PinSubCategory, InPinType.PinSubCategory.ToString());
		}

		if(InPinType.PinSubCategoryObject.IsValid())
		{
			// Write the full path because this can be an ambiguous blueprint class name
			InWriter->WriteValue(FFindInBlueprintSearchTags::FiB_ObjectClass, FText::FromString(InPinType.PinSubCategoryObject->GetPathName()));
		}
		InWriter->WriteValue(FFindInBlueprintSearchTags::FiB_IsArray, InPinType.IsArray());
		InWriter->WriteValue(FFindInBlueprintSearchTags::FiB_IsReference, InPinType.bIsReference);
	}

	/**
	 * Helper function to save a variable description to Json
	 *
	 * @param InWriter					Json writer object
	 * @param InBlueprint				Blueprint the property for the variable can be found in, if any
	 * @param InVariableDescription		The variable description being serialized to Json
	 */
	template<class PrintPolicy>
	void SaveVariableDescriptionToJson(const TSharedRef<TFindInBlueprintJsonStringWriter<PrintPolicy>>& InWriter, const UBlueprint* InBlueprint, const FBPVariableDescription& InVariableDescription)
	{
		FEdGraphPinType VariableType = InVariableDescription.VarType;

		InWriter->WriteObjectStart();

		InWriter->WriteValue(FFindInBlueprintSearchTags::FiB_Name, InVariableDescription.FriendlyName);

		// Find the variable's tooltip
		FString TooltipResult;
		
		if(InVariableDescription.HasMetaData(FBlueprintMetadata::MD_Tooltip))
		{
			TooltipResult = InVariableDescription.GetMetaData(FBlueprintMetadata::MD_Tooltip);
		}
		InWriter->WriteValue(FFindInBlueprintSearchTags::FiB_Tooltip, TooltipResult);

		// Save the variable's pin type
		SavePinTypeToJson(InWriter, VariableType);

		// Find the FProperty and convert it into a Json value.
		FProperty* VariableProperty = FindFProperty<FProperty>(InBlueprint->GeneratedClass, InVariableDescription.VarName);
		if (VariableProperty)
		{
			const uint8* PropData = VariableProperty->ContainerPtrToValuePtr<uint8>(InBlueprint->GeneratedClass->GetDefaultObject());
			auto JsonValue = FJsonObjectConverter::UPropertyToJsonValue(VariableProperty, PropData, 0, 0);

			// Only use the value if it is searchable
			if (BlueprintSearchMetaDataHelpers::CheckIfJsonValueIsSearchable(JsonValue, InVariableDescription))
			{
				TSharedRef< FJsonValue > JsonValueAsSharedRef = JsonValue.ToSharedRef();
				FJsonSerializer::Serialize<TCHAR, PrintPolicy>(JsonValue, FFindInBlueprintSearchTags::FiB_DefaultValue.ToString(), InWriter, false );
			}
		}

		InWriter->WriteObjectEnd();
	}

	/** Helper enum to gather searchable UProperties */
	enum EGatherSearchableType
	{
		SEARCHABLE_AS_DESIRED = 0,
		SEARCHABLE_FULL,
		SEARCHABLE_SHALLOW,
	};

	/**
	 * Gathers all searchable properties in a UObject and writes them out to Json
	 *
	 * @param InWriter				Json writer
	 * @param InValue				Value of the Object to serialize
	 * @param InStruct				Struct or class that represent the UObject's layout
	 * @param InSearchableType		Informs the system how it should examine the properties to determine if they are searchable. All sub-properties of searchable properties are automatically gathered unless marked as not being searchable
	 */
	template<class PrintPolicy>
	void GatherSearchableProperties(const TSharedRef<TFindInBlueprintJsonStringWriter<PrintPolicy>>& InWriter, const void* InValue, UStruct* InStruct, EGatherSearchableType InSearchableType = SEARCHABLE_AS_DESIRED);

	/**
	 * Examines a searchable property and digs in deeper if it is a UObject, UStruct, or an array, or serializes it straight out to Json
	 *
	 * @param InWriter				Json writer
	 * @param InProperty			Property to examine
	 * @param InValue				Value to find the property in the UStruct
	 * @param InStruct				Struct or class that represent the UObject's layout
	 */
	template<class PrintPolicy>
	void GatherSearchablesFromProperty(const TSharedRef<TFindInBlueprintJsonStringWriter<PrintPolicy>>& InWriter, FProperty* InProperty, const void* InValue, UStruct* InStruct)
	{
		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InProperty))
		{
			FScriptArrayHelper Helper(ArrayProperty, InValue);
			InWriter->WriteArrayStart(FText::FromString(InProperty->GetName()));
			for (int32 i=0, n=Helper.Num(); i<n; ++i)
			{
				GatherSearchablesFromProperty(InWriter, ArrayProperty->Inner, Helper.GetRawPtr(i), InStruct);
			}
			InWriter->WriteArrayEnd();
		}
		else if (FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
		{
			if (!InProperty->HasMetaData(*FFiBMD::FiBSearchableMD) || InProperty->GetBoolMetaData(*FFiBMD::FiBSearchableMD))
			{
				GatherSearchableProperties(InWriter, InValue, StructProperty->Struct, SEARCHABLE_FULL);
			}
		}
		else if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(InProperty))
		{
			UObject* SubObject = ObjectProperty->GetObjectPropertyValue(InValue);
			if (SubObject)
			{
				// Objects default to shallow unless they are marked as searchable
				EGatherSearchableType searchType = SEARCHABLE_SHALLOW;

				// Check if there is any Searchable metadata
				if (InProperty->HasMetaData(*FFiBMD::FiBSearchableMD))
				{
					// Check if that metadata informs us that the property should not be searchable
					bool bSearchable = InProperty->GetBoolMetaData(*FFiBMD::FiBSearchableMD);
					if (bSearchable)
					{
						GatherSearchableProperties(InWriter, SubObject, SubObject->GetClass(), SEARCHABLE_FULL);
					}
				}
				else
				{
					// Shallow conversion of property to string
					TSharedPtr<FJsonValue> JsonValue;
					JsonValue = FJsonObjectConverter::UPropertyToJsonValue(InProperty, InValue, 0, 0);
					FJsonSerializer::Serialize<TCHAR, PrintPolicy>(JsonValue, InProperty->GetName(), InWriter, false);
				}
			}
		}
		else
		{
			TSharedPtr<FJsonValue> JsonValue;
			JsonValue = FJsonObjectConverter::UPropertyToJsonValue(InProperty, InValue, 0, 0);
			FJsonSerializer::Serialize<TCHAR, PrintPolicy>(JsonValue, InProperty->GetName(), InWriter, false);
		}
	}

	template<class PrintPolicy>
	void GatherSearchableProperties(const TSharedRef<TFindInBlueprintJsonStringWriter<PrintPolicy>>& InWriter, const void* InValue, UStruct* InStruct, EGatherSearchableType InSearchableType)
	{
		if (InValue)
		{
			TArray<FSearchableProperty>* SearchablePropertyData = InWriter->CachedPropertyMapping.Find(InStruct);
			check(SearchablePropertyData);

			for (FSearchableProperty& SearchableProperty : *SearchablePropertyData)
			{
				FProperty* Property = SearchableProperty.TargetProperty;
				bool bIsSearchableMD = SearchableProperty.bIsSearchableMD;
				bool bIsShallowSearchableMD = SearchableProperty.bIsShallowSearchableMD;
				// It only is truly marked as not searchable if it has the metadata set to false, if the metadata is missing then we assume the searchable type that is passed in unless SEARCHABLE_AS_DESIRED
				bool bIsMarkedNotSearchableMD = SearchableProperty.bIsMarkedNotSearchableMD;

				const bool bShouldGatherSearchableProperty = (InSearchableType != SEARCHABLE_AS_DESIRED && !bIsMarkedNotSearchableMD)
					|| bIsShallowSearchableMD || bIsSearchableMD;
				if (bShouldGatherSearchableProperty && InWriter->GetFormatVersion() >= SearchableProperty.MinDataFormatVersion)
				{
					const void* Value = Property->ContainerPtrToValuePtr<uint8>(InValue);

					// Need to store the metadata on the property in a sub-object
					InWriter->WriteObjectStart(FText::FromString(Property->GetName()));
					{
						InWriter->WriteObjectStart(FFindInBlueprintSearchTags::FiBMetaDataTag);
						{
							if (Property->GetBoolMetaData(*FFiBMD::FiBSearchableHiddenExplicitMD))
							{
								InWriter->WriteValue(FText::FromString(FFiBMD::FiBSearchableHiddenExplicitMD), true);
							}
							else if (Property->GetBoolMetaData(*FFiBMD::FiBSearchableExplicitMD))
							{
								InWriter->WriteValue(FText::FromString(FFiBMD::FiBSearchableExplicitMD), true);
							}
						}
						InWriter->WriteObjectEnd();

						if (Property->ArrayDim == 1)
						{
							GatherSearchablesFromProperty(InWriter, Property, Value, InStruct);
						}
						else
						{
							TArray< TSharedPtr<FJsonValue> > Array;
							for (int Index = 0; Index != Property->ArrayDim; ++Index)
							{
								GatherSearchablesFromProperty(InWriter, Property, (char*)Value + Index * Property->GetElementSize(), InStruct);
							}
						}
					}
					InWriter->WriteObjectEnd();
				}
			}
		}
	}

	/**
	 * Caches all properties that have searchability metadata
	 *
	 * @param InOutCachePropertyMapping		Mapping of all the searchable properties that we are building
	 * @param InValue						Value of the Object to serialize
	 * @param InStruct						Struct or class that represent the UObject's layout
	 * @param InSearchableType				Informs the system how it should examine the properties to determine if they are searchable. All sub-properties of searchable properties are automatically gathered unless marked as not being searchable
	 */
	void CacheSearchableProperties(TMap<UStruct*, TArray<FSearchableProperty>>& InOutCachePropertyMapping, const void* InValue, UStruct* InStruct, EGatherSearchableType InSearchableType = SEARCHABLE_AS_DESIRED);

	/**
	 * Digs into a property for any sub-properties that might exist so it can recurse and cache them
	 *
	 * @param InOutCachePropertyMapping		Mapping of all the searchable properties that we are building
	 * @param InProperty					Property currently being cached
	 * @param InValue						Value of the Object to serialize
	 * @param InStruct						Struct or class that represent the UObject's layout
	 */
	void CacheSubPropertySearchables(TMap<UStruct*, TArray<FSearchableProperty>>& InOutCachePropertyMapping, FProperty* InProperty, const void* InValue, UStruct* InStruct)
	{
		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InProperty))
		{
			FScriptArrayHelper Helper(ArrayProperty, InValue);
			for (int32 i = 0, n = Helper.Num(); i < n; ++i)
			{
				CacheSubPropertySearchables(InOutCachePropertyMapping, ArrayProperty->Inner, Helper.GetRawPtr(i), InStruct);
			}
		}
		else if (FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
		{
			if (!InOutCachePropertyMapping.Find(StructProperty->Struct))
			{
				if (!InProperty->HasMetaData(*FFiBMD::FiBSearchableMD) || InProperty->GetBoolMetaData(*FFiBMD::FiBSearchableMD))
				{
					CacheSearchableProperties(InOutCachePropertyMapping, InValue, StructProperty->Struct, SEARCHABLE_FULL);
				}
			}
		}
		else if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(InProperty))
		{
			UObject* SubObject = ObjectProperty->GetObjectPropertyValue(InValue);
			if (SubObject)
			{
				// Objects default to shallow unless they are marked as searchable
				EGatherSearchableType SearchType = SEARCHABLE_SHALLOW;

				// Check if there is any Searchable metadata
				if (InProperty->HasMetaData(*FFiBMD::FiBSearchableMD))
				{
					if (!InOutCachePropertyMapping.Find(SubObject->GetClass()))
					{
						// Check if that metadata informs us that the property should not be searchable
						bool bSearchable = InProperty->GetBoolMetaData(*FFiBMD::FiBSearchableMD);
						if (bSearchable)
						{
							CacheSearchableProperties(InOutCachePropertyMapping, SubObject, SubObject->GetClass(), SEARCHABLE_FULL);
						}
					}
				}
			}
		}
	}

	void CacheSearchableProperties(TMap<UStruct*, TArray<FSearchableProperty>>& InOutCachePropertyMapping, const void* InValue, UStruct* InStruct, EGatherSearchableType InSearchableType)
	{
		if (InValue)
		{
			TArray<FSearchableProperty> SearchableProperties;

			for (TFieldIterator<FProperty> PropIt(InStruct); PropIt; ++PropIt)
			{
				FProperty* Property = *PropIt;
				bool bIsSearchableMD = Property->GetBoolMetaData(*FFiBMD::FiBSearchableMD);
				bool bIsShallowSearchableMD = Property->GetBoolMetaData(*FFiBMD::FiBSearchableShallowMD);
				// It only is truly marked as not searchable if it has the metadata set to false, if the metadata is missing then we assume the searchable type that is passed in unless SEARCHABLE_AS_DESIRED
				bool bIsMarkedNotSearchableMD = Property->HasMetaData(*FFiBMD::FiBSearchableMD) && !bIsSearchableMD;

				// Searchable properties default to the latest format version, unless otherwise specified in the property metadata.
				EFiBVersion MinDataFormatVersion = FIB_VER_LATEST;
				if (Property->HasMetaData(*FFiBMD::FiBSearchableFormatVersionMD))
				{
					const FString VersionString = Property->GetMetaData(*FFiBMD::FiBSearchableFormatVersionMD);
					if (!VersionString.IsEmpty())
					{
						const UEnum* FiBVersionEnum = StaticEnum<EFiBVersion>();
						const int64 VersionValue = FiBVersionEnum->GetValueByNameString(VersionString);
						if (VersionValue != INDEX_NONE)
						{
							MinDataFormatVersion = static_cast<EFiBVersion>(VersionValue);
						}
					}
				}

				if ((InSearchableType != SEARCHABLE_AS_DESIRED && !bIsMarkedNotSearchableMD)
					|| bIsShallowSearchableMD || bIsSearchableMD)
				{
					const void* Value = Property->ContainerPtrToValuePtr<uint8>(InValue);

					FSearchableProperty SearchableProperty;
					SearchableProperty.TargetProperty = Property;
					SearchableProperty.bIsSearchableMD = bIsSearchableMD;
					SearchableProperty.bIsShallowSearchableMD = bIsShallowSearchableMD;
					SearchableProperty.bIsMarkedNotSearchableMD = bIsMarkedNotSearchableMD;
					SearchableProperty.MinDataFormatVersion = MinDataFormatVersion;

					if (Property->ArrayDim == 1)
					{
						CacheSubPropertySearchables(InOutCachePropertyMapping, Property, Value, InStruct);
					}
					else
					{
						TArray< TSharedPtr<FJsonValue> > Array;
						for (int Index = 0; Index != Property->ArrayDim; ++Index)
						{
							CacheSubPropertySearchables(InOutCachePropertyMapping, Property, (char*)Value + Index * Property->GetElementSize(), InStruct);
						}
					}
					SearchableProperties.Add(MoveTemp(SearchableProperty));
				}
				InOutCachePropertyMapping.Add(InStruct, SearchableProperties);
			}
		}
	}

	/**
	 * Gathers all nodes from a specified graph and serializes their searchable data to Json
	 *
	 * @param InWriter		The Json writer to use for serialization
	 * @param InGraph		The graph to search through
	 */
	template<class PrintPolicy>
	void GatherNodesFromGraph(const TSharedRef<TFindInBlueprintJsonStringWriter<PrintPolicy>>& InWriter, const UEdGraph* InGraph)
	{
		// Collect all macro graphs
		InWriter->WriteArrayStart(FFindInBlueprintSearchTags::FiB_Nodes);
		{
			for(auto& Node : InGraph->Nodes)
			{
				if(Node)
				{
					{
						// Make sure we don't collect search data for nodes that are going away soon
						if (!IsValid(Node->GetOuter()))
						{
							continue;
						}

						InWriter->WriteObjectStart();

						// Retrieve the search metadata from the node, some node types may have extra metadata to be searchable.
						TArray<struct FSearchTagDataPair> Tags;
						Node->AddSearchMetaDataInfo(Tags);

						// Go through the node metadata tags and put them into the Json object.
						for (const FSearchTagDataPair& SearchData : Tags)
						{
							InWriter->WriteValue(SearchData.Key, SearchData.Value);
						}
					}

					{
						// Find all the pins and extract their metadata
						InWriter->WriteArrayStart(FFindInBlueprintSearchTags::FiB_Pins);
						for (const UEdGraphPin* Pin : Node->Pins)
						{
							// Hidden pins are not searchable, except for 'self' pins which represent the target type.
							// Indexing self pins, even if they are hidden, allows searching for BP function library
							// function calls and nodes that automatically target self like parent function calls.
							if (Pin->bHidden == false || Pin->PinName == UEdGraphSchema_K2::PN_Self)
							{
								InWriter->WriteObjectStart();
								{
									TArray<struct FSearchTagDataPair> Tags;
									Node->AddPinSearchMetaDataInfo(Pin, Tags);

									for (const FSearchTagDataPair& SearchData : Tags)
									{
										InWriter->WriteValue(SearchData.Key, SearchData.Value);
									}
								}
								SavePinTypeToJson(InWriter, Pin->PinType);
								InWriter->WriteObjectEnd();
							}
						}
						InWriter->WriteArrayEnd();

						if (!InWriter->CachedPropertyMapping.Find(Node->GetClass()))
						{
							CacheSearchableProperties(InWriter->CachedPropertyMapping, Node, Node->GetClass());
						}
						// Only support this for nodes for now, will gather all searchable properties
						GatherSearchableProperties(InWriter, Node, Node->GetClass());

						InWriter->WriteObjectEnd();
					}
				}
				
			}
		}
		InWriter->WriteArrayEnd();
	}

	/** 
	 * Gathers all graph's search data (and subojects) and serializes them to Json
	 *
	 * @param InWriter			The Json writer to use for serialization
	 * @param InGraphArray		All the graphs to process
	 * @param InTitle			The array title to place these graphs into
	 * @param InOutSubGraphs	All the subgraphs that need to be processed later
	 */
	template<class PrintPolicy>
	void GatherGraphSearchData(const TSharedRef<TFindInBlueprintJsonStringWriter<PrintPolicy>>& InWriter, const UBlueprint* InBlueprint, const TArray< UEdGraph* >& InGraphArray, FText InTitle, TArray< UEdGraph* >* InOutSubGraphs)
	{
		if(InGraphArray.Num() > 0)
		{
			// Collect all graphs
			InWriter->WriteArrayStart(InTitle);
			{
				for(const UEdGraph* Graph : InGraphArray)
				{
					// This is non-critical but should not happen and needs to be resolved
					if (!ensure(Graph != nullptr))
					{
						continue;
					}
					InWriter->WriteObjectStart();

					FGraphDisplayInfo DisplayInfo;
					if (auto GraphSchema = Graph->GetSchema())
					{
						GraphSchema->GetGraphDisplayInformation(*Graph, DisplayInfo);
					}
					InWriter->WriteValue(FFindInBlueprintSearchTags::FiB_Name, DisplayInfo.PlainName);

					FText GraphDescription = FBlueprintEditorUtils::GetGraphDescription(Graph);
					if(!GraphDescription.IsEmpty())
					{
						InWriter->WriteValue(FFindInBlueprintSearchTags::FiB_Description, GraphDescription);
					}
					// All nodes will appear as children to the graph in search results
					GatherNodesFromGraph(InWriter, Graph);

					// Collect local variables
					TArray<UK2Node_FunctionEntry*> FunctionEntryNodes;
					Graph->GetNodesOfClass<UK2Node_FunctionEntry>(FunctionEntryNodes);

					InWriter->WriteArrayStart(FFindInBlueprintSearchTags::FiB_Properties);
					{
						// Search in all FunctionEntry nodes for their local variables and add them to the list
						FString ActionCategory;
						for (UK2Node_FunctionEntry* const FunctionEntry : FunctionEntryNodes)
						{
							for( const FBPVariableDescription& Variable : FunctionEntry->LocalVariables )
							{
								SaveVariableDescriptionToJson(InWriter, InBlueprint, Variable);
							}
						}
					}
					InWriter->WriteArrayEnd(); // Properties

					InWriter->WriteObjectEnd();

					// Only if asked to do it
					if(InOutSubGraphs)
					{
						Graph->GetAllChildrenGraphs(*InOutSubGraphs);
					}
				}
			}
			InWriter->WriteArrayEnd();
		}
	}

	template<class PrintPolicy>
	void GatherExtensionsSearchData_Recursive(const TSharedRef<TFindInBlueprintJsonStringWriter<PrintPolicy>>& InWriter, const UBlueprintExtension::FSearchData& SearchData)
	{
		for (const UBlueprintExtension::FSearchTagDataPair& Data : SearchData.Datas)
		{
			InWriter->WriteValue(Data.Key, Data.Value);
		}

		for (const TUniquePtr<UBlueprintExtension::FSearchArrayData>& ArrayData : SearchData.SearchArrayDatas)
		{
			InWriter->WriteArrayStart(ArrayData->Identifier.IsEmpty() ? FFindInBlueprintSearchTags::FiB_Extensions : ArrayData->Identifier);
			for (const UBlueprintExtension::FSearchData& Data : ArrayData->SearchSubList)
			{
				InWriter->WriteObjectStart();
				GatherExtensionsSearchData_Recursive(InWriter, Data);
				InWriter->WriteObjectEnd();
			}
			InWriter->WriteArrayEnd();
		}
	}

	template<class PrintPolicy>
	void GatherExtensionsSearchData(const TSharedRef<TFindInBlueprintJsonStringWriter<PrintPolicy>>& InWriter, const UBlueprint* InBlueprint)
	{
		if (InBlueprint->GetExtensions().Num() > 0)
		{
			// Collect all extensions
			bool bExtensionArrayStarted = false;
			for (UBlueprintExtension* Extension : InBlueprint->GetExtensions())
			{
				if (Extension)
				{
					if (!bExtensionArrayStarted)
					{
						InWriter->WriteArrayStart(FFindInBlueprintSearchTags::FiB_Extensions);
						bExtensionArrayStarted = true;
					}

					InWriter->WriteObjectStart();
					InWriter->WriteValue(FFindInBlueprintSearchTags::FiB_Name, FText::FromString(Extension->GetName()));
					InWriter->WriteValue(FFindInBlueprintSearchTags::FiB_ClassName, FText::FromString(Extension->GetClass()->GetName()));

					// Retrieve the custom search metadata from the extension.
					UBlueprintExtension::FSearchData SearchData = Extension->GatherSearchData(InBlueprint);
					GatherExtensionsSearchData_Recursive(InWriter, SearchData);

					InWriter->WriteObjectEnd();
				}
			}

			if (bExtensionArrayStarted)
			{
				InWriter->WriteArrayEnd();
			}
		}
	}

	template<class PrintPolicy>
	void GatherBlueprintSearchMetadata(const TSharedRef<TFindInBlueprintJsonStringWriter<PrintPolicy>>& InWriter, const UBlueprint* Blueprint)
	{
		FTemporarilyUseFriendlyNodeTitles TemporarilyUseFriendlyNodeTitles;

		TMap<FString, TMap<FString, int>> AllPaths;
		InWriter->WriteObjectStart();

		// Only pull properties if the Blueprint has been compiled
		if (Blueprint->SkeletonGeneratedClass)
		{
			InWriter->WriteArrayStart(FFindInBlueprintSearchTags::FiB_Properties);
			{
				for (const FBPVariableDescription& Variable : Blueprint->NewVariables)
				{
					SaveVariableDescriptionToJson(InWriter, Blueprint, Variable);
				}
			}
			InWriter->WriteArrayEnd(); // Properties
		}

		// Gather all graph searchable data
		TArray< UEdGraph* > SubGraphs;

		// Gather normal event graphs
		GatherGraphSearchData(InWriter, Blueprint, Blueprint->UbergraphPages, FFindInBlueprintSearchTags::FiB_UberGraphs, &SubGraphs);

		// Gather extension graphs
		for (UBlueprintExtension* Extension : Blueprint->GetExtensions())
		{
			if (Extension)
			{
				TArray<UEdGraph*> ExtensionGraphs;
				Extension->GetAllGraphs(ExtensionGraphs);
				GatherGraphSearchData(InWriter, Blueprint, ExtensionGraphs, FFindInBlueprintSearchTags::FiB_ExtensionGraphs, &SubGraphs);
			}
		}

		// We have interface graphs and function graphs to put into the Functions category. We cannot do them separately, so we must compile the full list
		{
			TArray<UEdGraph*> CompleteGraphList;
			CompleteGraphList.Append(Blueprint->FunctionGraphs);

			// Gather all interface graphs as functions
			if (InWriter->GetFormatVersion() >= EFiBVersion::FIB_VER_INTERFACE_GRAPHS)
			{
				for (const FBPInterfaceDescription& InterfaceDesc : Blueprint->ImplementedInterfaces)
				{
					CompleteGraphList.Append(InterfaceDesc.Graphs);
				}
			}
			
			GatherGraphSearchData(InWriter, Blueprint, CompleteGraphList, FFindInBlueprintSearchTags::FiB_Functions, &SubGraphs);
		}

		// Gather Macros
		GatherGraphSearchData(InWriter, Blueprint, Blueprint->MacroGraphs, FFindInBlueprintSearchTags::FiB_Macros, &SubGraphs);

		// Sub graphs are processed separately so that they do not become children in the TreeView, cluttering things up if the tree is deep
		GatherGraphSearchData(InWriter, Blueprint, SubGraphs, FFindInBlueprintSearchTags::FiB_SubGraphs, nullptr);

		// Gather all SCS components
		// If we have an SCS but don't support it, then we remove it
		if (Blueprint->SimpleConstructionScript)
		{
			// Remove any SCS variable nodes
			const TArray<USCS_Node*>& AllSCSNodes = Blueprint->SimpleConstructionScript->GetAllNodes();
			InWriter->WriteArrayStart(FFindInBlueprintSearchTags::FiB_Components);
			for (TFieldIterator<FProperty> PropertyIt(Blueprint->SkeletonGeneratedClass, EFieldIteratorFlags::ExcludeSuper); PropertyIt; ++PropertyIt)
			{
				FProperty* Property = *PropertyIt;
				FObjectPropertyBase* Obj = CastField<FObjectPropertyBase>(Property);
				const bool bComponentProperty = Obj && Obj->PropertyClass ? Obj->PropertyClass->IsChildOf<UActorComponent>() : false;
				FName PropName = Property->GetFName();
				if (bComponentProperty && FBlueprintEditorUtils::FindSCS_Node(Blueprint, PropName))
				{
					FEdGraphPinType PropertyPinType;
					if (UEdGraphSchema_K2::StaticClass()->GetDefaultObject<UEdGraphSchema_K2>()->ConvertPropertyToPinType(Property, PropertyPinType))
					{
						InWriter->WriteObjectStart();
						{
							InWriter->WriteValue(FFindInBlueprintSearchTags::FiB_Name, FText::FromName(PropName));
							InWriter->WriteValue(FFindInBlueprintSearchTags::FiB_IsSCSComponent, true);
							SavePinTypeToJson(InWriter, PropertyPinType);
						}
						InWriter->WriteObjectEnd();
					}
				}
			}
			InWriter->WriteArrayEnd(); // Components
		}

		GatherExtensionsSearchData(InWriter, Blueprint);

		InWriter->WriteObjectEnd();
		InWriter->Close();
	}

	template<class JsonWriterType>
	FString GatherBlueprintSearchMetadata(const UBlueprint* Blueprint, int32 FiBDataVersion)
	{
		CSV_SCOPED_TIMING_STAT(FindInBlueprint, GatherBlueprintSearchMetadata);

		FString SearchMetaData;

		// The search registry tags for a Blueprint are all in Json
		TSharedRef<JsonWriterType> Writer = MakeShared<JsonWriterType>(&SearchMetaData, FiBDataVersion);

		typedef typename JsonWriterType::InnerPrintPolicy PrintPolicyType;
		GatherBlueprintSearchMetadata<PrintPolicyType>(Writer, Blueprint);

		return SearchMetaData;
	}
}

/** Interface for controlling the async indexing task. */
class IAsyncSearchIndexTaskController
{
public:
	/** Return true if there is still work available. */
	virtual bool IsWorkPending() const = 0;

	/** Return the next batch of asset paths for indexing. */
	virtual void GetAssetPathsToIndex(TArray<FSoftObjectPath>& OutAssetPaths) = 0;

	/** Return true if assets should be fully indexed. */
	virtual bool ShouldFullyIndexAssets() const = 0;

	/** Determine if multiprocessing should be enabled for indexing the next batch. */
	virtual bool ShouldEnableMultiprocessing() const = 0;

	/** Add the given asset path to the queue for gathering search data from a Blueprint. */
	virtual void AddAssetPathToGatherQueue(const FSoftObjectPath& InAssetPath) = 0;

	/** Called when indexing has been completed for the given asset path. */
	virtual void IndexCompletedForAssetPath(const FSoftObjectPath& InAssetPath) = 0;
};

/** Asynchronous indexing thread. Can spawn additional worker threads to index multiple assets in parallel. */
class FAsyncSearchIndexTaskRunnable : public FRunnable
{
public:
	FAsyncSearchIndexTaskRunnable(IAsyncSearchIndexTaskController* InController)
		:Controller(InController)
	{
		Thread = TUniquePtr<FRunnableThread>(FRunnableThread::Create(this, TEXT("FAsyncSearchIndexTaskRunnable"), 0, TPri_BelowNormal));

		// Make sure engine shut down is properly handled. Modules involved in the async search will be destroyed after engine pre exit.
		FCoreDelegates::OnEnginePreExit.AddRaw(this, &FAsyncSearchIndexTaskRunnable::OnEnginePreExit);
	}

	virtual ~FAsyncSearchIndexTaskRunnable()
	{
		bDestructing = true;
		EnsureCompletion();

		FCoreDelegates::OnEnginePreExit.RemoveAll(this);
	}

	virtual uint32 Run() override
	{
		while (
			!bDestructing &&
			Controller->IsWorkPending())
		{
			// Get the next batch of asset paths to be indexed.
			TArray<FSoftObjectPath> AssetPathsToIndex;
			Controller->GetAssetPathsToIndex(AssetPathsToIndex);

			// Determine whether this is a full or partial indexing operation.
			const bool bEnableFullIndexingPass = Controller->ShouldFullyIndexAssets();

			// Determine whether to disable multiprocessing for this batch. By default, use multiprocessing with low-priority worker threads.
			EParallelForFlags ParallelForFlags = EParallelForFlags::Unbalanced | EParallelForFlags::BackgroundPriority;
			if (!Controller->ShouldEnableMultiprocessing())
			{
				// This batch will be processed entirely on this thread.
				ParallelForFlags |= EParallelForFlags::ForceSingleThread;
			}

			ParallelFor(AssetPathsToIndex.Num(), [&AssetPathsToIndex, bEnableFullIndexingPass, ParallelForFlags, this](int32 ArrayIdx)
			{
				CSV_CUSTOM_STAT(FindInBlueprint, IndexedAssetCountThisFrame, 1, ECsvCustomStatOp::Accumulate);
				if (bDestructing)
				{
					return;
				}

				FSoftObjectPath AssetPath = AssetPathsToIndex[ArrayIdx];
				FFindInBlueprintSearchManager& FindManager = FFindInBlueprintSearchManager::Get();
				FSearchData SearchData = FindManager.GetSearchDataForAssetPath(AssetPath);
				if (SearchData.IsValid() && !SearchData.IsMarkedForDeletion() && !SearchData.IsIndexingCompleted())
				{
					// Generate the metadata tag value if it was not previously cached or loaded.
					if (!SearchData.HasEncodedValue())
					{
						// This must be done on the main thread, so enqueue it and continue.
						Controller->AddAssetPathToGatherQueue(AssetPath);
					}
					else
					{
						if (bEnableFullIndexingPass)
						{
							// Unpack the metadata tag and rebuild the index for this asset.
							if (FindManager.ProcessEncodedValueForUnloadedBlueprint(SearchData))
							{
								// Build the full index using a BFS traversal.
								TArray<FImaginaryFiBDataSharedPtr> IndexNodes = { SearchData.ImaginaryBlueprint };
								while (IndexNodes.Num() > 0)
								{
									if (bDestructing)
									{
										break;
									}

									ParallelFor(IndexNodes.Num(), [&IndexNodes](int32 NodeIdx)
									{
										IndexNodes[NodeIdx]->ParseAllChildData();
									}, ParallelForFlags);

									TArray<FImaginaryFiBDataSharedPtr> ChildNodes;
									for (FImaginaryFiBDataSharedPtr& NodePtr : IndexNodes)
									{
										ChildNodes.Append(NodePtr->GetAllParsedChildData());
									}

									IndexNodes = MoveTemp(ChildNodes);
								}
							}

							// Signal that this asset has now been fully indexed.
							SearchData.StateFlags |= ESearchDataStateFlags::IsIndexed;

							// Update this entry in the search database (thread-safe).
							FindManager.ApplySearchDataToDatabase(MoveTemp(SearchData));
						}

						// Signal that indexing has been completed for this asset path.
						Controller->IndexCompletedForAssetPath(AssetPath);
					}
				}
				else
				{
					// Signal that indexing has been completed for this asset path, even though its cache entry is invalid.
					Controller->IndexCompletedForAssetPath(AssetPath);
				}
			}, ParallelForFlags);
		}

		return 0;
	}

	void EnsureCompletion()
	{
		if (Thread.IsValid())
		{
			Thread->WaitForCompletion();
		}
	}

private:
	/** Called before engine exits. Useful to terminate async work before involved engine modules are being destroyed. */
	void OnEnginePreExit()
	{
		bDestructing = true;
		EnsureCompletion();
	}

	TUniquePtr<FRunnableThread> Thread;
	IAsyncSearchIndexTaskController* Controller;

	/** True when this object is being destructed, useful to terminate intermediate work instead of blocking until completion is ensured */
	TAtomic<bool> bDestructing = false;
};

class FCacheAllBlueprintsTickableObject : public IAsyncSearchIndexTaskController
{
public:
	DECLARE_DELEGATE_OneParam(FOnAssetCached, FSoftObjectPath);

	struct FCacheParams
	{
		/** Cache type */
		EFiBCacheOpType OpType;

		/** Control flags */
		EFiBCacheOpFlags OpFlags;

		/** Callback for when assets are cached */
		FOnAssetCached OnCached;

		/** Callback for when caching is finished */
		FSimpleDelegate OnFinished;

		/** Size of each batch (for async processing) */
		int32 AsyncTaskBatchSize;

		FCacheParams()
			:OpFlags(EFiBCacheOpFlags::None)
			,AsyncTaskBatchSize(0)
		{
		}
	};

	FCacheAllBlueprintsTickableObject(const TSet<FSoftObjectPath>& InAssets, const FCacheParams& InParams)
		: UncachedAssets(InAssets.Array())
		, CacheParams(InParams)
		, TickCacheIndex(0)
		, AsyncTaskBatchIndex(0)
		, bIsGatheringSearchMetadata(false)
		, bHasAssetsPendingGatherWork(false)
		, bIsStarted(false)
		, bIsCancelled(false)
	{
		if (CacheParams.AsyncTaskBatchSize <= 0)
		{
			CacheParams.AsyncTaskBatchSize = UncachedAssets.Num();
		}

		if (EnumHasAnyFlags(CacheParams.OpFlags, EFiBCacheOpFlags::ShowProgress)
			&& !EnumHasAnyFlags(CacheParams.OpFlags, EFiBCacheOpFlags::HideNotifications))
		{
			// Start the Blueprint indexing 'progress' notification
			FNotificationInfo Info(LOCTEXT("BlueprintIndexMessage", "Indexing Blueprints..."));
			Info.bFireAndForget = false;
			if (EnumHasAnyFlags(CacheParams.OpFlags, EFiBCacheOpFlags::AllowUserCancel))
			{
				Info.ButtonDetails.Add(FNotificationButtonInfo(
					LOCTEXT("BlueprintIndexCancel", "Cancel"),
					LOCTEXT("BlueprintIndexCancelToolTip", "Cancels indexing Blueprints."), FSimpleDelegate::CreateRaw(this, &FCacheAllBlueprintsTickableObject::OnCancelCaching, false)));
			}

			ProgressNotification = FSlateNotificationManager::Get().AddNotification(Info);
			if (ProgressNotification.IsValid())
			{
				ProgressNotification.Pin()->SetCompletionState(SNotificationItem::CS_Pending);
			}
		}
	}

	virtual ~FCacheAllBlueprintsTickableObject()
	{

	}

	/** Returns the current cache index of the object */
	int32 GetCurrentCacheIndex() const
	{
		return TickCacheIndex + 1;
	}

	/** Returns the current cache operation type */
	EFiBCacheOpType GetCurrentCacheOpType() const
	{
		return CacheParams.OpType;
	}

	/** Returns the current cache operation control flags */
	EFiBCacheOpFlags GetCurrentCacheOpFlags() const
	{
		return CacheParams.OpFlags;
	}

	/** Returns the path of the current Blueprint being cached */
	FSoftObjectPath GetCurrentCacheBlueprintPath() const
	{
		if(UncachedAssets.Num() && TickCacheIndex >= 0)
		{
			return UncachedAssets[TickCacheIndex];
		}
		return {};
	}

	/** Returns the progress as a percent */
	float GetCacheProgress() const
	{
		return UncachedAssets.Num() > 0 ? (float)TickCacheIndex / (float)UncachedAssets.Num() : 1.0f;
	}

	/** Returns the number of uncached assets */
	int32 GetUncachedAssetCount()
	{
		return UncachedAssets.Num();
	}

	/** Returns the entire list of uncached assets that this object will attempt to cache */
	const TArray<FSoftObjectPath>& GetUncachedAssetList() const
	{
		return UncachedAssets;
	}

	/** True if there is a callback when done caching, this will prevent a re-query from occuring */
	bool HasPostCacheWork() const
	{
		return CacheParams.OnFinished.IsBound();
	}

	/** Cancels caching and destroys this object */
	void OnCancelCaching(bool bIsImmediate)
	{
		if (!bIsCancelled)
		{
			if (ProgressNotification.IsValid())
			{
				ProgressNotification.Pin()->SetText(LOCTEXT("BlueprintIndexCancelled", "Cancelled Indexing Blueprints!"));

				ProgressNotification.Pin()->SetCompletionState(SNotificationItem::CS_Fail);
				ProgressNotification.Pin()->ExpireAndFadeout();
			}

			// Sometimes we can't wait another tick to shutdown, so make the callback immediately.
			if (bIsImmediate)
			{
				// Note: This will effectively delete this instance. It should not be used after this!
				FFindInBlueprintSearchManager::Get().FinishedCachingBlueprints(CacheParams.OpType, CacheParams.OpFlags, TickCacheIndex, FailedToCacheList);
			}
			else
			{
				bIsCancelled = true;
			}
		}
	}

	virtual bool IsWorkPending() const override
	{
		if (!GIsRunning)
		{
			return false;
		}

		const int32 AsyncTaskBatchSize = CacheParams.AsyncTaskBatchSize;
		const int32 StartIndex = AsyncTaskBatchIndex * AsyncTaskBatchSize;
		return StartIndex < UncachedAssets.Num() || !AssetsPendingAsyncIndexing.IsEmpty() || bHasAssetsPendingGatherWork || bIsGatheringSearchMetadata;
	}

	virtual bool ShouldFullyIndexAssets() const override
	{
		return !EnumHasAnyFlags(CacheParams.OpFlags, EFiBCacheOpFlags::ExecuteGatherPhaseOnly);
	}

	virtual bool ShouldEnableMultiprocessing() const override
	{
		if (CacheParams.OpType == EFiBCacheOpType::CachePendingAssets)
		{
			// Don't utilize the task graph if any of the following conditions hold TRUE:
			// a) The application has throttled the tick rate.
			// b) No global search tabs are currently open and visible, or there are no async search queries that are otherwise pending completion.
			// c) The initial asset discovery phase has not yet been completed.
			// d) Multiprocessing has been explicitly disabled for this operation.
			return FSlateThrottleManager::Get().IsAllowingExpensiveTasks()
				&& (FFindInBlueprintSearchManager::Get().IsGlobalFindResultsOpen() || FFindInBlueprintSearchManager::Get().IsAsyncSearchQueryInProgress())
				&& !FFindInBlueprintSearchManager::Get().IsAssetDiscoveryInProgress()
				&& !EnumHasAnyFlags(CacheParams.OpFlags, EFiBCacheOpFlags::ExecuteOnSingleThread);
		}

		return false;
	}

	virtual void GetAssetPathsToIndex(TArray<FSoftObjectPath>& OutAssetPaths) override
	{
		OutAssetPaths.Empty();

		const int32 AsyncTaskBatchSize = CacheParams.AsyncTaskBatchSize;
		const int32 StartIndex = AsyncTaskBatchIndex * AsyncTaskBatchSize;
		if (StartIndex < UncachedAssets.Num())
		{
			for (int32 i = StartIndex; i < FMath::Min(StartIndex + AsyncTaskBatchSize, UncachedAssets.Num()); ++i)
			{
				OutAssetPaths.Add(UncachedAssets[i]);
			}

			AsyncTaskBatchIndex++;
		}
		else
		{
			FSoftObjectPath AssetPath;
			int32 Count = 0;
			while (Count < AsyncTaskBatchSize && AssetsPendingAsyncIndexing.Dequeue(AssetPath))
			{
				OutAssetPaths.Add(AssetPath);
				++Count;
			}
		}
	}

	virtual void AddAssetPathToGatherQueue(const FSoftObjectPath& InAssetPath) override
	{
		bHasAssetsPendingGatherWork = true;
		AssetsPendingGatherQueue.Enqueue(InAssetPath);
	}

	virtual void IndexCompletedForAssetPath(const FSoftObjectPath& InAssetPath) override
	{
		FScopeLock Lock(&AsyncTaskCompletionMutex);

		CompletedAsyncTaskAssets.Add(InAssetPath);
	}

	/** Enables the caching process */
	void Start()
	{
		if (!bIsStarted)
		{
			bIsStarted = true;
			FFindInBlueprintSearchManager::Get().StartedCachingBlueprints(CacheParams.OpType, CacheParams.OpFlags);

			if (CacheParams.OpType == EFiBCacheOpType::CachePendingAssets && !EnumHasAnyFlags(CacheParams.OpFlags, EFiBCacheOpFlags::ExecuteOnMainThread))
			{
				AsyncIndexBuilderTask = MakeUnique<FAsyncSearchIndexTaskRunnable>(this);
			}
		}
	}

	void Tick(float InDeltaTime)
	{
		if (!bIsStarted)
		{
			return;
		}

		if (UncachedAssets.Num() == 0)
		{
			// Immediately finish if we have no assets to index. This will delete this instance!
			Finish();

			return;
		}

		if (bIsCancelled || GWarn->ReceivedUserCancel())
		{
			// Note: This will effectively delete this instance. It should not be used after this!
			FFindInBlueprintSearchManager::Get().FinishedCachingBlueprints(CacheParams.OpType, CacheParams.OpFlags, TickCacheIndex, FailedToCacheList);
		}
		else
		{
			if (CacheParams.OpType == EFiBCacheOpType::CachePendingAssets)
			{
				if (AsyncIndexBuilderTask.IsValid())
				{
					// Process assets that require search metadata regeneration on the main thread (one per tick).
					if (!AssetsPendingGatherQueue.IsEmpty())
					{
						// Since we may empty the queue below, this flag is used to indicate that work is still pending.
						bIsGatheringSearchMetadata = true;

						FSoftObjectPath AssetPath;
						if (AssetsPendingGatherQueue.Dequeue(AssetPath))
						{
							bool bEnqueueForAsyncIndexing = false;

							FSearchData SearchData = FFindInBlueprintSearchManager::Get().GetSearchDataForAssetPath(AssetPath);
							if (SearchData.IsValid() && !SearchData.IsMarkedForDeletion())
							{
								if (UBlueprint* Blueprint = SearchData.Blueprint.Get())
								{
									using namespace BlueprintSearchMetaDataHelpers;
									SearchData.Value = GatherBlueprintSearchMetadata<FFindInBlueprintJsonWriter>(Blueprint, SearchData.VersionInfo.FiBDataVersion);

									if (SearchData.Value.Len() > 0)
									{
										bEnqueueForAsyncIndexing = ShouldFullyIndexAssets();
									}
									else
									{
										SearchData.StateFlags |= ESearchDataStateFlags::IsIndexed;
									}
								}

								// Update the entry in the search database. Don't use search data after this line.
								FFindInBlueprintSearchManager::Get().ApplySearchDataToDatabase(MoveTemp(SearchData));
							}

							if (bEnqueueForAsyncIndexing)
							{
								// Enqueue this asset path to restart async indexing with the updated search metadata.
								AssetsPendingAsyncIndexing.Enqueue(AssetPath);
							}
							else
							{
								// We're done indexing this asset path, so invoke the completion callback.
								IndexCompletedForAssetPath(AssetPath);
							}
						}

						// Signal whether or not the queue has been emptied.
						bHasAssetsPendingGatherWork = !AssetsPendingGatherQueue.IsEmpty();

						// Indicate that gather work is no longer in progress.
						bIsGatheringSearchMetadata = false;
					}

					// Block additional task completions until we're done processing.
					FScopeLock Lock(&AsyncTaskCompletionMutex);

					// Process each completed asset path.
					for (const FSoftObjectPath& AssetPath : CompletedAsyncTaskAssets)
					{
						// Execute the completion callback, if bound.
						CacheParams.OnCached.ExecuteIfBound(AssetPath);

						// Increment the counter for progress/UI display.
						++TickCacheIndex;
					}

					// Reset for the next tick.
					CompletedAsyncTaskAssets.Empty();
				}
				else
				{
					// Generate the metadata tag value if it was not previously cached or loaded.
					FSoftObjectPath AssetPath = UncachedAssets[TickCacheIndex];
					FSearchData SearchData = FFindInBlueprintSearchManager::Get().GetSearchDataForAssetPath(AssetPath);
					if (SearchData.IsValid() && !SearchData.IsMarkedForDeletion() && SearchData.Value.Len() == 0)
					{
						if (UBlueprint* Blueprint = SearchData.Blueprint.Get())
						{
							using namespace BlueprintSearchMetaDataHelpers;
							SearchData.Value = GatherBlueprintSearchMetadata<FFindInBlueprintJsonWriter>(Blueprint, SearchData.VersionInfo.FiBDataVersion);
						}
					}

					// Signal that this deferred asset has now been indexed.
					SearchData.StateFlags |= ESearchDataStateFlags::IsIndexed;

					// Update the entry in the search database.
					FFindInBlueprintSearchManager::Get().ApplySearchDataToDatabase(MoveTemp(SearchData));

					// Execute the completion callback, if bound.
					CacheParams.OnCached.ExecuteIfBound(AssetPath);

					// Increment the counter for progress/UI display.
					++TickCacheIndex;
				}
			}
			else
			{
				const bool bIncludeOnlyOnDiskAssets = false;	// @todo - time this false (default) vs. true
				FAssetRegistryModule* AssetRegistryModule = &FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
				FAssetData AssetData = AssetRegistryModule->Get().GetAssetByObjectPath(UncachedAssets[TickCacheIndex], bIncludeOnlyOnDiskAssets);
				if (AssetData.IsValid())
				{
					const bool bIsWorldAsset = AssetData.AssetClassPath == UWorld::StaticClass()->GetClassPathName();

					// Construct a full package filename with path so we can query the read only status and save to disk
					FString FinalPackageFilename = FPackageName::LongPackageNameToFilename(AssetData.PackageName.ToString());
					if (FinalPackageFilename.Len() > 0 && FPaths::GetExtension(FinalPackageFilename).Len() == 0)
					{
						FinalPackageFilename += bIsWorldAsset ? FPackageName::GetMapPackageExtension() : FPackageName::GetAssetPackageExtension();
					}
					FText ErrorMessage;
					bool bValidFilename = FFileHelper::IsFilenameValidForSaving(FinalPackageFilename, ErrorMessage);
					if (bValidFilename)
					{
						bValidFilename = bIsWorldAsset ? FEditorFileUtils::IsValidMapFilename(FinalPackageFilename, ErrorMessage) : FPackageName::IsValidLongPackageName(FinalPackageFilename, false, &ErrorMessage);
					}

					const bool bCheckOutAndSave = EnumHasAnyFlags(CacheParams.OpFlags, EFiBCacheOpFlags::CheckOutAndSave);

					bool bIsAssetReadOnlyOnDisk = IFileManager::Get().IsReadOnly(*FinalPackageFilename);
					bool bFailedToCache = bCheckOutAndSave;

					if (!bIsAssetReadOnlyOnDisk || !bCheckOutAndSave)
					{
						UObject* Asset = AssetData.GetAsset();
						if (Asset && bCheckOutAndSave)
						{
							if (UBlueprint* BlueprintAsset = Cast<UBlueprint>(Asset))
							{
								if (BlueprintAsset->SkeletonGeneratedClass == nullptr)
								{
									// There is no skeleton class, something was wrong with the Blueprint during compile on load. This asset will be marked as failing to cache.
									bFailedToCache = false;
								}
							}

							// Still good to attempt to save
							if (bFailedToCache)
							{
								// Assume the package was correctly checked out from SCC
								bool bOutPackageLocallyWritable = true;

								UPackage* Package = AssetData.GetPackage();

								ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
								// Trusting the SCC status in the package file cache to minimize network activity during save.
								const FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(Package, EStateCacheUsage::Use);
								// If the package is in the depot, and not recognized as editable by source control, and not read-only, then we know the user has made the package locally writable!
								const bool bSCCCanEdit = !SourceControlState.IsValid() || SourceControlState->CanCheckIn() || SourceControlState->IsIgnored() || SourceControlState->IsUnknown();
								const bool bSCCIsCheckedOut = SourceControlState.IsValid() && SourceControlState->IsCheckedOut();
								const bool bInDepot = SourceControlState.IsValid() && SourceControlState->IsSourceControlled();
								if (!bSCCCanEdit && bInDepot && !bIsAssetReadOnlyOnDisk && SourceControlProvider.UsesLocalReadOnlyState() && !bSCCIsCheckedOut)
								{
									bOutPackageLocallyWritable = false;
								}

								// Save the package if the file is writable
								if (bOutPackageLocallyWritable)
								{
									UWorld* WorldAsset = Cast<UWorld>(Asset);

									// Save the package
									FSavePackageArgs SaveArgs;
									SaveArgs.TopLevelFlags = (WorldAsset == nullptr) ? RF_Standalone : RF_NoFlags;
									SaveArgs.SaveFlags = SAVE_NoError;
									if (GEditor->SavePackage(Package, WorldAsset, *FinalPackageFilename, SaveArgs))
									{
										bFailedToCache = false;
									}
								}
							}
						}
					}

					if (bFailedToCache)
					{
						FailedToCacheList.Add(UncachedAssets[TickCacheIndex]);
					}
					else
					{
						CacheParams.OnCached.ExecuteIfBound(UncachedAssets[TickCacheIndex]);
					}

					++TickCacheIndex;
				}
				else
				{
					FailedToCacheList.Add(UncachedAssets[TickCacheIndex]);
					++TickCacheIndex;
				}
			}

			// Check if done caching Blueprints
			if(TickCacheIndex == UncachedAssets.Num())
			{
				// Note: This will effectively delete this instance, do not use after this!
				Finish();
			}
			else if(ProgressNotification.IsValid())
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("Percent"), FText::AsPercent(GetCacheProgress()));
				ProgressNotification.Pin()->SetText(FText::Format(LOCTEXT("BlueprintIndexProgress", "Indexing Blueprints... ({Percent})"), Args));
			}
		}
	}

protected:
	/** Completes a successful caching process */
	void Finish()
	{
		if (AsyncIndexBuilderTask.IsValid())
		{
			AsyncIndexBuilderTask->EnsureCompletion();
		}

		if (ProgressNotification.IsValid())
		{
			ProgressNotification.Pin()->SetCompletionState(SNotificationItem::CS_Success);
			ProgressNotification.Pin()->ExpireAndFadeout();

			ProgressNotification.Pin()->SetText(LOCTEXT("BlueprintIndexComplete", "Finished indexing Blueprints!"));
		}

		// We have actually finished, use the OnFinished callback.
		CacheParams.OnFinished.ExecuteIfBound();

		// Note: This will effectively delete this instance. It should not be used after this!
		FFindInBlueprintSearchManager::Get().FinishedCachingBlueprints(CacheParams.OpType, CacheParams.OpFlags, TickCacheIndex, FailedToCacheList);
	}

private:
	/** The list of assets that are in the process of being cached */
	TArray<FSoftObjectPath> UncachedAssets;

	/** Notification that appears and details progress */
	TWeakPtr<SNotificationItem> ProgressNotification;

	/** Set of Blueprints that failed to be saved */
	TSet<FSoftObjectPath> FailedToCacheList;

	/** Parameters for task configuration */
	FCacheParams CacheParams;

	/** Async search index builder task */
	TUniquePtr<FAsyncSearchIndexTaskRunnable> AsyncIndexBuilderTask;

	/** The current index, increases at a rate of once per tick */
	int32 TickCacheIndex;

	/** Tracks the next available work index for async index builder tasks */
	TAtomic<int32> AsyncTaskBatchIndex;

	/** Tracks completed async index builder tasks since the previous tick */
	TSet<FSoftObjectPath> CompletedAsyncTaskAssets;

	/** Synchronize between async index builder task worker threads and completion logic */
	FCriticalSection AsyncTaskCompletionMutex;

	/** Thread-safe queue for tracking asset paths that need to gather search metadata from a loaded object. This must be done on the main thread */
	TQueue<FSoftObjectPath, EQueueMode::Mpsc> AssetsPendingGatherQueue;

	/** Thread-safe queue for tracking asset paths that have exited the gather queue on the main thread and have now been re-queued for async indexing */
	TQueue<FSoftObjectPath, EQueueMode::Spsc> AssetsPendingAsyncIndexing;

	/** TRUE if we're busy gathering search metadata from a loaded object on the main thread */
	TAtomic<bool> bIsGatheringSearchMetadata;

	/** TRUE if we have assets pending in the gather queue. It is not thread safe to check the queue state directly from a producer thread. */
	TAtomic<bool> bHasAssetsPendingGatherWork;

	/** TRUE if the caching process is started */
	bool bIsStarted;

	/** TRUE if the user has requested to cancel the caching process */
	bool bIsCancelled;
};

FFindInBlueprintSearchManager& FFindInBlueprintSearchManager::Get()
{
	if (Instance == NULL)
	{
		Instance = new FFindInBlueprintSearchManager();
		Instance->Initialize();
	}

	return *Instance;
}

FFindInBlueprintSearchManager::FFindInBlueprintSearchManager()
	: AssetRegistryModule(nullptr)
	, CachingObject(nullptr)
	, AsyncTaskBatchSize(0)
	, bIsPausing(false)
	, bHasFirstSearchOccurred(false)
	, bEnableGatheringData(true)
	, bDisableDeferredIndexing(false)
	, bEnableCSVStatsProfiling(false)
	, bEnableDeveloperMenuTools(false)
	, bDisableSearchResultTemplates(false)
	, bDisableImmediateAssetDiscovery(false)
{
	for (int32 TabIdx = 0; TabIdx < UE_ARRAY_COUNT(GlobalFindResultsTabIDs); TabIdx++)
	{
		const FName TabID = FName(*FString::Printf(TEXT("GlobalFindResults_%02d"), TabIdx + 1));
		GlobalFindResultsTabIDs[TabIdx] = TabID;
	}
}

FFindInBlueprintSearchManager::~FFindInBlueprintSearchManager()
{
	if (AssetRegistryModule)
	{
		IAssetRegistry* AssetRegistry = AssetRegistryModule->TryGet();
		if (AssetRegistry)
		{
			AssetRegistry->OnAssetAdded().RemoveAll(this);
			AssetRegistry->OnAssetRemoved().RemoveAll(this);
			AssetRegistry->OnAssetRenamed().RemoveAll(this);
			AssetRegistry->OnFilesLoaded().RemoveAll(this);
		}
	}
	FKismetEditorUtilities::OnBlueprintUnloaded.RemoveAll(this);
	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().RemoveAll(this);
	FCoreUObjectDelegates::GetPostGarbageCollect().RemoveAll(this);
	FCoreUObjectDelegates::OnAssetLoaded.RemoveAll(this);
	FCoreUObjectDelegates::ReloadCompleteDelegate.RemoveAll(this);

	// Shut down the global find results tab feature.
	EnableGlobalFindResults(false);
}

void FFindInBlueprintSearchManager::Initialize()
{
	// Init configuration
	GConfig->GetInt(TEXT("BlueprintSearchSettings"), TEXT("AsyncTaskBatchSize"), AsyncTaskBatchSize, GEditorIni);
	GConfig->GetBool(TEXT("BlueprintSearchSettings"), TEXT("bDisableDeferredIndexing"), bDisableDeferredIndexing, GEditorIni);
	GConfig->GetBool(TEXT("BlueprintSearchSettings"), TEXT("bDisableThreadedIndexing"), bDisableThreadedIndexing, GEditorIni);
	GConfig->GetBool(TEXT("BlueprintSearchSettings"), TEXT("bEnableCsvStatsProfiling"), bEnableCSVStatsProfiling, GEditorIni);
	GConfig->GetBool(TEXT("BlueprintSearchSettings"), TEXT("bEnableDeveloperMenuTools"), bEnableDeveloperMenuTools, GEditorIni);
	GConfig->GetBool(TEXT("BlueprintSearchSettings"), TEXT("bDisableSearchResultTemplates"), bDisableSearchResultTemplates, GEditorIni);
	GConfig->GetBool(TEXT("BlueprintSearchSettings"), TEXT("bDisableImmediateAssetDiscovery"), bDisableImmediateAssetDiscovery, GEditorIni);

#if CSV_PROFILER
	// If profiling has been enabled, turn on the stat category and begin a capture.
	if (bEnableCSVStatsProfiling)
	{
		FCsvProfiler::Get()->EnableCategoryByString(TEXT("FindInBlueprint"));
		if (!FCsvProfiler::Get()->IsCapturing())
		{
			const FString CaptureFolder = FPaths::ProfilingDir() + TEXT("CSV/FindInBlueprint");
			FCsvProfiler::Get()->BeginCapture(-1, CaptureFolder);
		}
	}
#endif

	// Must ensure we do not attempt to load the AssetRegistry Module while saving a package, however, if it is loaded already we can safely obtain it
	if (!UE::IsSavingPackage() || (UE::IsSavingPackage() && FModuleManager::Get().IsModuleLoaded(TEXT("AssetRegistry"))))
	{
		AssetRegistryModule = &FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		IAssetRegistry* AssetRegistry = AssetRegistryModule->TryGet();
		if (AssetRegistry)
		{
			AssetRegistry->OnFilesLoaded().AddRaw(this, &FFindInBlueprintSearchManager::OnAssetRegistryFilesLoaded);
		}
	}
	else
	{
		// Log a warning to inform the Asset Registry could not be initialized when FiB initialized due to saving package
		// The Asset Registry should be initialized before Find-in-Blueprints, or FiB should be explicitly initialized during a safe time
		// This message will not appear in commandlets because most commandlets do not care. If a search query is made, further warnings will be produced even in commandlets.
		if (!IsRunningCommandlet())
		{
			UE_LOGF(LogBlueprint, Warning, "Find-in-Blueprints could not pre-cache all unloaded Blueprints due to the Asset Registry module being unable to initialize because a package is currently being saved. Pre-cache will not be reattempted!");
		}
	}

	FKismetEditorUtilities::OnBlueprintUnloaded.AddRaw(this, &FFindInBlueprintSearchManager::OnBlueprintUnloaded);

	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddRaw(this, &FFindInBlueprintSearchManager::PauseFindInBlueprintSearch);
	FCoreUObjectDelegates::GetPostGarbageCollect().AddRaw(this, &FFindInBlueprintSearchManager::UnpauseFindInBlueprintSearch);
	FCoreUObjectDelegates::OnAssetLoaded.AddRaw(this, &FFindInBlueprintSearchManager::OnAssetLoaded);
	
	// Register to be notified of reloads
	FCoreUObjectDelegates::ReloadCompleteDelegate.AddRaw(this, &FFindInBlueprintSearchManager::OnReloadComplete);

	if(!UE::IsSavingPackage() && AssetRegistryModule && (!bDisableImmediateAssetDiscovery || !AssetRegistryModule->GetRegistry().IsLoadingAssets()))
	{
		// Do an immediate load of the cache to catch any Blueprints that were discovered by the asset registry before we initialized.
		BuildCache();
	}

	// Register global find results tabs.
	EnableGlobalFindResults(true);
}

void FFindInBlueprintSearchManager::OnAssetAdded(const FAssetData& InAssetData)
{
	const UClass* AssetClass = nullptr;
	{
		TWeakObjectPtr<const UClass> FoundClass = CachedAssetClasses.FindRef(InAssetData.AssetClassPath);
		if (FoundClass.IsValid())
		{
			AssetClass = FoundClass.Get();
		}
		else
		{
			AssetClass = InAssetData.GetClass();
			if (AssetClass)
			{
				CachedAssetClasses.Add(InAssetData.AssetClassPath, TWeakObjectPtr<const UClass>(AssetClass));
			}
		}
	}

	const IBlueprintAssetHandler* Handler = AssetClass ? FBlueprintAssetHandler::Get().FindHandler(AssetClass) : nullptr;

	// No handler means we can't process this asset
	if (!Handler)
	{
		return;
	}

	if (UObject* AssetObject = InAssetData.FastGetAsset(false))
	{
		if (ensureMsgf(AssetObject->IsA(AssetClass), TEXT("AssetClass (%s) matched handler, but does not match actual object type (%s) for asset: %s."), *AssetClass->GetName(), *AssetObject->GetClass()->GetName(), *AssetObject->GetPathName()))
		{
			UBlueprint* Blueprint = Handler->RetrieveBlueprint(AssetObject);
			if (Blueprint)
			{
				AddOrUpdateBlueprintSearchMetadata(Blueprint);
			}
		}
	}
	else if (Handler->AssetContainsBlueprint(InAssetData))
	{
		AddUnloadedBlueprintSearchMetadata(InAssetData);
	}
}

void FFindInBlueprintSearchManager::AddUnloadedBlueprintSearchMetadata(const FAssetData& InAssetData)
{
	// Check first for versioned FiB data (latest codepath)
	FAssetDataTagMapSharedView::FFindTagResult Result = InAssetData.TagsAndValues.FindTag(FBlueprintTags::FindInBlueprintsData);
	if (Result.IsSet())
	{
		if (bDisableImmediateAssetDiscovery)
		{
			// If the versioned key is set at all, we assume it is valid and will parse it later
			ExtractUnloadedFiBData(InAssetData, nullptr, FBlueprintTags::FindInBlueprintsData, EFiBVersion::FIB_VER_NONE);
		}
		else
		{
			// Extract it now
			FString FiBVersionedSearchData = Result.GetValue();
			if (FiBVersionedSearchData.Len() == 0)
			{
				UnindexedAssets.Add(InAssetData.GetSoftObjectPath());
			}
			else
			{
				ExtractUnloadedFiBData(InAssetData, &FiBVersionedSearchData, NAME_None, EFiBVersion::FIB_VER_NONE);
			}
		}
	}
	else
	{
		// Check for legacy (unversioned) FiB data
		FAssetDataTagMapSharedView::FFindTagResult ResultLegacy = InAssetData.TagsAndValues.FindTag(FBlueprintTags::UnversionedFindInBlueprintsData);
		if (ResultLegacy.IsSet())
		{
			if (bDisableImmediateAssetDiscovery)
			{
				ExtractUnloadedFiBData(InAssetData, nullptr, FBlueprintTags::UnversionedFindInBlueprintsData, EFiBVersion::FIB_VER_BASE);
			}
			else
			{
				FString FiBUnversionedSearchData = ResultLegacy.GetValue();
				ExtractUnloadedFiBData(InAssetData, &FiBUnversionedSearchData, NAME_None, EFiBVersion::FIB_VER_BASE);
			}
		}
		// The asset has no FiB data, keep track of it so we can inform the user
		else
		{
			UnindexedAssets.Add(InAssetData.GetSoftObjectPath());
		}
	}
}

bool FFindInBlueprintSearchManager::ProcessEncodedValueForUnloadedBlueprint(FSearchData& SearchData)
{
	const FString AssetPath = SearchData.AssetPath.ToString();
	FString TempEncodedString;
	if (!SearchData.AssetKeyForValue.IsNone() && SearchData.Value.IsEmpty())
	{
		// Get the string from the asset registry now, use the on disk version to avoid inconsistencies
		FAssetData AssetData;
		AssetData = AssetRegistryModule->Get().GetAssetByObjectPath(SearchData.AssetPath, true);
		if (ensure(AssetData.IsValid()))
		{
			FAssetDataTagMapSharedView::FFindTagResult Result = AssetData.TagsAndValues.FindTag(SearchData.AssetKeyForValue);
			if (Result.IsSet())
			{
				// This makes a large string copy because the version in the asset data cannot be treated as an FString
				TempEncodedString = Result.GetValue();	
			}
		}
		if (TempEncodedString.Len() == 0)
		{
			// This asset was saved with an empty tag which is invalid
			UE_LOGF(LogFindInBlueprint, Warning, "%ls AssetData loaded with %ls has an invalid FiB tag!", *AssetData.GetSoftObjectPath().ToString(), *AssetPath);

			return false;
		}
	}

	const FString& EncodedString = !TempEncodedString.IsEmpty() ? TempEncodedString : SearchData.Value;
	if (FiBSerializationHelpers::ValidateSearchDataVersionInfo(AssetPath, EncodedString, SearchData.VersionInfo))
	{
		// Parse the data into json and then clear the memory
		SearchData.ImaginaryBlueprint = MakeShareable(new FImaginaryBlueprint(FPaths::GetBaseFilename(AssetPath), AssetPath, SearchData.ParentClass, SearchData.Interfaces, EncodedString, SearchData.VersionInfo));
		SearchData.ClearEncodedValue();

		return true;
	}

	return false;
}

void FFindInBlueprintSearchManager::ExtractUnloadedFiBData(const FAssetData& InAssetData, FString* InFiBData, FName InKeyForFiBData, EFiBVersion InFiBDataVersion)
{
	CSV_SCOPED_TIMING_STAT(FindInBlueprint, ExtractUnloadedFiBData);
	CSV_CUSTOM_STAT(FindInBlueprint, ExtractUnloadedCountThisFrame, 1, ECsvCustomStatOp::Accumulate);

	// Check whether this asset has already had its search data cached. If marked for deletion, we will replace it with a new entry.
	FSearchData SearchData = GetSearchDataForAssetPath(InAssetData.GetSoftObjectPath());
	if (SearchData.IsValid() && !SearchData.IsMarkedForDeletion())
	{
		return;
	}

	FSearchData NewSearchData;
	NewSearchData.AssetPath = InAssetData.GetSoftObjectPath();
	InAssetData.GetTagValue(FBlueprintTags::ParentClassPath, NewSearchData.ParentClass);

	const FString ImplementedInterfaces = InAssetData.GetTagValueRef<FString>(FBlueprintTags::ImplementedInterfaces);
	if(!ImplementedInterfaces.IsEmpty())
	{
		// ImplementedInterfaces is an array of structs (FBPInterfaceDescription). When exported to an AR tag value, each entry will be formatted as:
		//
		//	Entry := (Interface=Type'Package.Class') OR
		//  Entry := (Interface=Type'Package.Class',Graphs=(Type'Package.Blueprint:Graph1',Type'Package.Blueprint:Graph2',...Type'Package.Blueprint:GraphN'))
		//
		// The full tag value (array of exported struct values) will then be formatted as follows:
		//
		//  Value := (Entry1,Entry2,...EntryN)
		//
		// Here we parse out the 'Interface' values, and add only the 'Name' part of the full interface path substrings into the new search data as keywords.

		auto FindSubStringPosLambda = [&ImplementedInterfaces](const FString& InSubString, int32 StartPosition) -> int32
		{
			return ImplementedInterfaces.Find(InSubString, ESearchCase::CaseSensitive, ESearchDir::FromStart, StartPosition);
		};

		static const FString InterfaceFieldName = GET_MEMBER_NAME_STRING_CHECKED(FBPInterfaceDescription, Interface);

		int32 CurPos = FindSubStringPosLambda(InterfaceFieldName, 0);
		while (CurPos != INDEX_NONE)
		{
			CurPos = FindSubStringPosLambda(TEXT("="), CurPos);
			if (CurPos != INDEX_NONE)
			{
				CurPos = FindSubStringPosLambda(TEXT("."), CurPos);
				if (CurPos != INDEX_NONE)
				{
					const int32 StartPos = CurPos + 1;
					CurPos = FindSubStringPosLambda(TEXT("\'"), StartPos);
					if (CurPos != INDEX_NONE)
					{
						const FString InterfaceName = ImplementedInterfaces.Mid(StartPos, CurPos - StartPos);
						if (!InterfaceName.IsEmpty())
						{
							NewSearchData.Interfaces.Add(InterfaceName.TrimQuotes());
						}

						CurPos = FindSubStringPosLambda(InterfaceFieldName, CurPos + 1);
					}
				}
			}
		}
	}

	if (InFiBData)
	{
		NewSearchData.Value = MoveTemp(*InFiBData);
	}
	else
	{
		NewSearchData.Value.Empty();
	}

	NewSearchData.AssetKeyForValue = InKeyForFiBData;

	// This will be set to 'None' if the data is versioned. Deserialization of the actual version from the tag value is deferred until later.
	NewSearchData.VersionInfo.FiBDataVersion = InFiBDataVersion;

	// In these modes, or if there is no tag data, no additional indexing work is deferred for unloaded assets.
	if (!bDisableDeferredIndexing && !bDisableThreadedIndexing && NewSearchData.HasEncodedValue())
	{
		// Add it to the list of assets that require a full index rebuild from the metadata. This work will not block the main thread and is decoupled from the search thread.
		PendingAssets.Add(NewSearchData.AssetPath);
	}
	else
	{
		// We're not going to defer any indexing work, so mark it as having been indexed (i.e. it's now searchable).
		NewSearchData.StateFlags |= ESearchDataStateFlags::IsIndexed;
	}

	// Add search data for the asset into the database
	const bool bAllowNewEntry = true;
	ApplySearchDataToDatabase(MoveTemp(NewSearchData), bAllowNewEntry);
}

FSearchData FFindInBlueprintSearchManager::GetSearchDataForIndex(int32 CacheIndex)
{
	FScopeLock ScopeLock(&SafeModifyCacheCriticalSection);

	if (CacheIndex >= 0 && CacheIndex < SearchArray.Num())
	{
		return SearchArray[CacheIndex];
	}

	return FSearchData();
}

FSearchData FFindInBlueprintSearchManager::GetSearchDataForAssetPath(const FSoftObjectPath& InAssetPath)
{
	FScopeLock ScopeLock(&SafeModifyCacheCriticalSection);

	const int32* ArrayIdx = SearchMap.Find(InAssetPath);
	if (ArrayIdx)
	{
		checkf(*ArrayIdx < SearchArray.Num(),
			TEXT("ArrayIdx:%d, SearchArray.Num():%d"),
			*ArrayIdx,
			SearchArray.Num());

		return SearchArray[*ArrayIdx];
	}

	return FSearchData();
}

void FFindInBlueprintSearchManager::ApplySearchDataToDatabase(FSearchData InSearchData, bool bAllowNewEntry)
{
	FScopeLock ScopeLock(&SafeModifyCacheCriticalSection);

	const int32* ArrayIdx = SearchMap.Find(InSearchData.AssetPath);
	if (ArrayIdx)
	{
		checkf(*ArrayIdx < SearchArray.Num(),
			TEXT("ArrayIdx:%d, SearchArray.Num():%d"),
			*ArrayIdx,
			SearchArray.Num());

		SearchArray[*ArrayIdx] = MoveTemp(InSearchData);
	}
	else if (bAllowNewEntry)
	{
		FSoftObjectPath AssetPath = InSearchData.AssetPath; // Copy before we move the data into the array

		int32 ArrayIndex = SearchArray.Add(MoveTemp(InSearchData));

		// Add the asset file path to the map along with the index into the array
		SearchMap.Add(AssetPath, ArrayIndex);
	}
}

void FFindInBlueprintSearchManager::RemoveBlueprintByPath(const FSoftObjectPath& InPath)
{
	FScopeLock ScopeLock(&SafeModifyCacheCriticalSection);

	int32* SearchIdx = SearchMap.Find(InPath);

	if(SearchIdx)
	{
		// Stale entries are flagged to be removed later, which happens after a GC pass.
		SearchArray[*SearchIdx].StateFlags |= ESearchDataStateFlags::WasRemoved;
	}
}
void FFindInBlueprintSearchManager::OnAssetRemoved(const struct FAssetData& InAssetData)
{
	if(InAssetData.IsAssetLoaded())
	{
		RemoveBlueprintByPath(InAssetData.GetSoftObjectPath());
	}
}

void FFindInBlueprintSearchManager::OnAssetRenamed(const struct FAssetData& InAssetData, const FString& InOldName)
{
	// Renaming removes the item from the manager, it will be re-added in the OnAssetAdded event under the new name.
	if(InAssetData.IsAssetLoaded())
	{
		RemoveBlueprintByPath(FSoftObjectPath(InOldName));
	}
}

void FFindInBlueprintSearchManager::OnAssetRegistryFilesLoaded()
{
	CSV_EVENT(FindInBlueprint, TEXT("OnAssetRegistryFilesLoaded"));

	// If we've deferred asset discovery, scan all registered assets now to extract search metadata from the asset tags.
	// Note: Depending on how many assets there are (loaded/unloaded), this may block the UI frame for an extended period.
	if (bDisableImmediateAssetDiscovery)
	{
		BuildCache();
	}

	if (!IsCacheInProgress() && PendingAssets.Num() == 0)
	{
		// Invoke the completion callback on any active global FiB tabs that are currently open to signal that the discovery stage is complete.
		for (TWeakPtr<SFindInBlueprints> FindResultsPtr : GlobalFindResults)
		{
			if (FindResultsPtr.IsValid())
			{
				FindResultsPtr.Pin()->OnCacheComplete(EFiBCacheOpType::CachePendingAssets, EFiBCacheOpFlags::None);
			}
		}
	}
}

void FFindInBlueprintSearchManager::OnAssetLoaded(UObject* InAsset)
{
	const IBlueprintAssetHandler* Handler = FBlueprintAssetHandler::Get().FindHandler(InAsset->GetClass());
	UBlueprint* BlueprintObject = Handler ? Handler->RetrieveBlueprint(InAsset) : nullptr;

	if (BlueprintObject)
	{
		FSoftObjectPath AssetPath(InAsset);

		// Find and update the item in the search array. Searches may currently be active, this will do no harm to them

		// Confirm that the Blueprint has not been added already, this can occur during duplication of Blueprints.
		FSearchData SearchData = GetSearchDataForAssetPath(AssetPath);

		// The asset registry might not have informed us of this asset yet.
		if (SearchData.IsValid())
		{
			// That index should never have a valid Blueprint already, but if it does, it should be the same Blueprint!
			ensureMsgf(SearchData.IsMarkedForDeletion() || !SearchData.Blueprint.IsValid() || SearchData.Blueprint == BlueprintObject,
				TEXT("Blueprint in database has path %s and is being stomped by %s"), *(SearchData.AssetPath.ToString()), *AssetPath.ToString());
			ensureMsgf(SearchData.IsMarkedForDeletion() || !SearchData.Blueprint.IsValid() || SearchData.AssetPath == AssetPath,
				TEXT("Blueprint in database has path %s and is being stomped by %s"), *(SearchData.AssetPath.ToString()), *AssetPath.ToString());
			SearchData.Blueprint = BlueprintObject;

			// Apply the updated entry to the database.
			ApplySearchDataToDatabase(MoveTemp(SearchData));
		}

		UnindexedAssets.Remove(AssetPath);
	}
}

void FFindInBlueprintSearchManager::OnBlueprintUnloaded(UBlueprint* InBlueprint)
{
	if (InBlueprint)
	{
		// Transient objects likely represent trashed or temporary Blueprint assets; we're not going to try and re-index those.
		if (InBlueprint->HasAnyFlags(RF_Transient))
		{
			FScopeLock ScopeLock(&SafeModifyCacheCriticalSection);

			// Invalidate any existing database entry that matches the reference. It's ok if we don't find a match here.
			for (int32 SearchIdx = 0; SearchIdx < SearchArray.Num(); ++SearchIdx)
			{
				if (SearchArray[SearchIdx].Blueprint.Get() == InBlueprint)
				{
					// Remove it from the lookup table so it can no longer be found indirectly.
					SearchMap.Remove(SearchArray[SearchIdx].AssetPath);

					// Stale entries are flagged to be removed later, which happens after a GC pass.
					SearchArray[SearchIdx].StateFlags |= ESearchDataStateFlags::WasRemoved;
					break;
				}
			}

			return;
		}

		// Reloaded assets will be re-indexed on load through the registry delegates; there's no need to handle any removal here.
		if (InBlueprint->HasAnyFlags(RF_NewerVersionExists))
		{
			return;
		}

		// Otherwise, the Blueprint is about to be unloaded. We need to re-index it as an unloaded asset.
		if(const UObject* AssetObject = GetAssetObject(InBlueprint))
		{
			// Mark any existing entry for deletion. This will allow the entry to be updated below.
			const FSoftObjectPath AssetPath{AssetObject};
			RemoveBlueprintByPath(AssetPath);

			// Add or update an existing entry to one that represents the data for the asset on disk, and re-index it.
			if (ensure(AssetRegistryModule != nullptr))
			{
				const bool bIncludeOnlyOnDiskAssets = true;
				FAssetData AssetData = AssetRegistryModule->Get().GetAssetByObjectPath(AssetPath, bIncludeOnlyOnDiskAssets);
				if (AssetData.IsValid() && AssetData.IsAssetLoaded())
				{
					// Re-scan the asset file on disk to ensure that the updated entry will be based on the serialized FiB tag.
					const FString PackageName = AssetData.PackageName.ToString();
					if (FPackageName::IsValidLongPackageName(PackageName))
					{
						FPackagePath PackagePath;
						if (FPackagePath::TryFromPackageName(PackageName, PackagePath))
						{
							FPackagePath OutPackagePath;
							const FPackageName::EPackageLocationFilter PackageLocation = FPackageName::DoesPackageExistEx(PackagePath, FPackageName::EPackageLocationFilter::Any, /*bMatchCaseOnDisk*/ false, &OutPackagePath);
							if (PackageLocation != FPackageName::EPackageLocationFilter::None)
							{
								if (PackageLocation == FPackageName::EPackageLocationFilter::FileSystem && OutPackagePath.HasLocalPath())
								{
									TArray<FString> FilesToScan = { OutPackagePath.GetLocalFullPath() };
									AssetRegistryModule->Get().ScanModifiedAssetFiles(FilesToScan);

									AssetData = AssetRegistryModule->Get().GetAssetByObjectPath(AssetPath, bIncludeOnlyOnDiskAssets);
								}

								if (AssetData.IsValid())
								{
									AddUnloadedBlueprintSearchMetadata(AssetData);
								}
							}
						}
					}
				}
			}
		}
	}
}

void FFindInBlueprintSearchManager::OnReloadComplete(EReloadCompleteReason Reason)
{
	CachedAssetClasses.Reset();
}

void FFindInBlueprintSearchManager::AddOrUpdateBlueprintSearchMetadata(UBlueprint* InBlueprint, EAddOrUpdateBlueprintSearchMetadataFlags InFlags/* = EAddOrUpdateBlueprintSearchMetadataFlags::None*/, EFiBVersion InVersion/* = EFiBVersion::FIB_VER_LATEST*/)
{
	CSV_SCOPED_TIMING_STAT(FindInBlueprint, AddOrUpdateBlueprintSearchMetadata);
	CSV_CUSTOM_STAT(FindInBlueprint, AddOrUpdateCountThisFrame, 1, ECsvCustomStatOp::Accumulate);

	check(InBlueprint);

	const bool bIsTransientBlueprint = InBlueprint->HasAnyFlags(RF_Transient) || InBlueprint->IsInPackage(GetTransientPackage());

	// No need to update the cache in the following cases:
	//	a) Indexing is disabled.
	//  b) The Blueprint is explicitly marked as being transient (i.e. internal utility-type assets that aren't saved).
	//	c) The Blueprint is not yet fully loaded. This ensures that we don't make attempts to re-index before load completion.
	//	d) The Blueprint was loaded for diffing. It makes search all very strange and allows you to fully open those Blueprints.
	//	e) The Blueprint was loaded/copied for PIE. These assets are temporarily created for a session and don't need to be re-indexed.
	if (!bEnableGatheringData
		|| bIsTransientBlueprint
		|| InBlueprint->HasAnyFlags(RF_NeedLoad | RF_NeedPostLoad)
		|| InBlueprint->GetOutermost()->HasAnyPackageFlags(PKG_ForDiffing | PKG_PlayInEditor))
	{
		return;
	}

	// Control flags
	const bool bForceRecache = EnumHasAllFlags(InFlags, EAddOrUpdateBlueprintSearchMetadataFlags::ForceRecache);
	const bool bClearCachedValue = EnumHasAllFlags(InFlags, EAddOrUpdateBlueprintSearchMetadataFlags::ClearCachedValue);

	UObject* AssetObject = GetAssetObject(InBlueprint);

	check(AssetObject);

	FSoftObjectPath AssetPath(AssetObject);
	FSearchData SearchData = GetSearchDataForAssetPath(AssetPath);

	if (SearchData.IsValid())
	{
		SearchData.Blueprint = InBlueprint; // Blueprint instance may change due to reloading
		SearchData.StateFlags &= ~(ESearchDataStateFlags::IsIndexed | ESearchDataStateFlags::WasRemoved);
	}
	else
	{
		SearchData = FSearchData();
		SearchData.AssetPath = AssetPath;
		SearchData.Blueprint = InBlueprint;
	}

	// Build the search data
	if (FProperty* ParentClassProp = InBlueprint->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UBlueprint, ParentClass)))
	{
		ParentClassProp->ExportTextItem_Direct(SearchData.ParentClass, ParentClassProp->ContainerPtrToValuePtr<uint8>(InBlueprint), nullptr, InBlueprint, 0);
	}

	if (SearchData.IsValid())
	{
		// Clear any previously-gathered data.
		SearchData.ClearEncodedValue();

		// Update version info stored in database. This indicates which format to use when regenerating the tag value.
		SearchData.VersionInfo = FSearchDataVersionInfo::Current();
		if (InVersion != EFiBVersion::FIB_VER_NONE)
		{
			SearchData.VersionInfo.FiBDataVersion = InVersion;
		}

		// During unindexed/out-of-date caching we will arrive here as a result of loading the asset, so don't remove the IsUnindexedCacheInProgress() check!
		if (bForceRecache || bClearCachedValue || IsUnindexedCacheInProgress() || bDisableDeferredIndexing)
		{
			// Cannot successfully gather most searchable data if there is no SkeletonGeneratedClass, so don't try, leave it as whatever it was last set to
			if (!bClearCachedValue && InBlueprint->SkeletonGeneratedClass != nullptr)
			{
				using namespace BlueprintSearchMetaDataHelpers;

				// Update search metadata string content
				SearchData.Value = GatherBlueprintSearchMetadata<FFindInBlueprintJsonWriter>(InBlueprint, SearchData.VersionInfo.FiBDataVersion);
			}

			// Mark it as having been indexed in the following cases:
			// a) Deferred or multithreaded indexing is disabled. In these cases, indexing will be handled by the search thread.
			// b) There is no generated metadata. In this case, there's nothing to search, so there's no need to index the asset.
			if (bDisableDeferredIndexing || bDisableThreadedIndexing || SearchData.Value.Len() == 0)
			{
				// Mark it as having been indexed (it's now searchable).
				SearchData.StateFlags |= ESearchDataStateFlags::IsIndexed;

				// Remove it from the list of pending assets (if it exists).
				PendingAssets.Remove(AssetPath);
			}
			else if(!bDisableThreadedIndexing)
			{
				// With multithreaded indexing, we defer additional work beyond regenerating the metadata, so ensure it's been added.
				PendingAssets.Add(AssetPath);
			}
		}
		else if (!bDisableDeferredIndexing)
		{
			// Add it to the list of assets to be indexed (deferred)
			PendingAssets.Add(AssetPath);
		}

		// Copy new/updated search data into the cache
		const bool bAllowNewEntry = true;
		ApplySearchDataToDatabase(MoveTemp(SearchData), bAllowNewEntry);
	}
}

FFindInBlueprintSearchManager::FActiveSearchQueryPtr FFindInBlueprintSearchManager::FindSearchQuery(const class FStreamSearch* InSearchOriginator) const
{
	// Must lock this behind a critical section to ensure that no other thread is accessing it at the same time
	FScopeLock ScopeLock(&SafeQueryModifyCriticalSection);

	return ActiveSearchQueries.FindRef(InSearchOriginator);
}

FSearchData FFindInBlueprintSearchManager::GetNextSearchDataForQuery(const FStreamSearch* SearchOriginator, FActiveSearchQueryPtr SearchQuery, bool bCheckDeferredList)
{
	check(SearchOriginator);

	// If terminated, don't advance, just return with no entry.
	if (SearchOriginator->WasStopped())
	{
		return FSearchData();
	}

	// Get the entry in the index cache for the next asset to search.
	FSearchData SearchData = GetSearchDataForIndex(SearchQuery->NextIndex);
	while (SearchData.IsValid())
	{
		// Advance the current search index.
		++SearchQuery->NextIndex;

		// If this asset has not been indexed, don't search it yet.
		if (!SearchData.IsIndexingCompleted())
		{
			SearchQuery->DeferredAssetPaths.Enqueue(SearchData.AssetPath);
		}
		else if (!SearchData.IsMarkedForDeletion())
		{
			// Ok to use this asset's index entry; break out of the loop.
			break;
		}

		// Move on to the next entry in the index cache.
		SearchData = GetSearchDataForIndex(SearchQuery->NextIndex);
	}

	// If we don't have valid search data, try the deferred list from above.
	if (!SearchData.IsValid() && bCheckDeferredList)
	{
		FSoftObjectPath AssetPath, FirstAssetPath;
		while (SearchQuery->DeferredAssetPaths.Dequeue(AssetPath))
		{
			// Skip invalid paths (shouldn't happen, but just in case).
			if (AssetPath.IsNull())
			{
				continue;
			}
			else if (AssetPath == FirstAssetPath)
			{
				if (bIsPausing)
				{
					// Wait here until the search query is unpaused.
					BlockSearchQueryIfPaused();
				}
				else
				{
					// Yield to allow indexing to progress a bit further.
					FPlatformProcess::Sleep(0.1f);
				}

				// If terminated, break out of the loop and return with no entry.
				if (SearchOriginator->WasStopped())
				{
					return FSearchData();
				}

				// Check for a new entry in case the cache has grown in size.
				SearchData = GetNextSearchDataForQuery(SearchOriginator, SearchQuery, /*bCheckDeferredList = */false);
				if (SearchData.IsValid())
				{
					// Ok to use this entry; break out of the loop.
					break;
				}
			}

			SearchData = GetSearchDataForAssetPath(AssetPath);
			if (SearchData.IsValid() && !SearchData.IsMarkedForDeletion())
			{
				// If this asset is still waiting to be indexed, put it back into the queue.
				if (!SearchData.IsIndexingCompleted())
				{
					// Only put it back into the queue if we're either still actively indexing or if the asset is queued up for the next indexing operation.
					if (IsCacheInProgress() || PendingAssets.Contains(AssetPath) || AssetsToIndexOnFirstSearch.Contains(AssetPath))
					{
						SearchQuery->DeferredAssetPaths.Enqueue(AssetPath);
					}
					else
					{
						UE_LOGF(LogFindInBlueprint, Warning, "%ls has been unexpectedly excluded from indexing and will not be included in the search.", *AssetPath.ToString());
					}

					// Keep track of the first dequeued asset path. If we wrap back around, we'll yield to give the indexing thread more time to work.
					if (FirstAssetPath.IsNull())
					{
						FirstAssetPath = AssetPath;
					}
				}
				else
				{
					// Ok to use this entry; break out of the loop.
					break;
				}
			}
		}
	}

	// Increment the search counter if we have valid search data.
	if (SearchData.IsValid())
	{
		++SearchQuery->SearchCount;
	}

	return SearchData;
}

void FFindInBlueprintSearchManager::BlockSearchQueryIfPaused()
{
	// Check if the thread has been told to pause, this occurs for the Garbage Collector and for saving to disk
	if (bIsPausing == true)
	{
		// Pause all searching, the GC is running and we will also be saving the database
		ActiveSearchCounter.Decrement();
		FScopeLock ScopeLock(&PauseThreadsCriticalSection);
		ActiveSearchCounter.Increment();
	}
}

void FFindInBlueprintSearchManager::BeginSearchQuery(const FStreamSearch* InSearchOriginator)
{
	if (AssetRegistryModule == nullptr)
	{
		UE_LOGF(LogBlueprint, Warning, "Find-in-Blueprints was not fully initialized, possibly due to problems being initialized while saving a package. Please explicitly initialize earlier!");
	}

	// Unblock caching operations from doing a full indexing pass on pending assets
	bHasFirstSearchOccurred = true;

	// Cannot begin a search thread while saving
	FScopeLock ScopeLock(&PauseThreadsCriticalSection);
	FScopeLock ScopeLock2(&SafeQueryModifyCriticalSection);

	FActiveSearchQueryPtr NewSearchQuery = MakeShared<FActiveSearchQuery, ESPMode::ThreadSafe>();
	check(NewSearchQuery.IsValid());

	ActiveSearchCounter.Increment();
	ActiveSearchQueries.Add(InSearchOriginator, NewSearchQuery);
}

bool FFindInBlueprintSearchManager::ContinueSearchQuery(const FStreamSearch* InSearchOriginator, FSearchData& OutSearchData)
{
	// If paused, wait here until searching is resumed.
	BlockSearchQueryIfPaused();

	check(InSearchOriginator);

	// Check for an explicit termination of the search thread.
	if (InSearchOriginator->WasStopped())
	{
		return false;
	}

	// We can have multiple active search queries, so find the matching one that corresponds to the originating search thread.
	FActiveSearchQueryPtr SearchQuery = FindSearchQuery(InSearchOriginator);
	if (!SearchQuery.IsValid())
	{
		// Terminate the search if there is no matching query.
		return false;
	}

	// Grab the next entry and update the active search query. Include the list of entries that were deferred pending completion of async indexing work.
	const bool bCheckDeferredList = true;
	FSearchData SearchData = GetNextSearchDataForQuery(InSearchOriginator, SearchQuery, bCheckDeferredList);
	if (SearchData.IsValid())
	{
		// In these modes, the full index may not have been parsed yet. We'll do that now on the search thread.
		if (bDisableDeferredIndexing || bDisableThreadedIndexing)
		{
			// Must lock this behind a critical section to ensure that no other thread is accessing it at the same time
			FScopeLock Lock(&SafeModifyCacheCriticalSection);

			// Grab the latest entry from the cache. In the case of parallel global searches, one of the other threads may have already updated it while we were blocked by the mutex above.
			SearchData = GetSearchDataForAssetPath(SearchData.AssetPath);

			// If there is FiB data, parse it into an ImaginaryBlueprint
			if (SearchData.IsValid() && SearchData.HasEncodedValue())
			{
				if (ProcessEncodedValueForUnloadedBlueprint(SearchData))
				{
					check(SearchData.ImaginaryBlueprint.IsValid());

					// In the case of parallel global searches, two search threads may be looking at the same entry. Thus, we only allow one search thread to be actively parsing JSON nodes.
					SearchData.ImaginaryBlueprint->EnableInterlockedParsing();
				}

				// Update the entry in the database
				ApplySearchDataToDatabase(SearchData);
			}
		}

		OutSearchData = SearchData;
		return true;
	}

	return false;
}

void FFindInBlueprintSearchManager::EnsureSearchQueryEnds(const class FStreamSearch* InSearchOriginator)
{
	// Must lock this behind a critical section to ensure that no other thread is accessing it at the same time
	FScopeLock ScopeLock(&SafeQueryModifyCriticalSection);

	// If the search thread is still considered active, remove it
	if (ActiveSearchQueries.Contains(InSearchOriginator))
	{
		ActiveSearchQueries.Remove(InSearchOriginator);
		ActiveSearchCounter.Decrement();
	}
}

float FFindInBlueprintSearchManager::GetPercentComplete(const FStreamSearch* InSearchOriginator) const
{
	FScopeLock ScopeLock(&SafeQueryModifyCriticalSection);
	FActiveSearchQueryPtr SearchQuery = FindSearchQuery(InSearchOriginator);

	float ReturnPercent = 0.0f;

	if (SearchQuery.IsValid())
	{
		ReturnPercent = (float)SearchQuery->SearchCount / (float)SearchArray.Num();
	}

	return ReturnPercent;
}

FSearchData FFindInBlueprintSearchManager::QuerySingleBlueprint(UBlueprint* InBlueprint, bool bInRebuildSearchData)
{
	// AddOrUpdateBlueprintSearchMetadata would fail to cache any data for a Blueprint loaded specifically for diffing, but the bigger question
	// here in this function is how you are doing a search specifically for data within this Blueprint. This function is limited to be called
	// only when querying within the specific Blueprint (somehow opened a diff Blueprint) and when gathering the Blueprint's tags (usually for saving)
	const bool bIsDiffingBlueprint = InBlueprint->GetOutermost()->HasAnyPackageFlags(PKG_ForDiffing);
	if (!bIsDiffingBlueprint)
	{
		if (bInRebuildSearchData)
		{
			// Update the Blueprint, make sure it is fully up-to-date
			AddOrUpdateBlueprintSearchMetadata(InBlueprint, EAddOrUpdateBlueprintSearchMetadataFlags::ForceRecache);
		}

		UObject* AssetObject = GetAssetObject(InBlueprint);
		check(AssetObject);

		FSoftObjectPath Key(AssetObject);
		FSearchData SearchData = GetSearchDataForAssetPath(Key);
		if (SearchData.IsValid())
		{
			return SearchData;
		}
		else if(bInRebuildSearchData)
		{
			// Warn here, since we make sure to refresh the search data for this Blueprint when doing the search, and we expect that it should have
			// been indexed. Note that there are some situations in which we never index a Blueprint asset (@see AddOrUpdateBlueprintSearchMetadata).
			UE_LOGF(LogBlueprint, Warning, "Attempted to query a Blueprint (%ls) that was not re-indexed even after rebuilding. No results can be returned.", *InBlueprint->GetPathName());
		}
	}
	else
	{
		// Also warn here as we do not index diff-only packages.
		UE_LOGF(LogBlueprint, Warning, "Attempted to query an old Blueprint package opened for diffing!");
	}
	return FSearchData();
}

void FFindInBlueprintSearchManager::PauseFindInBlueprintSearch()
{
	// Lock the critical section and flag that threads need to pause, they will pause when they can
	PauseThreadsCriticalSection.Lock();
	bIsPausing = true;

	// It is UNSAFE to lock any other critical section here, threads need them to finish a cycle of searching. Next cycle they will pause

	// We don't expect to be pausing a search off the main/game thread (i.e. GC).
	check(IsInGameThread());
	FTaskGraphInterface& TaskGraphInterface = FTaskGraphInterface::Get();

	// Wait until all threads have come to a stop, it won't take long
	while(ActiveSearchCounter.GetValue() > 0)
	{
		// Async tasks may have been registered to the game thread, so make sure
		// we're processing those here (e.g. FSearchableValueInfo::GetDisplayText).
		if (TaskGraphInterface.IsThreadProcessingTasks(ENamedThreads::GameThread))
		{
			TaskGraphInterface.ProcessThreadUntilIdle(ENamedThreads::GameThread);
		}

		// Yield some time to other threads.
		FPlatformProcess::Sleep(0.1f);
	}
}

void FFindInBlueprintSearchManager::UnpauseFindInBlueprintSearch()
{
	// Before unpausing, we clean the cache of any excess data to keep it from bloating in size
	CleanCache();
	bIsPausing = false;

	// Release the threads to continue searching.
	PauseThreadsCriticalSection.Unlock();
}

void FFindInBlueprintSearchManager::CleanCache()
{
	// We need to cache where the active queries are so that we can put them back in a safe and expected position
	TMap<const FStreamSearch*, FSoftObjectPath> CacheQueries;
	for( auto It = ActiveSearchQueries.CreateIterator() ; It ; ++It )
	{
	 	const FStreamSearch* ActiveSearch = It.Key();
	 	check(ActiveSearch);
	 	{
			FActiveSearchQueryPtr SearchQuery = FindSearchQuery(ActiveSearch);
			if (SearchQuery.IsValid())
			{
				// Don't check the deferred list here; in this case we are only fixing up the query's index position (state) if
				// we haven't yet reached the end of the database, and we don't care about assets that are waiting to be indexed.
				const bool bCheckDeferredList = false;
				FSearchData SearchData = GetNextSearchDataForQuery(ActiveSearch, SearchQuery, bCheckDeferredList);
				if (SearchData.IsValid())
				{
					FSoftObjectPath CachePath = SearchData.AssetPath;
					CacheQueries.Add(ActiveSearch, CachePath);
				}
			}
	 	}
	}

	TMap<FSoftObjectPath, int32> NewSearchMap;
	TArray<FSearchData> NewSearchArray;

	// Don't allow background indexing tasks to access the search database while we fix it up.
	FScopeLock Lock(&SafeModifyCacheCriticalSection);

	for(const TPair<FSoftObjectPath, int32>& SearchValuePair : SearchMap)
	{
		// Here it builds the new map/array, clean of deleted content.

		// If the database item is invalid, marked for deletion, stale or pending kill (if loaded), remove it from the database
		const bool bEvenIfPendingKill = true;
		FSearchData& SearchData = SearchArray[SearchValuePair.Value];
		if (!SearchData.IsValid()
			|| SearchData.IsMarkedForDeletion()
			|| SearchData.Blueprint.IsStale()
			|| (SearchData.Blueprint.IsValid(bEvenIfPendingKill) && !IsValid(SearchData.Blueprint.Get(bEvenIfPendingKill))))
		{
			// Also remove it from the list of loaded assets that require indexing
			PendingAssets.Remove(SearchData.AssetPath);
			UnindexedAssets.Remove(SearchData.AssetPath);
		}
		else
		{
			// Build the new map/array
			NewSearchMap.Add(SearchValuePair.Key, NewSearchArray.Add(MoveTemp(SearchData)));
		}
	}

	SearchMap = MoveTemp( NewSearchMap );
	SearchArray = MoveTemp( NewSearchArray );

	// After the search, we have to place the active search queries where they belong
	for( auto& CacheQuery : CacheQueries )
	{
	 	int32 NewMappedIndex = 0;
	 	// Is the CachePath is valid? Otherwise we are at the end and there are no more search results, leave the query there so it can handle shutdown on it's own
	 	if(!CacheQuery.Value.IsNull())
	 	{
	 		int32* NewMappedIndexPtr = SearchMap.Find(CacheQuery.Value);
	 		check(NewMappedIndexPtr);
	 
	 		NewMappedIndex = *NewMappedIndexPtr;
	 	}
	 	else
	 	{
	 		NewMappedIndex = SearchArray.Num();
	 	}
	 
		// Update the active search to the new index of where it is at in the search
		(ActiveSearchQueries.FindRef(CacheQuery.Key))->NextIndex = NewMappedIndex;
	}
}

void FFindInBlueprintSearchManager::BuildCache()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FFindInBlueprintSearchManager::BuildCache);

	if (!ensure(AssetRegistryModule))
	{
		return;
	}

	IAssetRegistry* AssetRegistry = AssetRegistryModule->TryGet();
	if (!AssetRegistry)
	{
		return;
	}

	TArray< FAssetData > BlueprintAssets;
	FARFilter ClassFilter;
	ClassFilter.bRecursiveClasses = true;
	ClassFilter.bIncludeOnlyOnDiskAssets = true;

	for (FTopLevelAssetPath ClassPathName : FBlueprintAssetHandler::Get().GetRegisteredClassNames())
	{
		ClassFilter.ClassPaths.Add(ClassPathName);
	}

	AssetRegistry->GetAssets(ClassFilter, BlueprintAssets);
	
	for( FAssetData& Asset : BlueprintAssets )
	{
		OnAssetAdded(Asset);
	}

	// Register to be notified for future asset registry events.
	AssetRegistry->OnAssetAdded().AddRaw(this, &FFindInBlueprintSearchManager::OnAssetAdded);
	AssetRegistry->OnAssetRemoved().AddRaw(this, &FFindInBlueprintSearchManager::OnAssetRemoved);
	AssetRegistry->OnAssetRenamed().AddRaw(this, &FFindInBlueprintSearchManager::OnAssetRenamed);
}

void FFindInBlueprintSearchManager::DumpCache(FArchive& Ar)
{
	FScopeLock ScopeLock(&SafeModifyCacheCriticalSection);

	for (const FSearchData& SearchData : SearchArray)
	{
		if (SearchData.ImaginaryBlueprint.IsValid())
		{
			SearchData.ImaginaryBlueprint->DumpParsedObject(Ar);
		}
	}
}

void FFindInBlueprintSearchManager::DoRandomIndexCacheRebuild()
{
	if (AssetRegistryModule && bHasFirstSearchOccurred && !IsCacheInProgress())
	{
		FARFilter ARFilter;
		ARFilter.bRecursiveClasses = true;
		for (FTopLevelAssetPath ClassPathName : FBlueprintAssetHandler::Get().GetRegisteredClassNames())
		{
			ARFilter.ClassPaths.Add(ClassPathName);
		}

		TArray<FAssetData> AssetList;
		AssetRegistryModule->Get().GetAssets(ARFilter, AssetList);
		if (AssetList.IsEmpty())
		{
			return;
		}

		const int32 MinSize = FMath::Min(1000, AssetList.Num());
		const int32 MaxSize = FMath::Min(10000, AssetList.Num());
		const int32 SubsetSize = MinSize + (FMath::Rand() % (MaxSize - MinSize + 1));
		const int32 StartingIndex = FMath::Rand() % (AssetList.Num() - SubsetSize + 1);
		for (int32 i = StartingIndex; i < StartingIndex + SubsetSize; ++i)
		{
			RemoveBlueprintByPath(AssetList[i].ToSoftObjectPath());

			if (AssetList[i].IsAssetLoaded())
			{
				UBlueprint* Blueprint = CastChecked<UBlueprint>(AssetList[i].GetAsset());
				AddOrUpdateBlueprintSearchMetadata(Blueprint);
			}
			else
			{
				AddUnloadedBlueprintSearchMetadata(AssetList[i]);
			}
		}
	}
}

FText FFindInBlueprintSearchManager::ConvertHexStringToFText(FString InHexString)
{
	TArray<uint8> SerializedData;
	SerializedData.AddZeroed(InHexString.Len());

	HexToBytes(InHexString, SerializedData.GetData());

	FText ResultText;

	FMemoryReader Ar(SerializedData);
	Ar << ResultText;
	Ar.Close();

	return ResultText;
}

FString FFindInBlueprintSearchManager::ConvertFTextToHexString(FText InValue)
{
	TArray<uint8> SerializedData;
	FMemoryWriter Ar(SerializedData);

	Ar << InValue;
	Ar.Close();

	return BytesToHex(SerializedData.GetData(), SerializedData.Num());
}

FString FFindInBlueprintSearchManager::GenerateSearchIndexForDebugging(UBlueprint* InBlueprint)
{
	using namespace BlueprintSearchMetaDataHelpers;
	return GatherBlueprintSearchMetadata<TFindInBlueprintJsonStringWriter<TPrettyJsonPrintPolicy<TCHAR>>>(InBlueprint, EFiBVersion::FIB_VER_LATEST);
}

void FFindInBlueprintSearchManager::OnCacheAllUnindexedAssets(bool bInSourceControlActive, bool bInCheckoutAndSave)
{
	// We need to check validity first in case the user has closed the initiating FiB tab before responding to the source control login dialog (which is modeless).
	if (CachingObject.IsValid())
	{
		if(bInSourceControlActive && bInCheckoutAndSave)
		{
			TArray<FString> UncachedAssetStrings;
			const TArray<FSoftObjectPath>& TotalUncachedAssets = CachingObject->GetUncachedAssetList();
		
			UncachedAssetStrings.Reserve(TotalUncachedAssets.Num());
			for (const FSoftObjectPath& UncachedAsset : TotalUncachedAssets)
			{
				UncachedAssetStrings.Add(UncachedAsset.ToString());
			}
			FEditorFileUtils::CheckoutPackages(UncachedAssetStrings);
		}

		// Start the cache process.
		CachingObject->Start();
	}
}

void FFindInBlueprintSearchManager::CacheAllAssets(TWeakPtr< SFindInBlueprints > InSourceWidget, const FFindInBlueprintCachingOptions& InOptions)
{
	// Do not start another caching process if one is in progress
	if(!IsCacheInProgress())
	{
		FCacheAllBlueprintsTickableObject::FCacheParams CacheParams;
		CacheParams.OpType = InOptions.OpType;

		if (CacheParams.OpType == EFiBCacheOpType::CachePendingAssets)
		{
			CacheParams.OnFinished = InOptions.OnFinished;
			CacheParams.AsyncTaskBatchSize = AsyncTaskBatchSize;

			// Determine if PIE is active - in that case we're potentially streaming assets in at random intervals, so just hide the progress UI while re-indexing those assets
			const bool bIsPIESimulating = (GEditor->bIsSimulatingInEditor || GEditor->PlayWorld);

			// Display progress during a re-indexing operation only if we have multiple assets to process (e.g. avoid showing after compiling a single asset) and we're not in PIE
			if ((PendingAssets.Num() > 1) && !bIsPIESimulating)
			{
				CacheParams.OpFlags |= EFiBCacheOpFlags::ShowProgress;
			}

			// Keep popup notifications hidden during this operation
			CacheParams.OpFlags |= EFiBCacheOpFlags::HideNotifications;

			const bool bIsAssetDiscoveryInProgress = IsAssetDiscoveryInProgress();

			// Determine if this operation has been continued from a previous operation.
			bool bIsCachingDiscoveredAssets = EnumHasAnyFlags(InOptions.OpFlags, EFiBCacheOpFlags::IsCachingDiscoveredAssets);
			if (!bIsCachingDiscoveredAssets)
			{
				// During the initial asset discovery and registration stage, we'll index an unknown number of assets in the background as a continuous caching operation.
				bIsCachingDiscoveredAssets = !bDisableDeferredIndexing && bIsAssetDiscoveryInProgress;
			}

			if (bIsCachingDiscoveredAssets)
			{
				CacheParams.OpFlags |= EFiBCacheOpFlags::IsCachingDiscoveredAssets;

				if (bIsAssetDiscoveryInProgress)
				{
					// Keep progress bars hidden during asset discovery since the endpoint will be a moving target. Also keep progress onscreen between operations.
					CacheParams.OpFlags |= EFiBCacheOpFlags::HideProgressBars;
					CacheParams.OpFlags |= EFiBCacheOpFlags::KeepProgressVisibleOnCompletion;
				}
			}

			if (bDisableThreadedIndexing)
			{
				CacheParams.OpFlags |= EFiBCacheOpFlags::ExecuteOnMainThread;
			}
//			else if (bEnableCSVStatsProfiling)
//			{
//				CacheParams.OpFlags |= EFiBCacheOpFlags::ExecuteOnSingleThread;
//			}

			// Keep track of which global FiB context started the operation (if any)
			SourceCachingWidget = InSourceWidget;

			// If async indexing is disabled, or if it's enabled and we're allowed to do a full index pass, use the full array
			if (bDisableThreadedIndexing || bHasFirstSearchOccurred)
			{
				// Create a new instance of the caching object.
				CachingObject = MakeUnique<FCacheAllBlueprintsTickableObject>(PendingAssets, CacheParams);
			}
			else
			{
				// Find all pending assets for which we can gather and cache search metadata
				TSet<FSoftObjectPath> AssetsToPartiallyCache;
				for (const FSoftObjectPath& AssetPath : PendingAssets)
				{
					if (const int32* IndexPtr = SearchMap.Find(AssetPath))
					{
						const FSearchData& SearchData = SearchArray[*IndexPtr];
						if (!SearchData.IsIndexingCompleted() && SearchData.Value.Len() == 0)
						{
							AssetsToPartiallyCache.Add(AssetPath);
						}
					}

					// Keep track of all assets for the full indexing pass, which is currently being deferred until the first global search is initiated.
					AssetsToIndexOnFirstSearch.Add(AssetPath);
				}

				// If we found one or more assets above...
				if (AssetsToPartiallyCache.Num() > 0)
				{
					// Run a partial indexing pass only.
					CacheParams.OpFlags |= EFiBCacheOpFlags::ExecuteGatherPhaseOnly;

					// Create a new instance of the caching object.
					CachingObject = MakeUnique<FCacheAllBlueprintsTickableObject>(MoveTemp(AssetsToPartiallyCache), CacheParams);
				}
			}

			// Immediately start the operation (non-interactive)
			if (CachingObject.IsValid())
			{
				CachingObject->Start();
			}

			// Reset for the next set of incoming deferred assets.
			PendingAssets.Empty();
		}
		else
		{
			TArray<FSoftObjectPath> BlueprintsToUpdate;
			// Add any out-of-date Blueprints to the list
			for (FSearchData SearchData : SearchArray)
			{
				if ((SearchData.Value.Len() != 0 || SearchData.ImaginaryBlueprint.IsValid()) && SearchData.VersionInfo.FiBDataVersion < InOptions.MinimiumVersionRequirement)
				{
					BlueprintsToUpdate.Add(SearchData.AssetPath);
				}
			}

			FText DialogTitle = LOCTEXT("ConfirmIndexAll_Title", "Indexing All");
			FFormatNamedArguments Args;
			Args.Add(TEXT("PackageCount"), UnindexedAssets.Num() + BlueprintsToUpdate.Num());
			Args.Add(TEXT("UnindexedCount"), UnindexedAssets.Num());
			Args.Add(TEXT("OutOfDateCount"), BlueprintsToUpdate.Num());

			// Retrieve from blueprint editor settings whether user is allowed to checkout and resave
			const bool bCanCheckoutResaveAll = GetDefault<UBlueprintEditorSettings>()->AllowIndexAllBlueprints == EFiBIndexAllPermission::CheckoutAndResave;

			// Present a prompt depending on whether checkout is allowed
			EAppReturnType::Type ReturnValue;
			if (bCanCheckoutResaveAll)
			{
				const FText DialogDisplayText = FText::Format(LOCTEXT("CacheAllConfirmationMessage_UnindexedAndOutOfDate_WithCheckout", "About to CHECKOUT and RESAVE {PackageCount} Blueprints ({UnindexedCount} unindexed/{OutOfDateCount} out-of-date)! The editor may become unresponsive while these assets are loaded for indexing. Save your work before initiating this: broken assets and memory usage can affect editor stability. \n\nLoaded assets must be resaved to make this indexing permanent, otherwise their updated searchability is for this editor session only. Select 'Yes' to checkout, load and resave all Blueprints with an outdated index. Select 'No' to load these Blueprints only without checking out and resaving them."), Args);
				ReturnValue = FMessageDialog::Open(EAppMsgType::YesNoCancel, DialogDisplayText, DialogTitle);
			}
			else
			{
				const FText DialogDisplayText = FText::Format(LOCTEXT("CacheAllConfirmationMessage_UnindexedAndOutOfDate_LoadOnly", "About to load {PackageCount} Blueprints ({UnindexedCount} unindexed/{OutOfDateCount} out-of-date)! The editor may become unresponsive while these assets are loaded for indexing. Save your work before initiating this: broken assets and memory usage can affect editor stability. \n\nLoaded assets must be resaved to make this indexing permanent, otherwise their updated searchability is for this editor session only. Your editor settings disallow resaving all these assets from this window, see Blueprint Editor Settings: AllowIndexAllBlueprints."), Args);
				ReturnValue = FMessageDialog::Open(EAppMsgType::OkCancel, DialogDisplayText, DialogTitle);
			}

			// If checkout is allowed and Yes is chosen, checkout and save all Blueprints. Otherwise only load all Blueprints. Cancel aborts everything.
			if (ReturnValue != EAppReturnType::Cancel)
			{
				FailedToCachePaths.Empty();

				TSet<FSoftObjectPath> TempUncachedAssets;
				TempUncachedAssets.Append(UnindexedAssets);
				TempUncachedAssets.Append(BlueprintsToUpdate);

				const bool bCheckOutAndSave = bCanCheckoutResaveAll && (ReturnValue == EAppReturnType::Yes);
				CacheParams.OpFlags = EFiBCacheOpFlags::ShowProgress | EFiBCacheOpFlags::AllowUserCancel | EFiBCacheOpFlags::AllowUserCloseProgress;
				if (bCheckOutAndSave)
				{
					CacheParams.OpFlags |= EFiBCacheOpFlags::CheckOutAndSave;
				}
				CacheParams.OnFinished = InOptions.OnFinished;
				CachingObject = MakeUnique<FCacheAllBlueprintsTickableObject>(TempUncachedAssets, CacheParams);

				const bool bIsSourceControlEnabled = ISourceControlModule::Get().IsEnabled();
				if (!bIsSourceControlEnabled && bCheckOutAndSave)
				{
					// Offer to start up Source Control
					ISourceControlModule::Get().ShowLoginDialog(FSourceControlLoginClosed::CreateRaw(this, &FFindInBlueprintSearchManager::OnCacheAllUnindexedAssets, bCheckOutAndSave), ELoginWindowMode::Modeless, EOnLoginWindowStartup::PreserveProvider);
				}
				else
				{
					OnCacheAllUnindexedAssets(bIsSourceControlEnabled, bCheckOutAndSave);
				}

				SourceCachingWidget = InSourceWidget;
			}
		}
	}
}

void FFindInBlueprintSearchManager::ExportOutdatedAssetList()
{
	// Construct path for output text file
	const FString FileLocation = FPaths::ConvertRelativePathToFull(FPaths::ProjectSavedDir());
	const FString FullPath = FString::Printf(TEXT("%s/FindInBlueprints_OutdatedAssetList.txt"), *FileLocation);

	if (FArchive* Ar = IFileManager::Get().CreateFileWriter(*FullPath))
	{
		// Write out all asset paths of unindexed blueprints
		const FString UnindexedHeader = TEXT("Unindexed assets:\n");
		Ar->Serialize(TCHAR_TO_ANSI(*UnindexedHeader), UnindexedHeader.Len());

		for (const FSoftObjectPath& SoftObjPath : UnindexedAssets)
		{
			const FString UnindexedEntry = FString::Printf(TEXT("%s\n"), *SoftObjPath.ToString());
			Ar->Serialize(TCHAR_TO_ANSI(*UnindexedEntry), UnindexedEntry.Len());
		}

		// Write out all asset paths of blueprints with out-of-data metadata
		const FString OutOfDateHeader = TEXT("\nOut-of-date assets:\n");
		Ar->Serialize(TCHAR_TO_ANSI(*OutOfDateHeader), OutOfDateHeader.Len());
		for (FSearchData SearchData : SearchArray)
		{
			if ((SearchData.Value.Len() != 0 || SearchData.ImaginaryBlueprint.IsValid()) && SearchData.VersionInfo.FiBDataVersion < EFiBVersion::FIB_VER_LATEST)
			{
				const FString OutdatedEntry = FString::Printf(TEXT("%s\n"), *SearchData.AssetPath.ToString());
				Ar->Serialize(TCHAR_TO_ANSI(*OutdatedEntry), OutdatedEntry.Len());
			}
		}

		Ar->Close();
		delete Ar;

		// Log success message
		FFormatNamedArguments Args;
		Args.Add(TEXT("OutputPath"), FText::FromString(FullPath));
		const FText ExportConfirmationText = FText::Format(LOCTEXT("ExportListConfirmationMessage", "Saved list of blueprints with out-of-date metadata to {OutputPath}"), Args);

		UE_LOGF(LogFindInBlueprint, Log, "%ls", *ExportConfirmationText.ToString());
		FMessageDialog::Open(EAppMsgType::Ok, ExportConfirmationText);
	}
	else
	{
		// Log failure message
		FFormatNamedArguments Args;
		Args.Add(TEXT("OutputPath"), FText::FromString(FullPath));
		const FText ExportFailureText = LOCTEXT("ExportListFailureMessage", "Failed to write to {OutputPath}");

		UE_LOGF(LogFindInBlueprint, Log, "%ls", *ExportFailureText.ToString());
		FMessageDialog::Open(EAppMsgType::Ok, ExportFailureText);
	}

}

void FFindInBlueprintSearchManager::CancelCacheAll(SFindInBlueprints* InFindInBlueprintWidget)
{
	if(IsCacheInProgress() && ((SourceCachingWidget.IsValid() && SourceCachingWidget.Pin().Get() == InFindInBlueprintWidget) || !SourceCachingWidget.IsValid()))
	{
		CachingObject->OnCancelCaching(!SourceCachingWidget.IsValid());
		SourceCachingWidget.Reset();
	}
}

int32 FFindInBlueprintSearchManager::GetCurrentCacheIndex() const
{
	int32 CachingIndex = 0;
	if(CachingObject.IsValid())
	{
		CachingIndex = CachingObject->GetCurrentCacheIndex();
	}

	return CachingIndex;
}

FSoftObjectPath FFindInBlueprintSearchManager::GetCurrentCacheBlueprintPath() const
{
	FSoftObjectPath CachingBPPath;
	if(CachingObject.IsValid())
	{
		CachingBPPath = CachingObject->GetCurrentCacheBlueprintPath();
	}

	return CachingBPPath;
}

float FFindInBlueprintSearchManager::GetCacheProgress() const
{
	float ReturnCacheValue = 1.0f;

	if(CachingObject.IsValid())
	{
		ReturnCacheValue = CachingObject->GetCacheProgress();
	}

	return ReturnCacheValue;
}

int32 FFindInBlueprintSearchManager::GetNumberUnindexedAssets() const
{
	return UnindexedAssets.Num();
}

int32 FFindInBlueprintSearchManager::GetNumberUncachedAssets() const
{
	if (CachingObject.IsValid())
	{
		return CachingObject->GetUncachedAssetCount();
	}
	
	return 0;
}

void FFindInBlueprintSearchManager::StartedCachingBlueprints(EFiBCacheOpType InCacheOpType, EFiBCacheOpFlags InCacheOpFlags)
{
	// Invoke the callback on any open global widgets
	for (TWeakPtr<SFindInBlueprints> FindResultsPtr : GlobalFindResults)
	{
		if (FindResultsPtr.IsValid())
		{
			FindResultsPtr.Pin()->OnCacheStarted(InCacheOpType, InCacheOpFlags);
		}
	}
}

void FFindInBlueprintSearchManager::FinishedCachingBlueprints(EFiBCacheOpType InCacheOpType, EFiBCacheOpFlags InCacheOpFlags, int32 InNumberCached, TSet<FSoftObjectPath>& InFailedToCacheList)
{
	// Update the list of cache failures
	FailedToCachePaths = InFailedToCacheList;

	bool bContinueCachingPendingAssets = false;
	if (InCacheOpType == EFiBCacheOpType::CachePendingAssets)
	{
		// In the discovery stage, check to see if discovery has ended and we have no additional assets to process. 
		const bool bIsCachingDiscoveredAssets = EnumHasAnyFlags(InCacheOpFlags, EFiBCacheOpFlags::IsCachingDiscoveredAssets);
		if (bIsCachingDiscoveredAssets && PendingAssets.Num() > 0)
		{
			// Discovered assets are still pending; continue immediately.
			bContinueCachingPendingAssets = true;
		}
		else if (!IsAssetDiscoveryInProgress())
		{
			// Update the flags for the completion callback.
			InCacheOpFlags &= ~(EFiBCacheOpFlags::HideProgressBars | EFiBCacheOpFlags::IsCachingDiscoveredAssets | EFiBCacheOpFlags::KeepProgressVisibleOnCompletion);
		}
	}

	// Invoke the completion callback on any open widgets that are not the initiating one (e.g. to hide progress bar)
	for (TWeakPtr<SFindInBlueprints> FindResultsPtr : GlobalFindResults)
	{
		if (FindResultsPtr.IsValid() && FindResultsPtr != SourceCachingWidget)
		{
			FindResultsPtr.Pin()->OnCacheComplete(InCacheOpType, InCacheOpFlags);
		}
	}

	// Invoke the completion callback on the initiating widget only if necessary (i.e. only if it doesn't already delegate it)
	if (SourceCachingWidget.IsValid() && !CachingObject->HasPostCacheWork())
	{
		SourceCachingWidget.Pin()->OnCacheComplete(InCacheOpType, InCacheOpFlags);
	}

	// Delete the object and NULL it out so we can do it again in the future if needed (if it was canceled)
	CachingObject.Reset();

	// If necessary, begin a new operation to continue caching the next set of pending assets.
	if (bContinueCachingPendingAssets)
	{
		FFindInBlueprintCachingOptions CacheOptions;
		CacheOptions.OpType = InCacheOpType;
		CacheOptions.OpFlags = InCacheOpFlags;
		CacheAllAssets(SourceCachingWidget, CacheOptions);
	}
	else
	{
		// Reset the reference to the widget that initiated the caching operation
		SourceCachingWidget.Reset();
	}
}

bool FFindInBlueprintSearchManager::IsCacheInProgress() const
{
	return CachingObject.IsValid();
}

bool FFindInBlueprintSearchManager::IsUnindexedCacheInProgress() const
{
	return IsCacheInProgress() && CachingObject->GetCurrentCacheOpType() == EFiBCacheOpType::CacheUnindexedAssets;
}

bool FFindInBlueprintSearchManager::IsAssetDiscoveryInProgress() const
{
	return GIsRunning && AssetRegistryModule && AssetRegistryModule->Get().IsLoadingAssets();
}

bool FFindInBlueprintSearchManager::IsAsyncSearchQueryInProgress() const
{
	// Note: Not using ActiveSearchCounter here, as that's used to block on pause and thus can be decremented during an active search.
	return ActiveSearchQueries.Num() > 0;
}

TSharedPtr< FJsonObject > FFindInBlueprintSearchManager::ConvertJsonStringToObject(FSearchDataVersionInfo InVersionInfo, const FString& InJsonString, TMap<int32, FText>& OutFTextLookupTable)
{
	/** The searchable data is more complicated than a Json string, the Json being the main searchable body that is parsed. Below is a diagram of the full data:
	 *  | int32 "Version" | int32 "Size" | TMap "Lookup Table" | Json String |
	 *
	 * Version: Version of the FiB data, which may impact searching
	 * Size: The size of the TMap in bytes
	 * Lookup Table: The Json's identifiers and string values are in Hex strings and stored in a TMap, the Json stores these values as ints and uses them as the Key into the TMap
	 * Json String: The Json string to be deserialized in full
	 */
	TArray<uint8> DerivedData;

	// SearchData is currently the full string
	// We want to first extract the size of the TMap we will be serializing
	int32 SizeOfData;
	FBufferReader ReaderStream((void*)*InJsonString, InJsonString.Len() * sizeof(TCHAR), false);

	// It's unexpected to not have valid search data versioning info at this point
	checkf(FiBSerializationHelpers::ValidateSearchDataVersionInfo(InVersionInfo), TEXT("FiB: Invalid JSON stream data version."));

	// If the stream is versioned, read past the version info
	if (InVersionInfo.FiBDataVersion > EFiBVersion::FIB_VER_BASE)
	{
		// Read the FiB search data version
		const int32 Version = FiBSerializationHelpers::Deserialize<int32>(ReaderStream);

		// Check that the deserialized version matches up with what's recorded in the search database
		ensureMsgf(Version == InVersionInfo.FiBDataVersion, TEXT("FiB: JSON stream data does not match search data version from database. This is unexpected."));
	}

	// Configure the JSON stream with the proper custom versions for serialization when reading the LUT
	ReaderStream.SetCustomVersions(InVersionInfo.CustomVersions);

 	// Read, as a byte string, the number of characters composing the Lookup Table for the Json.
	SizeOfData = FiBSerializationHelpers::Deserialize<int32>(ReaderStream);

 	// With the size of the TMap in hand, let's serialize JUST that (as a byte string)
	TMap<int32, FText> LookupTable;
	OutFTextLookupTable = LookupTable = FiBSerializationHelpers::Deserialize< TMap<int32, FText> >(ReaderStream, SizeOfData);

	// The original BufferReader should be positioned at the Json
	TSharedPtr<FJsonObject> JsonObject = nullptr;
	TSharedRef<TJsonReader<TCHAR>> Reader = TJsonReader<TCHAR>::Create(&ReaderStream);
	FJsonSerializer::Deserialize( Reader, JsonObject );

	return JsonObject;
}

void FFindInBlueprintSearchManager::GlobalFindResultsClosed(const TSharedRef<SFindInBlueprints>& FindResults)
{
	CSV_EVENT(FindInBlueprint, TEXT("GlobalFindResultsClosed: %s"), *FindResults->GetHostTabId().ToString());

	for (TWeakPtr<SFindInBlueprints> FindResultsPtr : GlobalFindResults)
	{
		if (FindResultsPtr.Pin() == FindResults)
		{
			GlobalFindResults.Remove(FindResultsPtr);

			if (IsCacheInProgress() && SourceCachingWidget == FindResultsPtr)
			{
				SourceCachingWidget.Reset();
			}
			break;
		}
	}
}

FText FFindInBlueprintSearchManager::GetGlobalFindResultsTabLabel(int32 TabIdx)
{
	int32 NumOpenGlobalFindResultsTabs = 0;
	for (int32 i = GlobalFindResults.Num() - 1; i >= 0; --i)
	{
		if (GlobalFindResults[i].IsValid())
		{
			++NumOpenGlobalFindResultsTabs;
		}
		else
		{
			GlobalFindResults.RemoveAt(i);
		}
	}

	if (NumOpenGlobalFindResultsTabs > 1 || TabIdx > 0)
	{
		return FText::Format(LOCTEXT("GlobalFindResultsTabNameWithIndex", "Find in Blueprints {0}"), FText::AsNumber(TabIdx + 1));
	}
	else
	{
		return LOCTEXT("GlobalFindResultsTabName", "Find in Blueprints");
	}
}

TSharedRef<SDockTab> FFindInBlueprintSearchManager::SpawnGlobalFindResultsTab(const FSpawnTabArgs& SpawnTabArgs, int32 TabIdx)
{
	CSV_EVENT(FindInBlueprint, TEXT("SpawnGlobalFindResultsTab: %d"), TabIdx);

	TAttribute<FText> Label = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateRaw(this, &FFindInBlueprintSearchManager::GetGlobalFindResultsTabLabel, TabIdx));

	TSharedRef<SDockTab> NewTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		.Label(Label)
		.ToolTipText(LOCTEXT("GlobalFindResultsTabTooltip", "Search for a string in all Blueprint assets."));

	TSharedRef<SFindInBlueprints> FindResults = SNew(SFindInBlueprints)
		.ContainingTab(NewTab);

	// If we're in the middle of a caching operation, signal the tab so that it can update the UI.
	if (IsCacheInProgress())
	{
		FindResults->OnCacheStarted(CachingObject->GetCurrentCacheOpType(), CachingObject->GetCurrentCacheOpFlags());
	}

	GlobalFindResults.Add(FindResults);

	NewTab->SetContent(FindResults);

	return NewTab;
}

TSharedPtr<SFindInBlueprints> FFindInBlueprintSearchManager::OpenGlobalFindResultsTab()
{
	TSet<FName> OpenGlobalTabIDs;

	for (TWeakPtr<SFindInBlueprints> FindResultsPtr : GlobalFindResults)
	{
		TSharedPtr<SFindInBlueprints> FindResults = FindResultsPtr.Pin();
		if (FindResults.IsValid())
		{
			OpenGlobalTabIDs.Add(FindResults->GetHostTabId());
		}
	}

	for (int32 Idx = 0; Idx < UE_ARRAY_COUNT(GlobalFindResultsTabIDs); ++Idx)
	{
		const FName GlobalTabId = GlobalFindResultsTabIDs[Idx];
		if (!OpenGlobalTabIDs.Contains(GlobalTabId))
		{
			TSharedPtr<SDockTab> NewTab = FGlobalTabmanager::Get()->TryInvokeTab(GlobalTabId);
			if (NewTab.IsValid())
			{
				TSharedRef<SFindInBlueprints> NewFindTab = StaticCastSharedRef<SFindInBlueprints>(NewTab->GetContent());
				return NewFindTab;
			}
			else
			{
				break;
			}
		}
	}

	return TSharedPtr<SFindInBlueprints>();
}

TSharedPtr<SFindInBlueprints> FFindInBlueprintSearchManager::GetGlobalFindResults()
{
	TSharedPtr<SFindInBlueprints> FindResultsToUse;

	for (TWeakPtr<SFindInBlueprints> FindResultsPtr : GlobalFindResults)
	{
		TSharedPtr<SFindInBlueprints> FindResults = FindResultsPtr.Pin();
		if (FindResults.IsValid() && !FindResults->IsLocked())
		{
			FindResultsToUse = FindResults;
			break;
		}
	}

	if (FindResultsToUse.IsValid())
	{
		FGlobalTabmanager::Get()->TryInvokeTab(FindResultsToUse->GetHostTabId());
	}
	else
	{
		FindResultsToUse = OpenGlobalFindResultsTab();
	}

	return FindResultsToUse;
}

void FFindInBlueprintSearchManager::EnableGlobalFindResults(bool bEnable)
{
	const TSharedRef<FGlobalTabmanager>& GlobalTabManager = FGlobalTabmanager::Get();

	if (bEnable)
	{
		// Register the spawners for all global Find Results tabs
		const FSlateIcon GlobalFindResultsIcon(FAppStyle::GetAppStyleSetName(), "BlueprintEditor.FindInBlueprints.MenuIcon");
		GlobalFindResultsMenuItem = WorkspaceMenu::GetMenuStructure().GetToolsCategory()->AddGroup(
			"FindInBlueprints",
			LOCTEXT("WorkspaceMenu_GlobalFindResultsCategory", "Find in Blueprints"),
			LOCTEXT("GlobalFindResultsMenuTooltipText", "Find references to functions, events and variables in all Blueprints."),
			GlobalFindResultsIcon,
			true);

		for (int32 TabIdx = 0; TabIdx < UE_ARRAY_COUNT(GlobalFindResultsTabIDs); TabIdx++)
		{
			const FName TabID = GlobalFindResultsTabIDs[TabIdx];
			if (!GlobalTabManager->HasTabSpawner(TabID))
			{
				const FText DisplayName = FText::Format(LOCTEXT("GlobalFindResultsDisplayName", "Find in Blueprints {0}"), FText::AsNumber(TabIdx + 1));

				GlobalTabManager->RegisterNomadTabSpawner(TabID, FOnSpawnTab::CreateRaw(this, &FFindInBlueprintSearchManager::SpawnGlobalFindResultsTab, TabIdx))
					.SetDisplayName(DisplayName)
					.SetIcon(GlobalFindResultsIcon)
					.SetGroup(GlobalFindResultsMenuItem.ToSharedRef());
			}
		}
	}
	else
	{
		// Close all Global Find Results tabs when turning the feature off, since these may not get closed along with the Blueprint Editor contexts above.
		TSet<TSharedPtr<SFindInBlueprints>> FindResultsToClose;

		for (TWeakPtr<SFindInBlueprints> FindResultsPtr : GlobalFindResults)
		{
			TSharedPtr<SFindInBlueprints> FindResults = FindResultsPtr.Pin();
			if (FindResults.IsValid())
			{
				FindResultsToClose.Add(FindResults);
			}
		}

		for (TSharedPtr<SFindInBlueprints> FindResults : FindResultsToClose)
		{
			FindResults->CloseHostTab();
		}

		GlobalFindResults.Empty();

		for (int32 TabIdx = 0; TabIdx < UE_ARRAY_COUNT(GlobalFindResultsTabIDs); TabIdx++)
		{
			const FName TabID = GlobalFindResultsTabIDs[TabIdx];
			if (GlobalTabManager->HasTabSpawner(TabID))
			{
				GlobalTabManager->UnregisterNomadTabSpawner(TabID);
			}
		}

		if (GlobalFindResultsMenuItem.IsValid())
		{
			WorkspaceMenu::GetMenuStructure().GetToolsCategory()->RemoveItem(GlobalFindResultsMenuItem.ToSharedRef());
			GlobalFindResultsMenuItem.Reset();
		}
	}
}

void FFindInBlueprintSearchManager::CloseOrphanedGlobalFindResultsTabs(TSharedPtr<class FTabManager> TabManager)
{
	if (TabManager.IsValid())
	{
		for (int32 TabIdx = 0; TabIdx < UE_ARRAY_COUNT(GlobalFindResultsTabIDs); TabIdx++)
		{
			const FName TabID = GlobalFindResultsTabIDs[TabIdx];
			if (!FGlobalTabmanager::Get()->HasTabSpawner(TabID))
			{
				TSharedPtr<SDockTab> OrphanedTab = TabManager->FindExistingLiveTab(FTabId(TabID));
				if (OrphanedTab.IsValid())
				{
					OrphanedTab->RequestCloseTab();
				}
			}
		}
	}
}

void FFindInBlueprintSearchManager::Tick(float DeltaTime)
{
	CSV_SCOPED_TIMING_STAT(FindInBlueprint, Tick);

	if (bHasFirstSearchOccurred && AssetsToIndexOnFirstSearch.Num() > 0)
	{
		PendingAssets = PendingAssets.Union(AssetsToIndexOnFirstSearch);
		AssetsToIndexOnFirstSearch.Empty();
	}

	if(IsCacheInProgress())
	{
		check(CachingObject);
		CachingObject->Tick(DeltaTime);
	}
	else if (PendingAssets.Num() > 0)
	{
		// Kick off a re-indexing operation to update the cache
		FFindInBlueprintCachingOptions CachingOptions;
		CachingOptions.OpType = EFiBCacheOpType::CachePendingAssets;
		CacheAllAssets(nullptr, CachingOptions);
	}
	else if (CVarRandomSearchIndexCacheRebuildEachTick.GetValueOnGameThread())
	{
		DoRandomIndexCacheRebuild();
	}
}

bool FFindInBlueprintSearchManager::IsTickable() const
{
	const bool bHasPendingAssets = PendingAssets.Num() > 0;
	const bool bNeedsFirstIndex = bHasFirstSearchOccurred && AssetsToIndexOnFirstSearch.Num() > 0;

	// Tick only if we have an active caching operation or if a search has occured before we're ready or if we have pending assets and an open FiB context or an active async search query
	return IsCacheInProgress() || bNeedsFirstIndex || (bHasPendingAssets && (IsGlobalFindResultsOpen() || IsAsyncSearchQueryInProgress())) || CVarRandomSearchIndexCacheRebuildEachTick.GetValueOnGameThread();
}

TStatId FFindInBlueprintSearchManager::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FFindInBlueprintSearchManager, STATGROUP_Tickables);
}

class FFiBDumpIndexCacheToFileExecHelper : public FSelfRegisteringExec
{
	virtual bool Exec_Editor(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
	{
		if (FParse::Command(&Cmd, TEXT("DUMPFIBINDEXCACHE")))
		{
			FString FileLocation = FPaths::ConvertRelativePathToFull(FPaths::ProjectIntermediateDir() + TEXT("BlueprintSearchTools"));
			FString FullPath = FString::Printf(TEXT("%s/FullBlueprintIndexCacheDump.csv"), *FileLocation);

			FArchive* DumpFile = IFileManager::Get().CreateFileWriter(*FullPath);
			if (DumpFile)
			{
				FFindInBlueprintSearchManager::Get().DumpCache(*DumpFile);

				DumpFile->Close();
				delete DumpFile;

				UE_LOGF(LogFindInBlueprint, Log, "Wrote full index cache to %ls", *FullPath);
			}
			return true;
		}
		return false;
	}
};
static FFiBDumpIndexCacheToFileExecHelper GFiBDumpIndexCacheToFileExec;

#undef LOCTEXT_NAMESPACE
