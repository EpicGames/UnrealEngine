// Copyright Epic Games, Inc. All Rights Reserved.
#include "FindInBlueprints.h"

#include "BlueprintEditor.h"
#include "BlueprintEditorModule.h"
#include "BlueprintEditorSettings.h"
#include "BlueprintEditorTabs.h"
#include "CoreGlobals.h"
#include "Dom/JsonValue.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraphSchema_K2.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Level.h"
#include "Engine/SCS_Node.h"
#include "Engine/SimpleConstructionScript.h"
#include "Engine/World.h"
#include "FiBSearchInstance.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/SlateDelegates.h"
#include "Framework/Views/ITypedTableView.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformTime.h"
#include "IDocumentation.h"
#include "ImaginaryBlueprintData.h"
#include "Input/Events.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CallFunction.h"
#include "K2Node_Event.h"
#include "K2Node_MacroInstance.h"
#include "K2Node_Variable.h"
#include "K2Node_VariableGet.h"
#include "K2Node_VariableSet.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/WidgetPath.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/CString.h"
#include "Misc/EnumClassFlags.h"
#include "SWarningOrErrorBox.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Trace/Detail/Channel.h"
#include "Types/SlateConstants.h"
#include "Types/SlateStructs.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Templates/GuardValueAccessors.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/SMultiLineEditableText.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FindInBlueprints)

class ITableRow;
class SWidget;
class UActorComponent;
struct FGeometry;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "FindInBlueprints"

KISMET_API UClass* FindInBlueprintsHelpers::GetFunctionOriginClass(const UFunction* Function)
{
	// Abort if invalid param
	if (!Function)
	{
		return nullptr;
	}

	// Get outermost super function
	while (const UFunction* SuperFunction = Function->GetSuperFunction())
	{
		Function = SuperFunction;
	}

	// Get that function's class
	UClass* OwnerClass = Function->GetOwnerClass() && Function->GetOwnerClass()->GetAuthoritativeClass() ? Function->GetOwnerClass()->GetAuthoritativeClass() : Function->GetOwnerClass();

	// Consider case where a blueprint implements an interface function
	if (const UBlueprintGeneratedClass* BPGC = Cast<UBlueprintGeneratedClass>(OwnerClass))
	{
		const FName FunctionName = Function->GetFName();
		for (const FImplementedInterface& Interface : BPGC->Interfaces)
		{
			if (!Interface.Class)
			{
				continue;
			}

			if (UFunction* InterfaceFunction = Interface.Class->FindFunctionByName(FunctionName))
			{
				if (InterfaceFunction->IsSignatureCompatibleWith(Function))
				{
					OwnerClass = Interface.Class;
					break;
				}
			}
		}
	}

	return OwnerClass;
}

bool FindInBlueprintsHelpers::ConstructSearchTermFromFunction(const UFunction* Function, FString& SearchTerm)
{
	if (!Function)
	{
		return false;
	}

	const UClass* FuncOriginClass = GetFunctionOriginClass(Function);
	if (!FuncOriginClass)
	{
		return false;
	}

	const FString FunctionNativeName = Function->GetName();
	const FString TargetTypeName = FuncOriginClass->GetPathName();
	SearchTerm = FString::Printf(TEXT("Nodes(\"Native Name\"=+\"%s\" && (Pins(Name=Target && ObjectClass=+\"%s\") || FuncOriginClass=+\"%s\"))"), *FunctionNativeName, *TargetTypeName, *TargetTypeName);
	return true;
}

FText FindInBlueprintsHelpers::AsFText(TSharedPtr< FJsonValue > InJsonValue, const TMap<int32, FText>& InLookupTable)
{
	if (const FText* LookupText = InLookupTable.Find(FCString::Atoi(*InJsonValue->AsString())))
	{
		return *LookupText;
	}
	// Let's never get here.
	return LOCTEXT("FiBSerializationError", "There was an error in serialization!");
}

FText FindInBlueprintsHelpers::AsFText(int32 InValue, const TMap<int32, FText>& InLookupTable)
{
	if (const FText* LookupText = InLookupTable.Find(InValue))
	{
		return *LookupText;
	}
	// Let's never get here.
	return LOCTEXT("FiBSerializationError", "There was an error in serialization!");
}

bool FindInBlueprintsHelpers::IsTextEqualToString(const FText& InText, const FString& InString)
{
	return InString == InText.ToString() || InString == *FTextInspector::GetSourceString(InText);
}

FString FindInBlueprintsHelpers::GetPinTypeAsString(const FEdGraphPinType& InPinType)
{
	FString Result = InPinType.PinCategory.ToString();
	if(UObject* SubCategoryObject = InPinType.PinSubCategoryObject.Get()) 
	{
		Result += FString(" '") + SubCategoryObject->GetName() + "'";
	}
	else
	{
		Result += FString(" '") + InPinType.PinSubCategory.ToString() + "'";
	}

	return Result;
}

bool FindInBlueprintsHelpers::ParsePinType(FText InKey, FText InValue, FEdGraphPinType& InOutPinType)
{
	bool bParsed = true;

	if(InKey.CompareTo(FFindInBlueprintSearchTags::FiB_PinCategory) == 0)
	{
		InOutPinType.PinCategory = *InValue.ToString();
	}
	else if(InKey.CompareTo(FFindInBlueprintSearchTags::FiB_PinSubCategory) == 0)
	{
		InOutPinType.PinSubCategory = *InValue.ToString();
	}
	else if(InKey.CompareTo(FFindInBlueprintSearchTags::FiB_ObjectClass) == 0)
	{
		InOutPinType.PinSubCategory = *InValue.ToString();
	}
	else if(InKey.CompareTo(FFindInBlueprintSearchTags::FiB_IsArray) == 0)
	{
		InOutPinType.ContainerType = (InValue.ToString().ToBool() ? EPinContainerType::Array : EPinContainerType::None);
	}
	else if(InKey.CompareTo(FFindInBlueprintSearchTags::FiB_IsReference) == 0)
	{
		InOutPinType.bIsReference = InValue.ToString().ToBool();
	}
	else
	{
		bParsed = false;
	}

	return bParsed;
}

void FindInBlueprintsHelpers::ExpandAllChildren(FSearchResult InTreeNode, TSharedPtr<STreeView<TSharedPtr<FFindInBlueprintsResult>>> InTreeView)
{
	if (InTreeNode->Children.Num())
	{
		InTreeView->SetItemExpansion(InTreeNode, true);
		for (int32 i = 0; i < InTreeNode->Children.Num(); i++)
		{
			ExpandAllChildren(InTreeNode->Children[i], InTreeView);
		}
	}
}

//////////////////////////////////////////////////////////////////////////
// FBlueprintSearchResult

FFindInBlueprintsResult::FFindInBlueprintsResult(const FText& InDisplayText ) 
	: DisplayText(InDisplayText)
{
}

FReply FFindInBlueprintsResult::OnClick()
{
	// If there is a parent, handle it using the parent's functionality
	if(Parent.IsValid())
	{
		return Parent.Pin()->OnClick();
	}
	else
	{
		// As a last resort, find the parent Blueprint, and open that, it will get the user close to what they want
		UBlueprint* Blueprint = GetParentBlueprint();
		if(Blueprint)
		{
			FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(Blueprint, false);
		}
	}
	
	return FReply::Handled();
}

UObject* FFindInBlueprintsResult::GetObject(UBlueprint* InBlueprint) const
{
	return GetParentBlueprint();
}

FText FFindInBlueprintsResult::GetCategory() const
{
	return FText::GetEmpty();
}

TSharedRef<SWidget> FFindInBlueprintsResult::CreateIcon() const
{
	const FSlateBrush* Brush = NULL;

	return 	SNew(SImage)
			.Image(Brush)
			.ColorAndOpacity(FStyleColors::Foreground)
			.ToolTipText( GetCategory() );
}

FString FFindInBlueprintsResult::GetCommentText() const
{
	return CommentText;
}

UBlueprint* FFindInBlueprintsResult::GetParentBlueprint() const
{
	UBlueprint* ResultBlueprint = nullptr;
	if (Parent.IsValid())
	{
		ResultBlueprint = Parent.Pin()->GetParentBlueprint();
	}
	else
	{
		UObject* Object;
		{
			TGuardValueAccessors<bool> IsEditorLoadingPackageGuard(UE::GetIsEditorLoadingPackage, UE::SetIsEditorLoadingPackage, true);
			Object = LoadObject<UObject>(NULL, *DisplayText.ToString(), NULL, 0, NULL);
		}

		if(UBlueprint* BlueprintObj = Cast<UBlueprint>(Object))
		{
			ResultBlueprint = BlueprintObj;
		}
		else if(UWorld* WorldObj = Cast<UWorld>(Object))
		{
			if(WorldObj->PersistentLevel)
			{
				ResultBlueprint = Cast<UBlueprint>((UObject*)WorldObj->PersistentLevel->GetLevelScriptBlueprint(true));
			}
		}

	}
	return ResultBlueprint;
}

FText FFindInBlueprintsResult::GetDisplayString() const
{
	return DisplayText;
}

//////////////////////////////////////////////////////////
// FFindInBlueprintsGraphNode

FFindInBlueprintsGraphNode::FFindInBlueprintsGraphNode()
	: Glyph(FAppStyle::GetAppStyleSetName(), "")
	, Class(nullptr)
{
}

FReply FFindInBlueprintsGraphNode::OnClick()
{
	UBlueprint* Blueprint = GetParentBlueprint();
	if(Blueprint)
	{
		UEdGraphNode* OutNode = NULL;
		if(	UEdGraphNode* GraphNode = FBlueprintEditorUtils::GetNodeByGUID(Blueprint, NodeGuid) )
		{
			FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(GraphNode, /*bRequestRename=*/false);
			return FReply::Handled();
		}
	}

	return FFindInBlueprintsResult::OnClick();
}

TSharedRef<SWidget> FFindInBlueprintsGraphNode::CreateIcon() const
{
	return 	SNew(SImage)
		.Image(Glyph.GetOptionalIcon())
		.ColorAndOpacity(GlyphColor)
		.ToolTipText( GetCategory() );
}

void FFindInBlueprintsGraphNode::ParseSearchInfo(FText InKey, FText InValue)
{
	if(InKey.CompareTo(FFindInBlueprintSearchTags::FiB_NodeGuid) == 0)
	{
		FString NodeGUIDAsString = InValue.ToString();
		FGuid::Parse(NodeGUIDAsString, NodeGuid);
	}

	if(InKey.CompareTo(FFindInBlueprintSearchTags::FiB_ClassName) == 0)
	{
		ClassName = InValue.ToString();
	}
	else if(InKey.CompareTo(FFindInBlueprintSearchTags::FiB_Name) == 0)
	{
		DisplayText = InValue;
	}
	else if(InKey.CompareTo(FFindInBlueprintSearchTags::FiB_Comment) == 0)
	{
		CommentText = InValue.ToString();
	}
	else if(InKey.CompareTo(FFindInBlueprintSearchTags::FiB_Glyph) == 0)
	{
		Glyph = FSlateIcon(Glyph.GetStyleSetName(), *InValue.ToString());
	}
	else if(InKey.CompareTo(FFindInBlueprintSearchTags::FiB_GlyphStyleSet) == 0)
	{
		Glyph = FSlateIcon(*InValue.ToString(), Glyph.GetStyleName());
	}
	else if(InKey.CompareTo(FFindInBlueprintSearchTags::FiB_GlyphColor) == 0)
	{
		GlyphColor.InitFromString(InValue.ToString());
	}
}

FText FFindInBlueprintsGraphNode::GetCategory() const
{
	if(Class == UK2Node_CallFunction::StaticClass())
	{
		return LOCTEXT("CallFuctionCat", "Function Call");
	}
	else if(Class == UK2Node_MacroInstance::StaticClass())
	{
		return LOCTEXT("MacroCategory", "Macro");
	}
	else if(Class == UK2Node_Event::StaticClass())
	{
		return LOCTEXT("EventCat", "Event");
	}
	else if(Class == UK2Node_VariableGet::StaticClass())
	{
		return LOCTEXT("VariableGetCategory", "Variable Get");
	}
	else if(Class == UK2Node_VariableSet::StaticClass())
	{
		return LOCTEXT("VariableSetCategory", "Variable Set");
	}

	return LOCTEXT("NodeCategory", "Node");
}

void FFindInBlueprintsGraphNode::FinalizeSearchData()
{
	if(!ClassName.IsEmpty())
	{
		// Check the node subclasses and look for one with the same short name
		TArray<UClass*> NodeClasses;
		GetDerivedClasses(UEdGraphNode::StaticClass(), NodeClasses, /*bRecursive=*/true);

		for (UClass* FoundClass : NodeClasses)
		{
			if (FoundClass->GetName() == ClassName)
			{
				Class = FoundClass;
				break;
			}
		}

		ClassName.Empty();
	}
}

UObject* FFindInBlueprintsGraphNode::GetObject(UBlueprint* InBlueprint) const
{
	return FBlueprintEditorUtils::GetNodeByGUID(InBlueprint, NodeGuid);
}

//////////////////////////////////////////////////////////
// FFindInBlueprintsPin

FFindInBlueprintsPin::FFindInBlueprintsPin(FString InSchemaName)
	: SchemaName(InSchemaName)
	, IconColor(FSlateColor::UseForeground())
{
}

TSharedRef<SWidget> FFindInBlueprintsPin::CreateIcon() const
{
	const FSlateBrush* Brush = nullptr;

	if( PinType.IsArray() )
	{
		Brush = FAppStyle::Get().GetBrush( TEXT("GraphEditor.ArrayPinIcon") );
	}
	else if( PinType.bIsReference )
	{
		Brush = FAppStyle::Get().GetBrush( TEXT("GraphEditor.RefPinIcon") );
	}
	else
	{
		Brush = FAppStyle::Get().GetBrush( TEXT("GraphEditor.PinIcon") );
	}

	return SNew(SImage)
		.Image(Brush)
		.ColorAndOpacity(IconColor)
		.ToolTipText(FText::FromString(FindInBlueprintsHelpers::GetPinTypeAsString(PinType)));
}

void FFindInBlueprintsPin::ParseSearchInfo(FText InKey, FText InValue)
{
	if(InKey.CompareTo(FFindInBlueprintSearchTags::FiB_Name) == 0)
	{
		DisplayText = InValue;
	}
	else
	{
		FindInBlueprintsHelpers::ParsePinType(InKey, InValue, PinType);
	}
}

FText FFindInBlueprintsPin::GetCategory() const
{
	return LOCTEXT("PinCategory", "Pin");
}

void FFindInBlueprintsPin::FinalizeSearchData()
{
	if(!PinType.PinSubCategory.IsNone())
	{
		// This can either be a full path to an object, or a short name specific to the category
		if (FPackageName::IsShortPackageName(PinType.PinSubCategory))
		{
			// This could also be an old class name without the full path, but it's fine to ignore in that case
		}
		else
		{
			PinType.PinSubCategoryObject = FindObject<UObject>(UObject::StaticClass(), *PinType.PinSubCategory.ToString());
			if (PinType.PinSubCategoryObject.IsValid())
			{
				PinType.PinSubCategory = NAME_None;
			}
		}
	}

	if(!SchemaName.IsEmpty())
	{
		// Get all subclasses of schema and find the one with a matching short name
		TArray<UClass*> SchemaClasses;
		GetDerivedClasses(UEdGraphSchema::StaticClass(), SchemaClasses, /*bRecursive=*/true);

		for (UClass* FoundClass : SchemaClasses)
		{
			if (FoundClass->GetName() == SchemaName)
			{
				UEdGraphSchema* Schema = FoundClass->GetDefaultObject<UEdGraphSchema>();
				IconColor = Schema->GetPinTypeColor(PinType);
				break;
			}
		}

		SchemaName.Empty();
	}
}

//////////////////////////////////////////////////////////
// FFindInBlueprintsProperty

FFindInBlueprintsProperty::FFindInBlueprintsProperty()
	: bIsSCSComponent(false)
{
}

FReply FFindInBlueprintsProperty::OnClick()
{
	if (bIsSCSComponent)
	{
		UBlueprint* Blueprint = GetParentBlueprint();
		if (Blueprint)
		{
			TSharedPtr<IBlueprintEditor> BlueprintEditor = FKismetEditorUtilities::GetIBlueprintEditorForObject(Blueprint, true);

			if (BlueprintEditor.IsValid())
			{
				// Open Viewport Tab
				BlueprintEditor->FocusWindow();
				BlueprintEditor->GetTabManager()->TryInvokeTab(FBlueprintEditorTabs::SCSViewportID);

				// Find and Select the Component in the Viewport tab view
				const TArray<USCS_Node*>& Nodes = Blueprint->SimpleConstructionScript->GetAllNodes();
				for (USCS_Node* Node : Nodes)
				{
					if (Node->GetVariableName().ToString() == DisplayText.ToString())
					{
						UBlueprintGeneratedClass* GeneratedClass = Cast<UBlueprintGeneratedClass>(Blueprint->GeneratedClass);
						if (GeneratedClass)
						{
							UActorComponent* Component = Node->GetActualComponentTemplate(GeneratedClass);
							if (Component)
							{
								BlueprintEditor->FindAndSelectSubobjectEditorTreeNode(Component, false);
							}
						}
						break;
					}
				}
			}
		}
	}
	else
	{
		return FFindInBlueprintsResult::OnClick();
	}

	return FReply::Handled();
}

TSharedRef<SWidget> FFindInBlueprintsProperty::CreateIcon() const
{
	FLinearColor IconColor = FStyleColors::Foreground.GetSpecifiedColor();
	const FSlateBrush* Brush = UK2Node_Variable::GetVarIconFromPinType(PinType, IconColor).GetOptionalIcon();
	IconColor = UEdGraphSchema_K2::StaticClass()->GetDefaultObject<UEdGraphSchema_K2>()->GetPinTypeColor(PinType);

	return 	SNew(SImage)
		.Image(Brush)
		.ColorAndOpacity(FStyleColors::Foreground)
		.ToolTipText( FText::FromString(FindInBlueprintsHelpers::GetPinTypeAsString(PinType)) );
}

void FFindInBlueprintsProperty::ParseSearchInfo(FText InKey, FText InValue)
{
	if(InKey.CompareTo(FFindInBlueprintSearchTags::FiB_Name) == 0)
	{
		DisplayText = InValue;
	}
	else if(InKey.CompareTo(FFindInBlueprintSearchTags::FiB_IsSCSComponent) == 0)
	{
		bIsSCSComponent = true;
	}
	else
	{
		FindInBlueprintsHelpers::ParsePinType(InKey, InValue, PinType);
	}
}

FText FFindInBlueprintsProperty::GetCategory() const
{
	if(bIsSCSComponent)
	{
		return LOCTEXT("Component", "Component");
	}
	return LOCTEXT("Variable", "Variable");
}

void FFindInBlueprintsProperty::FinalizeSearchData()
{
	if (!PinType.PinSubCategory.IsNone())
	{
		// This can either be a full path to an object, or a short name specific to the category
		if (FPackageName::IsShortPackageName(PinType.PinSubCategory))
		{
			// This could also be an old class name without the full path, but it's fine to ignore in that case
		}
		else
		{
			PinType.PinSubCategoryObject = FindObject<UObject>(UObject::StaticClass(), *PinType.PinSubCategory.ToString());
			if (PinType.PinSubCategoryObject.IsValid())
			{
				PinType.PinSubCategory = NAME_None;
			}
		}
	}
}

//////////////////////////////////////////////////////////
// FFindInBlueprintsGraph

FFindInBlueprintsGraph::FFindInBlueprintsGraph(EGraphType InGraphType)
	: GraphType(InGraphType)
{
}

FReply FFindInBlueprintsGraph::OnClick()
{
	UBlueprint* Blueprint = GetParentBlueprint();
	if(Blueprint)
	{
		TArray<UEdGraph*> BlueprintGraphs;
		Blueprint->GetAllGraphs(BlueprintGraphs);

		for( auto Graph : BlueprintGraphs)
		{
			FGraphDisplayInfo DisplayInfo;
			Graph->GetSchema()->GetGraphDisplayInformation(*Graph, DisplayInfo);

			if(DisplayInfo.PlainName.EqualTo(DisplayText))
			{
				FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(Graph);
				break;
			}
		}
	}
	else
	{
		return FFindInBlueprintsResult::OnClick();
	}
	return FReply::Handled();
}

TSharedRef<SWidget> FFindInBlueprintsGraph::CreateIcon() const
{
	const FSlateBrush* Brush = NULL;
	if(GraphType == GT_Function)
	{
		Brush = FAppStyle::GetBrush(TEXT("GraphEditor.Function_16x"));
	}
	else if(GraphType == GT_Macro)
	{
		Brush = FAppStyle::GetBrush(TEXT("GraphEditor.Macro_16x"));
	}

	return 	SNew(SImage)
		.Image(Brush)
		.ToolTipText( GetCategory() );
}

void FFindInBlueprintsGraph::ParseSearchInfo(FText InKey, FText InValue)
{
	if(InKey.CompareTo(FFindInBlueprintSearchTags::FiB_Name) == 0)
	{
		DisplayText = InValue;
	}
}

FText FFindInBlueprintsGraph::GetCategory() const
{
	if(GraphType == GT_Function)
	{
		return LOCTEXT("FunctionGraphCategory", "Function");
	}
	else if(GraphType == GT_Macro)
	{
		return LOCTEXT("MacroGraphCategory", "Macro");
	}
	return LOCTEXT("GraphCategory", "Graph");
}

//////////////////////////////////////////////////////////////////////////
// SBlueprintSearch

void SFindInBlueprints::Construct( const FArguments& InArgs, TSharedPtr<FBlueprintEditor> InBlueprintEditor)
{
	OutOfDateWithLastSearchBPCount = 0;
	LastSearchedFiBVersion = EFiBVersion::FIB_VER_LATEST;
	BlueprintEditorPtr = InBlueprintEditor;

	HostTab = InArgs._ContainingTab;
	bIsLocked = false;

	bHideProgressBars = false;
	bShowCacheBarCloseButton = false;
	bShowCacheBarCancelButton = false;
	bShowCacheBarUnresponsiveEditorWarningText = false;
	bKeepCacheBarProgressVisible = false;

	if (HostTab.IsValid())
	{
		HostTab.Pin()->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateSP(this, &SFindInBlueprints::OnHostTabClosed));
	}

	if (InArgs._bIsSearchWindow)
	{
		RegisterCommands();
	}

	bIsInFindWithinBlueprintMode = BlueprintEditorPtr.IsValid();
	
	this->ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		[
			SAssignNew(MainVerticalBox, SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(8.f, 5.f, 8.f, 5.f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(1)
				[
					SAssignNew(SearchTextField, SSearchBox)
					.HintText(LOCTEXT("BlueprintSearchHint", "Enter function or event name to find references..."))
					.OnTextChanged(this, &SFindInBlueprints::OnSearchTextChanged)
					.OnTextCommitted(this, &SFindInBlueprints::OnSearchTextCommitted)
					.Visibility(InArgs._bHideSearchBar? EVisibility::Collapsed : EVisibility::Visible)
					.DelayChangeNotificationsWhileTyping(false)
				]
				+SHorizontalBox::Slot()
				.Padding(4.f, 0.f, 2.f, 0.f)
				.AutoWidth()
				[
					SNew(SButton)
					.OnClicked(this, &SFindInBlueprints::OnOpenGlobalFindResults)
					.Visibility(!InArgs._bHideFindGlobalButton && BlueprintEditorPtr.IsValid() ? EVisibility::Visible : EVisibility::Collapsed)
					.ToolTipText(LOCTEXT("OpenInGlobalFindResultsButtonTooltip", "Find in all Blueprints"))
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "FindResults.FindInBlueprints")
						.Text(FText::FromString(FString(TEXT("\xf1e5"))) /*fa-binoculars*/)
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.VAlign(EVerticalAlignment::VAlign_Center)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.ContentPadding(4.f)
					.OnClicked(this, &SFindInBlueprints::OnLockButtonClicked)
					.Visibility(!InArgs._bHideSearchBar && !BlueprintEditorPtr.IsValid() ? EVisibility::Visible : EVisibility::Collapsed)
					[
						SNew(SImage)
						.Image(this, &SFindInBlueprints::OnGetLockButtonImage)
					]   
				]
			]
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
				.Padding(FMargin(8.f, 8.f, 4.f, 0.f))
				[
					SAssignNew(TreeView, STreeViewType)
					.TreeItemsSource( &ItemsFound )
					.OnGenerateRow( this, &SFindInBlueprints::OnGenerateRow )
					.OnGetChildren( this, &SFindInBlueprints::OnGetChildren )
					.OnMouseButtonDoubleClick(this,&SFindInBlueprints::OnTreeSelectionDoubleClicked)
					.SelectionMode( ESelectionMode::Multi )
					.OnContextMenuOpening(this, &SFindInBlueprints::OnContextMenuOpening)
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding( FMargin( 16.f, 8.f ) )
			[
				SNew(SHorizontalBox)

				// Text
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font( FAppStyle::Get().GetFontStyle("Text.Large") )
					.Text( LOCTEXT("SearchResults", "Searching...") )
					.Visibility(this, &SFindInBlueprints::GetSearchBarWidgetVisiblity, EFiBSearchBarWidget::StatusText)
				]

				// Throbber
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(12.f, 8.f, 16.f, 8.f))
				.VAlign(VAlign_Center)
				[
					SNew(SThrobber)
					.Visibility(this, &SFindInBlueprints::GetSearchBarWidgetVisiblity, EFiBSearchBarWidget::Throbber)
				]

				// Progress bar
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.Padding(FMargin(12.f, 8.f, 16.f, 8.f))
				.VAlign(VAlign_Center)
				[
					SNew(SProgressBar)
					.Visibility(this, &SFindInBlueprints::GetSearchBarWidgetVisiblity, EFiBSearchBarWidget::ProgressBar)
					.Percent(this, &SFindInBlueprints::GetPercentCompleteSearch)
				]
			]
		]
	];
}

void SFindInBlueprints::ConditionallyAddCacheBar()
{
	// Do not add when it should not be visible
	if(GetCacheBarVisibility() == EVisibility::Visible)
	{
		// Do not add a second cache bar
		if(MainVerticalBox.IsValid() && !CacheBarSlot.IsValid())
		{
			// Create a single string of all the Blueprint paths that failed to cache, on separate lines
			FString PathList;
			TSet<FSoftObjectPath> FailedToCacheList = FFindInBlueprintSearchManager::Get().GetFailedToCachePathList();
			for (const FSoftObjectPath& Path : FailedToCacheList)
			{
				PathList += Path.ToString() + TEXT("\n");
			}

			// Lambda to put together the popup menu detailing the failed to cache paths
			auto OnDisplayCacheFailLambda = [](TWeakPtr<SWidget> InParentWidget, FString InPathList)->FReply
			{
				if (InParentWidget.IsValid())
				{
					TSharedRef<SWidget> DisplayWidget = 
						SNew(SBox)
						.MaxDesiredHeight(512.0f)
						.MaxDesiredWidth(512.0f)
						.Content()
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							[
								SNew(SScrollBox)
								+SScrollBox::Slot()
								[
									SNew(SMultiLineEditableText)
									.AutoWrapText(true)
									.IsReadOnly(true)
									.Text(FText::FromString(InPathList))
								]
							]
						];

					FSlateApplication::Get().PushMenu(
						InParentWidget.Pin().ToSharedRef(),
						FWidgetPath(),
						DisplayWidget,
						FSlateApplication::Get().GetCursorPos(),
						FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup)
					);	
				}
				return FReply::Handled();
			};

			float VPadding = 8.f;
			float HPadding = 12.f;

			MainVerticalBox.Pin()->AddSlot()
			.AutoHeight()
			[
				SAssignNew(CacheBarSlot, SBorder)
				.Visibility( this, &SFindInBlueprints::GetCacheBarVisibility )
				.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
				.Padding( FMargin( 16.f, 8.f ) )
				[
					SNew(SVerticalBox)

					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SWarningOrErrorBox)
						.Message(this, &SFindInBlueprints::GetCacheBarStatusText)
						.Visibility(this, &SFindInBlueprints::GetCacheBarWidgetVisibility, EFiBCacheBarWidget::CacheAllUnindexedButton)
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(SButton)
								.Text(LOCTEXT("DismissIndexAllWarning", "Dismiss"))
								.Visibility( this, &SFindInBlueprints::GetCacheBarWidgetVisibility, EFiBCacheBarWidget::CloseButton )
								.OnClicked( this, &SFindInBlueprints::OnRemoveCacheBar )
							]

							// View of failed Blueprint paths
							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(SButton)
								.Text(LOCTEXT("ShowFailedPackages", "Show Failed Packages"))
								.OnClicked(FOnClicked::CreateLambda(OnDisplayCacheFailLambda, TWeakPtr<SWidget>(SharedThis(this)), PathList))
								.Visibility(this, &SFindInBlueprints::GetCacheBarWidgetVisibility, EFiBCacheBarWidget::ShowCacheFailuresButton)
								.ToolTip(IDocumentation::Get()->CreateToolTip(
									LOCTEXT("FailedCache_Tooltip", "Displays a list of packages that failed to save."),
									NULL,
									TEXT("Shared/Editors/BlueprintEditor"),
									TEXT("FindInBlueprint_FailedCache")
								))
							]

							+SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(HPadding, 0.f, 0.f, 0.f)
							.VAlign(VAlign_Center)
							[
								SNew(SButton)
								.Text(LOCTEXT("IndexAllBlueprints", "Index All"))
								.IsEnabled( this, &SFindInBlueprints::CanCacheAllUnindexedBlueprints )
								.OnClicked( this, &SFindInBlueprints::OnCacheAllUnindexedBlueprints )
								.ToolTip(IDocumentation::Get()->CreateToolTip(
									LOCTEXT("IndexAlLBlueprints_Tooltip", "Loads all Blueprints with an out-of-date index (search metadata) and resaves them with an up-to-date index. This can be a very slow process and the editor may become unresponsive. This action can be disabled via Blueprint Editor settings."),
									NULL,
									TEXT("Shared/Editors/BlueprintEditor"),
									TEXT("FindInBlueprint_IndexAll"))
								)
							]

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(HPadding, 0.f, 0.f, 0.f)
							.VAlign(VAlign_Center)
							[
								SNew(SButton)
									.Text(LOCTEXT("IndexExportList", "Export Asset List"))
									.OnClicked(this, &SFindInBlueprints::OnExportUnindexedAssetList)
									.ToolTip(IDocumentation::Get()->CreateToolTip(
										LOCTEXT("IndexExportList_Tooltip", "Exports a list of all Blueprints that have an out-of-date index (search metadata)."),
										NULL,
										TEXT("Shared/Editors/BlueprintEditor"),
										TEXT("FindInBlueprint_IndexExportList_Tooltip"))
									)
							]
						
						]
					]

					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.f, VPadding, 0.f, 0.f)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						[
							SNew(STextBlock)
							.Visibility(this, &SFindInBlueprints::GetCacheBarWidgetVisibility, EFiBCacheBarWidget::ShowCacheStatusText)
							.Text(this, &SFindInBlueprints::GetCacheBarStatusText)
						]

						// Cache progress bar
						+SHorizontalBox::Slot()
						.FillWidth(1.0f)
						.Padding(HPadding, 0.f, 0.f, 0.f)
						.VAlign(VAlign_Center)
						[
							SNew(SProgressBar)
							.Percent( this, &SFindInBlueprints::GetPercentCompleteCache )
							.Visibility( this, &SFindInBlueprints::GetCacheBarWidgetVisibility, EFiBCacheBarWidget::ProgressBar )
						]

						// Cancel button
						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(HPadding, 0.f, 0.f, 0.f)
						.VAlign(VAlign_Center)
						[
							SNew(SButton)
							.Text(LOCTEXT("CancelCacheAll", "Cancel"))
							.OnClicked( this, &SFindInBlueprints::OnCancelCacheAll )
							.Visibility( this, &SFindInBlueprints::GetCacheBarWidgetVisibility, EFiBCacheBarWidget::CancelButton )
							.ToolTipText( LOCTEXT("CancelCacheAll_Tooltip", "Stops the caching process from where ever it is, can be started back up where it left off when needed.") )
						]
					]

					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.f, VPadding, 0.f, 0.f)
					[
						SNew(STextBlock)
						.Text(this, &SFindInBlueprints::GetCacheBarCurrentAssetName)
						.Visibility( this, &SFindInBlueprints::GetCacheBarWidgetVisibility, EFiBCacheBarWidget::CurrentAssetNameText )
						.ColorAndOpacity( FCoreStyle::Get().GetColor("ErrorReporting.ForegroundColor") )
					]

					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.f, VPadding, 0.f, 0.f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("FiBUnresponsiveEditorWarning", "NOTE: The editor may become unresponsive while these assets are loaded for indexing. This may take some time!"))
						.TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>( "SmallText" ))
						.Visibility(this, &SFindInBlueprints::GetCacheBarWidgetVisibility, EFiBCacheBarWidget::UnresponsiveEditorWarningText)
					]
				]
			];
		}
	}
	else
	{
		// Because there are no uncached Blueprints, remove the bar
		OnRemoveCacheBar();
	}
}

FReply SFindInBlueprints::OnRemoveCacheBar()
{
	if(MainVerticalBox.IsValid() && CacheBarSlot.IsValid())
	{
		MainVerticalBox.Pin()->RemoveSlot(CacheBarSlot.Pin().ToSharedRef());
	}

	return FReply::Handled();
}

SFindInBlueprints::~SFindInBlueprints()
{
	if(StreamSearch.IsValid())
	{
		StreamSearch->Stop();
		StreamSearch->EnsureCompletion();
	}

	// Only cancel unindexed (slow) caching operations upon destruction
	if (FFindInBlueprintSearchManager::Get().IsUnindexedCacheInProgress())
	{
		FFindInBlueprintSearchManager::Get().CancelCacheAll(this);
	}
}

EActiveTimerReturnType SFindInBlueprints::UpdateSearchResults( double InCurrentTime, float InDeltaTime )
{
	if ( StreamSearch.IsValid() )
	{
		bool bShouldShutdownThread = false;
		bShouldShutdownThread = StreamSearch->IsComplete();

		TArray<FSearchResult> BackgroundItemsFound;

		StreamSearch->GetFilteredItems( BackgroundItemsFound );
		if ( BackgroundItemsFound.Num() )
		{
			for ( auto& Item : BackgroundItemsFound )
			{
				FindInBlueprintsHelpers::ExpandAllChildren(Item, TreeView);
				ItemsFound.Add( Item );
			}
			TreeView->RequestTreeRefresh();
		}

		// If the thread is complete, shut it down properly
		if ( bShouldShutdownThread )
		{
			if ( ItemsFound.Num() == 0 )
			{
				// Insert a fake result to inform user if none found
				ItemsFound.Add( FSearchResult( new FFindInBlueprintsNoResult( LOCTEXT( "BlueprintSearchNoResults", "No Results found" ) ) ) );
				TreeView->RequestTreeRefresh();
			}

			// Add the cache bar if needed.
			ConditionallyAddCacheBar();

			StreamSearch->EnsureCompletion();

			TArray<FImaginaryFiBDataSharedPtr> ImaginaryResults;
			if (OnSearchComplete.IsBound())
			{
				// Pull out the filtered imaginary results if there is a callback to pass them to
				StreamSearch->GetFilteredImaginaryResults(ImaginaryResults);
			}
			OutOfDateWithLastSearchBPCount = StreamSearch->GetOutOfDateCount();

			StreamSearch.Reset();

			OnSearchComplete.ExecuteIfBound(ImaginaryResults);
		}
	}

	return StreamSearch.IsValid() ? EActiveTimerReturnType::Continue : EActiveTimerReturnType::Stop;
}

void SFindInBlueprints::RegisterCommands()
{
	CommandList = BlueprintEditorPtr.IsValid() ? BlueprintEditorPtr.Pin()->GetToolkitCommands() : MakeShareable(new FUICommandList());

	CommandList->MapAction( FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP(this, &SFindInBlueprints::OnCopyAction) );

	CommandList->MapAction( FGenericCommands::Get().SelectAll,
		FExecuteAction::CreateSP(this, &SFindInBlueprints::OnSelectAllAction) );
}

void SFindInBlueprints::FocusForUse(bool bSetFindWithinBlueprint, FString NewSearchTerms, bool bSelectFirstResult)
{
	// NOTE: Careful, GeneratePathToWidget can be reentrant in that it can call visibility delegates and such
	FWidgetPath FilterTextBoxWidgetPath;
	FSlateApplication::Get().GeneratePathToWidgetUnchecked( SearchTextField.ToSharedRef(), FilterTextBoxWidgetPath );

	// Set keyboard focus directly
	FSlateApplication::Get().SetKeyboardFocus( FilterTextBoxWidgetPath, EFocusCause::SetDirectly );

	// Set the filter mode
	bIsInFindWithinBlueprintMode = bSetFindWithinBlueprint;

	if (!NewSearchTerms.IsEmpty())
	{
		SearchTextField->SetText(FText::FromString(NewSearchTerms));
		MakeSearchQuery(SearchValue, bIsInFindWithinBlueprintMode);

		// Select the first result
		if (bSelectFirstResult && ItemsFound.Num())
		{
			auto ItemToFocusOn = ItemsFound[0];

			// We want the first childmost item to select, as that is the item that is most-likely to be what was searched for (parents being graphs).
			// Will fail back upward as neccessary to focus on a focusable item
			while(ItemToFocusOn->Children.Num())
			{
				ItemToFocusOn = ItemToFocusOn->Children[0];
			}
			TreeView->SetSelection(ItemToFocusOn);
			ItemToFocusOn->OnClick();
		}
	}
}

void SFindInBlueprints::MakeSearchQuery(FString InSearchString, bool bInIsFindWithinBlueprint, const FStreamSearchOptions& InSearchOptions/* = FStreamSearchOptions()*/, FOnSearchComplete InOnSearchComplete/* = FOnSearchComplete()*/)
{
	SearchTextField->SetText(FText::FromString(InSearchString));
	LastSearchedFiBVersion = InSearchOptions.MinimiumVersionRequirement;

	if(ItemsFound.Num())
	{
		// Reset the scroll to the top
		TreeView->RequestScrollIntoView(ItemsFound[0]);
	}

	ItemsFound.Empty();

	if (InSearchString.Len() > 0)
	{
		// Remove the cache bar unless an active cache is in progress (so that we still show the status). It's ok to proceed with the new search while this is ongoing.
		if (!IsCacheInProgress())
		{
			OnRemoveCacheBar();
		}

		TreeView->RequestTreeRefresh();
		HighlightText = FText::FromString( InSearchString );

		if (bInIsFindWithinBlueprint
			&& ensureMsgf(BlueprintEditorPtr.IsValid(), TEXT("A local search was requested, but this widget does not support it.")))
		{
			const double StartTime = FPlatformTime::Seconds();

			if(StreamSearch.IsValid() && !StreamSearch->IsComplete())
			{
				StreamSearch->Stop();
				StreamSearch->EnsureCompletion();
				OutOfDateWithLastSearchBPCount = StreamSearch->GetOutOfDateCount();
				StreamSearch.Reset();
			}

			UBlueprint* Blueprint = BlueprintEditorPtr.Pin()->GetBlueprintObj();
			FString ParentClass;
			if (FProperty* ParentClassProp = Blueprint->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UBlueprint, ParentClass)))
			{
				ParentClassProp->ExportTextItem_Direct(ParentClass, ParentClassProp->ContainerPtrToValuePtr<uint8>(Blueprint), nullptr, Blueprint, 0);
			}

			TArray<FString> Interfaces;

			for (FBPInterfaceDescription& InterfaceDesc : Blueprint->ImplementedInterfaces)
			{
				Interfaces.Add(InterfaceDesc.Interface->GetPathName());
			}

			const bool bRebuildSearchData = true;
			FSearchData SearchData = FFindInBlueprintSearchManager::Get().QuerySingleBlueprint(Blueprint, bRebuildSearchData);
			const bool bHasValidSearchData = SearchData.IsValid() && !SearchData.Value.IsEmpty();

			if (bHasValidSearchData)
			{
				FImaginaryFiBDataSharedPtr ImaginaryBlueprint(new FImaginaryBlueprint(Blueprint->GetName(), Blueprint->GetPathName(), ParentClass, Interfaces, SearchData.Value, SearchData.VersionInfo));
				TSharedPtr< FFiBSearchInstance > SearchInstance(new FFiBSearchInstance);
				FSearchResult SearchResult = RootSearchResult = SearchInstance->StartSearchQuery(SearchValue, ImaginaryBlueprint);

				if (SearchResult.IsValid())
				{
					ItemsFound = SearchResult->Children;
				}

				// call SearchCompleted callback if bound (the only steps left are to update the TreeView, the search operation is complete)
				if (InOnSearchComplete.IsBound())
				{
					TArray<FImaginaryFiBDataSharedPtr> FilteredImaginaryResults;
					SearchInstance->CreateFilteredResultsListFromTree(InSearchOptions.ImaginaryDataFilter, FilteredImaginaryResults);
					InOnSearchComplete.Execute(FilteredImaginaryResults);
				}
			}

			if(ItemsFound.Num() == 0)
			{
				FText NoResultsText;
				if (bHasValidSearchData)
				{
					NoResultsText = LOCTEXT("BlueprintSearchNoResults", "No Results found");
				}
				else
				{
					NoResultsText = LOCTEXT("BlueprintSearchNotIndexed", "This Blueprint is not indexed for searching");
				}

				// Insert a fake result to inform user if none found
				ItemsFound.Add(FSearchResult(new FFindInBlueprintsNoResult(NoResultsText)));
				HighlightText = FText::GetEmpty();
			}
			else
			{
				for(auto Item : ItemsFound)
				{
					FindInBlueprintsHelpers::ExpandAllChildren(Item, TreeView);
				}
			}

			TreeView->RequestTreeRefresh();

			UE_LOGF(LogFindInBlueprint, Log, "Search completed in %0.2f seconds.", FPlatformTime::Seconds() - StartTime);
		}
		else
		{
			LaunchStreamThread(InSearchString, InSearchOptions, InOnSearchComplete);
		}
	}
}

void SFindInBlueprints::OnSearchTextChanged( const FText& Text)
{
	SearchValue = Text.ToString();
}

void SFindInBlueprints::OnSearchTextCommitted( const FText& Text, ETextCommit::Type CommitType )
{
	if (CommitType == ETextCommit::OnEnter)
	{
		MakeSearchQuery(SearchValue, bIsInFindWithinBlueprintMode);
	}
}

void SFindInBlueprints::LaunchStreamThread(const FString& InSearchValue, const FStreamSearchOptions& InSearchOptions, FOnSearchComplete InOnSearchComplete)
{
	if(StreamSearch.IsValid() && !StreamSearch->IsComplete())
	{
		StreamSearch->Stop();
		StreamSearch->EnsureCompletion();
	}
	else
	{
		// If the stream search wasn't already running, register the active timer
		RegisterActiveTimer( 0.f, FWidgetActiveTimerDelegate::CreateSP( this, &SFindInBlueprints::UpdateSearchResults ) );
	}

	StreamSearch = MakeShared<FStreamSearch>(InSearchValue, InSearchOptions);
	OnSearchComplete = InOnSearchComplete;
}

TSharedRef<ITableRow> SFindInBlueprints::OnGenerateRow( FSearchResult InItem, const TSharedRef<STableViewBase>& OwnerTable )
{
	// Finalize the search data, this does some non-thread safe actions that could not be done on the separate thread.
	InItem->FinalizeSearchData();

	bool bIsACategoryWidget = !bIsInFindWithinBlueprintMode && !InItem->Parent.IsValid();

	if (bIsACategoryWidget)
	{
		return SNew( STableRow< TSharedPtr<FFindInBlueprintsResult> >, OwnerTable )
			.Style( &FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("ShowParentsTableView.Row") )
			.Padding(FMargin(2.f, 3.f, 2.f, 3.f))
			[
				SNew(STextBlock)
				.Text(InItem.Get(), &FFindInBlueprintsResult::GetDisplayString)
				.ToolTipText(LOCTEXT("BlueprintCatSearchToolTip", "Blueprint"))
			];
	}
	else // Functions/Event/Pin widget
	{
		FText CommentText = FText::GetEmpty();

		if(!InItem->GetCommentText().IsEmpty())
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("Comment"), FText::FromString(InItem->GetCommentText()));

			CommentText = FText::Format(LOCTEXT("NodeComment", "Node Comment:[{Comment}]"), Args);
		}

		FFormatNamedArguments Args;
		Args.Add(TEXT("Category"), InItem->GetCategory());
		Args.Add(TEXT("DisplayTitle"), InItem->DisplayText);

		FText Tooltip = FText::Format(LOCTEXT("BlueprintResultSearchToolTip", "{Category} : {DisplayTitle}"), Args);

		return SNew( STableRow< TSharedPtr<FFindInBlueprintsResult> >, OwnerTable )
			.Style( &FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("ShowParentsTableView.Row") )
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					InItem->CreateIcon()
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(2.f)
				[
					SNew(STextBlock)
						.Text(InItem.Get(), &FFindInBlueprintsResult::GetDisplayString)
						.HighlightText(HighlightText)
						.ToolTipText(Tooltip)
				]
				+SHorizontalBox::Slot()
				.FillWidth(1)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(2.f)
				[
					SNew(STextBlock)
					.Text( CommentText )
					.HighlightText(HighlightText)
				]
			];
	}
}

void SFindInBlueprints::OnGetChildren( FSearchResult InItem, TArray< FSearchResult >& OutChildren )
{
	OutChildren += InItem->Children;
}

void SFindInBlueprints::OnTreeSelectionDoubleClicked( FSearchResult Item )
{
	if(Item.IsValid())
	{
		Item->OnClick();
	}
}

TOptional<float> SFindInBlueprints::GetPercentCompleteSearch() const
{
	if(StreamSearch.IsValid())
	{
		return StreamSearch->GetPercentComplete();
	}
	return 0.0f;
}

EVisibility SFindInBlueprints::GetSearchBarWidgetVisiblity(EFiBSearchBarWidget InSearchBarWidget) const
{
	const bool bShowSearchBarWidgets = StreamSearch.IsValid();
	if (bShowSearchBarWidgets)
	{
		EVisibility Result = EVisibility::Visible;
		const bool bShouldShowProgressBarWidget = !bHideProgressBars;

		switch (InSearchBarWidget)
		{
		case EFiBSearchBarWidget::Throbber:
			// Keep hidden if progress bar is visible.
			if (bShouldShowProgressBarWidget)
			{
				Result = EVisibility::Collapsed;
			}
			break;

		case EFiBSearchBarWidget::ProgressBar:
			// Keep hidden if not allowed to be shown.
			if (!bShouldShowProgressBarWidget)
			{
				Result = EVisibility::Collapsed;
			}
			break;

		default:
			// Always visible.
			break;
		}

		return Result;
	}
	else
	{
		return EVisibility::Collapsed;
	}
}

void SFindInBlueprints::CacheAllBlueprints(const FFindInBlueprintCachingOptions& InOptions)
{
	OnCacheAllBlueprints(InOptions);
}

FReply SFindInBlueprints::OnCacheAllUnindexedBlueprints()
{
	FFindInBlueprintCachingOptions CachingOptions;
	CachingOptions.OpType = EFiBCacheOpType::CacheUnindexedAssets;
	return OnCacheAllBlueprints(CachingOptions);
}

FReply SFindInBlueprints::OnExportUnindexedAssetList()
{
	FFindInBlueprintSearchManager& FindInBlueprintManager = FFindInBlueprintSearchManager::Get();
	FindInBlueprintManager.ExportOutdatedAssetList();
	return FReply::Handled();
}

FReply SFindInBlueprints::OnCacheAllBlueprints(const FFindInBlueprintCachingOptions& InOptions)
{
	if(!FFindInBlueprintSearchManager::Get().IsCacheInProgress())
	{
		FFindInBlueprintSearchManager::Get().CacheAllAssets(SharedThis(this), InOptions);
	}

	return FReply::Handled();
}

FReply SFindInBlueprints::OnCancelCacheAll()
{
	FFindInBlueprintSearchManager::Get().CancelCacheAll(this);

	// Resubmit the last search
	OnSearchTextCommitted(SearchTextField->GetText(), ETextCommit::OnEnter);

	return FReply::Handled();
}

int32 SFindInBlueprints::GetCurrentCacheIndex() const
{
	return FFindInBlueprintSearchManager::Get().GetCurrentCacheIndex();
}

TOptional<float> SFindInBlueprints::GetPercentCompleteCache() const
{
	return FFindInBlueprintSearchManager::Get().GetCacheProgress();
}

EVisibility SFindInBlueprints::GetCacheBarVisibility() const
{
	const bool bIsPIESimulating = (GEditor->bIsSimulatingInEditor || GEditor->PlayWorld);
	FFindInBlueprintSearchManager& FindInBlueprintManager = FFindInBlueprintSearchManager::Get();
	return (bKeepCacheBarProgressVisible || FindInBlueprintManager.GetNumberUncachedAssets() > 0 || (!bIsPIESimulating && (FindInBlueprintManager.GetNumberUnindexedAssets() > 0 || FindInBlueprintManager.GetFailedToCacheCount()))) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SFindInBlueprints::GetCacheBarWidgetVisibility(EFiBCacheBarWidget InCacheBarWidget) const
{
	EVisibility Result = EVisibility::Visible;

	const bool bIsCaching = IsCacheInProgress() || bKeepCacheBarProgressVisible;
	const bool bNotCurrentlyCaching = !bIsCaching;

	switch (InCacheBarWidget)
	{
	case EFiBCacheBarWidget::ProgressBar:
		// Keep hidden when not caching or when progress bars are explicitly not being shown.
		if (bNotCurrentlyCaching || bHideProgressBars)
		{
			Result = EVisibility::Hidden;
		}
		break;

	case EFiBCacheBarWidget::CloseButton:
		// Keep hidden while caching if explicitly not being shown.
		if ((bIsCaching && !bShowCacheBarCloseButton))
		{
			Result = EVisibility::Collapsed;
		}
		break;

	case EFiBCacheBarWidget::CancelButton:
		// Keep hidden when not caching or when explicitly not being shown.
		if (bNotCurrentlyCaching || !bShowCacheBarCancelButton)
		{
			Result = EVisibility::Collapsed;
		}
		break;


	case EFiBCacheBarWidget::CacheAllUnindexedButton:
		// Always keep hidden while caching.
		if (bIsCaching)
		{
			Result = EVisibility::Collapsed;
		}
		break;

	case EFiBCacheBarWidget::CurrentAssetNameText:
		// Keep hidden when not caching.
		if (bNotCurrentlyCaching)
		{
			Result = EVisibility::Collapsed;
		}
		break;

	case EFiBCacheBarWidget::UnresponsiveEditorWarningText:

		// Keep hidden while caching if explicitly not being shown.
		if (bNotCurrentlyCaching && !bShowCacheBarUnresponsiveEditorWarningText)
		{
			Result = EVisibility::Collapsed;
		}
		break;

	case EFiBCacheBarWidget::ShowCacheFailuresButton:
		// Always keep hidden while caching. Also keep hidden if there are no assets that failed to be cached. 
		if (bIsCaching || FFindInBlueprintSearchManager::Get().GetFailedToCacheCount() == 0 )
		{
			Result = EVisibility::Collapsed;
		}
		break;

	case EFiBCacheBarWidget::ShowCacheStatusText:
	{
		// Keep hidden if not currently caching 
		if (bNotCurrentlyCaching)
		{
			Result = EVisibility::Collapsed;
		}
		break;
	}

	default:
		// Always visible.
		break;
	}

	return Result;
}

bool SFindInBlueprints::IsCacheInProgress() const
{
	return FFindInBlueprintSearchManager::Get().IsCacheInProgress();
}

const FSlateBrush* SFindInBlueprints::GetCacheBarImage() const
{
	const FSlateBrush* ReturnBrush = FCoreStyle::Get().GetBrush("ErrorReporting.Box");
	if ((IsCacheInProgress() || bKeepCacheBarProgressVisible) && !FFindInBlueprintSearchManager::Get().IsUnindexedCacheInProgress())
	{
		// Allow the content area to show through for a non-unindexed operation.
		ReturnBrush = FAppStyle::GetBrush("NoBorder");
	}
	return ReturnBrush;
}

FText SFindInBlueprints::GetCacheBarStatusText() const
{
	FFindInBlueprintSearchManager& FindInBlueprintManager = FFindInBlueprintSearchManager::Get();

	FFormatNamedArguments Args;
	FText ReturnDisplayText;
	if (IsCacheInProgress() || bKeepCacheBarProgressVisible)
	{
		if (bHideProgressBars)
		{
			ReturnDisplayText = LOCTEXT("CachingBlueprintsWithUnknownEndpoint", "Indexing Blueprints...");
		}
		else
		{
			Args.Add(TEXT("CurrentIndex"), FindInBlueprintManager.GetCurrentCacheIndex());
			Args.Add(TEXT("Count"), FindInBlueprintManager.GetNumberUncachedAssets());

			ReturnDisplayText = FText::Format(LOCTEXT("CachingBlueprints", "Indexing Blueprints... {CurrentIndex}/{Count}"), Args);
		}
	}
	else
	{
		const int32 UnindexedCount = FindInBlueprintManager.GetNumberUnindexedAssets();
		Args.Add(TEXT("UnindexedCount"), UnindexedCount);
		Args.Add(TEXT("OutOfDateCount"), OutOfDateWithLastSearchBPCount);
		Args.Add(TEXT("Count"), UnindexedCount + OutOfDateWithLastSearchBPCount);

		// Show a different instruction depending on the "Index All" permission level in editor settings
		const EFiBIndexAllPermission IndexAllPermission = GetDefault<UBlueprintEditorSettings>()->AllowIndexAllBlueprints;
		const FText IndexAllDisabledText = LOCTEXT("IndexAllDisabled", "Your editor settings disallow loading all these assets from this window, see Blueprint Editor Settings: AllowIndexAllBlueprints. Export the asset list to inspect which assets do not have optimal searchability.");
		const FText IndexAllWarningText_LoadOnly = LOCTEXT("IndexAllWarning_LoadOnly", "Press \"Index All\" to load these assets right now. The editor may become unresponsive while these assets are loaded for indexing. Save your work before initiating this: broken assets and memory usage can affect editor stability. Alternatively, export the asset list to inspect which assets do not have optimal searchability.");
		const FText IndexAllWarningText_Checkout = LOCTEXT("IndexAllWarning_Checkout", "Press \"Index All\" to load, and optionally checkout and resave, these assets right now. The editor may become unresponsive while these assets are loaded for indexing. Save your work before initiating this: broken assets and memory usage can affect editor stability. Alternatively, export the asset list to inspect which assets do not have optimal searchability.");
		switch (IndexAllPermission)
		{
		case EFiBIndexAllPermission::CheckoutAndResave:
			Args.Add(TEXT("Instruction"), IndexAllWarningText_Checkout);
			break;
		case EFiBIndexAllPermission::LoadOnly:
			Args.Add(TEXT("Instruction"), IndexAllWarningText_LoadOnly);
			break;
		case EFiBIndexAllPermission::None:
			Args.Add(TEXT("Instruction"), IndexAllDisabledText);
			break;
		default:
			ensureMsgf(false, TEXT("Unhandled case"));
		}
		
		ReturnDisplayText = FText::Format(LOCTEXT("UncachedAssets", "Search incomplete: {Count} blueprints don't have an up-to-date index ({UnindexedCount} unindexed/{OutOfDateCount} out-of-date). These assets are searchable but some results may be missing. Load and resave these assets to improve their searchability. \n\n{Instruction}"), Args);

		const int32 FailedToCacheCount = FindInBlueprintManager.GetFailedToCacheCount();
		if (FailedToCacheCount > 0)
		{
			FFormatNamedArguments ArgsWithCacheFails;
			ArgsWithCacheFails.Add(TEXT("BaseMessage"), ReturnDisplayText);
			ArgsWithCacheFails.Add(TEXT("CacheFails"), FailedToCacheCount);
			ReturnDisplayText = FText::Format(LOCTEXT("UncachedAssetsWithCacheFails", "{BaseMessage} {CacheFails} Blueprints failed to cache."), ArgsWithCacheFails);
		}
	}

	return ReturnDisplayText;
}

FText SFindInBlueprints::GetCacheBarCurrentAssetName() const
{
	if (IsCacheInProgress())
	{
		LastCachedAssetPath = FFindInBlueprintSearchManager::Get().GetCurrentCacheBlueprintPath();
	}

	return FText::FromString(LastCachedAssetPath.ToString());
}

bool SFindInBlueprints::CanCacheAllUnindexedBlueprints() const
{
	return GetDefault<UBlueprintEditorSettings>()->AllowIndexAllBlueprints != EFiBIndexAllPermission::None;
}

void SFindInBlueprints::OnCacheStarted(EFiBCacheOpType InOpType, EFiBCacheOpFlags InOpFlags)
{
	const bool bShowProgress = EnumHasAnyFlags(InOpFlags, EFiBCacheOpFlags::ShowProgress);
	if (bShowProgress)
	{
		// Whether to keep both the cache and search bar progress indicators hidden.
		bHideProgressBars = EnumHasAnyFlags(InOpFlags, EFiBCacheOpFlags::HideProgressBars);

		// Whether to show the cache bar close button and allow users to dismiss the progress display.
		bShowCacheBarCloseButton = EnumHasAnyFlags(InOpFlags, EFiBCacheOpFlags::AllowUserCloseProgress);

		// Whether to show the cache bar cancel button allowing users to cancel the operation.
		bShowCacheBarCancelButton = EnumHasAnyFlags(InOpFlags, EFiBCacheOpFlags::AllowUserCancel);

		// Whether to show the unresponsive editor warning text in the cache bar status area.
		bShowCacheBarUnresponsiveEditorWarningText = (InOpType == EFiBCacheOpType::CacheUnindexedAssets);

		// Ensure that the cache bar is visible to show progress
		const bool bIsCacheBarAdded = CacheBarSlot.IsValid();
		if (!bIsCacheBarAdded)
		{
			ConditionallyAddCacheBar();
		}
	}
}

void SFindInBlueprints::OnCacheComplete(EFiBCacheOpType InOpType, EFiBCacheOpFlags InOpFlags)
{
	// Indicate whether to keep the search bar progress indicator hidden.
	bHideProgressBars = EnumHasAnyFlags(InOpFlags, EFiBCacheOpFlags::HideProgressBars);

	// Indicate whether to keep cache bar progress visible. Used to seamlessly transition to the next operation.
	bKeepCacheBarProgressVisible = EnumHasAnyFlags(InOpFlags, EFiBCacheOpFlags::KeepProgressVisibleOnCompletion);

	TWeakPtr<SFindInBlueprints> SourceCachingWidgetPtr = FFindInBlueprintSearchManager::Get().GetSourceCachingWidget();
	if (InOpType == EFiBCacheOpType::CacheUnindexedAssets
		&& SourceCachingWidgetPtr.IsValid() && SourceCachingWidgetPtr.Pin() == SharedThis(this))
	{
		// Resubmit the last search, which will also remove the bar if needed
		OnSearchTextCommitted(SearchTextField->GetText(), ETextCommit::OnEnter);
	}
	else if (CacheBarSlot.IsValid() && !bKeepCacheBarProgressVisible)
	{
		// Remove the cache bar, unless this is not the true end of the operation
		OnRemoveCacheBar();
	}
}

TSharedPtr<SWidget> SFindInBlueprints::OnContextMenuOpening()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder( bShouldCloseWindowAfterMenuSelection, CommandList);

	MenuBuilder.BeginSection("BasicOperations");
	{
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().SelectAll);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
	}

	return MenuBuilder.MakeWidget();
}

void SFindInBlueprints::SelectAllItemsHelper(FSearchResult InItemToSelect)
{
	// Iterates over all children and recursively selects all items in the results
	TreeView->SetItemSelection(InItemToSelect, true);

	for( const auto& Child : InItemToSelect->Children )
	{
		SelectAllItemsHelper(Child);
	}
}

void SFindInBlueprints::OnSelectAllAction()
{
	for( const auto& Item : ItemsFound )
	{
		SelectAllItemsHelper(Item);
	}
}

void SFindInBlueprints::OnCopyAction()
{
	TArray< FSearchResult > SelectedItems = TreeView->GetSelectedItems();

	FString SelectedText;

	for( const auto& SelectedItem : SelectedItems)
	{
		// Add indents for each layer into the tree the item is
		for(auto ParentItem = SelectedItem->Parent; ParentItem.IsValid(); ParentItem = ParentItem.Pin()->Parent)
		{
			SelectedText += TEXT("\t");
		}

		// Add the display string
		SelectedText += SelectedItem->GetDisplayString().ToString();

		// If there is a comment, add two indents and then the comment
		FString CommentText = SelectedItem->GetCommentText();
		if(!CommentText.IsEmpty())
		{
			SelectedText += TEXT("\t\t") + CommentText;
		}
		
		// Line terminator so the next item will be on a new line
		SelectedText += LINE_TERMINATOR;
	}

	// Copy text to clipboard
	FPlatformApplicationMisc::ClipboardCopy( *SelectedText );
}

FReply SFindInBlueprints::OnOpenGlobalFindResults()
{
	TSharedPtr<SFindInBlueprints> GlobalFindResults = FFindInBlueprintSearchManager::Get().GetGlobalFindResults();
	if (GlobalFindResults.IsValid())
	{
		GlobalFindResults->FocusForUse(false, SearchValue, true);
	}

	return FReply::Handled();
}

void SFindInBlueprints::OnHostTabClosed(TSharedRef<SDockTab> DockTab)
{
	FFindInBlueprintSearchManager::Get().GlobalFindResultsClosed(SharedThis(this));
}

FReply SFindInBlueprints::OnLockButtonClicked()
{
	bIsLocked = !bIsLocked;
	return FReply::Handled();
}

const FSlateBrush* SFindInBlueprints::OnGetLockButtonImage() const
{
	if (bIsLocked)
	{
		return FAppStyle::Get().GetBrush("Icons.Lock");
	}
	else
	{
		return FAppStyle::Get().GetBrush("Icons.Unlock");
	}
}

FName SFindInBlueprints::GetHostTabId() const
{
	TSharedPtr<SDockTab> HostTabPtr = HostTab.Pin();
	if (HostTabPtr.IsValid())
	{
		return HostTabPtr->GetLayoutIdentifier().TabType;
	}

	return NAME_None;
}

void SFindInBlueprints::CloseHostTab()
{
	TSharedPtr<SDockTab> HostTabPtr = HostTab.Pin();
	if (HostTabPtr.IsValid())
	{
		HostTabPtr->RequestCloseTab();
	}
}

bool SFindInBlueprints::IsSearchInProgress() const
{
	return StreamSearch.IsValid() && !StreamSearch->IsComplete();
}

FReply SFindInBlueprints::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	// BlueprintEditor's IToolkit code will handle shortcuts itself - but we can just use 
	// simple slate handlers when we're standalone:
	if(!BlueprintEditorPtr.IsValid() && CommandList.IsValid())
	{
		if( CommandList->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();
		}
	}
	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

void SFindInBlueprints::ClearResults()
{
	ItemsFound.Empty();

	if (TreeView.IsValid())
	{
		TreeView->RequestTreeRefresh();
	}
}

#undef LOCTEXT_NAMESPACE
