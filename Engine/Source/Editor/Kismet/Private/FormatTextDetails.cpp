// Copyright Epic Games, Inc. All Rights Reserved.

#include "FormatTextDetails.h"

#include "Containers/UnrealString.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "HAL/PlatformCrt.h"
#include "IDetailChildrenBuilder.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_FormatText.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyCustomizationHelpers.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/Package.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"

class IPropertyHandle;
class SWidget;
class UObject;

#define LOCTEXT_NAMESPACE "FormatTextDetails"

void FFormatTextDetails::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{
	const TArray<TWeakObjectPtr<UObject>>& Objects = DetailLayout.GetSelectedObjects();
	check(Objects.Num() > 0);

	if (Objects.Num() == 1)
	{
		TargetNode = CastChecked<UK2Node_FormatText>(Objects[0].Get());
		TSharedRef<IPropertyHandle> PropertyHandle = DetailLayout.GetProperty(FName("PinNames"), UK2Node_FormatText::StaticClass());

		IDetailCategoryBuilder& InputsCategory = DetailLayout.EditCategory("Arguments", LOCTEXT("FormatTextDetailsArguments", "Arguments"));

		InputsCategory.AddCustomRow( LOCTEXT("FunctionNewInputArg", "New") )
			[
				SNew(SBox)
				.HAlign(HAlign_Right)
				[
					SNew(SButton)
					.Text(LOCTEXT("FunctionNewInputArg", "New"))
					.OnClicked(this, &FFormatTextDetails::OnAddNewArgument)
					.IsEnabled(this, &FFormatTextDetails::CanEditArguments)
				]
			];

		Layout = MakeShareable( new FFormatTextLayout(TargetNode) );
		InputsCategory.AddCustomBuilder( Layout.ToSharedRef() );
	}

	UPackage::PackageDirtyStateChangedEvent.AddSP(this, &FFormatTextDetails::OnEditorPackageModified);
}

FFormatTextDetails::~FFormatTextDetails()
{
	UPackage::PackageDirtyStateChangedEvent.RemoveAll(this);
}

void FFormatTextDetails::OnForceRefresh()
{
	Layout->Refresh();
}

FReply FFormatTextDetails::OnAddNewArgument()
{
	TargetNode->AddArgumentPin();
	OnForceRefresh();
	return FReply::Handled();
}

void FFormatTextDetails::OnEditorPackageModified(UPackage* Package)
{
	if (TargetNode 
		&& Package 
		&& Package->IsDirty() 
		&& (Package == TargetNode->GetOutermost())
		&& (!Layout.IsValid() || !Layout->CausedChange()))
	{
		OnForceRefresh();
	}
}

bool FFormatTextDetails::CanEditArguments() const
{
	return TargetNode->CanEditArguments();
}


void FFormatTextLayout::GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder )
{
	Children.Empty();
	for (int32 ArgIdx = 0; ArgIdx < TargetNode->GetArgumentCount(); ++ArgIdx)
	{
		TSharedRef<class FFormatTextArgumentLayout> ArgumentIndexLayout = MakeShareable(new FFormatTextArgumentLayout(TargetNode, ArgIdx) );
		ChildrenBuilder.AddCustomBuilder(ArgumentIndexLayout);
		Children.Add(ArgumentIndexLayout);
	}
}

bool FFormatTextLayout::CausedChange() const
{
	for(auto Iter = Children.CreateConstIterator(); Iter; ++Iter)
	{
		const auto Child = (*Iter);
		if (Child.IsValid() && Child.Pin()->CausedChange())
		{
			return true;
		}
	}
	return false;
}

void FFormatTextArgumentLayout::GenerateHeaderRowContent( FDetailWidgetRow& NodeRow )
{
	const bool bIsMoveUpEnabled = (TargetNode->GetArgumentCount() != 1) && (ArgumentIndex != 0);
	const bool bIsMoveDownEnabled = (TargetNode->GetArgumentCount() != 1) && (ArgumentIndex < TargetNode->GetArgumentCount() - 1);

	TSharedRef< SWidget > ClearButton = PropertyCustomizationHelpers::MakeClearButton(FSimpleDelegate::CreateSP(this, &FFormatTextArgumentLayout::OnArgumentRemove));

	NodeRow
		.WholeRowWidget
		[
			SNew(SHorizontalBox)
			.IsEnabled(this, &FFormatTextArgumentLayout::CanEditArguments)

			+SHorizontalBox::Slot()
			[
				SAssignNew(ArgumentNameWidget, SEditableTextBox)
				.OnTextCommitted(this, &FFormatTextArgumentLayout::OnArgumentNameCommitted)
				.OnTextChanged(this, &FFormatTextArgumentLayout::OnArgumentNameChanged)
				.Text(this, &FFormatTextArgumentLayout::GetArgumentName)
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2, 0)
			[
				SNew(SButton)
				.ContentPadding(0.0f)
				.OnClicked(this, &FFormatTextArgumentLayout::OnMoveArgumentUp)
				.IsEnabled( bIsMoveUpEnabled )
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.ChevronUp"))
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2, 0)
			[
				SNew(SButton)
				.ContentPadding(0.0f)
				.OnClicked(this, &FFormatTextArgumentLayout::OnMoveArgumentDown)
				.IsEnabled( bIsMoveDownEnabled )
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.ChevronDown"))
				]
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2, 0)
			[
				ClearButton
			]
		];
}

FText FFormatTextArgumentLayout::GetArgumentName() const
{
	return TargetNode->GetArgumentName(ArgumentIndex);
}

void FFormatTextArgumentLayout::OnArgumentRemove()
{
	TargetNode->RemoveArgument(ArgumentIndex);
}

FReply FFormatTextArgumentLayout::OnMoveArgumentUp()
{
	TargetNode->SwapArguments(ArgumentIndex, ArgumentIndex - 1);

	return FReply::Handled();
}

FReply FFormatTextArgumentLayout::OnMoveArgumentDown()
{
	TargetNode->SwapArguments(ArgumentIndex, ArgumentIndex + 1);

	return FReply::Handled();
}

bool FFormatTextArgumentLayout::CanEditArguments() const
{
	return TargetNode->CanEditArguments();
}

struct FScopeTrue
{
	bool& bRef;

	FScopeTrue(bool& bInRef) : bRef(bInRef)
	{
		ensure(!bRef);
		bRef = true;
	}

	~FScopeTrue()
	{
		ensure(bRef);
		bRef = false;
	}
};

void FFormatTextArgumentLayout::OnArgumentNameCommitted(const FText& NewText, ETextCommit::Type InTextCommit)
{
	if(IsValidArgumentName(NewText))
	{
		FScopeTrue ScopeTrue(bCausedChange);
		TargetNode->SetArgumentName(ArgumentIndex, *NewText.ToString());
	}
	ArgumentNameWidget.Pin()->SetError(FString());
}

void FFormatTextArgumentLayout::OnArgumentNameChanged(const FText& NewText)
{
	IsValidArgumentName(NewText);
}

bool FFormatTextArgumentLayout::IsValidArgumentName(const FText& InNewText) const
{
	if(TargetNode->FindArgumentPin(*InNewText.ToString()))
	{
		ArgumentNameWidget.Pin()->SetError(LOCTEXT("UniqueName_Error", "Name must be unique."));
		return false;
	}
	ArgumentNameWidget.Pin()->SetError(FString());
	return true;
}

#undef LOCTEXT_NAMESPACE
