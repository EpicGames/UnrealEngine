// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformMath.h"
#include "IDetailCustomNodeBuilder.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IDetailLayoutBuilder;
class SEditableTextBox;
class UK2Node_FormatText;
class UPackage;

/** Details customization for the "Format Text" node */
class FFormatTextDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FFormatTextDetails);
	}

	FFormatTextDetails()
		: TargetNode(NULL)
	{
	}

	~FFormatTextDetails();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailLayout ) override;

	/** Forces a refresh on the details customization */
	void OnForceRefresh();

private:
	/** Handles new argument request */
	FReply OnAddNewArgument();

	/** Callback whenever a package is marked dirty, will refresh the node being represented by this details customization */
	void OnEditorPackageModified(UPackage* Package);

	bool CanEditArguments() const;
private:
	TSharedPtr<class FFormatTextLayout> Layout;
	/** The target node that this argument is on */
	UK2Node_FormatText* TargetNode;
};

/** Custom struct for each group of arguments in the function editing details */
class FFormatTextLayout : public IDetailCustomNodeBuilder, public TSharedFromThis<FFormatTextLayout>
{
public:
	FFormatTextLayout(UK2Node_FormatText* InTargetNode)
		: TargetNode(InTargetNode)
	{}

	void Refresh()
	{
		OnRebuildChildren.ExecuteIfBound();
	}

	bool CausedChange() const;

private:
	/** IDetailCustomNodeBuilder Interface*/
	virtual void SetOnRebuildChildren( FSimpleDelegate InOnRegenerateChildren ) override { OnRebuildChildren = InOnRegenerateChildren; }
	virtual void GenerateHeaderRowContent( FDetailWidgetRow& NodeRow ) override {}
	virtual void GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder ) override;
	virtual void Tick( float DeltaTime ) override {}
	virtual bool RequiresTick() const override { return false; }
	virtual FName GetName() const override { return NAME_None; }
	virtual bool InitiallyCollapsed() const override { return false; }

private:
	FSimpleDelegate OnRebuildChildren;

	/** The target node that this argument is on */
	UK2Node_FormatText* TargetNode;

	TArray<TWeakPtr<class FFormatTextArgumentLayout>> Children;
};

/** Custom struct for each group of arguments in the function editing details */
class FFormatTextArgumentLayout : public IDetailCustomNodeBuilder, public TSharedFromThis<FFormatTextArgumentLayout>
{
public:
	FFormatTextArgumentLayout(UK2Node_FormatText* InTargetNode, int32 InArgumentIndex)
		: TargetNode(InTargetNode)
		, ArgumentIndex(InArgumentIndex)
		, bCausedChange(false)
	{}

	bool CausedChange() const { return bCausedChange; }

private:
	/** IDetailCustomNodeBuilder Interface*/
	virtual void SetOnRebuildChildren( FSimpleDelegate InOnRegenerateChildren ) override {};
	virtual void GenerateHeaderRowContent( FDetailWidgetRow& NodeRow ) override;
	virtual void GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder ) override {};
	virtual void Tick( float DeltaTime ) override {}
	virtual bool RequiresTick() const override { return false; }
	virtual FName GetName() const override { return NAME_None; }
	virtual bool InitiallyCollapsed() const override { return false; }

private:
	/** Retrieves the argument's name */
	FText GetArgumentName() const;

	/** Moves the argument up in the list */
	FReply OnMoveArgumentUp();
	
	/** Moves the argument down in the list */
	FReply OnMoveArgumentDown();

	/** Deletes the argument */
	void OnArgumentRemove();

	/** Callback when the argument's name is committed */
	void OnArgumentNameCommitted(const FText& NewText, ETextCommit::Type InTextCommit);
	
	/** Callback when changing the argument's name to verify the name */
	void OnArgumentNameChanged(const FText& NewText);

	/** 
	 * Helper function to validate the argument's name
	 *
	 * @param InNewText		The name to validate
	 *
	 * @return				Returns true if the name is valid
	 */
	bool IsValidArgumentName(const FText& InNewText) const;

	bool CanEditArguments() const;
private:
	/** The target node that this argument is on */
	UK2Node_FormatText* TargetNode;

	/** Index of argument */
	int32 ArgumentIndex;

	/** The argument's name widget, used for setting a argument's name */
	TWeakPtr< SEditableTextBox > ArgumentNameWidget;

	bool bCausedChange;
};
