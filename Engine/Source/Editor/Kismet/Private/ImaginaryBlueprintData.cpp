// Copyright Epic Games, Inc. All Rights Reserved.
#include "ImaginaryBlueprintData.h"

#include "Async/Async.h"
#include "Async/Future.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/StringConv.h"
#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Engine/Blueprint.h"
#include "EdGraphSchema_K2.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/StringTableCore.h"
#include "Misc/CString.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Serialization/Archive.h"
#include "Serialization/JsonTypes.h"
#include "Templates/Casts.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"

class UObject;

#define LOCTEXT_NAMESPACE "FindInBlueprints"

///////////////////////
// FSearchableValueInfo

FText FSearchableValueInfo::GetDisplayText(const TMap<int32, FText>& InLookupTable) const
{
	FText Result;
	if (!DisplayText.IsEmpty() || LookupTableKey == -1)
	{
		Result = DisplayText;
	}
	else
	{
		Result = FindInBlueprintsHelpers::AsFText(LookupTableKey, InLookupTable);
	}

	if (Result.IsFromStringTable() && FTextInspector::GetSourceString(Result) == &FStringTableEntry::GetPlaceholderSourceString() && !IsInGameThread())
	{
		// String Table asset references in FiB may be unresolved as we can't load the asset on the search thread
		// To solve this we send a request to the game thread to load the asset and wait for the result
		FName TableId;
		FString Key;
		if (FTextInspector::GetTableIdAndKey(Result, TableId, Key) && IStringTableEngineBridge::IsStringTableFromAsset(TableId))
		{
			TPromise<bool> Promise;

			// Run the request on the game thread, filling the promise when done
			AsyncTask(ENamedThreads::GameThread, [TableId, &Promise]()
			{
				FName ResolvedTableId = TableId;
				if (IStringTableEngineBridge::CanFindOrLoadStringTableAsset())
				{
					IStringTableEngineBridge::FullyLoadStringTableAsset(ResolvedTableId); // Trigger the asset load
				}
				Promise.SetValue(true); // Signal completion
			});

			// Get the promise value to block until the AsyncTask has completed
			Promise.GetFuture().Get();
		}
	}

	return Result;
}

////////////////////////////
// FComponentUniqueDisplay

bool FComponentUniqueDisplay::operator==(const FComponentUniqueDisplay& Other)
{
	// Two search results in the same object/sub-object should never have the same display string ({Key}: {Value} pairing)
	return SearchResult.IsValid() && Other.SearchResult.IsValid() && SearchResult->GetDisplayString().CompareTo(Other.SearchResult->GetDisplayString()) == 0;
}

///////////////////////
// FImaginaryFiBData

FCriticalSection FImaginaryFiBData::ParseChildDataCriticalSection;

FImaginaryFiBData::FImaginaryFiBData(FImaginaryFiBDataWeakPtr InOuter, TSharedPtr< FJsonObject > InUnparsedJsonObject, TMap<int32, FText>* InLookupTablePtr)
	: UnparsedJsonObject(InUnparsedJsonObject)
	, LookupTablePtr(InLookupTablePtr)
	, Outer(InOuter)
	, bHasParsedJsonObject(false)
	, bRequiresInterlockedParsing(false)
{
	// Backwards-compatibility; inherit the flag that only allows one thread at a time into the JSON parsing logic.
	const FImaginaryFiBDataSharedPtr OuterPtr = Outer.Pin();
	if (OuterPtr.IsValid())
	{
		bRequiresInterlockedParsing = OuterPtr->bRequiresInterlockedParsing;
	}
}

FSearchResult FImaginaryFiBData::CreateSearchResult(FSearchResult InParent) const
{
	CSV_SCOPED_TIMING_STAT(FindInBlueprint, CreateSearchResult);

	FSearchResult ReturnSearchResult = CreateSearchResult_Internal(SearchResultTemplate);
	if (ReturnSearchResult.IsValid())
	{	
		ReturnSearchResult->Parent = InParent;

		if (!FFindInBlueprintSearchManager::Get().ShouldEnableSearchResultTemplates())
		{
			for (const TPair<FindInBlueprintsHelpers::FSimpleFTextKeyStorage, FSearchableValueInfo>& TagsAndValues : ParsedTagsAndValues)
			{
				if (TagsAndValues.Value.IsCoreDisplay() || !TagsAndValues.Value.IsSearchable())
				{
					FText Value = TagsAndValues.Value.GetDisplayText(*LookupTablePtr);
					ReturnSearchResult->ParseSearchInfo(TagsAndValues.Key.Text, Value);
				}
			}
		}
	}
	
	return ReturnSearchResult;
}

FSearchResult FImaginaryFiBData::CreateSearchTree(FSearchResult InParentSearchResult, FImaginaryFiBDataWeakPtr InCurrentPointer, TArray< const FImaginaryFiBData* >& InValidSearchResults, TMultiMap< const FImaginaryFiBData*, FComponentUniqueDisplay >& InMatchingSearchComponents)
{
	CSV_SCOPED_TIMING_STAT(FindInBlueprint, CreateSearchTree);
	CSV_CUSTOM_STAT(FindInBlueprint, CreateSearchTreeIterations, 1, ECsvCustomStatOp::Accumulate);

	FImaginaryFiBDataSharedPtr CurrentDataPtr = InCurrentPointer.Pin();
	if (FImaginaryFiBData* CurrentData = CurrentDataPtr.Get())
	{
		FSearchResult CurrentSearchResult = CurrentData->CreateSearchResult(InParentSearchResult);
		bool bValidSearchResults = false;

		// Check all children first, to see if they are valid in the search results
		for (FImaginaryFiBDataSharedPtr ChildData : CurrentData->ParsedChildData)
		{
			FSearchResult Result = CreateSearchTree(CurrentSearchResult, ChildData, InValidSearchResults, InMatchingSearchComponents);
			if (Result.IsValid())
			{
				bValidSearchResults = true;
				CurrentSearchResult->Children.Add(Result);
			}
		}

		// If the children did not match the search results but this item does, then we will want to return true.
		// Include "tag+value" categories in the search tree, as the relevant results need to be added as children.
		const bool bInvalidSearchResultsCategory = CurrentData->IsCategory() && !CurrentData->IsTagAndValueCategory();
		if (!bValidSearchResults && !bInvalidSearchResultsCategory && (InValidSearchResults.Find(CurrentData) != INDEX_NONE || InMatchingSearchComponents.Find(CurrentData)))
		{
			bValidSearchResults = true;
		}

		if (bValidSearchResults)
		{
			TArray< FComponentUniqueDisplay > SearchResultList;
			InMatchingSearchComponents.MultiFind(CurrentData, SearchResultList, true);
			CurrentSearchResult->Children.Reserve(CurrentSearchResult->Children.Num() + SearchResultList.Num());

			// Add any data that matched the search results as a child of our search result
			for (FComponentUniqueDisplay& SearchResultWrapper : SearchResultList)
			{
				SearchResultWrapper.SearchResult->Parent = CurrentSearchResult;
				CurrentSearchResult->Children.Add(SearchResultWrapper.SearchResult);
			}
			return CurrentSearchResult;
		}
	}
	return nullptr;
}

bool FImaginaryFiBData::IsCompatibleWithFilter(ESearchQueryFilter InSearchQueryFilter) const
{
	return true;
}

bool FImaginaryFiBData::CanCallFilter(ESearchQueryFilter InSearchQueryFilter) const
{
	// Always compatible with the AllFilter
	return InSearchQueryFilter == ESearchQueryFilter::AllFilter;
}

void FImaginaryFiBData::ParseAllChildData_Internal(ESearchableValueStatus InSearchabilityOverride/* = ESearchableValueStatus::Searchable*/)
{
	if (UnparsedJsonObject.IsValid())
	{
		if (InSearchabilityOverride & ESearchableValueStatus::Searchable)
		{
			TSharedPtr< FJsonObject > MetaDataField;
			for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& MapValues : UnparsedJsonObject->Values)
			{
				FText KeyText = FindInBlueprintsHelpers::AsFText(FCString::Atoi(*MapValues.Key), *LookupTablePtr);
				if (!KeyText.CompareTo(FFindInBlueprintSearchTags::FiBMetaDataTag))
				{
					MetaDataField = MapValues.Value->AsObject();
					break;
				}
			}

			if (MetaDataField.IsValid())
			{
				TSharedPtr<FFiBMetaData, ESPMode::ThreadSafe> MetaDataFiBInfo = MakeShareable(new FFiBMetaData(AsShared(), MetaDataField, LookupTablePtr));
				MetaDataFiBInfo->ParseAllChildData_Internal();

				if (MetaDataFiBInfo->IsHidden() && MetaDataFiBInfo->IsExplicit())
				{
					InSearchabilityOverride = ESearchableValueStatus::ExplicitySearchableHidden;
				}
				else if (MetaDataFiBInfo->IsExplicit())
				{
					InSearchabilityOverride = ESearchableValueStatus::ExplicitySearchable;
				}
			}
		}

		for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& MapValues : UnparsedJsonObject->Values)
		{
			FText KeyText = FindInBlueprintsHelpers::AsFText(FCString::Atoi(*MapValues.Key), *LookupTablePtr);
			TSharedPtr< FJsonValue > JsonValue = MapValues.Value;

			if (!KeyText.CompareTo(FFindInBlueprintSearchTags::FiBMetaDataTag))
			{
				// Do not let this be processed again
				continue;
			}
			if (!TrySpecialHandleJsonValue(KeyText, JsonValue))
			{
				TArray<FSearchableValueInfo> ParsedValues;

				// Parse the value making it searchable
				ParseJsonValue(KeyText, KeyText, JsonValue, ParsedValues, false, InSearchabilityOverride);

				// For some property types like TMaps we want KeyText to be a search candidate since it's entered by the user.
				// In most other cases KeyText represents property or category names which we don't want to return as search results.
				if (AreKeysSearchable())
				{
					FSearchableValueInfo& ParsedValue = ParsedValues.AddDefaulted_GetRef();
					ParsedValue = FSearchableValueInfo(FText::FromString("Key"), KeyText, InSearchabilityOverride);
					ParsedTagsAndValues.Add(FindInBlueprintsHelpers::FSimpleFTextKeyStorage(KeyText), ParsedValue);
				}

				if (FFindInBlueprintSearchManager::Get().ShouldEnableSearchResultTemplates())
				{
					for (const FSearchableValueInfo& ParsedValue : ParsedValues)
					{
						if (ParsedValue.IsCoreDisplay() || !ParsedValue.IsSearchable())
						{
							// If necessary, create the search result template.
							if (!SearchResultTemplate.IsValid())
							{
								FSearchResult NullTemplate;
								SearchResultTemplate = CreateSearchResult_Internal(NullTemplate);
								check(SearchResultTemplate.IsValid());
							}

							// Parse out meta values used for display and cache them in the template.
							SearchResultTemplate->ParseSearchInfo(KeyText, ParsedValue.GetDisplayText(*LookupTablePtr));
						}
					}
				}
			}
		}
	}

	UnparsedJsonObject.Reset();
}

void FImaginaryFiBData::ParseAllChildData(ESearchableValueStatus InSearchabilityOverride/* = ESearchableValueStatus::Searchable*/)
{
	CSV_SCOPED_TIMING_STAT(FindInBlueprint, ParseAllChildData);
	CSV_CUSTOM_STAT(FindInBlueprint, ParseAllChildDataIterations, 1, ECsvCustomStatOp::Accumulate);

	if (bRequiresInterlockedParsing)
	{
		ParseChildDataCriticalSection.Lock();
	}

	if (!bHasParsedJsonObject)
	{
		ParseAllChildData_Internal(InSearchabilityOverride);
		bHasParsedJsonObject = true;
	}

	if (bRequiresInterlockedParsing)
	{
		ParseChildDataCriticalSection.Unlock();
	}
}

void FImaginaryFiBData::ParseJsonValue(FText InKey, FText InDisplayKey, TSharedPtr< FJsonValue > InJsonValue, TArray<FSearchableValueInfo>& OutParsedValues, bool bIsInArray/*=false*/, ESearchableValueStatus InSearchabilityOverride/* = ESearchableValueStatus::Searchable*/)
{
	ESearchableValueStatus SearchabilityStatus = (InSearchabilityOverride == ESearchableValueStatus::Searchable)? GetSearchabilityStatus(InKey.ToString()) : InSearchabilityOverride;
	if( InJsonValue->Type == EJson::Array)
	{
		TSharedPtr< FCategorySectionHelper, ESPMode::ThreadSafe > ArrayCategory = MakeShareable(new FCategorySectionHelper(AsShared(), LookupTablePtr, InKey, /*bIsTagAndValueCategory=*/true));
		ParsedChildData.Add(ArrayCategory);
		TArray<TSharedPtr< FJsonValue > > ArrayList = InJsonValue->AsArray();
		for( int32 ArrayIdx = 0; ArrayIdx < ArrayList.Num(); ++ArrayIdx)
		{
			TSharedPtr< FJsonValue > ArrayValue = ArrayList[ArrayIdx];
			ArrayCategory->ParseJsonValue(InKey, FText::FromString(FString::FromInt(ArrayIdx)), ArrayValue, OutParsedValues, /*bIsInArray=*/true, SearchabilityStatus);
		}		
	}
	else if (InJsonValue->Type == EJson::Object)
	{
		// Create a FCategorySectionHelper for Json objects, which are Json representations of structs, instanced objects, array elements.
		// bIsTagAndValueCategory represents whether a match with the category's contents should return its parent as a search result.
		// Let the following category return this as search result when:
		// - The following category represents an array element, return this (the array)
		// - The following category represents a map entry, return this (the map)
		// - The following category represents the default value for a struct or object, return this (the default value)
		// - The following category represents a nested element for one of the above
		const bool bIsDefaultValueKey = InDisplayKey.EqualTo(FFindInBlueprintSearchTags::FiB_DefaultValue);
		const bool bIsTagAndValueCategory = bIsInArray || IsTagAndValueCategory() || bIsDefaultValueKey;

		// Evaluate now whether this child's keys should be searchable. Currently this requires peeking at the value syntax 
		// because we don't store metadata to detect that we're processing a TMap. It's detectable from value though.
		TSharedPtr<FJsonObject> JsonValueObject = InJsonValue->AsObject();
		const bool bAreChildKeysSearchable = bIsDefaultValueKey && ShouldSearchChildKeys(JsonValueObject);

		ECategorySectionHelperFlags CategoryFlags = ECategorySectionHelperFlags::None;
		if (bIsTagAndValueCategory)
		{
			CategoryFlags |= ECategorySectionHelperFlags::IsTagAndValue;
		}
		if (bAreChildKeysSearchable)
		{
			CategoryFlags |= ECategorySectionHelperFlags::AreKeysSearchable;
		}

		TSharedPtr< FCategorySectionHelper, ESPMode::ThreadSafe > SubObjectCategory = MakeShareable(new FCategorySectionHelper(AsShared(), JsonValueObject, LookupTablePtr, InDisplayKey, CategoryFlags));
		SubObjectCategory->ParseAllChildData(SearchabilityStatus);
		ParsedChildData.Add(SubObjectCategory);
	}
	else
	{
		FSearchableValueInfo& ParsedValue = OutParsedValues.AddDefaulted_GetRef();
		if (InJsonValue->Type == EJson::String)
		{
			ParsedValue = FSearchableValueInfo(InDisplayKey, FCString::Atoi(*InJsonValue->AsString()), SearchabilityStatus);
		}
		else
		{
			// For everything else, there's this. Numbers come here and will be treated as strings
			ParsedValue = FSearchableValueInfo(InDisplayKey, FText::FromString(InJsonValue->AsString()), SearchabilityStatus);
		}

		ParsedTagsAndValues.Add(FindInBlueprintsHelpers::FSimpleFTextKeyStorage(InKey), ParsedValue);
	}
}

FText FImaginaryFiBData::CreateSearchComponentDisplayText(FText InKey, FText InValue) const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("Key"), InKey);
	Args.Add(TEXT("Value"), InValue);
	return FText::Format(LOCTEXT("ExtraSearchInfo", "{Key}: {Value}"), Args);
}

bool FImaginaryFiBData::TestBasicStringExpression(const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode, TMultiMap< const FImaginaryFiBData*, FComponentUniqueDisplay >& InOutMatchingSearchComponents) const
{
	bool bMatchesSearchQuery = false;
	for(const TPair< FindInBlueprintsHelpers::FSimpleFTextKeyStorage, FSearchableValueInfo >& ParsedValues : ParsedTagsAndValues )
	{
		if (ParsedValues.Value.IsSearchable() && !ParsedValues.Value.IsExplicitSearchable())
		{
			FText Value = ParsedValues.Value.GetDisplayText(*LookupTablePtr);
			FString ValueAsString = Value.ToString();
			ValueAsString.ReplaceInline(TEXT(" "), TEXT(""));
			bool bMatchesSearch = TextFilterUtils::TestBasicStringExpression(MoveTemp(ValueAsString), InValue, InTextComparisonMode) || TextFilterUtils::TestBasicStringExpression(Value.BuildSourceString(), InValue, InTextComparisonMode);
			
			if (bMatchesSearch && !ParsedValues.Value.IsCoreDisplay())
			{
				FSearchResult SearchResult = MakeShared<FFindInBlueprintsResult>(CreateSearchComponentDisplayText(ParsedValues.Value.GetDisplayKey(), Value));
				InOutMatchingSearchComponents.Add(this, FComponentUniqueDisplay(SearchResult));
			}

			bMatchesSearchQuery |= bMatchesSearch;
		}
	}
	// Any children that are treated as a TagAndValue Category should be added for independent searching
	for (const FImaginaryFiBDataSharedPtr& Child : ParsedChildData)
	{
		if (Child->IsTagAndValueCategory())
		{
			bMatchesSearchQuery |= Child->TestBasicStringExpression(InValue, InTextComparisonMode, InOutMatchingSearchComponents);
		}
	}

	return bMatchesSearchQuery;
}

bool FImaginaryFiBData::TestComplexExpression(const FName& InKey, const FTextFilterString& InValue, const ETextFilterComparisonOperation InComparisonOperation, const ETextFilterTextComparisonMode InTextComparisonMode, TMultiMap< const FImaginaryFiBData*, FComponentUniqueDisplay >& InOutMatchingSearchComponents) const
{
	bool bMatchesSearchQuery = false;
	for (const TPair< FindInBlueprintsHelpers::FSimpleFTextKeyStorage, FSearchableValueInfo >& TagsValuePair : ParsedTagsAndValues)
	{
		if (TagsValuePair.Value.IsSearchable())
		{
			if (TagsValuePair.Key.Text.ToString() == InKey.ToString() || TagsValuePair.Key.Text.BuildSourceString() == InKey.ToString())
			{
				FText Value = TagsValuePair.Value.GetDisplayText(*LookupTablePtr);
				FString ValueAsString = Value.ToString();
				ValueAsString.ReplaceInline(TEXT(" "), TEXT(""));
				bool bMatchesSearch = TextFilterUtils::TestComplexExpression(MoveTemp(ValueAsString), InValue, InComparisonOperation, InTextComparisonMode) || TextFilterUtils::TestComplexExpression(Value.BuildSourceString(), InValue, InComparisonOperation, InTextComparisonMode);

				if (bMatchesSearch && !TagsValuePair.Value.IsCoreDisplay())
				{
					FSearchResult SearchResult = MakeShared<FFindInBlueprintsResult>(CreateSearchComponentDisplayText(TagsValuePair.Value.GetDisplayKey(), Value));
					InOutMatchingSearchComponents.Add(this, FComponentUniqueDisplay(SearchResult));
				}
				bMatchesSearchQuery |= bMatchesSearch;
			}
		}
	}

	// Any children that are treated as a TagAndValue Category should be added for independent searching
	for (const FImaginaryFiBDataSharedPtr& Child : ParsedChildData)
	{
		if (Child->IsTagAndValueCategory())
		{
			bMatchesSearchQuery |= Child->TestComplexExpression(InKey, InValue, InComparisonOperation, InTextComparisonMode, InOutMatchingSearchComponents);
		}
	}
	return bMatchesSearchQuery;
}

UObject* FImaginaryFiBData::GetObject(UBlueprint* InBlueprint) const
{
	return CreateSearchResult(nullptr)->GetObject(InBlueprint);
}

void FImaginaryFiBData::DumpParsedObject(FArchive& Ar, int32 InTreeLevel) const
{
	FString CommaStr = TEXT(",");
	for (int32 i = 0; i < InTreeLevel; ++i)
	{
		Ar.Serialize(TCHAR_TO_ANSI(*CommaStr), CommaStr.Len());
	}

	DumpParsedObject_Internal(Ar);

	for (const TPair< FindInBlueprintsHelpers::FSimpleFTextKeyStorage, FSearchableValueInfo >& TagsValuePair : ParsedTagsAndValues)
	{
		FText Value = TagsValuePair.Value.GetDisplayText(*LookupTablePtr);
		FString ValueAsString = Value.ToString();
		ValueAsString.ReplaceInline(TEXT(" "), TEXT(""));

		FString LineStr = FString::Printf(TEXT(",%s:%s"), *TagsValuePair.Key.Text.ToString(), *ValueAsString);
		Ar.Serialize(TCHAR_TO_ANSI(*LineStr), LineStr.Len());
	}

	FString NewLine(TEXT("\n"));
	Ar.Serialize(TCHAR_TO_ANSI(*NewLine), NewLine.Len());

	for (const FImaginaryFiBDataSharedPtr& Child : ParsedChildData)
	{
		Child->DumpParsedObject(Ar, InTreeLevel + 1);
	}

	if (InTreeLevel == 0)
	{
		Ar.Serialize(TCHAR_TO_ANSI(*NewLine), NewLine.Len());
	}
}

///////////////////////////
// FFiBMetaData

FFiBMetaData::FFiBMetaData(FImaginaryFiBDataWeakPtr InOuter, TSharedPtr< FJsonObject > InUnparsedJsonObject, TMap<int32, FText>* InLookupTablePtr)
	: FImaginaryFiBData(InOuter, InUnparsedJsonObject, InLookupTablePtr)
	, bIsHidden(false)
	, bIsExplicit(false)
{
}

bool FFiBMetaData::TrySpecialHandleJsonValue(FText InKey, TSharedPtr< FJsonValue > InJsonValue)
{
	bool bResult = false;
	if (InKey.ToString() == FFiBMD::FiBSearchableExplicitMD)
	{
		bIsExplicit = true;
		bResult = true;
	}
	else if (InKey.ToString() == FFiBMD::FiBSearchableHiddenExplicitMD)
	{
		bIsExplicit = true;
		bIsHidden = true;
		bResult = true;
	}
	ensure(bResult);
	return bResult;
}

///////////////////////////
// FCategorySectionHelper

FCategorySectionHelper::FCategorySectionHelper(FImaginaryFiBDataWeakPtr InOuter, TMap<int32, FText>* InLookupTablePtr, FText InCategoryName, bool bInTagAndValueCategory)
	: FImaginaryFiBData(InOuter, nullptr, InLookupTablePtr)
	, CategoryName(InCategoryName)
	, bIsTagAndValue(bInTagAndValueCategory)
	, bAreKeysSearchable(false)
{
}

FCategorySectionHelper::FCategorySectionHelper(FImaginaryFiBDataWeakPtr InOuter, TSharedPtr< FJsonObject > InUnparsedJsonObject, TMap<int32, FText>* InLookupTablePtr, FText InCategoryName, bool bInTagAndValueCategory)
	: FImaginaryFiBData(InOuter, InUnparsedJsonObject, InLookupTablePtr)
	, CategoryName(InCategoryName)
	, bIsTagAndValue(bInTagAndValueCategory)
	, bAreKeysSearchable(false)
{
}

FCategorySectionHelper::FCategorySectionHelper(FImaginaryFiBDataWeakPtr InOuter, TSharedPtr< FJsonObject > InUnparsedJsonObject, TMap<int32, FText>* InLookupTablePtr, FText InCategoryName, bool bInTagAndValueCategory, FCategorySectionHelperCallback InSpecialHandlingCallback)
	: FImaginaryFiBData(InOuter, InUnparsedJsonObject, InLookupTablePtr)
	, SpecialHandlingCallback(InSpecialHandlingCallback)
	, CategoryName(InCategoryName)
	, bIsTagAndValue(bInTagAndValueCategory)
	, bAreKeysSearchable(false)
{
}

FCategorySectionHelper::FCategorySectionHelper(FImaginaryFiBDataWeakPtr InOuter, TSharedPtr< FJsonObject > InUnparsedJsonObject, TMap<int32, FText>* InLookupTablePtr, FText InCategoryName, ECategorySectionHelperFlags Flags)
	: FImaginaryFiBData(InOuter, InUnparsedJsonObject, InLookupTablePtr)
	, CategoryName(InCategoryName)
	, bIsTagAndValue((Flags & ECategorySectionHelperFlags::IsTagAndValue) != ECategorySectionHelperFlags::None)
	, bAreKeysSearchable((Flags & ECategorySectionHelperFlags::AreKeysSearchable) != ECategorySectionHelperFlags::None)
{
}

bool FCategorySectionHelper::CanCallFilter(ESearchQueryFilter InSearchQueryFilter) const
{
	return true;
}

FString FCategorySectionHelper::GetCategoryPath() const
{
	const FCategorySectionHelper* OuterCategory = this;
	FString Path;
	do
	{
		// Prepend category
		Path = FString::Printf(TEXT("%s.%s"), *OuterCategory->GetCategoryFunctionName(), *Path);

		// Resolve outer category, or null if the outer isn't a category
		OuterCategory = (OuterCategory->GetOuter().Pin().IsValid() && OuterCategory->GetOuter().Pin()->IsCategory())
			? (FCategorySectionHelper*)(OuterCategory->GetOuter().Pin().Get())
			: nullptr;
	} while (OuterCategory);

	// Remove redundant . at the end
	return Path.LeftChop(1);
}

FSearchResult FCategorySectionHelper::CreateSearchResult_Internal(FSearchResult InTemplate) const
{
	if (InTemplate.IsValid())
	{
		return MakeShared<FFindInBlueprintsResult>(*InTemplate);
	}
	else
	{
		return MakeShared<FFindInBlueprintsResult>(CategoryName);
	}
}

void FCategorySectionHelper::ParseAllChildData_Internal(ESearchableValueStatus InSearchabilityOverride/* = ESearchableValueStatus::Searchable*/)
{
	if (UnparsedJsonObject.IsValid() && SpecialHandlingCallback.IsBound())
	{
		SpecialHandlingCallback.Execute(UnparsedJsonObject, ParsedChildData);
		UnparsedJsonObject.Reset();
	}
	else
	{
		bool bHasMetaData = false;
		bool bHasOneOtherItem = false;
		if (UnparsedJsonObject.IsValid() && UnparsedJsonObject->Values.Num() == 2)
		{
			for(const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& MapValues : UnparsedJsonObject->Values)
			{
				FText KeyText = FindInBlueprintsHelpers::AsFText(FCString::Atoi(*MapValues.Key), *LookupTablePtr);
				if (!KeyText.CompareTo(FFindInBlueprintSearchTags::FiBMetaDataTag))
				{
					bHasMetaData = true;
				}
				else
				{
					bHasOneOtherItem = true;
				}
			}

			// If we have metadata and only one other item, we should be treated like a tag and value category
			bIsTagAndValue |= (bHasOneOtherItem && bHasMetaData);
		}

		FImaginaryFiBData::ParseAllChildData_Internal(InSearchabilityOverride);
	}
}

void FCategorySectionHelper::DumpParsedObject_Internal(FArchive& Ar) const
{
	FString OutputString = FString::Printf(TEXT("FCategorySectionHelper,CategoryName:%s,IsTagAndValueCategory:%s"), *CategoryName.ToString(), IsTagAndValueCategory() ? TEXT("true") : TEXT("false"));
	Ar.Serialize(TCHAR_TO_ANSI(*OutputString), OutputString.Len());
}

//////////////////////////////////////////
// FImaginaryBlueprint

FImaginaryBlueprint::FImaginaryBlueprint(const FString& InBlueprintName, const FString& InBlueprintPath, const FString& InBlueprintParentClass, const TArray<FString>& InInterfaces, const FString& InUnparsedStringData, FSearchDataVersionInfo InVersionInfo)
	: FImaginaryFiBData(nullptr)
	, BlueprintPath(InBlueprintPath)
{
	ParseToJson(InVersionInfo, InUnparsedStringData);
	LookupTablePtr = &LookupTable;
	ParsedTagsAndValues.Add(FindInBlueprintsHelpers::FSimpleFTextKeyStorage(FFindInBlueprintSearchTags::FiB_Name), FSearchableValueInfo(FFindInBlueprintSearchTags::FiB_Name, FText::FromString(InBlueprintName), ESearchableValueStatus::ExplicitySearchable));
	ParsedTagsAndValues.Add(FindInBlueprintsHelpers::FSimpleFTextKeyStorage(FFindInBlueprintSearchTags::FiB_Path), FSearchableValueInfo(FFindInBlueprintSearchTags::FiB_Path, FText::FromString(InBlueprintPath), ESearchableValueStatus::ExplicitySearchable));
	ParsedTagsAndValues.Add(FindInBlueprintsHelpers::FSimpleFTextKeyStorage(FFindInBlueprintSearchTags::FiB_ParentClass), FSearchableValueInfo(FFindInBlueprintSearchTags::FiB_ParentClass, FText::FromString(InBlueprintParentClass), ESearchableValueStatus::ExplicitySearchable));

	TSharedPtr< FCategorySectionHelper, ESPMode::ThreadSafe > InterfaceCategory = MakeShareable(new FCategorySectionHelper(nullptr, &LookupTable, FFindInBlueprintSearchTags::FiB_Interfaces, /*bIsTagAndValueCategory=*/true));
	for( int32 InterfaceIdx = 0; InterfaceIdx < InInterfaces.Num(); ++InterfaceIdx)
	{
		const FString& Interface = InInterfaces[InterfaceIdx];
		FText Key = FText::FromString(FString::FromInt(InterfaceIdx));
		FSearchableValueInfo Value(Key, FText::FromString(Interface), ESearchableValueStatus::ExplicitySearchable);
		InterfaceCategory->AddKeyValuePair(FFindInBlueprintSearchTags::FiB_Interfaces, Value);
	}		
	ParsedChildData.Add(InterfaceCategory);
}

FSearchResult FImaginaryBlueprint::CreateSearchResult_Internal(FSearchResult InTemplate) const
{
	if (InTemplate.IsValid())
	{
		return MakeShared<FFindInBlueprintsResult>(*InTemplate);
	}
	else
	{
		return MakeShared<FFindInBlueprintsResult>(ParsedTagsAndValues.Find(FindInBlueprintsHelpers::FSimpleFTextKeyStorage(FFindInBlueprintSearchTags::FiB_Path))->GetDisplayText(LookupTable));
	}
}

void FImaginaryBlueprint::DumpParsedObject_Internal(FArchive& Ar) const
{
	FString OutputString = FString::Printf(TEXT("FImaginaryBlueprint"));
	Ar.Serialize(TCHAR_TO_ANSI(*OutputString), OutputString.Len());
}

UBlueprint* FImaginaryBlueprint::GetBlueprint() const
{
	return Cast<UBlueprint>(GetObject(nullptr));
}

bool FImaginaryBlueprint::IsCompatibleWithFilter(ESearchQueryFilter InSearchQueryFilter) const
{
	return InSearchQueryFilter == ESearchQueryFilter::AllFilter || InSearchQueryFilter == ESearchQueryFilter::BlueprintFilter;
}

bool FImaginaryBlueprint::CanCallFilter(ESearchQueryFilter InSearchQueryFilter) const
{
	return InSearchQueryFilter == ESearchQueryFilter::NodesFilter ||
		InSearchQueryFilter == ESearchQueryFilter::PinsFilter ||
		InSearchQueryFilter == ESearchQueryFilter::GraphsFilter ||
		InSearchQueryFilter == ESearchQueryFilter::UberGraphsFilter ||
		InSearchQueryFilter == ESearchQueryFilter::FunctionsFilter ||
		InSearchQueryFilter == ESearchQueryFilter::MacrosFilter ||
		InSearchQueryFilter == ESearchQueryFilter::PropertiesFilter ||
		InSearchQueryFilter == ESearchQueryFilter::VariablesFilter ||
		InSearchQueryFilter == ESearchQueryFilter::ComponentsFilter ||
		FImaginaryFiBData::CanCallFilter(InSearchQueryFilter);
}

void FImaginaryBlueprint::ParseToJson(FSearchDataVersionInfo InVersionInfo, const FString& UnparsedStringData)
{
	UnparsedJsonObject = FFindInBlueprintSearchManager::ConvertJsonStringToObject(InVersionInfo, UnparsedStringData, LookupTable);
}

bool FImaginaryBlueprint::TrySpecialHandleJsonValue(FText InKey, TSharedPtr< FJsonValue > InJsonValue)
{
	bool bResult = false;

	if(!InKey.CompareTo(FFindInBlueprintSearchTags::FiB_Properties))
	{
		// Pulls out all properties (variables) for this Blueprint
		TArray<TSharedPtr< FJsonValue > > PropertyList = InJsonValue->AsArray();
		for( TSharedPtr< FJsonValue > PropertyValue : PropertyList )
		{
			ParsedChildData.Add(MakeShareable(new FImaginaryProperty(AsShared(), PropertyValue->AsObject(), &LookupTable)));
		}
		bResult = true;
	}
	else if (!InKey.CompareTo(FFindInBlueprintSearchTags::FiB_Functions))
	{
		ParseGraph(InJsonValue, FFindInBlueprintSearchTags::FiB_Functions.ToString(), GT_Function);
		bResult = true;
	}
	else if (!InKey.CompareTo(FFindInBlueprintSearchTags::FiB_Macros))
	{
		ParseGraph(InJsonValue, FFindInBlueprintSearchTags::FiB_Macros.ToString(), GT_Macro);
		bResult = true;
	}
	else if (!InKey.CompareTo(FFindInBlueprintSearchTags::FiB_UberGraphs))
	{
		ParseGraph(InJsonValue, FFindInBlueprintSearchTags::FiB_UberGraphs.ToString(), GT_Ubergraph);
		bResult = true;
	}
	else if (!InKey.CompareTo(FFindInBlueprintSearchTags::FiB_SubGraphs))
	{
		ParseGraph(InJsonValue, FFindInBlueprintSearchTags::FiB_SubGraphs.ToString(), GT_Ubergraph);
		bResult = true;
	}
	else if (!InKey.CompareTo(FFindInBlueprintSearchTags::FiB_ExtensionGraphs))
	{
		ParseGraph(InJsonValue, FFindInBlueprintSearchTags::FiB_ExtensionGraphs.ToString(), GT_Ubergraph);
		bResult = true;
	}
	else if(!InKey.CompareTo(FFindInBlueprintSearchTags::FiB_Components))
	{
		TArray<TSharedPtr< FJsonValue > > ComponentList = InJsonValue->AsArray();
		TSharedPtr< FJsonObject > ComponentsWrapperObject(new FJsonObject);
		ComponentsWrapperObject->SetField(FFindInBlueprintSearchTags::FiB_Components.ToString(), InJsonValue);
		ParsedChildData.Add(MakeShareable(new FCategorySectionHelper(AsShared(), ComponentsWrapperObject, &LookupTable, FFindInBlueprintSearchTags::FiB_Components, /*bIsTagAndValueCategory=*/false, FCategorySectionHelper::FCategorySectionHelperCallback::CreateRaw(this, &FImaginaryBlueprint::ParseComponents))));
		bResult = true;
	}

	if (!bResult)
	{
		bResult = FImaginaryFiBData::TrySpecialHandleJsonValue(InKey, InJsonValue);
	}
	return bResult;
}

void FImaginaryBlueprint::ParseGraph( TSharedPtr< FJsonValue > InJsonValue, FString InCategoryTitle, EGraphType InGraphType )
{
	TArray<TSharedPtr< FJsonValue > > GraphList = InJsonValue->AsArray();
	for( TSharedPtr< FJsonValue > GraphValue : GraphList )
	{
		ParsedChildData.Add(MakeShareable(new FImaginaryGraph(AsShared(), GraphValue->AsObject(), &LookupTable, InGraphType)));
	}
}

void FImaginaryBlueprint::ParseComponents(TSharedPtr< FJsonObject > InJsonObject, TArray<FImaginaryFiBDataSharedPtr>& OutParsedChildData)
{
	// Pulls out all properties (variables) for this Blueprint
	TArray<TSharedPtr< FJsonValue > > ComponentList = InJsonObject->GetArrayField(FFindInBlueprintSearchTags::FiB_Components.ToString());
	for( TSharedPtr< FJsonValue > ComponentValue : ComponentList )
	{
		OutParsedChildData.Add(MakeShareable(new FImaginaryComponent(AsShared(), ComponentValue->AsObject(), &LookupTable)));
	}
}

//////////////////////////
// FImaginaryGraph

FImaginaryGraph::FImaginaryGraph(FImaginaryFiBDataWeakPtr InOuter, TSharedPtr< FJsonObject > InUnparsedJsonObject, TMap<int32, FText>* InLookupTablePtr, EGraphType InGraphType)
	: FImaginaryFiBData(InOuter, InUnparsedJsonObject, InLookupTablePtr)
	, GraphType(InGraphType)
{
}

FSearchResult FImaginaryGraph::CreateSearchResult_Internal(FSearchResult InTemplate) const
{
	if (InTemplate.IsValid())
	{
		return MakeShared<FFindInBlueprintsGraph>(*StaticCastSharedPtr<FFindInBlueprintsGraph>(InTemplate));
	}
	else
	{
		return MakeShared<FFindInBlueprintsGraph>(GraphType);
	}
}

void FImaginaryGraph::DumpParsedObject_Internal(FArchive& Ar) const
{
	FString OutputString = FString::Printf(TEXT("FImaginaryGraph"));
	Ar.Serialize(TCHAR_TO_ANSI(*OutputString), OutputString.Len());
}

bool FImaginaryGraph::IsCompatibleWithFilter(ESearchQueryFilter InSearchQueryFilter) const
{
	return InSearchQueryFilter == ESearchQueryFilter::AllFilter || 
		InSearchQueryFilter == ESearchQueryFilter::GraphsFilter ||
		(GraphType == GT_Ubergraph && InSearchQueryFilter == ESearchQueryFilter::UberGraphsFilter) ||
		(GraphType == GT_Function && InSearchQueryFilter == ESearchQueryFilter::FunctionsFilter) ||
		(GraphType == GT_Macro && InSearchQueryFilter == ESearchQueryFilter::MacrosFilter);
}

bool FImaginaryGraph::CanCallFilter(ESearchQueryFilter InSearchQueryFilter) const
{
	return InSearchQueryFilter == ESearchQueryFilter::PinsFilter ||
		InSearchQueryFilter == ESearchQueryFilter::NodesFilter ||
		(GraphType == GT_Function && InSearchQueryFilter == ESearchQueryFilter::PropertiesFilter) ||
		(GraphType == GT_Function && InSearchQueryFilter == ESearchQueryFilter::VariablesFilter) ||
		FImaginaryFiBData::CanCallFilter(InSearchQueryFilter);
}

ESearchableValueStatus FImaginaryGraph::GetSearchabilityStatus(FString InKey)
{
	// This is a non-ideal way to assign searchability vs being a core display item and will be resolved in future versions of the FiB data in the AR
	if (FindInBlueprintsHelpers::IsTextEqualToString(FFindInBlueprintSearchTags::FiB_Name, InKey)
		|| FindInBlueprintsHelpers::IsTextEqualToString(FFindInBlueprintSearchTags::FiB_NativeName, InKey)
		)
	{
		return ESearchableValueStatus::CoreDisplayItem;
	}

	return ESearchableValueStatus::Searchable;
}

bool FImaginaryGraph::TrySpecialHandleJsonValue(FText InKey, TSharedPtr< FJsonValue > InJsonValue)
{
	if (!InKey.CompareTo(FFindInBlueprintSearchTags::FiB_Nodes))
	{
		TArray< TSharedPtr< FJsonValue > > NodeList = InJsonValue->AsArray();

		for( TSharedPtr< FJsonValue > NodeValue : NodeList )
		{
			ParsedChildData.Add(MakeShareable(new FImaginaryGraphNode(AsShared(), NodeValue->AsObject(), LookupTablePtr)));
		}
		return true;
	}
	else if (!InKey.CompareTo(FFindInBlueprintSearchTags::FiB_Properties))
	{
		// Pulls out all properties (local variables) for this graph
		TArray<TSharedPtr< FJsonValue > > PropertyList = InJsonValue->AsArray();
		for( TSharedPtr< FJsonValue > PropertyValue : PropertyList )
		{
			ParsedChildData.Add(MakeShareable(new FImaginaryProperty(AsShared(), PropertyValue->AsObject(), LookupTablePtr)));
		}
		return true;
	}
	return false;
}

//////////////////////////////////////
// FImaginaryGraphNode

FImaginaryGraphNode::FImaginaryGraphNode(FImaginaryFiBDataWeakPtr InOuter, TSharedPtr< FJsonObject > InUnparsedJsonObject, TMap<int32, FText>* InLookupTablePtr)
	: FImaginaryFiBData(InOuter, InUnparsedJsonObject, InLookupTablePtr)
{
}

FSearchResult FImaginaryGraphNode::CreateSearchResult_Internal(FSearchResult InTemplate) const
{
	if (InTemplate.IsValid())
	{
		return MakeShared<FFindInBlueprintsGraphNode>(*StaticCastSharedPtr<FFindInBlueprintsGraphNode>(InTemplate));
	}
	else
	{
		return MakeShared<FFindInBlueprintsGraphNode>();
	}
}

void FImaginaryGraphNode::DumpParsedObject_Internal(FArchive& Ar) const
{
	FString OutputString = FString::Printf(TEXT("FImaginaryGraphNode"));
	Ar.Serialize(TCHAR_TO_ANSI(*OutputString), OutputString.Len());
}

bool FImaginaryGraphNode::IsCompatibleWithFilter(ESearchQueryFilter InSearchQueryFilter) const
{
	return InSearchQueryFilter == ESearchQueryFilter::AllFilter || InSearchQueryFilter == ESearchQueryFilter::NodesFilter;
}

bool FImaginaryGraphNode::CanCallFilter(ESearchQueryFilter InSearchQueryFilter) const
{
	return InSearchQueryFilter == ESearchQueryFilter::PinsFilter ||
		FImaginaryFiBData::CanCallFilter(InSearchQueryFilter);
}

ESearchableValueStatus FImaginaryGraphNode::GetSearchabilityStatus(FString InKey)
{
	// This is a non-ideal way to assign searchability vs being a core display item and will be resolved in future versions of the FiB data in the AR
	if (FindInBlueprintsHelpers::IsTextEqualToString(FFindInBlueprintSearchTags::FiB_Name, InKey)
		|| FindInBlueprintsHelpers::IsTextEqualToString(FFindInBlueprintSearchTags::FiB_NativeName, InKey)
		|| FindInBlueprintsHelpers::IsTextEqualToString(FFindInBlueprintSearchTags::FiB_Comment, InKey)
		)
	{
		return ESearchableValueStatus::CoreDisplayItem;
	}
	else if (FindInBlueprintsHelpers::IsTextEqualToString(FFindInBlueprintSearchTags::FiB_Glyph, InKey)
		|| FindInBlueprintsHelpers::IsTextEqualToString(FFindInBlueprintSearchTags::FiB_GlyphStyleSet, InKey)
		|| FindInBlueprintsHelpers::IsTextEqualToString(FFindInBlueprintSearchTags::FiB_GlyphColor, InKey)
		|| FindInBlueprintsHelpers::IsTextEqualToString(FFindInBlueprintSearchTags::FiB_NodeGuid, InKey)
		)
	{
		return ESearchableValueStatus::NotSearchable;
	}
	else if (FindInBlueprintsHelpers::IsTextEqualToString(FFindInBlueprintSearchTags::FiB_ClassName, InKey))
	{
		return ESearchableValueStatus::ExplicitySearchable;
	}
	return ESearchableValueStatus::Searchable;
}

bool FImaginaryGraphNode::TrySpecialHandleJsonValue(FText InKey, TSharedPtr< FJsonValue > InJsonValue)
{
	if (!InKey.CompareTo(FFindInBlueprintSearchTags::FiB_Pins))
	{
		TArray< TSharedPtr< FJsonValue > > PinsList = InJsonValue->AsArray();

		for( TSharedPtr< FJsonValue > Pin : PinsList )
		{
			ParsedChildData.Add(MakeShareable(new FImaginaryPin(AsShared(), Pin->AsObject(), LookupTablePtr, SchemaName)));
		}
		return true;
	}
	else if (!InKey.CompareTo(FFindInBlueprintSearchTags::FiB_SchemaName))
	{
		// Previously extracted
		return true;
	}

	return false;
}

void FImaginaryGraphNode::ParseAllChildData_Internal(ESearchableValueStatus InSearchabilityOverride/* = ESearchableValueStatus::Searchable*/)
{
	if (UnparsedJsonObject.IsValid())
	{
		TSharedPtr< FJsonObject > JsonObject = UnparsedJsonObject;
		// Very important to get the schema first, other bits of data depend on it
		for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& MapValues : UnparsedJsonObject->Values)
		{
			FText KeyText = FindInBlueprintsHelpers::AsFText(FCString::Atoi(*MapValues.Key), *LookupTablePtr);
			if (!KeyText.CompareTo(FFindInBlueprintSearchTags::FiB_SchemaName))
			{
				TSharedPtr< FJsonValue > SchemaNameValue = MapValues.Value;
				SchemaName = FindInBlueprintsHelpers::AsFText(SchemaNameValue, *LookupTablePtr).ToString();
				break;
			}
		}

		FImaginaryFiBData::ParseAllChildData_Internal(InSearchabilityOverride);
	}
}

///////////////////////////////////////////
// FImaginaryProperty

FImaginaryProperty::FImaginaryProperty(FImaginaryFiBDataWeakPtr InOuter, TSharedPtr< FJsonObject > InUnparsedJsonObject, TMap<int32, FText>* InLookupTablePtr)
	: FImaginaryFiBData(InOuter, InUnparsedJsonObject, InLookupTablePtr)
	, bIsStructProperty(false)
{
}

bool FImaginaryProperty::IsCompatibleWithFilter(ESearchQueryFilter InSearchQueryFilter) const
{
	return InSearchQueryFilter == ESearchQueryFilter::AllFilter || 
		InSearchQueryFilter == ESearchQueryFilter::PropertiesFilter || 
		InSearchQueryFilter == ESearchQueryFilter::VariablesFilter;
}

void FImaginaryProperty::ParseAllChildData_Internal(ESearchableValueStatus InSearchabilityOverride)
{
	// Detect and cache whether PinCategory == PC_Struct. If so, it can represent:
	// a) a struct, or
	// b) a TMap whose key type is a struct
	// The purpose for caching this is to detect TMap with struct keys, which must be searchable.
	// We don't explicitly save whether some search metadata represents a TMap default value.
	for (auto MapValues : UnparsedJsonObject->Values)
	{
		FText KeyText = FindInBlueprintsHelpers::AsFText(FCString::Atoi(*MapValues.Key), *LookupTablePtr);
		if (KeyText.EqualTo(FFindInBlueprintSearchTags::FiB_PinCategory))
		{
			const FText PinCategoryText = FindInBlueprintsHelpers::AsFText(FCString::Atoi(*MapValues.Value->AsString()), *LookupTablePtr);
			static FText PinCategoryStructValue = FText::FromName(UEdGraphSchema_K2::PC_Struct);
			bIsStructProperty = PinCategoryText.EqualTo(PinCategoryStructValue);
		}
	}

	FImaginaryFiBData::ParseAllChildData_Internal(InSearchabilityOverride);
}

bool FImaginaryProperty::ShouldSearchChildKeys(TSharedPtr<FJsonObject> ChildNode) const
{
	// There are two scenarios where we want to search child keys:
	// - This is a TMap with a struct key type
	// - This is a TMap with a plain-old-data (POD) key type
	// In both cases ChildNode represents a key-value pair where we want to search the keys.
	if (bIsStructProperty)
	{
		// Entering here implies this FProperty is either a struct or a TMap with a struct key. Then:
		// - For structs, child nodes represent member properties where the key is the member name like 'MyMemberProperty'.
		// - For TMaps, child nodes represent key-value pairs where the keys store the struct's FStructProperty::ExportText output like '(MyMemberProperty=111)'.
		// Detect the latter case by peaking at the first element's syntax. This is not great, but implementing
		// TMap key searchability isn't worth changing the metadata format for.
		return !ChildNode->Values.IsEmpty() && FindInBlueprintsHelpers::AsFText(FCString::Atoi(*ChildNode->Values.Array()[0].Key), *LookupTablePtr).ToString().StartsWith("(");
	}

	// Entering here implies this is a TMap with a POD key type, because actual 
	// POD FProperties don't have FJsonObject child nodes.
	return true;
}

FSearchResult FImaginaryProperty::CreateSearchResult_Internal(FSearchResult InTemplate) const
{
	if (InTemplate.IsValid())
	{
		return MakeShared<FFindInBlueprintsProperty>(*StaticCastSharedPtr<FFindInBlueprintsProperty>(InTemplate));
	}
	else
	{
		return MakeShared<FFindInBlueprintsProperty>();
	}
}

void FImaginaryProperty::DumpParsedObject_Internal(FArchive& Ar) const
{
	FString OutputString = FString::Printf(TEXT("FImaginaryProperty"));
	Ar.Serialize(TCHAR_TO_ANSI(*OutputString), OutputString.Len());
}

ESearchableValueStatus FImaginaryProperty::GetSearchabilityStatus(FString InKey)
{
	// This is a non-ideal way to assign searchability vs being a core display item and will be resolved in future versions of the FiB data in the AR
	if (FindInBlueprintsHelpers::IsTextEqualToString(FFindInBlueprintSearchTags::FiB_Name, InKey)
		|| FindInBlueprintsHelpers::IsTextEqualToString(FFindInBlueprintSearchTags::FiB_NativeName, InKey)
		)
	{
		return ESearchableValueStatus::CoreDisplayItem;
	}
	else if (FindInBlueprintsHelpers::IsTextEqualToString(FFindInBlueprintSearchTags::FiB_PinCategory, InKey)
		|| FindInBlueprintsHelpers::IsTextEqualToString(FFindInBlueprintSearchTags::FiB_PinSubCategory, InKey)
		|| FindInBlueprintsHelpers::IsTextEqualToString(FFindInBlueprintSearchTags::FiB_ObjectClass, InKey)
		|| FindInBlueprintsHelpers::IsTextEqualToString(FFindInBlueprintSearchTags::FiB_IsArray, InKey)
		|| FindInBlueprintsHelpers::IsTextEqualToString(FFindInBlueprintSearchTags::FiB_IsReference, InKey)
		|| FindInBlueprintsHelpers::IsTextEqualToString(FFindInBlueprintSearchTags::FiB_IsSCSComponent, InKey)
		)
	{
		return ESearchableValueStatus::ExplicitySearchableHidden;
	}
	return ESearchableValueStatus::Searchable;
}

//////////////////////////////
// FImaginaryComponent

FImaginaryComponent::FImaginaryComponent(FImaginaryFiBDataWeakPtr InOuter, TSharedPtr< FJsonObject > InUnparsedJsonObject, TMap<int32, FText>* InLookupTablePtr)
	: FImaginaryProperty(InOuter, InUnparsedJsonObject, InLookupTablePtr)
{
}

bool FImaginaryComponent::IsCompatibleWithFilter(ESearchQueryFilter InSearchQueryFilter) const
{
	return FImaginaryProperty::IsCompatibleWithFilter(InSearchQueryFilter) || InSearchQueryFilter == ESearchQueryFilter::ComponentsFilter;
}

//////////////////////////////
// FImaginaryPin

FImaginaryPin::FImaginaryPin(FImaginaryFiBDataWeakPtr InOuter, TSharedPtr< FJsonObject > InUnparsedJsonObject, TMap<int32, FText>* InLookupTablePtr, FString InSchemaName)
	: FImaginaryFiBData(InOuter, InUnparsedJsonObject, InLookupTablePtr)
	, SchemaName(InSchemaName)
{
}

bool FImaginaryPin::IsCompatibleWithFilter(ESearchQueryFilter InSearchQueryFilter) const
{
	return InSearchQueryFilter == ESearchQueryFilter::AllFilter || InSearchQueryFilter == ESearchQueryFilter::PinsFilter;
}

FSearchResult FImaginaryPin::CreateSearchResult_Internal(FSearchResult InTemplate) const
{
	if (InTemplate.IsValid())
	{
		return MakeShared<FFindInBlueprintsPin>(*StaticCastSharedPtr<FFindInBlueprintsPin>(InTemplate));
	}
	else
	{
		return MakeShared<FFindInBlueprintsPin>(SchemaName);
	}
}

void FImaginaryPin::DumpParsedObject_Internal(FArchive& Ar) const
{
	FString OutputString = FString::Printf(TEXT("FImaginaryPin"));
	Ar.Serialize(TCHAR_TO_ANSI(*OutputString), OutputString.Len());
}

ESearchableValueStatus FImaginaryPin::GetSearchabilityStatus(FString InKey)
{
	if (FindInBlueprintsHelpers::IsTextEqualToString(FFindInBlueprintSearchTags::FiB_Name, InKey)
		|| FindInBlueprintsHelpers::IsTextEqualToString(FFindInBlueprintSearchTags::FiB_NativeName, InKey)
		)
	{
		return ESearchableValueStatus::CoreDisplayItem;
	}
	else if (FindInBlueprintsHelpers::IsTextEqualToString(FFindInBlueprintSearchTags::FiB_PinCategory, InKey)
		|| FindInBlueprintsHelpers::IsTextEqualToString(FFindInBlueprintSearchTags::FiB_PinSubCategory, InKey)
		|| FindInBlueprintsHelpers::IsTextEqualToString(FFindInBlueprintSearchTags::FiB_ObjectClass, InKey)
		|| FindInBlueprintsHelpers::IsTextEqualToString(FFindInBlueprintSearchTags::FiB_IsArray, InKey)
		|| FindInBlueprintsHelpers::IsTextEqualToString(FFindInBlueprintSearchTags::FiB_IsReference, InKey)
		|| FindInBlueprintsHelpers::IsTextEqualToString(FFindInBlueprintSearchTags::FiB_IsSCSComponent, InKey)
		)
	{
		return ESearchableValueStatus::ExplicitySearchableHidden;
	}
	return ESearchableValueStatus::Searchable;
}

bool FImaginaryPin::TrySpecialHandleJsonValue(FText InKey, TSharedPtr< FJsonValue > InJsonValue)
{
	return false;
}

#undef LOCTEXT_NAMESPACE
