// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedStaticMeshSCSEditorCustomization.h"

#include "BlueprintEditorModule.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Components/SceneComponent.h"
#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "EditorViewportClient.h"
#include "HitProxies.h"
#include "Math/Quat.h"
#include "Math/RotationMatrix.h"
#include "Math/ScaleMatrix.h"
#include "Math/Transform.h"
#include "Math/Vector.h"
#include "Math/Vector4.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "UObject/Object.h"
#include "UnrealClient.h"

TSharedRef<ISCSEditorCustomization> FInstancedStaticMeshSCSEditorCustomization::MakeInstance(TSharedRef< IBlueprintEditor > InBlueprintEditor)
{
	TSharedRef<FInstancedStaticMeshSCSEditorCustomization> NewCustomization = MakeShareable(new FInstancedStaticMeshSCSEditorCustomization());
	NewCustomization->BlueprintEditorPtr = InBlueprintEditor;
	return NewCustomization;
}

void FInstancedStaticMeshSCSEditorCustomization::ValidateSelectedInstances(UInstancedStaticMeshComponent* InComponent)
{
	check(InComponent);

	// @TODO - revisit; this might be better done as post-edit handling in the InstancedStaticMesh class itself.
	// Ensure that the number of selection bits matches the number of instances; if not, it's likely that the user has
	// just added or removed an instance, in which case we'll reset the selection to select the last instance in the list.
	if(InComponent->SelectedInstances.Num() != InComponent->PerInstanceSMData.Num())
	{
		InComponent->SelectedInstances.Init(false, InComponent->PerInstanceSMData.Num());
		if(InComponent->PerInstanceSMData.Num() > 0)
		{
			InComponent->SelectInstance(true, InComponent->PerInstanceSMData.Num() - 1);
			InComponent->MarkRenderStateDirty();
		}
	}
}

bool FInstancedStaticMeshSCSEditorCustomization::HandleViewportClick(const TSharedRef<FEditorViewportClient>& InViewportClient, class FSceneView& InView, class HHitProxy* InHitProxy, FKey InKey, EInputEvent InEvent, uint32 InHitX, uint32 InHitY)
{
	if (InHitProxy != NULL && InHitProxy->IsA(HInstancedStaticMeshInstance::StaticGetType()))
	{
		HInstancedStaticMeshInstance* InstancedStaticMeshInstanceProxy = static_cast<HInstancedStaticMeshInstance*>(InHitProxy);

		const bool bIsCtrlKeyDown = InViewportClient->Viewport->KeyState(EKeys::LeftControl) || InViewportClient->Viewport->KeyState(EKeys::RightControl);
		const bool bIsAltKeyDown = InViewportClient->Viewport->KeyState(EKeys::LeftAlt);
		const bool bCurrentSelection = InstancedStaticMeshInstanceProxy->Component->IsInstanceSelected(InstancedStaticMeshInstanceProxy->InstanceIndex);
		if (!bIsAltKeyDown)
		{
			InstancedStaticMeshInstanceProxy->Component->SelectInstance(false, 0, InstancedStaticMeshInstanceProxy->Component->PerInstanceSMData.Num());
		}
		InstancedStaticMeshInstanceProxy->Component->SelectInstance(bIsAltKeyDown ? !bCurrentSelection : true, InstancedStaticMeshInstanceProxy->InstanceIndex);
		InstancedStaticMeshInstanceProxy->Component->MarkRenderStateDirty();

		if (BlueprintEditorPtr.IsValid())
		{
			// Note: This will find and select any node associated with the component instance that's attached to the proxy (including visualizers)
			BlueprintEditorPtr.Pin()->FindAndSelectSubobjectEditorTreeNode(InstancedStaticMeshInstanceProxy->Component, bIsCtrlKeyDown);
		}

		return true;
	}

	return false;
}

bool FInstancedStaticMeshSCSEditorCustomization::HandleViewportDrag(const USceneComponent* InSceneComponent, USceneComponent* InComponentTemplate, const FVector& InDeltaTranslation, const FRotator& InDeltaRotation, const FVector& InDeltaScale, const FVector& InPivot)
{
	check(InSceneComponent->IsA(UInstancedStaticMeshComponent::StaticClass()));

	UInstancedStaticMeshComponent* InstancedStaticMeshComponentScene = const_cast<UInstancedStaticMeshComponent*>(CastChecked<const UInstancedStaticMeshComponent>(InSceneComponent));
	UInstancedStaticMeshComponent* InstancedStaticMeshComponentTemplate = const_cast<UInstancedStaticMeshComponent*>(CastChecked<const UInstancedStaticMeshComponent>(InComponentTemplate));

	// transform pivot into component's space
	const FTransform ComponentToWorldTransform = InstancedStaticMeshComponentScene->GetComponentToWorld();
	const FVector LocalPivot = ComponentToWorldTransform.InverseTransformPosition(InPivot);

	// Ensure that selected instances are up-to-date
	ValidateSelectedInstances(InstancedStaticMeshComponentScene);

	bool bMovedInstance = false;
	check(InstancedStaticMeshComponentScene->SelectedInstances.Num() == InstancedStaticMeshComponentScene->PerInstanceSMData.Num());
	for(int32 InstanceIndex = 0; InstanceIndex < InstancedStaticMeshComponentScene->SelectedInstances.Num(); InstanceIndex++)
	{
		if (InstancedStaticMeshComponentScene->SelectedInstances[InstanceIndex] && InstancedStaticMeshComponentTemplate->PerInstanceSMData.IsValidIndex(InstanceIndex))
		{
			const FMatrix& MatrixScene = InstancedStaticMeshComponentScene->PerInstanceSMData[InstanceIndex].Transform;

			FVector Translation = MatrixScene.GetOrigin();
			FRotator Rotation = MatrixScene.Rotator();
			FVector Scale = MatrixScene.GetScaleVector();

			FVector NewTranslation = Translation;
			FRotator NewRotation = Rotation;
			FVector NewScale = Scale;

			if( !InDeltaRotation.IsZero() )
			{
				NewRotation = FRotator( InDeltaRotation.Quaternion() * Rotation.Quaternion() );

				NewTranslation -= LocalPivot;
				NewTranslation = FRotationMatrix( InDeltaRotation ).TransformPosition( NewTranslation );
				NewTranslation += LocalPivot;
			}

			//transform translation from world space into to component space
			NewTranslation += ComponentToWorldTransform.InverseTransformVector(InDeltaTranslation);

			if( !InDeltaScale.IsNearlyZero() )
			{
				const FScaleMatrix ScaleMatrix( InDeltaScale );

				FVector DeltaScale3D = ScaleMatrix.TransformPosition( Scale );
				NewScale = Scale + DeltaScale3D;

				NewTranslation -= LocalPivot;
				NewTranslation += ScaleMatrix.TransformPosition( NewTranslation );
				NewTranslation += LocalPivot;
			}

			FMatrix& DefaultValue = InstancedStaticMeshComponentTemplate->PerInstanceSMData[InstanceIndex].Transform;
			const FTransform NewTransform(NewRotation, NewTranslation, NewScale);
			InstancedStaticMeshComponentScene->UpdateInstanceTransform(InstanceIndex, NewTransform);

			// Propagate the change to all other instances of the template.
			TArray<UObject*> ArchetypeInstances;
			InstancedStaticMeshComponentTemplate->GetArchetypeInstances(ArchetypeInstances);
			for (UObject* ArchetypeInstance : ArchetypeInstances)
			{
				UInstancedStaticMeshComponent* InstancedStaticMeshComponent = CastChecked<UInstancedStaticMeshComponent>(ArchetypeInstance);
				if (InstancedStaticMeshComponent->PerInstanceSMData.IsValidIndex(InstanceIndex))
				{
					if (InstancedStaticMeshComponent->PerInstanceSMData[InstanceIndex].Transform.Equals(DefaultValue))
					{
						InstancedStaticMeshComponent->UpdateInstanceTransform(InstanceIndex, NewTransform, false, true, true);
					}
				}
			}

			// Update the template.
			InstancedStaticMeshComponentTemplate->Modify();
			DefaultValue = InstancedStaticMeshComponentScene->PerInstanceSMData[InstanceIndex].Transform;

			bMovedInstance = true;
		}
	}

	return bMovedInstance;
}

bool FInstancedStaticMeshSCSEditorCustomization::HandleGetWidgetLocation(const USceneComponent* InSceneComponent, FVector& OutWidgetLocation)
{
	// location is average of selected instances
	float SelectedInstanceCount = 0.0f;
	FVector AverageLocation = FVector::ZeroVector;
	UInstancedStaticMeshComponent* InstancedStaticMeshComponent = const_cast<UInstancedStaticMeshComponent*>(CastChecked<UInstancedStaticMeshComponent>(InSceneComponent));

	// Ensure that selected instances are up-to-date
	ValidateSelectedInstances(InstancedStaticMeshComponent);

	for (int32 InstanceIndex = 0; InstanceIndex < InstancedStaticMeshComponent->SelectedInstances.Num(); InstanceIndex++)
	{
		if (InstancedStaticMeshComponent->SelectedInstances[InstanceIndex])
		{
			AverageLocation += InstancedStaticMeshComponent->GetComponentToWorld().TransformPosition(InstancedStaticMeshComponent->PerInstanceSMData[InstanceIndex].Transform.GetOrigin());
			SelectedInstanceCount += 1.0f;
		}
	}

	if (SelectedInstanceCount > 0.0f)
	{
		OutWidgetLocation = AverageLocation / SelectedInstanceCount;
		return true;
	}

	return false;
}

bool FInstancedStaticMeshSCSEditorCustomization::HandleGetWidgetTransform(const USceneComponent* InSceneComponent, FMatrix& OutWidgetTransform)
{
	// transform is first selected instance
	bool bInstanceSelected = false;
	const UInstancedStaticMeshComponent* InstancedStaticMeshComponent = CastChecked<UInstancedStaticMeshComponent>(InSceneComponent);

	// Ensure that selected instances are up-to-date
	ValidateSelectedInstances(const_cast<UInstancedStaticMeshComponent*>(InstancedStaticMeshComponent));

	for (int32 InstanceIndex = 0; InstanceIndex < InstancedStaticMeshComponent->SelectedInstances.Num(); InstanceIndex++)
	{
		if (InstancedStaticMeshComponent->SelectedInstances[InstanceIndex])
		{
			OutWidgetTransform = FRotationMatrix(InstancedStaticMeshComponent->PerInstanceSMData[InstanceIndex].Transform.Rotator());
			return true;
		}
	}

	return false;
}
