// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineBaseTypes.h"
#include "ISCSEditorCustomization.h"
#include "InputCoreTypes.h"
#include "Math/Matrix.h"
#include "Math/Rotator.h"
#include "Math/UnrealMathSSE.h"
#include "Templates/SharedPointer.h"

class FEditorViewportClient;
class USceneComponent;

class FInstancedStaticMeshSCSEditorCustomization : public ISCSEditorCustomization
{
public:
	static TSharedRef<ISCSEditorCustomization> MakeInstance(TSharedRef< class IBlueprintEditor > InBlueprintEditor);

public:
	/** ISCSEditorCustomization interface */
	virtual bool HandleViewportClick(const TSharedRef<FEditorViewportClient>& InViewportClient, class FSceneView& InView, class HHitProxy* InHitProxy, FKey InKey, EInputEvent InEvent, uint32 InHitX, uint32 InHitY) override;
	virtual bool HandleViewportDrag(const USceneComponent* InComponentScene, class USceneComponent* InComponentTemplate, const FVector& InDeltaTranslation, const FRotator& InDeltaRotation, const FVector& InDeltaScale, const FVector& InPivot) override;
	virtual bool HandleGetWidgetLocation(const USceneComponent* InSceneComponent, FVector& OutWidgetLocation) override;
	virtual bool HandleGetWidgetTransform(const USceneComponent* InSceneComponent, FMatrix& OutWidgetTransform) override;

protected:
	/** Ensure that selection bits are in sync w/ the number of instances */
	void ValidateSelectedInstances(class UInstancedStaticMeshComponent* InComponent);

private:
	/** The blueprint editor we are bound to */
	TWeakPtr<class IBlueprintEditor> BlueprintEditorPtr;
};
