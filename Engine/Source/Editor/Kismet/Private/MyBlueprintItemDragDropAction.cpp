// Copyright Epic Games, Inc. All Rights Reserved.

#include "MyBlueprintItemDragDropAction.h"

#include "BlueprintEditor.h"
#include "CoreTypes.h"
#include "EdGraph/EdGraphSchema.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Styling/AppStyle.h"
#include "UObject/NameTypes.h"

struct FSlateBrush;

#define LOCTEXT_NAMESPACE "FMyBlueprintItemDragDropAction"

FMyBlueprintItemDragDropAction::FMyBlueprintItemDragDropAction()
	: bControlDrag(false)
	, bAltDrag(false)
{
}

FReply FMyBlueprintItemDragDropAction::DroppedOnAction(TSharedRef<FEdGraphSchemaAction> Action)
{
	if (SourceAction.IsValid() && (SourceAction->GetTypeId() == Action->GetTypeId()))
	{
		if (SourceAction->GetPersistentItemDefiningObject() == Action->GetPersistentItemDefiningObject())
		{
			SourceAction->ReorderToBeforeAction(Action);
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

FReply FMyBlueprintItemDragDropAction::DroppedOnCategory(FText Category)
{
	if (SourceAction.IsValid())
	{
		SourceAction->MovePersistentItemToCategory(Category);
	}
	return FReply::Handled();
}

void FMyBlueprintItemDragDropAction::HoverTargetChanged()
{
	if (SourceAction.IsValid())
	{
		if (!HoveredCategoryName.IsEmpty())
		{
			const bool bIsNative = !SourceAction->GetPersistentItemDefiningObject().IsPotentiallyEditable();

			FFormatNamedArguments Args;
			Args.Add(TEXT("DisplayName"), SourceAction->GetMenuDescription());
			Args.Add(TEXT("HoveredCategoryName"), HoveredCategoryName);

			if (bIsNative)
			{
				SetFeedbackMessageError(FText::Format(LOCTEXT("ChangingCatagoryNotEditable", "Cannot change category for '{DisplayName}' because it is declared in C++"), Args));
			}
			else if (HoveredCategoryName.EqualTo(SourceAction->GetCategory()))
			{
				SetFeedbackMessageError(FText::Format(LOCTEXT("ChangingCatagoryAlreadyIn", "'{DisplayName}' is already in category '{HoveredCategoryName}'"), Args));
			}
			else
			{
				SetFeedbackMessageOK(FText::Format(LOCTEXT("ChangingCatagoryOk", "Move '{DisplayName}' to category '{HoveredCategoryName}'"), Args));
			}
			return;
		}
		else if (HoveredAction.IsValid())
		{
			TSharedPtr<FEdGraphSchemaAction> HoveredActionPtr = HoveredAction.Pin();
			FFormatNamedArguments Args;
			Args.Add(TEXT("DraggedDisplayName"), SourceAction->GetMenuDescription());
			Args.Add(TEXT("DropTargetDisplayName"), HoveredActionPtr->GetMenuDescription());

			if (HoveredActionPtr->GetTypeId() == SourceAction->GetTypeId())
			{
				if (SourceAction->GetPersistentItemDefiningObject() == HoveredActionPtr->GetPersistentItemDefiningObject())
				{
					const int32 MovingItemIndex = SourceAction->GetReorderIndexInContainer();
					const int32 TargetVarIndex = HoveredActionPtr->GetReorderIndexInContainer();

					if (MovingItemIndex == INDEX_NONE)
					{
						SetFeedbackMessageError(FText::Format(LOCTEXT("ReorderNonOrderedItem", "Cannot reorder '{DraggedDisplayName}'."), Args));
					}
					else if (TargetVarIndex == INDEX_NONE)
					{
						SetFeedbackMessageError(FText::Format(LOCTEXT("ReorderOntoNonOrderedItem", "Cannot reorder '{DraggedDisplayName}' before '{DropTargetDisplayName}'."), Args));
					}
					else if (HoveredActionPtr == SourceAction)
					{
						SetFeedbackMessageError(FText::Format(LOCTEXT("ReorderOntoSameItem", "Cannot reorder '{DraggedDisplayName}' before itself."), Args));
					}
					else
					{
						SetFeedbackMessageOK(FText::Format(LOCTEXT("ReorderActionOK", "Reorder '{DraggedDisplayName}' before '{DropTargetDisplayName}'"), Args));
					}
				}
				else
				{
					SetFeedbackMessageError(FText::Format(LOCTEXT("ReorderActionDifferentScope", "Cannot reorder '{DraggedDisplayName}' into a different scope."), Args));
				}
			}
			else
			{
				SetFeedbackMessageError(FText::Format(LOCTEXT("ReorderActionDifferentAction", "Cannot reorder '{DraggedDisplayName}' into a different section."), Args));
			}

			return;
		}
	}

	FGraphSchemaActionDragDropAction::HoverTargetChanged();
}

void FMyBlueprintItemDragDropAction::SetFeedbackMessageError(const FText& Message)
{
	const FSlateBrush* StatusSymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
	SetSimpleFeedbackMessage(StatusSymbol, FLinearColor::White, Message);
}

void FMyBlueprintItemDragDropAction::SetFeedbackMessageOK(const FText& Message)
{
	const FSlateBrush* StatusSymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
	SetSimpleFeedbackMessage(StatusSymbol, FLinearColor::White, Message);
}

#undef LOCTEXT_NAMESPACE
