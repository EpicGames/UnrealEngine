// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProjectUtilities/BuildTargetSet.h"

#include "Interfaces/IProjectManager.h"
#include "Interfaces/IPluginManager.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Logging/StructuredLog.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "ProjectDescriptor.h"
#include "IHotReload.h"
#if WITH_LIVE_CODING
#include "ILiveCodingModule.h"
#endif

namespace UE::Private
{
	const UClass* GetNativeClass(const UFunction* ForFunction);
	const UClass* GetNativeClass(const UClass* ForClass);
	void InvalidateCache(bool bIsAsyncCompile);
	void InvalidateCache();

	static TMap<FName, EHostType::Type> DescriptorCache;
	static FCriticalSection DescriptorCacheLock;
}

const UClass* UE::Private::GetNativeClass(const UFunction* ForFunction)
{
	const UClass* NativeClass = GetNativeClass(ForFunction->GetOwnerClass());
	if (NativeClass)
	{
		return NativeClass;
	}

	ensureMsgf(false, TEXT("Found no native base class for function - cannot validate native constraints: %s"), *ForFunction->GetPathName());
	return nullptr;
}

const UClass* UE::Private::GetNativeClass(const UClass* ForClass)
{
	return FBlueprintEditorUtils::FindFirstNativeClass(const_cast<UClass*>(ForClass));
}

void UE::Private::InvalidateCache(bool bIsAsyncCompile)
{
	InvalidateCache();
}

void UE::Private::InvalidateCache()
{
	FScopeLock WriteLock(&DescriptorCacheLock);
	DescriptorCache.Reset();
}


namespace UE::ProjectUtilities
{
const EHostType::Type FindModuleDescriptorHostType(const UPackage* ForNativePackage)
{
	if (!ForNativePackage)
	{
		return EHostType::Max;
	}

	// The loops at the bottom are naive and may need to be optimized if we begin
	// validating build target compatibility widely:
	FString FunctionPackageName = ForNativePackage->GetName();
	if (!FunctionPackageName.StartsWith(TEXT("/Script/")))
	{
		return EHostType::Max;
	}

	FunctionPackageName = FunctionPackageName.RightChop(8); // Chop "/Script/" from the name
	if (FunctionPackageName == TEXT("Engine"))
	{ // common case, there is no ModuleDescriptor for Engine
		return EHostType::Max;
	}

	const IProjectManager& ProjectManager = IProjectManager::Get();
	const FProjectDescriptor* PD = ProjectManager.GetCurrentProject();
	if (!PD)
	{
		return EHostType::Max;
	}

	const FName FunctionPackageFName(FunctionPackageName.Len(), *FunctionPackageName);
	const auto AddToCache = [](FName Name, EHostType::Type HostType)
		{
			// must be a write lock:
			FScopeLock WriteLock(&UE::Private::DescriptorCacheLock);
			static bool bCheckedForHotReloadOrLiveCoding = false;
			if(!bCheckedForHotReloadOrLiveCoding)
			{
				bCheckedForHotReloadOrLiveCoding = true;
				// Register the hot-reload callback - invalidating our cache when a module is compiled:
				if(IHotReloadModule::IsAvailable())
				{
					IHotReloadModule::Get().OnModuleCompilerStarted().AddStatic( &UE::Private::InvalidateCache );
				}
			
	#if WITH_LIVE_CODING
				if (ILiveCodingModule* LiveCoding = FModuleManager::LoadModulePtr<ILiveCodingModule>(LIVE_CODING_MODULE_NAME))
				{
					LiveCoding->GetOnPatchCompleteDelegate().AddStatic(&UE::Private::InvalidateCache);
				}
	#endif
			}
			UE::Private::DescriptorCache.Add(Name, HostType);
		};
	const auto FindFromCache = [](FName Name) -> TOptional<EHostType::Type>
		{
			// could be a read lock:
			FScopeLock FindLock(&UE::Private::DescriptorCacheLock);
			const EHostType::Type* Result = UE::Private::DescriptorCache.Find(Name);
			return Result ? *Result : TOptional<EHostType::Type>();
		};
	TOptional<EHostType::Type> CachedDescriptor = FindFromCache(FunctionPackageFName);
	if (CachedDescriptor)
	{
		return *CachedDescriptor;
	}

	const FModuleDescriptor* OwningMD = nullptr;

	for (const FModuleDescriptor& MD : PD->Modules)
	{
		if (MD.Name == FunctionPackageFName)
		{
			AddToCache(FunctionPackageFName, MD.Type);
			return MD.Type;
		}
	}

	// This is a little expensive, hence the cache - IPluginManager could take over
	// acceleration structure duties:
	TArray<TSharedRef<IPlugin>> AllPlugins = IPluginManager::Get().GetDiscoveredPlugins();
	for (const TSharedRef<IPlugin>& Plugin : AllPlugins)
	{
		if (Plugin->IsHidden())
		{
			continue;
		}

		for (const FModuleDescriptor& MD : Plugin->GetDescriptor().Modules)
		{
			if (MD.Name == FunctionPackageFName)
			{
				AddToCache(FunctionPackageFName, MD.Type);
				return MD.Type;
			}
		}
	}

	AddToCache(FunctionPackageFName, EHostType::Max);
	return EHostType::Max;
}

FBuildTargetSet FBuildTargetSet::GetCallerTargetsUnsupportedByCallee(const UClass* Caller, const UFunction* Callee)
{
	using namespace UE::Private;
	check(Caller && Callee);
	if (Caller == Callee->GetOwnerClass())
	{
		return FBuildTargetSet();
	}
	
	const UClass* NativeCaller = GetNativeClass(Caller);
	const UClass* NativeCallee = GetNativeClass(Callee);

	// if the native modules are the same type then we are fine:
	const UPackage* CallerNativePackage = NativeCaller->GetPackage();
	const UPackage* CalleeNativePackage = NativeCallee->GetPackage();
	if (CallerNativePackage == CalleeNativePackage)
	{
		return FBuildTargetSet();
	}
	
	FBuildTargetSet CallerSet;
	CallerSet.BuildTargetFlags = GetSupportedTargetsForNativeClass(NativeCaller);
	FBuildTargetSet CalleeSet;
	CalleeSet.BuildTargetFlags = GetSupportedTargetsForNativeClass(NativeCallee);
	
	FBuildTargetSet Result;
	Result.BuildTargetFlags = GetCallerTargetsUnsupportedByCalleeImpl(CallerSet.BuildTargetFlags, CalleeSet.BuildTargetFlags);

	return Result;
}

FString FBuildTargetSet::LexToString() const
{
	return LexToStringImpl(BuildTargetFlags);
}

FString FBuildTargetSet::LexToStringImpl(EBuildTargetFlags Flags)
{
	FString Result;
	const TCHAR* Seperator = TEXT("");
	if ((Flags & EBuildTargetFlags::Server) != EBuildTargetFlags::None)
	{
		Result += TEXT("Server");
		Seperator = TEXT("|");
	}
	if ((Flags & EBuildTargetFlags::Client) != EBuildTargetFlags::None)
	{
		Result += Seperator;
		Result += TEXT("Client");
		Seperator = TEXT("|");
	}
	if ((Flags & EBuildTargetFlags::Editor) != EBuildTargetFlags::None)
	{
		Result += Seperator;
		Result += TEXT("Editor");
		Seperator = TEXT("|");
	}
	return Result;
}

FBuildTargetSet::EBuildTargetFlags FBuildTargetSet::GetSupportedTargetsForNativeClass(const UClass* NativeBase)
{
	check(NativeBase);
	const EBuildTargetFlags ALL = EBuildTargetFlags::Server
		| EBuildTargetFlags::Client
		| EBuildTargetFlags::Editor;
	using namespace UE::Private;
	const UPackage* Package = NativeBase->GetPackage();
	const EHostType::Type ModuleHostType = UE::ProjectUtilities::FindModuleDescriptorHostType(Package);
	EBuildTargetFlags SupportedTargets = ALL;
	if (ModuleHostType != EHostType::Max)
	{
		switch (ModuleHostType)
		{
		case EHostType::Runtime:
		case EHostType::RuntimeNoCommandlet:
		case EHostType::RuntimeAndProgram:
		case EHostType::CookedOnly:
			break;
		case EHostType::UncookedOnly:
		case EHostType::Developer:
		case EHostType::DeveloperTool:
		case EHostType::Editor:
		case EHostType::EditorNoCommandlet:
		case EHostType::EditorAndProgram:
		case EHostType::Program:
			SupportedTargets = EBuildTargetFlags::Editor;
			break;
		case EHostType::ServerOnly:
			// Loads on all targets except dedicated clients.
			SupportedTargets = EBuildTargetFlags::Server
				| EBuildTargetFlags::Editor;
			break;
		case EHostType::ClientOnly:
		case EHostType::ClientOnlyNoCommandlet:
			// Loads on all targets except dedicated servers.
			SupportedTargets = EBuildTargetFlags::Client
				| EBuildTargetFlags::Editor;
			break;
		default:
			ensureMsgf(false, TEXT("Encountered unexpected module type: %d in module %s"), ModuleHostType, Package ? *Package->GetName() : TEXT("nullpackage"));
			break;
		}
	}

	// honor IsEditorOnlyObject
	if (IsEditorOnlyObject(NativeBase))
	{
		SupportedTargets = EBuildTargetFlags::Editor;
	}

	// if the class itself doesn't think it needs load for server
	if (!NativeBase->NeedsLoadForServer() ||
		!NativeBase->GetDefaultObject()->NeedsLoadForServer())
	{
		SupportedTargets &= ~EBuildTargetFlags::Server;
	}
	// or client, then we can't support those:
	if (!NativeBase->NeedsLoadForClient()||
		!NativeBase->GetDefaultObject()->NeedsLoadForClient())
	{
		SupportedTargets &= ~EBuildTargetFlags::Client;
	}
	return SupportedTargets;
}

FBuildTargetSet::EBuildTargetFlags FBuildTargetSet::GetCallerTargetsUnsupportedByCalleeImpl(FBuildTargetSet::EBuildTargetFlags CallerTargets, FBuildTargetSet::EBuildTargetFlags CalleeTargets)
{
	using namespace UE::Private;
	// caller targets must be a subset of valid callee targets,
	// otherwise caller could be deployed into a context within which
	// the callee is not available:
	EBuildTargetFlags Differences = CallerTargets ^ CalleeTargets;
	return (Differences & ~CalleeTargets);	
}

}

