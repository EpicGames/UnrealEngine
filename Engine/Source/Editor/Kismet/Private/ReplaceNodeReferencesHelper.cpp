// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplaceNodeReferencesHelper.h"

#include "Containers/UnrealString.h"
#include "HAL/PlatformCrt.h"
#include "ImaginaryBlueprintData.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ScopedSlowTask.h"
#include "ScopedTransaction.h"
#include "Templates/Casts.h"
#include "Templates/UnrealTemplate.h"

class UBlueprint;

#define LOCTEXT_NAMESPACE "FReplaceNodeReferencesHelper"

FReplaceNodeReferencesHelper::FReplaceNodeReferencesHelper(const FMemberReference& Source, const FMemberReference& Replacement, UBlueprint* InBlueprint)
	: SourceReference(Source)
	, ReplacementReference(Replacement)
	, Blueprint(InBlueprint)
{
}

FReplaceNodeReferencesHelper::FReplaceNodeReferencesHelper(FMemberReference&& Source, FMemberReference&& Replacement, UBlueprint* InBlueprint)
	: SourceReference(MoveTemp(Source))
	, ReplacementReference(MoveTemp(Replacement))
	, Blueprint(InBlueprint)
{
}

FReplaceNodeReferencesHelper::~FReplaceNodeReferencesHelper()
{
}

void FReplaceNodeReferencesHelper::BeginFindAndReplace(const FSimpleDelegate& InOnCompleted /*=FSimpleDelegate()*/)
{
	bCompleted = false;
	OnCompleted = InOnCompleted;
	FFindInBlueprintCachingOptions CachingOptions;
	CachingOptions.MinimiumVersionRequirement = EFiBVersion::FIB_VER_VARIABLE_REFERENCE;
	CachingOptions.OnFinished.BindRaw(this, &FReplaceNodeReferencesHelper::OnSubmitSearchQuery);
	FFindInBlueprintSearchManager::Get().CacheAllAssets(nullptr, CachingOptions);
	SlowTask = MakeUnique<FScopedSlowTask>(3.f, LOCTEXT("Caching", "Caching Blueprints..."));
	SlowTask->MakeDialog();
}

void FReplaceNodeReferencesHelper::ReplaceReferences(TArray<FImaginaryFiBDataSharedPtr>& InRawDataList)
{
	ReplaceReferences(SourceReference, ReplacementReference, Blueprint, InRawDataList);
}

void FReplaceNodeReferencesHelper::ReplaceReferences(const FMemberReference& InSource, const FMemberReference& InReplacement, UBlueprint* InBlueprint, TArray<FImaginaryFiBDataSharedPtr>& InRawDataList)
{
	const FScopedTransaction Transaction(FText::Format(LOCTEXT("ReplaceRefs", "Replace References with {0}"), FText::FromName(InReplacement.GetMemberName())));
	TArray< UBlueprint* > BlueprintsModified;
	for (FImaginaryFiBDataSharedPtr ImaginaryData : InRawDataList)
	{
		UBlueprint* Blueprint = ImaginaryData->GetBlueprint();
		BlueprintsModified.AddUnique(Blueprint);
		UK2Node* Node = Cast<UK2Node>(ImaginaryData->GetObject(Blueprint));
		if (ensure(Node))
		{
			Node->Modify();
			Node->ReplaceReferences(InBlueprint, Blueprint, InSource, InReplacement);
			Node->ReconstructNode();
		}
	}

	for (UBlueprint* ModifiedBlueprint : BlueprintsModified)
	{
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(ModifiedBlueprint);
		FFindInBlueprintSearchManager::Get().AddOrUpdateBlueprintSearchMetadata(ModifiedBlueprint);
	}
}

const void FReplaceNodeReferencesHelper::SetTransaction(TSharedPtr<FScopedTransaction> InTransaction)
{
	Transaction = InTransaction;
}

bool FReplaceNodeReferencesHelper::IsTickable() const
{
	return SlowTask.IsValid();
}

void FReplaceNodeReferencesHelper::Tick(float DeltaSeconds)
{
	if (StreamSearch.IsValid())
	{
		UpdateSearchQuery();
	}
	else
	{
		SlowTask->CompletedWork = FFindInBlueprintSearchManager::Get().GetCacheProgress();
	}
}

TStatId FReplaceNodeReferencesHelper::GetStatId() const
{
	return TStatId();
}

void FReplaceNodeReferencesHelper::OnSubmitSearchQuery()
{
	SlowTask->FrameMessage = LOCTEXT("Searching", "Searching Blueprints...");
	FString SearchTerm = SourceReference.GetReferenceSearchString(SourceReference.GetMemberParentClass());

	FStreamSearchOptions SearchOptions;
	SearchOptions.ImaginaryDataFilter = ESearchQueryFilter::NodesFilter;
	SearchOptions.MinimiumVersionRequirement = EFiBVersion::FIB_VER_VARIABLE_REFERENCE;

	StreamSearch = MakeShared<FStreamSearch>(SearchTerm, SearchOptions);
}

void FReplaceNodeReferencesHelper::UpdateSearchQuery()
{
	if (!StreamSearch->IsComplete())
	{
		SlowTask->CompletedWork = 1.f + FFindInBlueprintSearchManager::Get().GetPercentComplete(StreamSearch.Get());
	}
	else
	{
		TArray<FImaginaryFiBDataSharedPtr> ImaginaryData;
		StreamSearch->GetFilteredImaginaryResults(ImaginaryData);
		ReplaceReferences(ImaginaryData);
		
		StreamSearch->EnsureCompletion();

		// End the SlowTask
		SlowTask.Reset();

		OnCompleted.ExecuteIfBound();
		bCompleted = true;
	}
}

#undef LOCTEXT_NAMESPACE
