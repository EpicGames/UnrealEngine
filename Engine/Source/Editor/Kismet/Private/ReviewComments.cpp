// Copyright Epic Games, Inc. All Rights Reserved.
#include "ReviewComments.h"
#include "UObject/Class.h"
#include "Serialization/JsonSerializer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReviewComments)

namespace
{

	FString LowerCamelCase(const FString& In)
	{
		FString Result = In;
		if (!Result.IsEmpty())
		{
			Result[0] = tolower(Result[0]);
		}
		return Result;
	}

	bool TryGetJsonObjectField(FReviewTopic* OutVal, const TSharedPtr<FJsonObject>& JsonObject, const FString& Key)
	{
		if (const TSharedPtr<FJsonValue> &Value = JsonObject->TryGetField(Key))
		{
			if (Value->IsNull())
			{
				return false;
			}
			*OutVal = FReviewTopic::FromString(Value->AsString());
			return true;
		}
		return false;
	}
	
	void TrySetJsonObjectField(TSharedPtr<FJsonObject> JsonObject, const FReviewTopic& Val, const FString& Key)
	{
		JsonObject->SetStringField(Key, Val.ToString());
	}

	bool TryGetJsonObjectField(int32* OutVal, const TSharedPtr<FJsonObject>& JsonObject, const FString& Key)
	{
		if (const TSharedPtr<FJsonValue> &Value = JsonObject->TryGetField(Key))
		{
			if (Value->IsNull())
			{
				return false;
			}
			*OutVal = (int32)Value->AsNumber();
			return true;
		}
		return false;
	}
	
	void TrySetJsonObjectField(TSharedPtr<FJsonObject> JsonObject, const int32& Val, const FString& Key)
	{
		JsonObject->SetNumberField(Key, Val);
	}

	bool TryGetJsonObjectField(FString* OutVal, const TSharedPtr<FJsonObject>& JsonObject, const FString& Key)
	{
		if (const TSharedPtr<FJsonValue> &Value = JsonObject->TryGetField(Key))
		{
			if (Value->IsNull())
			{
				return false;
			}
			*OutVal = Value->AsString();
			return true;
		}
		return false;
	}
	
	void TrySetJsonObjectField(TSharedPtr<FJsonObject> JsonObject, const FString& Val, const FString& Key)
	{
		JsonObject->SetStringField(Key, Val);
	}

	bool TryGetJsonObjectField(FReviewCommentContext* OutVal, const TSharedPtr<FJsonObject>& JsonObject, const FString& Key)
	{
		if (const TSharedPtr<FJsonValue> &Value = JsonObject->TryGetField(Key))
		{
			if (Value->IsNull())
			{
				return false;
			}
			// for some reason the API stores null-contexts as empty arrays. 
			if (Value->Type == EJson::Array)
			{
				checkSlow(Value->AsArray().IsEmpty());
				return false;
			}
			*OutVal = FReviewCommentContext::FromJson(Value->AsObject());
			return true;
		}
		return false;
	}
	
	void TrySetJsonObjectField(TSharedPtr<FJsonObject> JsonObject, const FReviewCommentContext& Val, const FString& Key)
	{
		JsonObject->SetObjectField(Key, Val.ToJsonObject());
	}

	bool TryGetJsonObjectField(TSet<FString>* OutVal, const TSharedPtr<FJsonObject>& JsonObject, const FString& Key)
	{
		if (const TSharedPtr<FJsonValue> &Value = JsonObject->TryGetField(Key))
		{
			if (Value->IsNull())
			{
				return false;
			}
			for (const TSharedPtr<FJsonValue>& Item : Value->AsArray())
			{
				OutVal->Add(Item->AsString());
			}
			return true;
		}
		return false;
	}
	
	void TrySetJsonObjectField(TSharedPtr<FJsonObject> JsonObject, const TSet<FString>& Val, const FString& Key)
	{
		TArray<TSharedPtr<FJsonValue>> Array;
		for (const FString& Element : Val)
		{
			Array.Add(MakeShared<FJsonValueString>(Element));
		}
		JsonObject->SetArrayField(Key, Array);
	}

	bool TryGetJsonObjectField(TArray<FString>* OutVal, const TSharedPtr<FJsonObject>& JsonObject, const FString& Key)
	{
		if (const TSharedPtr<FJsonValue> &Value = JsonObject->TryGetField(Key))
		{
			if (Value->IsNull())
			{
				return false;
			}
			for (const TSharedPtr<FJsonValue>& Item : Value->AsArray())
			{
				OutVal->Add(Item->AsString());
			}
			return true;
		}
		return false;
	}
	
	void TrySetJsonObjectField(TSharedPtr<FJsonObject> JsonObject, const TArray<FString>& Val, const FString& Key)
	{
		TArray<TSharedPtr<FJsonValue>> Array;
		for (const FString& Element : Val)
		{
			Array.Add(MakeShared<FJsonValueString>(Element));
		}
		JsonObject->SetArrayField(Key, Array);
	}

	bool TryGetJsonObjectField(EReviewCommentTaskState* OutVal, const TSharedPtr<FJsonObject>& JsonObject, const FString& Key)
	{
		if (const TSharedPtr<FJsonValue> &Value = JsonObject->TryGetField(Key))
		{
			if (Value->IsNull())
			{
				return false;
			}
			*OutVal = (EReviewCommentTaskState)StaticEnum<EReviewCommentTaskState>()->GetValueByNameString(Value->AsString());
			return true;
		}
		return false;
	}
	
	void TrySetJsonObjectField(TSharedPtr<FJsonObject> JsonObject, const EReviewCommentTaskState& Val, const FString& Key)
	{
		const FString EnumString = StaticEnum<EReviewCommentTaskState>()->GetValueAsString(Val);
		int32 ChopIndex = INDEX_NONE;
		if (EnumString.FindLastChar(':', ChopIndex))
		{
			JsonObject->SetStringField(Key, LowerCamelCase(EnumString.RightChop(ChopIndex + 1)));
		}
	}

	bool TryGetJsonObjectField(FDateTime* OutVal, const TSharedPtr<FJsonObject>& JsonObject, const FString& Key)
	{
		if (const TSharedPtr<FJsonValue> &Value = JsonObject->TryGetField(Key))
		{
			if (Value->IsNull())
			{
				return false;
			}
			*OutVal = FDateTime::FromUnixTimestamp((int64)Value->AsNumber());
			return true;
		}
		return false;
	}
	
	void TrySetJsonObjectField(TSharedPtr<FJsonObject> JsonObject, const FDateTime& Val, const FString& Key)
	{
		JsonObject->SetNumberField(Key, Val.ToUnixTimestamp());
	}

	template<typename OptionalType>
	bool TryGetJsonObjectField(TOptional<OptionalType>* OutVal, const TSharedPtr<FJsonObject>& JsonObject, const FString& Key)
	{
		OptionalType Val;
		if (TryGetJsonObjectField(&Val, JsonObject, Key))
		{
			*OutVal = Val;
			return true;
		}
		return false;
	}
	
	template<typename OptionalType>
	void TrySetJsonObjectField(TSharedPtr<FJsonObject> JsonObject, const TOptional<OptionalType>& Val, const FString& Key)
	{
		if (Val.IsSet())
		{
			TrySetJsonObjectField(JsonObject, Val.GetValue(), Key);
		}
	}

	#define TRY_SET_FROM_JSON(OwningObject, Key, JsonObject) TryGetJsonObjectField(&(OwningObject).Key, JsonObject, LowerCamelCase(#Key))
	#define TRY_SET_FROM_JSON_NAMED(OwningObject, Key, JsonObject, Name) TryGetJsonObjectField(&(OwningObject).Key, JsonObject, LowerCamelCase(Name))
	#define SET_JSON_FIELD(JsonObject, OwningObject, Key) TrySetJsonObjectField(JsonObject, (OwningObject).Key, LowerCamelCase(#Key))
	#define SET_JSON_FIELD_NAMED(JsonObject, OwningObject, Key, Name) TrySetJsonObjectField(JsonObject, (OwningObject).Key, LowerCamelCase(Name))
}


FString FReviewCommentContext::ToJson() const
{
	FString Result;
    const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&Result);
    FJsonSerializer::Serialize(ToJsonObject().ToSharedRef(), Writer);
	return Result;
}

TSharedPtr<FJsonObject> FReviewCommentContext::ToJsonObject() const
{
	TSharedPtr<FJsonObject> Result = MakeShared<FJsonObject>();
	SET_JSON_FIELD(Result, *this, File);
	SET_JSON_FIELD(Result, *this, Version);
	SET_JSON_FIELD(Result, *this, Content);
	SET_JSON_FIELD(Result, *this, LeftLine);
	SET_JSON_FIELD(Result, *this, RightLine);
	SET_JSON_FIELD_NAMED(Result, *this, ReplyTo, TEXT("comment"));
	return Result;
}

FReviewCommentContext FReviewCommentContext::FromJson(const FString& Json)
{
	const TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(Json);
	TSharedPtr<FJsonValue> JsonValue;
	if (FJsonSerializer::Deserialize(JsonReader, JsonValue))
	{
		const TSharedPtr< FJsonObject > JsonObject = JsonValue->AsObject();
		return FromJson(JsonObject);
	}
	return {};
}

FReviewCommentContext FReviewCommentContext::FromJson(const TSharedPtr<FJsonObject>& JsonObject)
{
	FReviewCommentContext Result = {};
	if (!JsonObject.IsValid())
	{
		return Result;
	}

	TRY_SET_FROM_JSON(Result, File, JsonObject);
	TRY_SET_FROM_JSON(Result, Version, JsonObject);
	TRY_SET_FROM_JSON(Result, Content, JsonObject);
	TRY_SET_FROM_JSON(Result, LeftLine, JsonObject);
	TRY_SET_FROM_JSON(Result, RightLine, JsonObject);
	TRY_SET_FROM_JSON_NAMED(Result, ReplyTo, JsonObject, TEXT("comment"));
	return Result;
}

FString FReviewTopic::ToString() const
{
	switch(TopicType)
	{
	case EReviewTopicType::Review:
		return FString::Format(TEXT("reviews/{0}"), {ChangelistNum});
	case EReviewTopicType::Change:
		return FString::Format(TEXT("changes/{0}"), {ChangelistNum});
	}
	return {};
}

FReviewTopic FReviewTopic::FromString(const FString& ReviewTopic)
{
	FReviewTopic Result;
	FString TopicTypeString;
	ReviewTopic.Split("/", &TopicTypeString, &Result.ChangelistNum);
	if (TopicTypeString == TEXT("reviews"))
	{
		Result.TopicType = EReviewTopicType::Review;
	}
	else if (TopicTypeString == TEXT("changes"))
	{
		Result.TopicType = EReviewTopicType::Change;
	}
	else
	{
		Result.TopicType = EReviewTopicType::Unrecognised;
	}
	return Result;
}

FString FReviewComment::ToJson() const
{
	FString Result;
	const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&Result);
	FJsonSerializer::Serialize(ToJsonObject().ToSharedRef(), Writer);
	return Result;
}

TSharedPtr<FJsonObject> FReviewComment::ToJsonObject() const
{
	TSharedPtr<FJsonObject> Result = MakeShared<FJsonObject>();
	SET_JSON_FIELD(Result, *this, Topic);
	SET_JSON_FIELD(Result, *this, Body);
	SET_JSON_FIELD(Result, *this, Context);
	// don't include context if it doesn't have any fields
	const TMap<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& ContextValues = Result->GetObjectField(TEXT("Context"))->Values;
	if (ContextValues.IsEmpty())
	{
		Result->RemoveField(TEXT("Context"));
	}
	SET_JSON_FIELD(Result, *this, Likes);
	SET_JSON_FIELD(Result, *this, ReadBy);
	SET_JSON_FIELD(Result, *this, TaskState);
	SET_JSON_FIELD(Result, *this, User);
	
	SET_JSON_FIELD_NAMED(Result, *this, CommentID, TEXT("id"));
	SET_JSON_FIELD_NAMED(Result, *this, CreatedTime, TEXT("time"));
	SET_JSON_FIELD_NAMED(Result, *this, EditedTime, TEXT("edited"));
	SET_JSON_FIELD_NAMED(Result, *this, UpdatedTime, TEXT("updated"));
	if (bIsClosed)
	{
		TrySetJsonObjectField(Result, TArray<FString>{TEXT("closed")}, TEXT("flags"));
	}
	
	return Result;
	
}

FReviewComment FReviewComment::FromJson(const FString& Json)
{
	const TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(Json);
	TSharedPtr<FJsonValue> JsonValue;
	if (FJsonSerializer::Deserialize(JsonReader, JsonValue))
	{
		const TSharedPtr< FJsonObject > JsonObject = JsonValue->AsObject();
		return FromJson(JsonObject);
	}
	return {};
}

FReviewComment FReviewComment::FromJson(const TSharedPtr<FJsonObject>& JsonObject)
{
	FReviewComment Result = {};
	if (!JsonObject.IsValid())
	{
		return Result;
	}

	TRY_SET_FROM_JSON(Result, Topic, JsonObject);
	TRY_SET_FROM_JSON(Result, Body, JsonObject);
	TRY_SET_FROM_JSON(Result, Context, JsonObject);
	TRY_SET_FROM_JSON(Result, Likes, JsonObject);
	TRY_SET_FROM_JSON(Result, ReadBy, JsonObject);
	TRY_SET_FROM_JSON(Result, TaskState, JsonObject);
	TRY_SET_FROM_JSON(Result, User, JsonObject);
	
	TRY_SET_FROM_JSON_NAMED(Result, CommentID, JsonObject, TEXT("id"));
	TRY_SET_FROM_JSON_NAMED(Result, CreatedTime, JsonObject, TEXT("time"));
	TRY_SET_FROM_JSON_NAMED(Result, EditedTime, JsonObject, TEXT("edited"));
	TRY_SET_FROM_JSON_NAMED(Result, UpdatedTime, JsonObject, TEXT("updated"));
	
	// set flags
	Result.bIsClosed = false;
	if (const TSharedPtr<FJsonValue> &FlagsField = JsonObject->TryGetField(TEXT("flags")))
	{
		if (!FlagsField->IsNull())
		{
			for (const TSharedPtr<FJsonValue>& Item : FlagsField->AsArray())
            {
            	FString Flag = Item->AsString();
				if (Flag == TEXT("closed"))
				{
					Result.bIsClosed = true;
				}
				else
				{
					// swarm documentation doesn't mention any flags except 'closed'. if this is hit, let jordan.hoffmann know
					ensure(false);
				}
            }
		}
	}
	
	return Result;
}
