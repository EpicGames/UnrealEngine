// Copyright Epic Games, Inc. All Rights Reserved.


#include "SBlueprintActionMenu.h"

#include "BlueprintActionFilter.h"
#include "BlueprintActionMenuBuilder.h"
#include "BlueprintActionMenuUtils.h"
#include "BlueprintEditor.h"
#include "BlueprintEditorSettings.h"
#include "BlueprintNamespaceUtilities.h"
#include "BlueprintPaletteFavorites.h"
#include "Containers/EnumAsByte.h"
#include "Containers/SparseArray.h"
#include "CoreGlobals.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphSchema_K2_Actions.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Engine/Blueprint.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformCrt.h"
#include "IDocumentation.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Layout/Children.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Optional.h"
#include "SBlueprintContextTargetMenu.h"
#include "SBlueprintNamespaceEntry.h"
#include "SBlueprintPalette.h"
#include "SGraphActionMenu.h"
#include "SMyBlueprint.h"
#include "SSubobjectEditor.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/GarbageCollection.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "SBlueprintGraphContextMenu"

/** Action to promote a pin to a variable */
USTRUCT()
struct FBlueprintAction_PromoteVariable : public FEdGraphSchemaAction
{
	FBlueprintAction_PromoteVariable(bool bInToMemberVariable)
		: FEdGraphSchemaAction(	FText(), 
								bInToMemberVariable? LOCTEXT("PromoteToVariable", "Promote to variable") : LOCTEXT("PromoteToLocalVariable", "Promote to local variable"),
								bInToMemberVariable ? LOCTEXT("PromoteToVariable", "Promote to variable") : LOCTEXT("PromoteToLocalVariable", "Promote to local variable"),
								1)
		, bToMemberVariable(bInToMemberVariable)
	{
	}

	// FEdGraphSchemaAction interface
	virtual UEdGraphNode* PerformAction( class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override
	{
		if( ( ParentGraph != NULL ) && ( FromPin != NULL ) )
		{
			UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(ParentGraph);
			if( ( MyBlueprintEditor.IsValid() == true ) && ( Blueprint != NULL ) )
			{
				MyBlueprintEditor.Pin()->DoPromoteToVariable2f( Blueprint, FromPin, bToMemberVariable, Location);
			}
		}
		return NULL;		
	}
	// End of FEdGraphSchemaAction interface

	/* Pointer to the blueprint editor containing the blueprint in which we will promote the variable. */
	TWeakPtr<class FBlueprintEditor> MyBlueprintEditor;

	/* TRUE if promoting to member variable, FALSE if promoting to local variable */
	bool bToMemberVariable;
};

/**
 * Static method for binding with delegates. Spawns an instance of the custom
 * expander.
 * 
 * @param  ActionMenuData	A set of useful data for detailing the specific action menu row this is for.
 * @return A new widget, intended to lead entries in an SGraphActionMenu.
 */
static TSharedRef<SExpanderArrow> CreateCustomBlueprintActionExpander(const FCustomExpanderData& ActionMenuData)
{
	return SNew(SBlueprintActionMenuExpander, ActionMenuData);
}

/*******************************************************************************
* SBlueprintActionFavoriteToggle
*******************************************************************************/

class SBlueprintActionFavoriteToggle : public SCompoundWidget
{
	SLATE_BEGIN_ARGS( SBlueprintActionFavoriteToggle ) {}
	SLATE_END_ARGS()

public:
	/**
	 * Constructs a favorite-toggle widget (so that user can easily modify the 
	 * item's favorited state).
	 * 
	 * @param  InArgs			A set of slate arguments, defined above.
	 * @param  ActionPtrIn		The FEdGraphSchemaAction that the parent item represents.
	 * @param  BlueprintEdPtrIn	A pointer to the blueprint editor that the palette belongs to.
	 */
	void Construct(const FArguments& InArgs, const FCustomExpanderData& CustomExpanderData)
	{
		Container = CustomExpanderData.WidgetContainer;
		ActionPtr = CustomExpanderData.RowAction;

		ChildSlot
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Center)
				.FillWidth(1.0)
			[
				SNew( SCheckBox )
					.Visibility(this, &SBlueprintActionFavoriteToggle::IsVisible)
					.ToolTipText(this, &SBlueprintActionFavoriteToggle::GetToolTipText)
					.IsChecked(this, &SBlueprintActionFavoriteToggle::GetFavoritedState)
					.OnCheckStateChanged(this, &SBlueprintActionFavoriteToggle::OnFavoriteToggled)
					.Style(FAppStyle::Get(), "Kismet.Palette.FavoriteToggleStyle")
			]
		];
	}

private:
	/**
	 * Used to determine the toggle's visibility (this is only visible when the 
	 * owning item is being hovered over, and the associated action can be favorited).
	 *
	 * @return True if this toggle switch should be showing, false if not.
	 */
	EVisibility IsVisible() const
	{
		bool bNoFavorites = false;
		GConfig->GetBool(TEXT("BlueprintEditor.Palette"), TEXT("bUseLegacyLayout"), bNoFavorites, GEditorIni);

		UBlueprintPaletteFavorites const* const BlueprintFavorites = GetDefault<UEditorPerProjectUserSettings>()->BlueprintFavorites;

		EVisibility CurrentVisibility = EVisibility::Hidden;
		if (!bNoFavorites && BlueprintFavorites && BlueprintFavorites->CanBeFavorited(ActionPtr.Pin()))
		{
			if (BlueprintFavorites->IsFavorited(ActionPtr.Pin()) || Container->IsHovered())
			{
				CurrentVisibility = EVisibility::Visible;
			}			
		}

		return CurrentVisibility;
	}

	/**
	 * Retrieves tooltip that describes the current favorited state of the 
	 * associated action.
	 * 
	 * @return Text describing what this toggle will do when you click on it.
	 */
	FText GetToolTipText() const
	{
		if (GetFavoritedState() == ECheckBoxState::Checked)
		{
			return LOCTEXT("Unfavorite", "Click to remove this item from your favorites.");
		}
		return LOCTEXT("Favorite", "Click to add this item to your favorites.");
	}

	/**
	 * Checks on the associated action's favorite state, and returns a 
	 * corresponding checkbox state to match.
	 * 
	 * @return ECheckBoxState::Checked if the associated action is already favorited, ECheckBoxState::Unchecked if not.
	 */
	ECheckBoxState GetFavoritedState() const
	{
		ECheckBoxState FavoriteState = ECheckBoxState::Unchecked;
		if (ActionPtr.IsValid())
		{
			const UEditorPerProjectUserSettings& EditorSettings = *GetDefault<UEditorPerProjectUserSettings>();
			if (UBlueprintPaletteFavorites* BlueprintFavorites = EditorSettings.BlueprintFavorites)
			{
				FavoriteState = BlueprintFavorites->IsFavorited(ActionPtr.Pin()) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
		}
		return FavoriteState;
	}

	/**
	 * Triggers when the user clicks this toggle, adds or removes the associated
	 * action to the user's favorites.
	 * 
	 * @param  InNewState	The new state that the user set the checkbox to.
	 */
	void OnFavoriteToggled(ECheckBoxState InNewState)
	{
		if (InNewState == ECheckBoxState::Checked)
		{
			GetMutableDefault<UEditorPerProjectUserSettings>()->BlueprintFavorites->AddFavorite(ActionPtr.Pin());
		}
		else
		{
			GetMutableDefault<UEditorPerProjectUserSettings>()->BlueprintFavorites->RemoveFavorite(ActionPtr.Pin());
		}
	}	

private:
	/** The action that the owning palette entry represents */
	TWeakPtr<FEdGraphSchemaAction> ActionPtr;

	/** The widget that this widget is nested inside */
	TSharedPtr<SPanel> Container;
};

/*******************************************************************************
* SBlueprintActionMenu
*******************************************************************************/

SBlueprintActionMenu::~SBlueprintActionMenu()
{
	OnClosedCallback.ExecuteIfBound();
	OnCloseReasonCallback.ExecuteIfBound(bActionExecuted, ContextToggleIsChecked() == ECheckBoxState::Checked, DraggedFromPins.Num() > 0);
}

void SBlueprintActionMenu::Construct( const FArguments& InArgs, TSharedPtr<FBlueprintEditor> InEditor )
{
	bActionExecuted = false;

	this->GraphObj = InArgs._GraphObj;
	this->DraggedFromPins = InArgs._DraggedFromPins;
	this->NewNodePosition = InArgs._NewNodePosition;
	this->OnClosedCallback = InArgs._OnClosedCallback;
	this->bAutoExpandActionMenu = InArgs._AutoExpandActionMenu;
	this->EditorPtr = InEditor;
	this->OnCloseReasonCallback = InArgs._OnCloseReason;

	// Generate the context display; showing the user what they're picking something for
	//@TODO: Should probably be somewhere more schema-sensitive than the graph panel!
	FSlateColor TypeColor;
	FString TypeOfDisplay;
	const FSlateBrush* ContextIcon = nullptr;

	if (DraggedFromPins.Num() == 1)
	{
		UEdGraphPin* OnePin = DraggedFromPins[0];

		const UEdGraphSchema* Schema = OnePin->GetSchema();
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

		if (!Schema->IsA(UEdGraphSchema_K2::StaticClass()) || !K2Schema->IsExecPin(*OnePin))
		{
			// Get the type color and icon
			TypeColor = Schema->GetPinTypeColor(OnePin->PinType);
			ContextIcon = FAppStyle::GetBrush( OnePin->PinType.IsArray() ? TEXT("Graph.ArrayPin.Connected") : TEXT("Graph.Pin.Connected") );
		}
	}

	FBlueprintActionContext MenuContext;
	ConstructActionContext(MenuContext);

	TSharedPtr<SWidget> AddImportTargetContent = SNullWidget::NullWidget;
	if (GetDefault<UBlueprintEditorSettings>()->bEnableNamespaceImportingFeatures)
	{
		SAssignNew(AddImportTargetContent, SBox)
			.ToolTipText(LOCTEXT("ImportActionLabelTooltip", "Choose a namespace to import and load additional actions."))
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ImportActionButtonLabel", "Import Actions From:"))
			]
			+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4.f, 0.f)
			[
				SNew(SBlueprintNamespaceEntry)
					.AllowTextEntry(false)
					.OnNamespaceSelected(this, &SBlueprintActionMenu::OnNamespaceSelectedForImport)
					.OnGetNamespacesToExclude(this, &SBlueprintActionMenu::OnGetNamespacesToExcludeFromImportMenu)
					.ExcludedNamespaceTooltipText(LOCTEXT("CannotSelectNamespaceForImport", "This namespace has already been imported by this Blueprint."))
			]
		];
	}

	TSharedPtr<SComboButton> TargetContextSubMenuButton;
	// @TODO: would be nice if we could use a checkbox style for this, and have a different state for open/closed
	SAssignNew(TargetContextSubMenuButton, SComboButton)
		.MenuPlacement(MenuPlacement_MenuRight)
		.HasDownArrow(false)
		.ButtonStyle(FAppStyle::Get(), "BlueprintEditor.ContextMenu.TargetsButton")
		.ContentPadding(FMargin(5))
		.MenuContent()
		[
			SAssignNew(ContextTargetSubMenu, SBlueprintContextTargetMenu, MenuContext)
				.OnTargetMaskChanged(this, &SBlueprintActionMenu::OnContextTargetsChanged)
				.CustomTargetContent()
				[
					AddImportTargetContent.ToSharedRef()
				]
		];

	// Build the widget layout
	SBorder::Construct( SBorder::FArguments()
		.BorderImage( FAppStyle::GetBrush("Menu.Background") )
		.Padding(5.0f)
		[
			// Achieving fixed width by nesting items within a fixed width box.
			SNew(SBox)
			.WidthOverride(400.0f)
			.HeightOverride(400.0f)
			[
				SNew(SVerticalBox)

				// TYPE OF SEARCH INDICATOR
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2, 2, 2, 5)
				[
					SNew(SHorizontalBox)

					// Type pill
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(0.0f, 0.0f, (ContextIcon != nullptr) ? 5.0f : 0.0f, 0.0f)
					[
						SNew(SImage)
						.ColorAndOpacity(TypeColor)
						.Visibility(this, &SBlueprintActionMenu::GetTypeImageVisibility)
						.Image(ContextIcon)
					]

					// Search context description
					+SHorizontalBox::Slot()
					.FillWidth(1.f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(this, &SBlueprintActionMenu::GetSearchContextDesc)
						.Font(FAppStyle::GetFontStyle(FName("BlueprintEditor.ActionMenu.ContextDescriptionFont")))
						.ToolTip(IDocumentation::Get()->CreateToolTip(
							LOCTEXT("BlueprintActionMenuContextTextTooltip", "Describes the current context of the action list"),
							NULL,
							TEXT("Shared/Editors/BlueprintEditor"),
							TEXT("BlueprintActionMenuContextText")))
						.AutoWrapText(true)
					]

					// Context Toggle
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SCheckBox)
						.OnCheckStateChanged(this, &SBlueprintActionMenu::OnContextToggleChanged)
						.IsChecked(this, &SBlueprintActionMenu::ContextToggleIsChecked)
						.ToolTip(IDocumentation::Get()->CreateToolTip(
							LOCTEXT("BlueprintActionMenuContextToggleTooltip", "Should the list be filtered to only actions that make sense in the current context?"),
							NULL,
							TEXT("Shared/Editors/BlueprintEditor"),
							TEXT("BlueprintActionMenuContextToggle")))
						[
							SNew(STextBlock)
							.Text(LOCTEXT("BlueprintActionMenuContextToggle", "Context Sensitive"))
						]
					]

					+SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(3.f, 0.f, 0.f, 0.f)
					[
						TargetContextSubMenuButton.ToSharedRef()
					]
				]

				// ACTION LIST 
				+SVerticalBox::Slot()
				[
					SAssignNew(GraphActionMenu, SGraphActionMenu)
						.OnActionSelected(this, &SBlueprintActionMenu::OnActionSelected)
						.OnCreateWidgetForAction(SGraphActionMenu::FOnCreateWidgetForAction::CreateSP(this, &SBlueprintActionMenu::OnCreateWidgetForAction))
						.OnGetActionList(this, &SBlueprintActionMenu::OnGetActionList)
						.OnCreateCustomRowExpander_Static(&CreateCustomBlueprintActionExpander)
						.DraggedFromPins(DraggedFromPins)
						.GraphObj(GraphObj)
				]

				// PROGRESS BAR
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SBox)
					.HeightOverride(2)
					.Visibility_Lambda([this]()
					{
						return ContextMenuBuilder.IsValid() && ContextMenuBuilder->GetNumPendingActions() > 0 ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed;
					})
					[
						SNew(SProgressBar)
						.BorderPadding(FVector2D(0, 0))
						.Percent_Lambda([this]()
						{
							return ContextMenuBuilder.IsValid() ? ContextMenuBuilder->GetPendingActionsProgress() : 0.0f;
						})
					]
				]
			]
		]
	);
}

EVisibility SBlueprintActionMenu::GetTypeImageVisibility() const
{
	if (DraggedFromPins.Num() == 1 && EditorPtr.Pin()->GetIsContextSensitive())
	{
		UEdGraphPin* OnePin = DraggedFromPins[0];

		const UEdGraphSchema* Schema = OnePin->GetSchema();
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

		if (!Schema->IsA(UEdGraphSchema_K2::StaticClass()) || !K2Schema->IsExecPin(*OnePin))
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}

FText SBlueprintActionMenu::GetSearchContextDesc() const
{
	bool bIsContextSensitive = EditorPtr.Pin()->GetIsContextSensitive();
	bool bHasPins = DraggedFromPins.Num() > 0;
	if (!bIsContextSensitive)
	{
		return LOCTEXT("MenuPrompt_AllPins", "All Possible Actions");
	}
	else if (!bHasPins)
	{
		return LOCTEXT("MenuPrompt_BlueprintActions", "All Actions for this Blueprint");
	}
	else if (DraggedFromPins.Num() == 1)
	{
		UEdGraphPin* OnePin = DraggedFromPins[0];

		const UEdGraphSchema* Schema = OnePin->GetSchema();
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

		if (Schema->IsA(UEdGraphSchema_K2::StaticClass()) && K2Schema->IsExecPin(*OnePin))
		{
			return LOCTEXT("MenuPrompt_ExecPin", "Executable actions");
		}
		else
		{
			// Get the type string
			const FString TypeStringRaw = UEdGraphSchema_K2::TypeToText(OnePin->PinType).ToString();

			//@TODO: Add a parameter to TypeToText indicating the kind of formating requested
			const FString TypeString = (TypeStringRaw.Replace(TEXT("'"), TEXT(" "))).TrimEnd();

			if (OnePin->Direction == EGPD_Input)
			{
				return FText::Format(LOCTEXT("MenuPrompt_InputPin", "Actions providing a(n) {0}"), FText::FromString(TypeString));
			}
			else
			{
				return FText::Format(LOCTEXT("MenuPrompt_OutputPin", "Actions taking a(n) {0}"), FText::FromString(TypeString));
			}
		}
	}
	else
	{
		return FText::Format(LOCTEXT("MenuPrompt_ManyPins", "Actions for {0} pins"), FText::AsNumber(DraggedFromPins.Num()));
	}
}

void SBlueprintActionMenu::OnContextToggleChanged(ECheckBoxState CheckState)
{
	EditorPtr.Pin()->GetIsContextSensitive() = CheckState == ECheckBoxState::Checked;
	GraphActionMenu->RefreshAllActions(true, false);
}

void SBlueprintActionMenu::OnContextTargetsChanged(uint32 /*ContextTargetMask*/)
{
	GraphActionMenu->RefreshAllActions(/*bPreserveExpansion =*/true, /*bHandleOnSelectionEvent =*/false);
}

ECheckBoxState SBlueprintActionMenu::ContextToggleIsChecked() const
{
	return EditorPtr.Pin()->GetIsContextSensitive() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

TSharedRef<FGraphActionListBuilderBase> SBlueprintActionMenu::OnGetActionList()
{	
	check(EditorPtr.IsValid());
	TSharedPtr<FBlueprintEditor> BlueprintEditor = EditorPtr.Pin();
	bool const bIsContextSensitive = BlueprintEditor->GetIsContextSensitive();

	uint32 ContextTargetMask = 0;
	if (bIsContextSensitive && ContextTargetSubMenu.IsValid())
	{
		ContextTargetMask = ContextTargetSubMenu->GetContextTargetMask();
	}

	FBlueprintActionContext FilterContext;
	ConstructActionContext(FilterContext);

	FBlueprintActionMenuBuilder::EConfigFlags ConfigFlags = FBlueprintActionMenuBuilder::DefaultConfig;
	if (GetDefault<UBlueprintEditorSettings>()->bEnableContextMenuTimeSlicing)
	{
		ConfigFlags |= FBlueprintActionMenuBuilder::UseTimeSlicing;
	}

	ContextMenuBuilder = MakeShared<FBlueprintActionMenuBuilder>(ConfigFlags);

	// NOTE: cannot call GetGraphContextActions() during serialization and GC due to its use of FindObject()
	if (!UE::IsSavingPackage() && !IsGarbageCollecting() && FilterContext.Blueprints.Num() > 0)
	{
		FBlueprintActionMenuUtils::MakeContextMenu(FilterContext, bIsContextSensitive, ContextTargetMask, *ContextMenuBuilder);
	}
	
	// also try adding promote to variable if we can do so.
	TryInsertPromoteToVariable(FilterContext, *ContextMenuBuilder);

	// give the schema the opportunity to add another action
	if (const UEdGraphSchema* Schema = Cast<const UEdGraphSchema>(GraphObj->GetSchema()))
	{
		Schema->InsertAdditionalActions(FilterContext.Blueprints, FilterContext.Graphs, FilterContext.Pins, *ContextMenuBuilder);
	}

	return ContextMenuBuilder.ToSharedRef();
}

void SBlueprintActionMenu::ConstructActionContext(FBlueprintActionContext& ContextDescOut)
{
	check(EditorPtr.IsValid());
	TSharedPtr<FBlueprintEditor> BlueprintEditor = EditorPtr.Pin();
	bool const bIsContextSensitive = BlueprintEditor->GetIsContextSensitive();

	// we still want context from the graph (even if the user has unchecked
	// "Context Sensitive"), otherwise the user would be presented with nodes
	// that can't be placed in the graph... if the user isn't being presented
	// with a valid node, then fix it up in filtering
	ContextDescOut.Graphs.Add(GraphObj);

	UBlueprint* Blueprint = BlueprintEditor->GetBlueprintObj();
	const bool bBlueprintIsValid = IsValid(Blueprint) && Blueprint->GeneratedClass && (Blueprint->GeneratedClass->ClassGeneratedBy == Blueprint);
	if (!ensure(bBlueprintIsValid))  // to track UE-11597 and UE-11595
	{
		return;
	}

	ContextDescOut.EditorPtr = EditorPtr;
	ContextDescOut.Blueprints.Add(Blueprint);

	if (bIsContextSensitive)
	{
		ContextDescOut.Pins = DraggedFromPins;

		// Get selection from the "My Blueprint" view.
		FEdGraphSchemaAction_K2Var* SelectedVar = BlueprintEditor->GetMyBlueprintWidget()->SelectionAsVar();
		if ((SelectedVar != nullptr) && (SelectedVar->GetProperty() != nullptr))
		{
			ContextDescOut.SelectedObjects.Add(SelectedVar->GetProperty());
		}
		// If the selection come from the SCS editor, add it to the filter context.
		else if (Blueprint->SkeletonGeneratedClass && BlueprintEditor->GetSubobjectEditor().IsValid())
		{
			TArray<FSubobjectEditorTreeNodePtrType> Nodes = BlueprintEditor->GetSubobjectEditor()->GetSelectedNodes();
			if (Nodes.Num() == 1 && Nodes[0]->IsComponentNode())
			{
				FName PropertyName = Nodes[0]->GetVariableName();
				FObjectProperty* VariableProperty = FindFProperty<FObjectProperty>(Blueprint->SkeletonGeneratedClass, PropertyName);
				ContextDescOut.SelectedObjects.Add(VariableProperty);
			}
		}
	}
}

TSharedRef<SEditableTextBox> SBlueprintActionMenu::GetFilterTextBox()
{
	return GraphActionMenu->GetFilterTextBox();
}


TSharedRef<SWidget> SBlueprintActionMenu::OnCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData)
{
	InCreateData->bHandleMouseButtonDown = true;
	return SNew(SBlueprintPaletteItem, InCreateData, EditorPtr.Pin()); 
}

void SBlueprintActionMenu::OnActionSelected( const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedAction, ESelectInfo::Type InSelectionType )
{
	if (InSelectionType == ESelectInfo::OnMouseClick  || InSelectionType == ESelectInfo::OnKeyPress || SelectedAction.Num() == 0)
	{
		for ( int32 ActionIndex = 0; ActionIndex < SelectedAction.Num(); ActionIndex++ )
		{
			if ( SelectedAction[ActionIndex].IsValid() && GraphObj != nullptr )
			{
				// Don't dismiss when clicking on dummy action
				if ( !bActionExecuted && (SelectedAction[ActionIndex]->GetTypeId() != FEdGraphSchemaAction_Dummy::StaticGetTypeId()))
				{
					FSlateApplication::Get().DismissAllMenus();
					bActionExecuted = true;
				}

				UEdGraphNode* ResultNode = SelectedAction[ActionIndex]->PerformAction(GraphObj, DraggedFromPins, NewNodePosition);

				if ( ResultNode != nullptr )
				{
					NewNodePosition.Y += UEdGraphSchema_K2::EstimateNodeHeight( ResultNode );

					TSharedPtr<FBlueprintEditor> BlueprintEditorPtr = EditorPtr.Pin();
					if (BlueprintEditorPtr.IsValid())
					{
						// Determine which namespace(s) to import, based on the node's external dependencies.
						TSet<FString> NamespacesToImport;
						TArray<UStruct*> ExternalDependencies;
						if (ResultNode->HasExternalDependencies(&ExternalDependencies))
						{
							for (const UStruct* ExternalDependency : ExternalDependencies)
							{
								FBlueprintNamespaceUtilities::GetDefaultImportsForObject(ExternalDependency, NamespacesToImport);
							}
						}

						if (NamespacesToImport.Num() > 0)
						{
							// Auto-import the namespace(s) gathered above. Additional type objects within the imported scope may be loaded here.
							FBlueprintEditor::FImportNamespaceExParameters Params;
							Params.NamespacesToImport = MoveTemp(NamespacesToImport);
							BlueprintEditorPtr->ImportNamespaceEx(Params);
						}
					}
				}
			}
		}
	}
}

void SBlueprintActionMenu::TryInsertPromoteToVariable(FBlueprintActionContext const& MenuContext, FGraphActionListBuilderBase& OutAllActions)
{
	// If we can promote this to a variable add a menu entry to do so.
	const UEdGraphSchema_K2* K2Schema = Cast<const UEdGraphSchema_K2>(GraphObj->GetSchema());
	if ((K2Schema != nullptr) && (MenuContext.Pins.Num() > 0))
	{
		if (K2Schema->CanPromotePinToVariable(*MenuContext.Pins[0], true))
		{
			TSharedPtr<FBlueprintAction_PromoteVariable> PromoteAction = TSharedPtr<FBlueprintAction_PromoteVariable>(new FBlueprintAction_PromoteVariable(true));
			PromoteAction->MyBlueprintEditor = EditorPtr;
			OutAllActions.AddAction(PromoteAction);
		}

		if (MenuContext.Graphs.Num() == 1 && FBlueprintEditorUtils::DoesSupportLocalVariables(MenuContext.Graphs[0]) && K2Schema->CanPromotePinToVariable(*MenuContext.Pins[0], false))
		{
			TSharedPtr<FBlueprintAction_PromoteVariable> LocalPromoteAction = TSharedPtr<FBlueprintAction_PromoteVariable>(new FBlueprintAction_PromoteVariable(false));
			LocalPromoteAction->MyBlueprintEditor = EditorPtr;
			OutAllActions.AddAction( LocalPromoteAction );
		}
	}
}

void SBlueprintActionMenu::OnGetNamespacesToExcludeFromImportMenu(TSet<FString>& OutNamespacesToExclude)
{
	FBlueprintActionContext MenuContext;
	ConstructActionContext(MenuContext);

	FBlueprintNamespaceUtilities::GetSharedGlobalImports(OutNamespacesToExclude);

	for (const UBlueprint* Blueprint : MenuContext.Blueprints)
	{
		FBlueprintNamespaceUtilities::GetDefaultImportsForObject(Blueprint, OutNamespacesToExclude);
		OutNamespacesToExclude.Append(Blueprint->ImportedNamespaces);
	}
}

void SBlueprintActionMenu::OnNamespaceSelectedForImport(const FString& InNamespace)
{
	TSharedPtr<FBlueprintEditor> BlueprintEditorPtr = EditorPtr.Pin();
	if (BlueprintEditorPtr.IsValid())
	{
		FBlueprintEditor::FImportNamespaceExParameters Params;
		Params.NamespacesToImport.Add(InNamespace);
		Params.OnPostImportCallback = FSimpleDelegate::CreateLambda([GraphActionMenu = this->GraphActionMenu]()
		{
			// Now that additional types have been loaded/imported, update the menu to include any additional action(s).
			const bool bPreserveExpansion = true;
			const bool bHandleOnSelectionEvent = false;
			GraphActionMenu->RefreshAllActions(bPreserveExpansion, bHandleOnSelectionEvent);
		});

		// Auto-import the namespace into the current editor context. This may load additional type assets.
		BlueprintEditorPtr->ImportNamespaceEx(Params);
	}
}

void SBlueprintActionMenu::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SBorder::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	int32 NewIdxStart = ContextMenuBuilder->GetNumActions();
	if (ContextMenuBuilder.IsValid() && ContextMenuBuilder->ProcessPendingActions())
	{
		GraphActionMenu->UpdateForNewActions(NewIdxStart);
	}
}

/*******************************************************************************
* SBlueprintActionMenuExpander
*******************************************************************************/

void SBlueprintActionMenuExpander::Construct(const FArguments& InArgs, const FCustomExpanderData& ActionMenuData)
{
	OwnerRowPtr  = ActionMenuData.TableRow;
	SetIndentAmount(InArgs._IndentAmount);
	ActionPtr    = ActionMenuData.RowAction;

	if (!ActionPtr.IsValid())
	{
		SExpanderArrow::FArguments SuperArgs;
		SuperArgs._IndentAmount = InArgs._IndentAmount;

		SExpanderArrow::Construct(SuperArgs, ActionMenuData.TableRow);
	}
	else
	{			
		ChildSlot
			.Padding(TAttribute<FMargin>(this, &SBlueprintActionMenuExpander::GetCustomIndentPadding))
			[
				SNew(SBlueprintActionFavoriteToggle, ActionMenuData)
			];
	}
}

FMargin SBlueprintActionMenuExpander::GetCustomIndentPadding() const
{
	FMargin CustomPadding = SExpanderArrow::GetExpanderPadding();
	// if this is a action row (not a category or separator)
	if (ActionPtr.IsValid())
	{
		// flip the left/right margins (we want the favorite toggle aligned to the far left)
		//CustomPadding = FMargin(CustomPadding.Right, CustomPadding.Top, CustomPadding.Left, CustomPadding.Bottom);
	}
	return CustomPadding;
}

#undef LOCTEXT_NAMESPACE
