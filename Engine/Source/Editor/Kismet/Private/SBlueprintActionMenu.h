// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "GraphEditor.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Views/SExpanderArrow.h"

class FBlueprintEditor;
class SBlueprintContextTargetMenu;
class SEditableTextBox;
class SGraphActionMenu;
class SWidget;
class UEdGraph;
class UEdGraphPin;
struct FBlueprintActionContext;
struct FCustomExpanderData;
struct FEdGraphSchemaAction;
struct FGraphActionListBuilderBase;
struct FBlueprintActionMenuBuilder;

/*******************************************************************************
* SBlueprintActionMenu
*******************************************************************************/

class SBlueprintActionMenu : public SBorder
{
public:
	/** Delegate for the OnCloseReason event which is always raised when the SBlueprintActionMenu closes */
	DECLARE_DELEGATE_ThreeParams( FClosedReason, bool /*bActionExecuted*/, bool /*bContextSensitiveChecked*/, bool /*bGraphPinContext*/);

	SLATE_BEGIN_ARGS( SBlueprintActionMenu )
		: _GraphObj( static_cast<UEdGraph*>(NULL) )
		, _NewNodePosition( FVector2D::ZeroVector )
		, _AutoExpandActionMenu( false )
	{}

		SLATE_ARGUMENT( UEdGraph*, GraphObj )
		SLATE_ARGUMENT( FVector2D, NewNodePosition )
		SLATE_ARGUMENT( TArray<UEdGraphPin*>, DraggedFromPins )
		SLATE_ARGUMENT( SGraphEditor::FActionMenuClosed, OnClosedCallback )
		SLATE_ARGUMENT( bool, AutoExpandActionMenu )
		SLATE_EVENT( FClosedReason, OnCloseReason )
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, TSharedPtr<FBlueprintEditor> InEditor );

	~SBlueprintActionMenu();

	TSharedRef<SEditableTextBox> GetFilterTextBox();

	// SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	// End of SWidget interface

protected:
	/** UI Callback functions */
	EVisibility GetTypeImageVisibility() const;
	FText GetSearchContextDesc() const;
	void OnContextToggleChanged(ECheckBoxState CheckState);
	ECheckBoxState ContextToggleIsChecked() const;
	void OnContextTargetsChanged(uint32 ContextTargetMask);

	void OnActionSelected( const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedAction, ESelectInfo::Type InSelectionType );

	TSharedRef<SWidget> OnCreateWidgetForAction(struct FCreateWidgetForActionData* const InCreateData);

	/** Callback used to populate all actions list in SGraphActionMenu */
	TSharedRef<FGraphActionListBuilderBase> OnGetActionList();

	/**  */
	void ConstructActionContext(FBlueprintActionContext& ContextDescOut);

	/** Function to try to insert a promote to variable entry if it is possible to do so. */
	void TryInsertPromoteToVariable(FBlueprintActionContext const& Context, FGraphActionListBuilderBase& OutAllActions);

	/** Invoked when an available namespace is selected for import. */
	void OnNamespaceSelectedForImport(const FString& InNamespace);

	/** Gathers the list of namespaces to exclude from the import menu. */
	void OnGetNamespacesToExcludeFromImportMenu(TSet<FString>& OutNamespacesToExclude);

private:
	UEdGraph* GraphObj;
	TArray<UEdGraphPin*> DraggedFromPins;
	FDeprecateSlateVector2D NewNodePosition;
	bool bAutoExpandActionMenu;

	SGraphEditor::FActionMenuClosed OnClosedCallback;
	FClosedReason OnCloseReasonCallback;

	TSharedPtr<SGraphActionMenu> GraphActionMenu;
	TWeakPtr<FBlueprintEditor> EditorPtr;
	TSharedPtr<SBlueprintContextTargetMenu> ContextTargetSubMenu;
	TSharedPtr<FBlueprintActionMenuBuilder> ContextMenuBuilder;

	bool bActionExecuted;
};

/*******************************************************************************
* SBlueprintActionMenuExpander
*******************************************************************************/

class SBlueprintActionMenuExpander : public SExpanderArrow
{
	SLATE_BEGIN_ARGS( SBlueprintActionMenuExpander ) {}
		SLATE_ATTRIBUTE(float, IndentAmount)
	SLATE_END_ARGS()

public:
	/**
	 * Constructs a standard SExpanderArrow widget if the associated menu item 
	 * is a category or separator, otherwise, for action items, it constructs
	 * a favoriting toggle (plus indent) in front of the action entry.
	 * 
	 * @param  InArgs			A set of slate arguments for this widget type (defined above).
	 * @param  ActionMenuData	A set of useful data for detailing the specific action menu row this is for.
	 */
	void Construct(const FArguments& InArgs, const FCustomExpanderData& ActionMenuData);

private:
	/**
	 * Action menu expanders are also responsible for properly indenting the 
	 * menu entries, so this returns the proper margin padding for the menu row
	 * (based off its indent level).
	 * 
	 * @return Padding to construct around this widget (so the menu entry is properly indented).
	 */
	FMargin GetCustomIndentPadding() const;

	/** The action associated with the menu row this belongs to */
	TWeakPtr<FEdGraphSchemaAction> ActionPtr;
};
