// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBlueprintBookmarks.h"

#include "BlueprintEditor.h"
#include "BlueprintEditorSettings.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraphNode_Comment.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Guid.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Views/SHeaderRow.h"

class ITableRow;
class SWidget;

#define LOCTEXT_NAMESPACE "BlueprintBookmarks"

void SBlueprintBookmarks::Construct(const FArguments& InArgs)
{
	EditorContext = InArgs._EditorContext;

	BookmarksRootNode.DisplayName = LOCTEXT("BookmarksRootNodeLabel", "Bookmarks");
	CommentsRootNode.DisplayName = LOCTEXT("CommentsRootNodeLabel", "Comment Nodes");

	CommandList = MakeShareable(new FUICommandList);
	CommandList->MapAction(FGenericCommands::Get().Delete,
		FUIAction(
			FExecuteAction::CreateSP(this, &SBlueprintBookmarks::OnDeleteSelectedTreeViewItems),
			FCanExecuteAction::CreateSP(this, &SBlueprintBookmarks::CanDeleteSelectedTreeViewItems)
		)
	);
	CommandList->MapAction(FGenericCommands::Get().Rename,
		FUIAction(
			FExecuteAction::CreateSP(this, &SBlueprintBookmarks::OnRenameSelectedTreeViewItems),
			FCanExecuteAction::CreateSP(this, &SBlueprintBookmarks::CanRenameSelectedTreeViewItem)
		)
	);

	FMenuBuilder ViewOptions(true, nullptr);

	ViewOptions.AddMenuEntry(
		LOCTEXT("ShowCommentNodes", "Show Comment Blocks"),
		LOCTEXT("ShowCommentNodesTooltip", "Should comment blocks be shown in the tree view?"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SBlueprintBookmarks::OnToggleShowCommentNodes),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SBlueprintBookmarks::IsShowCommentNodesChecked)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton);

	ViewOptions.AddMenuEntry(
		LOCTEXT("ShowBookmarksForCurrentDocumentOnly", "Show Bookmarks for Current Graph Only"),
		LOCTEXT("ShowBookmarksForCurrentDocumentOnlyTooltip", "Only show bookmarks for the current graph."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SBlueprintBookmarks::OnToggleShowBookmarksForCurrentDocumentOnly),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SBlueprintBookmarks::IsShowBookmarksForCurrentDocumentOnlyChecked)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton);

	SAssignNew(SearchBoxWidget, SSearchBox)
	.SelectAllTextWhenFocused(true)
	.OnTextCommitted(this, &SBlueprintBookmarks::OnFilterTextCommitted)
	.OnTextChanged(this, &SBlueprintBookmarks::OnFilterTextCommitted, ETextCommit::Default);

	SAssignNew(TreeViewWidget, STreeView<FTreeViewItemPtr>)
	.TreeItemsSource(&TreeViewRootItems)
	.OnGenerateRow(this, &SBlueprintBookmarks::OnGenerateTreeViewRow)
	.OnGetChildren(this, &SBlueprintBookmarks::OnGetTreeViewChildren)
	.OnContextMenuOpening(this, &SBlueprintBookmarks::OnOpenTreeViewContextMenu)
	.OnMouseButtonDoubleClick(this, &SBlueprintBookmarks::OnTreeViewItemDoubleClick)
	.HeaderRow
	(
		SNew(SHeaderRow)
		+SHeaderRow::Column("Name")
		.DefaultLabel(LOCTEXT("NameColumnHeader", "Name"))
		.FillWidth(0.6f)
		+SHeaderRow::Column("Graph")
		.DefaultLabel(LOCTEXT("GraphColumnHeader", "Graph"))
		.FillWidth(0.4f)
	);

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.Padding(2.f)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(1.f)
				.Padding(2.f, 0.f)
				.VAlign(VAlign_Center)
				[
					SearchBoxWidget.ToSharedRef()
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.f, 0.f)
				[
					SNew(SButton)
					.Visibility(EVisibility::Collapsed) // @TODO: remove once folders are supported
					.ButtonStyle(FAppStyle::Get(), "FlatButton")
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Bookmarks.AddFolderButtonIcon"))
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.f, 0.f)
				[
					SNew(SComboButton)
					.ComboButtonStyle(FAppStyle::Get(), "ToolbarComboButton")
					.ForegroundColor(FSlateColor::UseForeground())
					.HasDownArrow(true)
					.ContentPadding(FMargin(1, 0))
					.MenuContent()
					[
						ViewOptions.MakeWidget()
					]
					.ButtonContent()
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("GenericViewButton"))
					]
				]
			]
		]
		+SVerticalBox::Slot()
		.FillHeight(1.f)
		.Padding(0.f, 2.0f, 0.f, 0.f)
		[
			SNew(SBorder)
			.Padding(2.f)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				TreeViewWidget.ToSharedRef()
			]
		]
	];

	RefreshBookmarksTree();
}

void SBlueprintBookmarks::STreeItemRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TWeakPtr<FBlueprintEditor> InEditorContext)
{
	ItemPtr = InArgs._ItemPtr;
	check(ItemPtr.IsValid());

	EditorContext = InEditorContext;
	SMultiColumnTableRow<FTreeViewItemPtr>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
}

TSharedRef<SWidget> SBlueprintBookmarks::STreeItemRow::GenerateWidgetForColumn(const FName& InColumnName)
{
	if (InColumnName == "Name")
	{
		TSharedPtr<SWidget> TextWidget;

		if (ItemPtr->IsBookmarkNode())
		{
			TSharedPtr<SInlineEditableTextBlock> InlineEditableTextWidget = SNew(SInlineEditableTextBlock)
				.Text(this, &SBlueprintBookmarks::STreeItemRow::GetItemNameText)
				.IsSelected(this, &STreeItemRow::IsSelectedExclusively)
				.OnTextCommitted(this, &STreeItemRow::OnNameTextCommitted);

			ItemPtr->OnRequestRenameDelegate = FTreeViewItem::FOnRequestRename::CreateSP(InlineEditableTextWidget.Get(), &SInlineEditableTextBlock::EnterEditingMode);

			TextWidget = InlineEditableTextWidget;
		}
		else
		{
			TextWidget = SNew(STextBlock)
				.Text(this, &SBlueprintBookmarks::STreeItemRow::GetItemNameText)
				.Font(FAppStyle::GetFontStyle(ItemPtr->IsRootNode() ? "Bookmarks.TreeViewRootItemFont" : "Bookmarks.TreeViewItemFont"));
		}

		return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SExpanderArrow, SharedThis(this))
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		[
			TextWidget.ToSharedRef()
		];
	}
	else if (InColumnName == "Graph")
	{
		const UEdGraph* GraphContext = nullptr;

		if (ItemPtr->IsBookmarkNode())
		{
			if (ensure(ItemPtr->DocumentInfo))
			{
				GraphContext = Cast<UEdGraph>(ItemPtr->DocumentInfo->EditedObjectPath.ResolveObject());
			}
		}
		else if (ItemPtr->IsCommentNode())
		{
			if (ensure(ItemPtr->DocumentInfo))
			{
				if (const UEdGraphNode* NodeContext = Cast<UEdGraphNode>(ItemPtr->DocumentInfo->EditedObjectPath.ResolveObject()))
				{
					GraphContext = NodeContext->GetGraph();
				}
			}
		}
		
		if (GraphContext)
		{
			return SNew(STextBlock)
				.Text_Lambda([GraphContext]() { return FText::FromName(GraphContext->GetFName()); })
				.Font(FAppStyle::GetFontStyle("Bookmarks.TreeViewItemFont"));
		}
	}

	return SNullWidget::NullWidget;
}

FText SBlueprintBookmarks::STreeItemRow::GetItemNameText() const
{
	return ItemPtr->BookmarkNode.DisplayName;
}

void SBlueprintBookmarks::STreeItemRow::OnNameTextCommitted(const FText& InNewName, ETextCommit::Type InTextCommit)
{
	TSharedPtr<FBlueprintEditor> BlueprintEditor = EditorContext.Pin();
	if (BlueprintEditor.IsValid())
	{
		UBlueprint* Blueprint = BlueprintEditor->GetBlueprintObj();
		check(Blueprint);

		Blueprint->Modify();
		ItemPtr->BookmarkNode.DisplayName = InNewName;
	}
}

void SBlueprintBookmarks::OnFilterTextCommitted(const FText& InText, ETextCommit::Type CommitType)
{
	FilterText = InText;

	RefreshBookmarksTree();
}

void SBlueprintBookmarks::OnDeleteSelectedTreeViewItems()
{
	TSharedPtr<FBlueprintEditor> BlueprintEditor = EditorContext.Pin();
	if (BlueprintEditor.IsValid())
	{
		for(FTreeViewItemPtr SelectedItem : TreeViewWidget->GetSelectedItems())
		{
			// The item contains a reference to the node in the source array, so use a local copy of the GUID since this will use RemoveSwap().
			const FGuid NodeGuid = SelectedItem->BookmarkNode.NodeGuid;
			BlueprintEditor->RemoveBookmark(NodeGuid, false);
		}

		RefreshBookmarksTree();
	}
}

bool SBlueprintBookmarks::CanDeleteSelectedTreeViewItems() const
{
	TArray<FTreeViewItemPtr> SelectedItems = TreeViewWidget->GetSelectedItems();
	bool bCanDelete = SelectedItems.Num() > 0;
	if(bCanDelete)
	{
		for (FTreeViewItemPtr SelectedItem : SelectedItems)
		{
			if (!SelectedItem->IsBookmarkNode())
			{
				bCanDelete = false;
				break;
			}
		}
	}

	return bCanDelete;
}

void SBlueprintBookmarks::OnRenameSelectedTreeViewItems()
{
	TArray<FTreeViewItemPtr> SelectedItems = TreeViewWidget->GetSelectedItems();
	if (SelectedItems.Num() == 1)
	{
		SelectedItems[0]->OnRequestRenameDelegate.ExecuteIfBound();
	}
}

bool SBlueprintBookmarks::CanRenameSelectedTreeViewItem() const
{
	TArray<FTreeViewItemPtr> SelectedItems = TreeViewWidget->GetSelectedItems();
	return SelectedItems.Num() == 1 && SelectedItems[0]->IsBookmarkNode();
}

TSharedRef<ITableRow> SBlueprintBookmarks::OnGenerateTreeViewRow(FTreeViewItemPtr TreeItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	check(TreeItem.IsValid());

	return SNew(STreeItemRow, OwnerTable, EditorContext)
		.ItemPtr(TreeItem);
}

void SBlueprintBookmarks::OnGetTreeViewChildren(FTreeViewItemPtr TreeItem, TArray<FTreeViewItemPtr>& OutChildren)
{
	OutChildren += TreeItem->Children;
}

bool SBlueprintBookmarks::CanNavigateToSelection() const
{
	TArray<FTreeViewItemPtr> SelectedItems = TreeViewWidget->GetSelectedItems();
	bool bCanNavigate = SelectedItems.Num() > 0;
	if (bCanNavigate)
	{
		for (FTreeViewItemPtr SelectedItem : SelectedItems)
		{
			if (SelectedItem->DocumentInfo == nullptr)
			{
				bCanNavigate = false;
				break;
			}
		}
	}

	return bCanNavigate;
}

void SBlueprintBookmarks::OnTreeViewItemDoubleClick(FTreeViewItemPtr TreeItem)
{
	TSharedPtr<FBlueprintEditor> BlueprintEditor = EditorContext.Pin();
	if (BlueprintEditor.IsValid() && TreeItem->DocumentInfo)
	{
		BlueprintEditor->JumpToHyperlink(TreeItem->DocumentInfo->EditedObjectPath.ResolveObject());

		if (TreeItem->IsBookmarkNode())
		{
			FGuid CurrentBookmarkId;
			BlueprintEditor->GetViewBookmark(CurrentBookmarkId);
			if (CurrentBookmarkId != TreeItem->BookmarkNode.NodeGuid)
			{
				BlueprintEditor->SetViewLocation(TreeItem->DocumentInfo->SavedViewOffset, TreeItem->DocumentInfo->SavedZoomAmount, TreeItem->BookmarkNode.NodeGuid);
			}
		}
	}
}

TSharedPtr<SWidget> SBlueprintBookmarks::OnOpenTreeViewContextMenu()
{
	TArray<FTreeViewItemPtr> SelectedItems = TreeViewWidget->GetSelectedItems();
	if (SelectedItems.Num() > 0)
	{
		FMenuBuilder ContextMenuBuilder(true, CommandList);

		ContextMenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
		ContextMenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename);
		ContextMenuBuilder.AddMenuEntry(LOCTEXT("ContextMenuItem_JumpToBookmark", "Jump to Bookmark"),
			LOCTEXT("ContextMenuItemTooltip_JumpToBookmark", "Navigate to this bookmark's location."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SBlueprintBookmarks::OnTreeViewItemDoubleClick, SelectedItems[0]),
				FCanExecuteAction::CreateSP(this, &SBlueprintBookmarks::CanNavigateToSelection)
			)
		);

		return ContextMenuBuilder.MakeWidget();
	}

	return SNullWidget::NullWidget;
}

void SBlueprintBookmarks::RefreshBookmarksTree()
{
	CommentNodes.Empty();
	CommentNodeInfo.Empty();
	TreeViewRootItems.Empty();

	UBlueprintEditorSettings* BlueprintEditorSettings = GetMutableDefault<UBlueprintEditorSettings>();

	FTreeViewItemPtr BookmarksTreeViewRoot = MakeShareable(new FTreeViewItem(ETreeViewNodeType::Root, BookmarksRootNode));
	TreeViewRootItems.Add(BookmarksTreeViewRoot);

	TreeViewWidget->SetItemExpansion(BookmarksTreeViewRoot, true);

	FTreeViewItemPtr CommentsTreeViewRoot;
	if (BlueprintEditorSettings->bIncludeCommentNodesInBookmarksTab)
	{
		CommentsTreeViewRoot = MakeShareable(new FTreeViewItem(ETreeViewNodeType::Root, CommentsRootNode));
		TreeViewRootItems.Add(CommentsTreeViewRoot);
	}

	TSharedPtr<FBlueprintEditor> BlueprintEditor = EditorContext.Pin();
	if (BlueprintEditor.IsValid())
	{
		UBlueprint* Blueprint = BlueprintEditor->GetBlueprintObj();
		check(Blueprint);

		const UEdGraph* FocusedGraph = BlueprintEditor->GetFocusedGraph();
		const bool bCurrentGraphOnly = BlueprintEditorSettings->bShowBookmarksForCurrentDocumentOnlyInTab;

		const FText& LocalFilterText = FilterText;
		auto IsNodeFilteredOutLambda = [LocalFilterText](const FBPEditorBookmarkNode& InNode)
		{
			return !LocalFilterText.IsEmpty() && !InNode.DisplayName.ToString().Contains(LocalFilterText.ToString());
		};

		for (FBPEditorBookmarkNode& BookmarkNode : BlueprintEditorSettings->BookmarkNodes)
		{
			if (!IsNodeFilteredOutLambda(BookmarkNode))
			{
				if (const FEditedDocumentInfo* BookmarkInfo = BlueprintEditorSettings->Bookmarks.Find(BookmarkNode.NodeGuid))
				{
					if (const UEdGraph* GraphContext = Cast<UEdGraph>(BookmarkInfo->EditedObjectPath.ResolveObject()))
					{
						if (Blueprint == FBlueprintEditorUtils::FindBlueprintForGraph(GraphContext))
						{
							if (!bCurrentGraphOnly || (GraphContext == FocusedGraph))
							{
								BookmarksTreeViewRoot->Children.Add(MakeShareable(new FTreeViewItem(ETreeViewNodeType::LocalBookmark, BookmarkNode, BookmarkInfo)));
							}
						}
					}
				}
			}
		}

		for (FBPEditorBookmarkNode& BookmarkNode : Blueprint->BookmarkNodes)
		{
			if (!IsNodeFilteredOutLambda(BookmarkNode))
			{
				const FEditedDocumentInfo* BookmarkInfo = Blueprint->Bookmarks.Find(BookmarkNode.NodeGuid);
				if (!bCurrentGraphOnly || (BookmarkInfo && BookmarkInfo->EditedObjectPath.ResolveObject() == FocusedGraph))
				{
					BookmarksTreeViewRoot->Children.Add(MakeShareable(new FTreeViewItem(ETreeViewNodeType::SharedBookmark, BookmarkNode, BookmarkInfo)));
				}
			}
		}

		if (CommentsTreeViewRoot.IsValid())
		{
			TArray<UEdGraph*> AllGraphs;
			Blueprint->GetAllGraphs(AllGraphs);

			TArray<UEdGraphNode_Comment*> GraphCommentNodes;
			for (UEdGraph* Graph : AllGraphs)
			{
				if (!bCurrentGraphOnly || Graph == FocusedGraph)
				{
					Graph->GetNodesOfClass(GraphCommentNodes);
				}
			}

			if (GraphCommentNodes.Num() > 0)
			{
				CommentNodes.Reserve(GraphCommentNodes.Num());
				CommentNodeInfo.Reserve(GraphCommentNodes.Num());

				for (UEdGraphNode_Comment* GraphCommentNode : GraphCommentNodes)
				{
					FBPEditorBookmarkNode& CommentNode = *new(CommentNodes) FBPEditorBookmarkNode;
					CommentNode.NodeGuid = FGuid::NewGuid();
					CommentNode.DisplayName = GraphCommentNode->GetNodeTitle(ENodeTitleType::EditableTitle);

					FEditedDocumentInfo& CommentInfo = CommentNodeInfo.Add(CommentNode.NodeGuid);
					CommentInfo.EditedObjectPath = GraphCommentNode;
					
					if (!IsNodeFilteredOutLambda(CommentNode))
					{
						CommentsTreeViewRoot->Children.Add(MakeShareable(new FTreeViewItem(ETreeViewNodeType::Comment, CommentNode, &CommentInfo)));
					}
				}
			}

			TreeViewWidget->SetItemExpansion(CommentsTreeViewRoot, true);
		}
	}

	TreeViewWidget->RequestTreeRefresh();
}

bool SBlueprintBookmarks::IsShowCommentNodesChecked() const
{
	return GetDefault<UBlueprintEditorSettings>()->bIncludeCommentNodesInBookmarksTab;
}

void SBlueprintBookmarks::OnToggleShowCommentNodes()
{
	UBlueprintEditorSettings* Settings = GetMutableDefault<UBlueprintEditorSettings>();
	Settings->bIncludeCommentNodesInBookmarksTab = !Settings->bIncludeCommentNodesInBookmarksTab;
	Settings->PostEditChange();
	Settings->SaveConfig();

	RefreshBookmarksTree();
}

bool SBlueprintBookmarks::IsShowBookmarksForCurrentDocumentOnlyChecked() const
{
	return GetDefault<UBlueprintEditorSettings>()->bShowBookmarksForCurrentDocumentOnlyInTab;
}

void SBlueprintBookmarks::OnToggleShowBookmarksForCurrentDocumentOnly()
{
	UBlueprintEditorSettings* Settings = GetMutableDefault<UBlueprintEditorSettings>();
	Settings->bShowBookmarksForCurrentDocumentOnlyInTab = !Settings->bShowBookmarksForCurrentDocumentOnlyInTab;
	Settings->PostEditChange();
	Settings->SaveConfig();

	RefreshBookmarksTree();
}

#undef LOCTEXT_NAMESPACE
