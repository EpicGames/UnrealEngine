// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Delegates/Delegate.h"
#include "Engine/Blueprint.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateConstants.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

class FBlueprintEditor;
class FUICommandList;
class ITableRow;
class SSearchBox;
class SWidget;
struct FGuid;

class SBlueprintBookmarks : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SBlueprintBookmarks)
	{}
		SLATE_ARGUMENT(TSharedPtr<FBlueprintEditor>, EditorContext)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	void RefreshBookmarksTree();

protected:
	enum class ETreeViewNodeType
	{
		Root,
		Folder,
		Comment,
		LocalBookmark,
		SharedBookmark
	};

	struct FTreeViewItem : public TSharedFromThis<FTreeViewItem>
	{
		DECLARE_DELEGATE(FOnRequestRename);

		ETreeViewNodeType NodeType;
		FBPEditorBookmarkNode& BookmarkNode;
		const FEditedDocumentInfo* DocumentInfo;
		TArray<TSharedPtr<FTreeViewItem>> Children;
		FOnRequestRename OnRequestRenameDelegate;

		FTreeViewItem(ETreeViewNodeType InNodeType, FBPEditorBookmarkNode& InBookmarkNode, const FEditedDocumentInfo* InDocumentInfo = nullptr)
			:NodeType(InNodeType)
			,BookmarkNode(InBookmarkNode)
			,DocumentInfo(InDocumentInfo)
		{}

		FORCEINLINE bool IsRootNode() const
		{
			return NodeType == ETreeViewNodeType::Root;
		}

		FORCEINLINE bool IsBookmarkNode() const
		{
			return NodeType == ETreeViewNodeType::LocalBookmark || NodeType == ETreeViewNodeType::SharedBookmark;
		}

		FORCEINLINE bool IsCommentNode() const
		{
			return NodeType == ETreeViewNodeType::Comment;
		}

		FORCEINLINE bool IsFolderNode() const
		{
			return NodeType == ETreeViewNodeType::Folder;
		}
	};

	typedef TSharedPtr<FTreeViewItem> FTreeViewItemPtr;

	class STreeItemRow : public SMultiColumnTableRow<FTreeViewItemPtr>
	{
	public:
		SLATE_BEGIN_ARGS(STreeItemRow) {}
			SLATE_ARGUMENT(FTreeViewItemPtr, ItemPtr)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TWeakPtr<FBlueprintEditor> InEditorContext);
		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;

	protected:
		FText GetItemNameText() const;
		void OnNameTextCommitted(const FText& InNewName, ETextCommit::Type InTextCommit);

	private:
		FTreeViewItemPtr ItemPtr;
		TWeakPtr<FBlueprintEditor> EditorContext;
	};

	void OnFilterTextCommitted(const FText& InText, ETextCommit::Type CommitType);

	void OnDeleteSelectedTreeViewItems();
	bool CanDeleteSelectedTreeViewItems() const;
	void OnRenameSelectedTreeViewItems();
	bool CanRenameSelectedTreeViewItem() const;

	bool CanNavigateToSelection() const;
	TSharedRef<ITableRow> OnGenerateTreeViewRow(FTreeViewItemPtr TreeItem, const TSharedRef<STableViewBase>& OwnerTable);
	void OnGetTreeViewChildren(FTreeViewItemPtr TreeItem, TArray<FTreeViewItemPtr>& OutChildren);
	void OnTreeViewItemDoubleClick(FTreeViewItemPtr TreeItem);
	TSharedPtr<SWidget> OnOpenTreeViewContextMenu();

	bool IsShowCommentNodesChecked() const;
	void OnToggleShowCommentNodes();

	bool IsShowBookmarksForCurrentDocumentOnlyChecked() const;
	void OnToggleShowBookmarksForCurrentDocumentOnly();

private:
	TWeakPtr<FBlueprintEditor> EditorContext;

	FBPEditorBookmarkNode CommentsRootNode;
	FBPEditorBookmarkNode BookmarksRootNode;
	TArray<FBPEditorBookmarkNode> CommentNodes;
	TMap<FGuid, FEditedDocumentInfo> CommentNodeInfo;

	TArray<FTreeViewItemPtr> TreeViewRootItems;

	TSharedPtr<FUICommandList> CommandList;
	TSharedPtr<SSearchBox> SearchBoxWidget;
	TSharedPtr<STreeView<FTreeViewItemPtr> > TreeViewWidget;

	FText FilterText;
};
