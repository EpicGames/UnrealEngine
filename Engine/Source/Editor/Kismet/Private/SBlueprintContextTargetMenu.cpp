// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBlueprintContextTargetMenu.h"

#include "BlueprintActionFilter.h"
#include "BlueprintEditorSettings.h"
#include "Components/ActorComponent.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "CoreGlobals.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Misc/ConfigCacheIni.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ReflectedTypeAccessors.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

#define LOCTEXT_NAMESPACE "SBlueprintContextTargetMenu"

/*******************************************************************************
 * FContextMenuTargetProfile
 ******************************************************************************/
 
namespace ContextMenuTargetProfileImpl
{
	const FString ConfigSection("BlueprintEditor.ContextTargets");
	const FString SharedProfileName("SharedMenuProfile");

	/**
	 * Query methods to check and see if the user is dragging off a specific pin
	 * type (for the context menu).
	 * 
	 * @param  MenuContext    Defines the context from which the menu was requested.
	 * @return True if the context contains a pin matching the type in question, otherwise false.
	 */
	static bool HasExecPinContext(const FBlueprintActionContext& MenuContext);
	static bool HasComponentPinContext(const FBlueprintActionContext& MenuContext);
	static bool HasObjectPinContext(const FBlueprintActionContext& MenuContext);
	static bool HasClassPinContext(const FBlueprintActionContext& MenuContext);
	static bool HasInterfacePinContext(const FBlueprintActionContext& MenuContext);

	static bool IsObjectPin(UEdGraphPin* Pin);
	static UClass* GetPinClass(UEdGraphPin const* Pin);
	static bool HasAnyExposedComponents(UClass* TargetClass);

	/**
	 * Returns a user facing description that provides a short succinct title,
	 * discerning this profile from others.
	 */
	static FText GetProfileDescription(const FBlueprintActionContext& MenuContext);

	/**
	 * Determines which target options are unused, given the current context.
	 */
	static uint32 GetIncompatibleTargetFlags(const FBlueprintActionContext& MenuContext, uint32& HasComponentsMaskOut);

	/**
	 * Controls how user set targets are saved. Determines how to split context 
	 * scenarios by providing a separate save name for each.
	 */
	static FString GetProfileSaveName(const FBlueprintActionContext& MenuContext);
}

//------------------------------------------------------------------------------
static bool ContextMenuTargetProfileImpl::HasExecPinContext(const FBlueprintActionContext& MenuContext)
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	for (UEdGraphPin* ContextPin : MenuContext.Pins)
	{
		if (K2Schema->IsExecPin(*ContextPin))
		{
			return true;
		}
	}
	return false;
}

//------------------------------------------------------------------------------
static bool ContextMenuTargetProfileImpl::HasComponentPinContext(const FBlueprintActionContext& MenuContext)
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	for (UEdGraphPin* ContextPin : MenuContext.Pins)
	{
		UClass* PinClass = GetPinClass(ContextPin);
		if ((PinClass != nullptr) && PinClass->IsChildOf<UActorComponent>())
		{
			return true;
		}
	}
	return false;
}

//------------------------------------------------------------------------------
static bool ContextMenuTargetProfileImpl::HasObjectPinContext(const FBlueprintActionContext& MenuContext)
{
	for (UEdGraphPin* ContextPin : MenuContext.Pins)
	{
		if (IsObjectPin(ContextPin))
		{
			return true;
		}
	}
	return false;
}

//------------------------------------------------------------------------------
static bool ContextMenuTargetProfileImpl::HasClassPinContext(const FBlueprintActionContext& MenuContext)
{
	for (UEdGraphPin* ContextPin : MenuContext.Pins)
	{
		if (ContextPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Class)
		{
			return true;
		}
	}
	return false;
}

//------------------------------------------------------------------------------
static bool ContextMenuTargetProfileImpl::HasInterfacePinContext(const FBlueprintActionContext& MenuContext)
{
	for (UEdGraphPin* ContextPin : MenuContext.Pins)
	{
		if (ContextPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Interface)
		{
			return true;
		}
	}
	return false;
}

//------------------------------------------------------------------------------
static bool ContextMenuTargetProfileImpl::IsObjectPin(UEdGraphPin* Pin)
{
	return GetPinClass(Pin) != nullptr;
}

//------------------------------------------------------------------------------
static UClass* ContextMenuTargetProfileImpl::GetPinClass(UEdGraphPin const* Pin)
{
	UClass* PinObjClass = nullptr;

	FEdGraphPinType const& PinType = Pin->PinType;
	if ((PinType.PinCategory == UEdGraphSchema_K2::PC_Object) ||
		(PinType.PinCategory == UEdGraphSchema_K2::PC_Interface))
	{
		bool const bIsSelfPin = !PinType.PinSubCategoryObject.IsValid();
		if (bIsSelfPin)
		{
			PinObjClass = CastChecked<UK2Node>(Pin->GetOwningNode())->GetBlueprint()->SkeletonGeneratedClass;
		}
		else
		{
			PinObjClass = Cast<UClass>(PinType.PinSubCategoryObject.Get());
			if (PinObjClass != nullptr)
			{
				if (UBlueprint* ClassBlueprint = Cast<UBlueprint>(PinObjClass->ClassGeneratedBy))
				{
					if (ClassBlueprint->SkeletonGeneratedClass != nullptr)
					{
						PinObjClass = ClassBlueprint->SkeletonGeneratedClass;
					}
				}
			}
		}
	}

	return PinObjClass;
}

//------------------------------------------------------------------------------
static bool ContextMenuTargetProfileImpl::HasAnyExposedComponents(UClass* TargetClass)
{
	for (TFieldIterator<FObjectProperty> PropertyIt(TargetClass, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
	{
		FObjectProperty* ObjectProperty = *PropertyIt;
		if (!ObjectProperty->HasAnyPropertyFlags(CPF_BlueprintVisible) || !ObjectProperty->PropertyClass->IsChildOf<UActorComponent>())
		{
			continue;
		}

		return true;
	}
	return false;
}

//------------------------------------------------------------------------------
FText ContextMenuTargetProfileImpl::GetProfileDescription(const FBlueprintActionContext& MenuContext)
{
	if (MenuContext.Pins.Num() == 0)
	{
		return LOCTEXT("BlueprintContextTarget", "Blueprint Context Target(s)");
	}
	else if (HasComponentPinContext(MenuContext))
	{
		return LOCTEXT("ComponentContextTarget", "Component Context Target(s)");
	}
// 	else if (HasClassPinContext(MenuContext))
// 	{
// 		return LOCTEXT("ClassContextTarget", "Class Context Target(s)");
// 	}
//	else if (HasInterfacePinContext(MenuContext))
// 	{
// 		return LOCTEXT("InterfaceContextTarget", "Interface Context Target(s)");
// 	}
	else if (HasObjectPinContext(MenuContext))
	{
		return LOCTEXT("ObjectContextTarget", "Object Context Target(s)");
	}
	else if (HasExecPinContext(MenuContext))
	{
		return LOCTEXT("ExecContextTarget", "Exec Context Target(s)");
	}
	else
	{
// 		const FEdGraphPinType& PinType = MenuContext.Pins[0]->PinType;
// 		FText PinTypename = UEdGraphSchema_K2::TypeToText(PinType);
// 		return FText::Format(LOCTEXT("TypedPinContextTarget", "{0} Context Target(s)"), PinTypename);

		return LOCTEXT("PODPinContextTarget", "Pin Context Target(s)");
	}
}

//------------------------------------------------------------------------------
uint32 ContextMenuTargetProfileImpl::GetIncompatibleTargetFlags(const FBlueprintActionContext& MenuContext, uint32& HasComponentsMaskOut)
{
	uint32 IncompatibleFlags = (EContextTargetFlags::TARGET_NodeTarget | EContextTargetFlags::TARGET_PinObject | 
		EContextTargetFlags::TARGET_SiblingPinObjects | EContextTargetFlags::TARGET_SubComponents);

	TArray<UClass*> BlueprintClasses;

	HasComponentsMaskOut = 0;
	for (UBlueprint* Blueprint : MenuContext.Blueprints)
	{
		UClass* BpClass = Blueprint->SkeletonGeneratedClass;
		BlueprintClasses.Add(BpClass);

		if (HasAnyExposedComponents(BpClass))
		{
			HasComponentsMaskOut |= EContextTargetFlags::TARGET_Blueprint;
		}
	}

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	for (UEdGraphPin* ContextPin : MenuContext.Pins)
	{
		UEdGraphNode* OwnerNode = ContextPin->GetOwningNodeUnchecked();
		if (OwnerNode != nullptr)
		{
			UEdGraphPin* SelfPin = K2Schema->FindSelfPin(*OwnerNode, EGPD_Input);
			UClass* SelfClass = (SelfPin != nullptr) ? GetPinClass(SelfPin) : nullptr;
			// if this node has a hidden self pin, then that probably means its 
			// target is a static lib, or the same as the current blueprint
			bool const bHasUniqueTarget = (SelfPin != nullptr) && !SelfPin->bHidden;// && !BlueprintClasses.Contains(SelfClass);
			// @TODO: check that "self" is different from this blueprint
			if (bHasUniqueTarget)
			{
				IncompatibleFlags &= ~EContextTargetFlags::TARGET_NodeTarget;
				if (HasAnyExposedComponents(SelfClass))
				{
					HasComponentsMaskOut |= EContextTargetFlags::TARGET_NodeTarget;
				}
			}

			for (UEdGraphPin* SiblingPin : OwnerNode->Pins)
			{
				UClass* PinClass = GetPinClass(SiblingPin);
				if ((SiblingPin != ContextPin) && (SiblingPin->Direction == EGPD_Output) && (PinClass != nullptr))
				{
					IncompatibleFlags &= ~EContextTargetFlags::TARGET_SiblingPinObjects;
					if (HasAnyExposedComponents(PinClass))
					{
						HasComponentsMaskOut |= EContextTargetFlags::TARGET_SiblingPinObjects;
					}
				}
			}
		}

		if (ContextPin->Direction == EGPD_Input)
		{
			continue;
		}

		if (UClass* PinClass = GetPinClass(ContextPin))
		{
			IncompatibleFlags &= ~EContextTargetFlags::TARGET_PinObject;
			if (HasAnyExposedComponents(PinClass))
			{
				HasComponentsMaskOut |= EContextTargetFlags::TARGET_PinObject;
			}
		}
	}

	if ((HasComponentsMaskOut != 0) && !HasComponentPinContext(MenuContext))
	{
		IncompatibleFlags &= ~EContextTargetFlags::TARGET_SubComponents;
	}

	return IncompatibleFlags;
}


//------------------------------------------------------------------------------
static FString ContextMenuTargetProfileImpl::GetProfileSaveName(const FBlueprintActionContext& MenuContext)
{
	const UBlueprintEditorSettings* BpSettings = GetDefault<UBlueprintEditorSettings>();
	if (!BpSettings->bSplitContextTargetSettings)
	{
		return ContextMenuTargetProfileImpl::SharedProfileName;
	}

	if (MenuContext.Pins.Num() == 0)
	{
		return TEXT("NoPinProfile");
	}
	else if (HasExecPinContext(MenuContext))
	{
		return TEXT("ExecPinProfile");
	}
	else if (HasComponentPinContext(MenuContext))
	{
		return TEXT("ComponentPinProfile");
	}
// 	else if (HasClassPinContext(MenuContext))
// 	{
// 		return TEXT("ClassPinProfile");
// 	}
// 	else if (HasInterfacePinContext(MenuContext))
// 	{
// 		return TEXT("InterfacePinProfile");
// 	}

	uint32 HasComponentMask = 0;
	uint32 IncompatibleTargetFlags = GetIncompatibleTargetFlags(MenuContext, HasComponentMask);
	return FString::Printf(TEXT("MenuProfile_%d"), IncompatibleTargetFlags);

// 	if (MenuContext.Pins.Num() == 0)
// 	{
// 		return TEXT("NoPin");
// 	}
// 	else if (ContextMenuTargetProfileImpl::HasObjectPinContext(MenuContext))
// 	{
// 		return TEXT("ObjectPin");
// 	}
// 	else
// 	{
// 		return TEXT("PODPin");
// 	}	
}

//------------------------------------------------------------------------------
FContextMenuTargetProfile::FContextMenuTargetProfile()
	: ProfileSaveName(TEXT("Default"))
	, HasComponentsMask(0)
	, IncompatibleTargetFlags(0)
	, SavedTargetFlags(~IncompatibleTargetFlags)
{
}

//------------------------------------------------------------------------------
FContextMenuTargetProfile::FContextMenuTargetProfile(const FBlueprintActionContext& MenuContext)
	: ProfileSaveName(ContextMenuTargetProfileImpl::GetProfileSaveName(MenuContext))
	, HasComponentsMask(0)
	, IncompatibleTargetFlags(ContextMenuTargetProfileImpl::GetIncompatibleTargetFlags(MenuContext, HasComponentsMask))
	, SavedTargetFlags(~IncompatibleTargetFlags)
{
	const UBlueprintEditorSettings* BpSettings = GetDefault<UBlueprintEditorSettings>();
	FString const BpConfigKey = BpSettings->GetClass()->GetPathName();

	bool bOldUseTargetContextForNodeMenu = true;
	if (GConfig->GetBool(*BpConfigKey, TEXT("bUseTargetContextForNodeMenu"), bOldUseTargetContextForNodeMenu, GEditorPerProjectIni) && !bOldUseTargetContextForNodeMenu)
	{
		SavedTargetFlags = 0;
	}

	if (BpSettings->bEnableNamespaceFilteringFeatures)
	{
		// Filter out non-imported types by default, but do so only if namespace filtering features are also enabled.
		SavedTargetFlags &= ~EContextTargetFlags::TARGET_NonImportedTypes;
	}

	if (!LoadProfile())
	{
		// maybe they were originally using the shared context profile? so let's default to that
		FString CachedProfileSaveName = ProfileSaveName;
		ProfileSaveName = ContextMenuTargetProfileImpl::SharedProfileName;
		LoadProfile();
		ProfileSaveName = CachedProfileSaveName;
	}
}

//------------------------------------------------------------------------------
uint32 FContextMenuTargetProfile::GetContextTargetMask() const
{
	return SavedTargetFlags & ~IncompatibleTargetFlags;
}

//------------------------------------------------------------------------------
void FContextMenuTargetProfile::SetContextTarget(EContextTargetFlags::Type Flag, bool bClear)
{
	if (bClear)
	{
		SavedTargetFlags &= ~Flag;
	}
	else
	{
		SavedTargetFlags |= Flag;
	}
	SaveProfile();
}

//------------------------------------------------------------------------------
uint32 FContextMenuTargetProfile::GetIncompatibleTargetsMask() const
{
	return IncompatibleTargetFlags;
}

//------------------------------------------------------------------------------
bool FContextMenuTargetProfile::IsTargetEnabled(EContextTargetFlags::Type Flag) const 
{
	if (IncompatibleTargetFlags & Flag)
	{
		return false;
	}
	else if (Flag == EContextTargetFlags::TARGET_SubComponents)
	{
		return ((SavedTargetFlags|EContextTargetFlags::TARGET_Blueprint) & HasComponentsMask) != 0;
	}
	return true;
}

//------------------------------------------------------------------------------
void FContextMenuTargetProfile::SaveProfile() const
{
	// want to save with all bits set (except for ones matching flags that
	// currently exist)... this is so we can later add flags, and a user's 
	// saved value doesn't immediately disable them (default to on)
	uint32 const GreatestUsedFlag = (EContextTargetFlags::ContextTargetFlagsEnd & ~1);
	uint32 const UnusedFlasgMask  = ((0xFFFFFFFF & GreatestUsedFlag) & ~GreatestUsedFlag);
	uint32 const SaveValue = UnusedFlasgMask | SavedTargetFlags;

	GConfig->SetInt(*ContextMenuTargetProfileImpl::ConfigSection, *ProfileSaveName, SaveValue, GEditorIni);
}

//------------------------------------------------------------------------------
bool FContextMenuTargetProfile::LoadProfile() 
{
	int32 SavedFlags = SavedTargetFlags;
	if (GConfig->GetInt(*ContextMenuTargetProfileImpl::ConfigSection, *ProfileSaveName, SavedFlags, GEditorIni))
	{
		SavedTargetFlags = SavedFlags;

		uint32 const GreatestUsedFlag = (EContextTargetFlags::ContextTargetFlagsEnd & ~1);
		uint32 const LowestUnusedFlag = GreatestUsedFlag << 1;
		// before we saved values with the unused bits all set (to support future
		// flags), we saved only the bits that were explicitly set by the user
		if ((SavedFlags & LowestUnusedFlag) == 0)
		{
			uint32 const OriginalFlagsMask = EContextTargetFlags::TARGET_Blueprint | EContextTargetFlags::TARGET_SubComponents |
				EContextTargetFlags::TARGET_NodeTarget | EContextTargetFlags::TARGET_PinObject | EContextTargetFlags::TARGET_SiblingPinObjects;
			// add in any new flags that have been added since this profile was last saved
			SavedTargetFlags = (0xFFFFFFFF & ~OriginalFlagsMask) | SavedFlags;

		}
		return true;
	}
	return false;
}

/*******************************************************************************
 * SBlueprintContextTargetMenu
 ******************************************************************************/

/**  */
namespace BlueprintContextTargetMenuImpl
{
	static FText GetContextTargetDisplayName(const UEnum* Enum, int32 EnumIndex)
	{
		if (Enum != nullptr)
		{
			return Enum->GetDisplayNameTextByIndex(EnumIndex);
		}
		return LOCTEXT("UnrecognizedTarget", "Error: <UNRECOGNIZED>");
	}
}

//------------------------------------------------------------------------------
void SBlueprintContextTargetMenu::Construct(const FArguments& InArgs, const FBlueprintActionContext& MenuContext)
{
	TargetProfile = FContextMenuTargetProfile(MenuContext);
	OnTargetMaskChanged = InArgs._OnTargetMaskChanged;

	FSlateFontInfo HeaderFontStyle = FAppStyle::GetFontStyle("BlueprintEditor.ActionMenu.ContextDescriptionFont");
	HeaderFontStyle.Size -= 2;
	const FText HeaderText = ContextMenuTargetProfileImpl::GetProfileDescription(MenuContext);

	const FText MenuToolTip = LOCTEXT("MenuToolTip", "Select whose functions/variables you want to see.\nNOTE: Unchecking everything is akin to 'SHOW EVERYTHING' (you're choosing to have NO target context and to not limit the scope)");

	TSharedRef<SWidget> CustomContentWidget = InArgs._CustomTargetContent.Widget;
	if (CustomContentWidget != SNullWidget::NullWidget)
	{
		SAssignNew(CustomContentWidget, SBox)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.f, 4.f, 0.f, 0.f)
			[
				SNew(SSeparator)
			]
			+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(4.f)
			[
				CustomContentWidget
			]
		];
	}

	TSharedPtr<SHorizontalBox> MenuBody;
	SBorder::Construct(SBorder::FArguments()
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		.Padding(5.f)
		.ToolTipText(MenuToolTip)
		[
			SNew(SBox)
				.MinDesiredWidth(200.0f)
				.ToolTipText(MenuToolTip)
				.Padding(FMargin(0.f, 0.f, 0.f, 18.f))
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
						.ForegroundColor(FAppStyle::GetSlateColor("DefaultForeground"))
					[
						SNew(STextBlock)
							.Text(HeaderText)
							.Font(HeaderFontStyle)
					]
				]

				+SVerticalBox::Slot()
					.VAlign(VAlign_Fill)
					.HAlign(HAlign_Fill)
				[
					SAssignNew(MenuBody, SHorizontalBox)
						.ToolTipText(MenuToolTip)
				]
				+SVerticalBox::Slot()
					.AutoHeight()
				[
					CustomContentWidget
				]
			]
		]
	);

	const uint32 ColumnCount = 2;
	TSharedPtr<SVerticalBox> Columns[ColumnCount];

	for (int32 Col = 0; Col < ColumnCount; ++Col)
	{
		MenuBody->AddSlot()
			.AutoWidth()
		[
			SAssignNew(Columns[Col], SVerticalBox)
				.ToolTipText(MenuToolTip)
		];
	}

	const UEnum* TargetEnum = StaticEnum<EContextTargetFlags::Type>();
	const uint32 GreatestFlag = (EContextTargetFlags::ContextTargetFlagsEnd & ~1);

	int32  ColIndex = 0;
	for (int32 BitMaskOffset = 0; (1 << BitMaskOffset) <= GreatestFlag; ++BitMaskOffset)
	{
		EContextTargetFlags::Type ContextTarget = (EContextTargetFlags::Type)(1 << BitMaskOffset);

		if (TargetEnum && TargetEnum->HasMetaData(TEXT("Hidden"), ContextTarget))
		{
			continue;
		}

		// Keep this target hidden for now unless namespace filtering features are turned on.
		if (ContextTarget == EContextTargetFlags::TARGET_NonImportedTypes && !GetDefault<UBlueprintEditorSettings>()->bEnableNamespaceFilteringFeatures)
		{
			continue;
		}
		
		const FText MenuName = BlueprintContextTargetMenuImpl::GetContextTargetDisplayName(TargetEnum, BitMaskOffset);
		const FContextMenuTargetProfile& ProfileRef = TargetProfile;

		Columns[ColIndex]->AddSlot()
			.AutoHeight()
			.VAlign(VAlign_Top)
			.Padding(3.f, 2.5f)
		[
			SNew(SCheckBox)
				.IsEnabled_Raw(&TargetProfile, &FContextMenuTargetProfile::IsTargetEnabled, ContextTarget)
				.IsChecked(this, &SBlueprintContextTargetMenu::GetTargetCheckedState, ContextTarget)
				.OnCheckStateChanged(this, &SBlueprintContextTargetMenu::OnTargetCheckStateChanged, ContextTarget)
				.ToolTipText_Lambda([TargetEnum, BitMaskOffset, ContextTarget, &ProfileRef]()->FText
				{
					if (!ProfileRef.IsTargetEnabled(ContextTarget))
					{
						return LOCTEXT("DisabledTargetTooltip", "This target is invalid or redundant for this context.");
					}
					else if (TargetEnum != nullptr)
					{
						return TargetEnum->GetToolTipTextByIndex(BitMaskOffset);
					}
					return LOCTEXT("GenericTargetTooltip", "Include variables/functions that belong to this target.");
				})
			[
				SNew(STextBlock).Text(MenuName)
			]
		];
		
		
		ColIndex = (ColIndex + 1) % ColumnCount;
	}
}

//------------------------------------------------------------------------------
uint32 SBlueprintContextTargetMenu::GetContextTargetMask() const
{
	return TargetProfile.GetContextTargetMask();
}

//------------------------------------------------------------------------------
void SBlueprintContextTargetMenu::OnTargetCheckStateChanged(const ECheckBoxState NewCheckedState, EContextTargetFlags::Type ContextTarget)
{
	TargetProfile.SetContextTarget(ContextTarget, (NewCheckedState != ECheckBoxState::Checked));
	OnTargetMaskChanged.ExecuteIfBound(TargetProfile.GetContextTargetMask());
}

//------------------------------------------------------------------------------
ECheckBoxState SBlueprintContextTargetMenu::GetTargetCheckedState(EContextTargetFlags::Type ContextTarget) const
{
	return ((TargetProfile.GetContextTargetMask() & ContextTarget) != 0) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

#undef LOCTEXT_NAMESPACE
