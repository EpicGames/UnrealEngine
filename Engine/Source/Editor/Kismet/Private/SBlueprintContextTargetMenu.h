// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintActionMenuUtils.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBorder.h"

struct FBlueprintActionContext;

enum class ECheckBoxState : uint8;

/*******************************************************************************
 * FContextMenuTargetProfile
 ******************************************************************************/

/** Used internally by SBlueprintContextTargetMenu, to track and save the user's context target settings */
struct FContextMenuTargetProfile
{
public:
	FContextMenuTargetProfile();
	FContextMenuTargetProfile(const FBlueprintActionContext& MenuContext);

	uint32 GetContextTargetMask() const;
	void   SetContextTarget(EContextTargetFlags::Type Flag, bool bClear = false);
	uint32 GetIncompatibleTargetsMask() const;

	bool   IsTargetEnabled(EContextTargetFlags::Type Flag) const;

private:
	void SaveProfile() const;
	bool LoadProfile();

	FString ProfileSaveName;
	uint32  HasComponentsMask;
	uint32  IncompatibleTargetFlags;
	uint32  SavedTargetFlags;
};

/*******************************************************************************
 * SBlueprintContextTargetMenu
 ******************************************************************************/

/**  */
class SBlueprintContextTargetMenu : public SBorder
{
	DECLARE_DELEGATE_OneParam(FOnTargetMaskChanged, uint32);

public:
	SLATE_BEGIN_ARGS(SBlueprintContextTargetMenu){}
		SLATE_EVENT(FOnTargetMaskChanged, OnTargetMaskChanged)
		SLATE_NAMED_SLOT(FArguments, CustomTargetContent)
	SLATE_END_ARGS()

	// SWidget interface
	void Construct(const FArguments& InArgs, const FBlueprintActionContext& MenuContext);
	// End SWidget interface

	/**
	 * Returns a EContextTargetFlags::Type bitmask, defining which targets the 
	 * user currently has enabled.
	 */
	uint32 GetContextTargetMask() const;

private:
	/**
	 * Internal UI callback that handles when one of the context targets is 
	 * checked/unchecked.
	 * 
	 * @param  NewCheckedState	The checkbox's new state.
	 * @param  ContextTarget    A flag identifying which checkbox was altered.
	 */
	void OnTargetCheckStateChanged(const ECheckBoxState NewCheckedState, EContextTargetFlags::Type ContextTarget);

	/**
	 * Internal UI callback that determines the checkbox state for one of the 
	 * context target options.
	 * 
	 * @param  ContextTarget    A flag identifying which checkbox we want the state of.
	 * @return ECheckBoxState::Checked if the user has the flag set, otherwise ECheckBoxState::UnChecked
	 */
	ECheckBoxState GetTargetCheckedState(EContextTargetFlags::Type ContextTarget) const;

	/** Determines what flags are incompatible with the current context, and saves/loads the user's choice settings */
	FContextMenuTargetProfile TargetProfile;
	/** Delegate for external users to hook into (so they can act when the menu's settings are changed). */
	FOnTargetMaskChanged OnTargetMaskChanged;
};
