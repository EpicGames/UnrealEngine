// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBlueprintDiff.h"

#include "BlueprintEditor.h"
#include "Containers/Set.h"
#include "DiffControl.h"
#include "DiffUtils.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphUtilities.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Blueprint.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "GameFramework/Actor.h"
#include "GraphDiffControl.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformCrt.h"
#include "IAssetTypeActions.h"
#include "Internationalization/Text.h"
#include "K2Node_MathExpression.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "PropertyEditorDelegates.h"
#include "ReviewComments.h"
#include "SDetailsSplitter.h"
#include "SKismetInspector.h"
#include "SMyBlueprint.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "AsyncDetailViewDiff.h"

class FProperty;

#define LOCTEXT_NAMESPACE "SBlueprintDif"

typedef TMap< FName, const FProperty* > FNamePropertyMap;

static const FName BlueprintTypeMode = FName(TEXT("BlueprintTypeMode"));
static const FName MyBlueprintMode = FName(TEXT("MyBlueprintMode"));
static const FName DefaultsMode = FName(TEXT("DefaultsMode"));
static const FName ClassSettingsMode = FName(TEXT("ClassSettingsMode"));
static const FName ComponentsMode = FName(TEXT("ComponentsMode"));
static const FName CommentsMode = FName(TEXT("CommentsMode"));
static const FName GraphMode = FName(TEXT("GraphMode"));

TSharedRef<SWidget>	FDiffResultItem::GenerateWidget() const
{
	FText ToolTip = Result.ToolTip;
	FLinearColor Color = Result.GetDisplayColor();
	FText Text = Result.DisplayString;
	if (Text.IsEmpty())
	{
		Text = LOCTEXT("DIF_UnknownDiff", "Unknown Diff");
		ToolTip = LOCTEXT("DIF_Confused", "There is an unspecified difference");
	}
	return SNew(STextBlock)
		.ToolTipText(ToolTip)
		.ColorAndOpacity(Color)
		.Text(Text);
}

FDiffPanel::FDiffPanel()
{
	Blueprint = nullptr;
}

void FDiffPanel::InitializeDiffPanel()
{
	TSharedRef< SKismetInspector > Inspector = SNew(SKismetInspector)
		.HideNameArea(true)
		.ViewIdentifier(FName("BlueprintInspector"))
		.MyBlueprintWidget(MyBlueprint)
		.IsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateStatic([] { return false; }))
		.ShowLocalVariables(true);
	DetailsView = Inspector;
	MyBlueprint->SetInspector(DetailsView);
}

static int32 GetCurrentIndex( SListView< TSharedPtr< FDiffSingleResult> > const& ListView, const TArray< TSharedPtr< FDiffSingleResult > >& ListViewSource )
{
	const TArray< TSharedPtr<FDiffSingleResult> >& Selected = ListView.GetSelectedItems();
	if (Selected.Num() == 1)
	{
		int32 Index = 0;
		for (const TSharedPtr<FDiffSingleResult>& Diff : ListViewSource)
		{
			if (Diff == Selected[0])
			{
				return Index;
			}
		}
	}
	return -1;
}

void DiffWidgetUtils::SelectNextRow( SListView< TSharedPtr< FDiffSingleResult> >& ListView, const TArray< TSharedPtr< FDiffSingleResult > >& ListViewSource )
{
	int32 CurrentIndex = GetCurrentIndex(ListView, ListViewSource);
	if (CurrentIndex == ListViewSource.Num() - 1)
	{
		return;
	}

	ListView.SetSelection(ListViewSource[CurrentIndex + 1]);
}

void DiffWidgetUtils::SelectPrevRow(SListView< TSharedPtr< FDiffSingleResult> >& ListView, const TArray< TSharedPtr< FDiffSingleResult > >& ListViewSource )
{
	int32 CurrentIndex = GetCurrentIndex(ListView, ListViewSource);
	if (CurrentIndex == 0)
	{
		return;
	}

	ListView.SetSelection(ListViewSource[CurrentIndex - 1]);
}

bool DiffWidgetUtils::HasNextDifference(SListView< TSharedPtr< FDiffSingleResult> >& ListView, const TArray< TSharedPtr< FDiffSingleResult > >& ListViewSource)
{
	int32 CurrentIndex = GetCurrentIndex(ListView, ListViewSource);
	return ListViewSource.IsValidIndex(CurrentIndex+1);
}

bool DiffWidgetUtils::HasPrevDifference(SListView< TSharedPtr< FDiffSingleResult> >& ListView, const TArray< TSharedPtr< FDiffSingleResult > >& ListViewSource)
{
	int32 CurrentIndex = GetCurrentIndex(ListView, ListViewSource);
	return ListViewSource.IsValidIndex(CurrentIndex - 1);
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SBlueprintDiff::Construct( const FArguments& InArgs)
{
	check(InArgs._BlueprintOld || InArgs._BlueprintNew);
	
	// make sure that both blueprints have the same category sorting so that they diff properly.
	if (InArgs._BlueprintOld && InArgs._BlueprintNew)
	{
		if (FPackageName::IsTempPackage(InArgs._BlueprintOld->GetPackage()->GetName()))
        {
        	const_cast<UBlueprint*>(InArgs._BlueprintOld)->CategorySorting = InArgs._BlueprintNew->CategorySorting;
        }
        else if(FPackageName::IsTempPackage(InArgs._BlueprintNew->GetPackage()->GetName()))
        {
        	const_cast<UBlueprint*>(InArgs._BlueprintNew)->CategorySorting = InArgs._BlueprintOld->CategorySorting;
        }
        else
        {
        	// Neither New or Old BPs are temp so we need to scope this change to the lifetime of this diff window.
        	// when the window is closed, ScopedCategorySortChange will revert BlueprintOld->CategorySorting back to it's original state.
        	ScopedCategorySortChange.SetBlueprint(const_cast<UBlueprint*>(InArgs._BlueprintOld));
        	const_cast<UBlueprint*>(InArgs._BlueprintOld)->CategorySorting = InArgs._BlueprintNew->CategorySorting;
        }
	}
	
	PanelOld.Blueprint = InArgs._BlueprintOld;
	PanelNew.Blueprint = InArgs._BlueprintNew;
	PanelOld.RevisionInfo = InArgs._OldRevision;
	PanelNew.RevisionInfo = InArgs._NewRevision;

	// Create a skeleton if we don't have one, this is true for revision history diffs
	if (PanelOld.Blueprint && !PanelOld.Blueprint->SkeletonGeneratedClass)
	{
		FKismetEditorUtilities::GenerateBlueprintSkeleton(const_cast<UBlueprint*>(PanelOld.Blueprint));
	}
	
	if (PanelNew.Blueprint && !PanelNew.Blueprint->SkeletonGeneratedClass)
	{
		FKismetEditorUtilities::GenerateBlueprintSkeleton(const_cast<UBlueprint*>(PanelNew.Blueprint));
	}

	// sometimes we want to clearly identify the assets being diffed (when it's
	// not the same asset in each panel)
	PanelOld.bShowAssetName = InArgs._ShowAssetNames;
	PanelNew.bShowAssetName = InArgs._ShowAssetNames;

	bLockViews = true;

	if (InArgs._ParentWindow.IsValid())
	{
		WeakParentWindow = InArgs._ParentWindow;

		AssetEditorCloseDelegate = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnAssetEditorRequestClose().AddSP(this, &SBlueprintDiff::OnCloseAssetEditor);
	}

	FToolBarBuilder NavToolBarBuilder(TSharedPtr< const FUICommandList >(), FMultiBoxCustomization::None);
	NavToolBarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateSP(this, &SBlueprintDiff::PrevDiff),
			FCanExecuteAction::CreateSP( this, &SBlueprintDiff::HasPrevDiff)
		)
		, NAME_None
		, LOCTEXT("PrevDiffLabel", "Prev")
		, LOCTEXT("PrevDiffTooltip", "Go to previous difference")
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintDif.PrevDiff")
	);
	NavToolBarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateSP(this, &SBlueprintDiff::NextDiff),
			FCanExecuteAction::CreateSP(this, &SBlueprintDiff::HasNextDiff)
		)
		, NAME_None
		, LOCTEXT("NextDiffLabel", "Next")
		, LOCTEXT("NextDiffTooltip", "Go to next difference")
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintDif.NextDiff")
	);

	FToolBarBuilder GraphToolbarBuilder(TSharedPtr< const FUICommandList >(), FMultiBoxCustomization::None);
	GraphToolbarBuilder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateSP(this, &SBlueprintDiff::OnToggleLockView))
		, NAME_None
		, LOCTEXT("LockGraphsLabel", "Lock/Unlock")
		, LOCTEXT("LockGraphsTooltip", "Force all graph views to change together, or allow independent scrolling/zooming")
		, TAttribute<FSlateIcon>(this, &SBlueprintDiff::GetLockViewImage)
	);
	GraphToolbarBuilder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateSP(this, &SBlueprintDiff::OnToggleSplitViewMode))
		, NAME_None
		, LOCTEXT("SplitGraphsModeLabel", "Vertical/Horizontal")
		, LOCTEXT("SplitGraphsModeLabelTooltip", "Toggles the split view of graphs between vertical and horizontal")
		, TAttribute<FSlateIcon>(this, &SBlueprintDiff::GetSplitViewModeImage)
	);

	DifferencesTreeView = DiffTreeView::CreateTreeView(&PrimaryDifferencesList);

	GenerateDifferencesList();

	const auto TextBlock = [](FText Text) -> TSharedRef<SWidget>
	{
		return SNew(SBox)
		.Padding(FMargin(4.0f,10.0f))
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		[
			SNew(STextBlock)
			.Visibility(EVisibility::HitTestInvisible)
			.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
			.Text(Text)
		];
	};

	TopRevisionInfoWidget =
		SNew(SSplitter)
		.Visibility(EVisibility::HitTestInvisible)
		+ SSplitter::Slot()
		.Value(.2f)
		[
			SNew(SBox)
		]
		+ SSplitter::Slot()
		.Value(.8f)
		[
			SNew(SSplitter)
			.PhysicalSplitterHandleSize(10.0f)
			+ SSplitter::Slot()
			.Value(.5f)
			[
				TextBlock(DiffViewUtils::GetPanelLabel(PanelOld.Blueprint, PanelOld.RevisionInfo, FText()))
			]
			+ SSplitter::Slot()
			.Value(.5f)
			[
				TextBlock(DiffViewUtils::GetPanelLabel(PanelNew.Blueprint, PanelNew.RevisionInfo, FText()))
			]
		];

	GraphToolBarWidget = 
		SNew(SSplitter)
		.Visibility(EVisibility::HitTestInvisible)
		+ SSplitter::Slot()
		.Value(.2f)
		[
			SNew(SBox)
		]
		+ SSplitter::Slot()
		.Value(.8f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				GraphToolbarBuilder.MakeWidget()
			]	
		];

	this->ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush( "Docking.Tab", ".ContentAreaBrush" ))
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				.VAlign(VAlign_Top)
				[
					TopRevisionInfoWidget.ToSharedRef()		
				]
				+ SOverlay::Slot()
				.VAlign(VAlign_Top)
				.Padding(0.0f, 6.0f, 0.0f, 4.0f)
				[
					GraphToolBarWidget.ToSharedRef()
				]
				+ SOverlay::Slot()
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 2.0f, 0.0f, 2.0f)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.Padding(4.f)
						.AutoWidth()
						[
							NavToolBarBuilder.MakeWidget()
						]
						+ SHorizontalBox::Slot()
						[
							SNew(SSpacer)
						]
					]
					+ SVerticalBox::Slot()
					[
						SNew(SSplitter)
						+ SSplitter::Slot()
						.Value(.2f)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							[
								DifferencesTreeView.ToSharedRef()
							]
						]
						+ SSplitter::Slot()
						.Value(.8f)
						[
							SAssignNew(ModeContents, SBox)
						]
					]
				]
			]
		];

	SetCurrentMode(MyBlueprintMode);

	// Bind to blueprint changed events as they may be real in memory blueprints that will be modified
	if (PanelNew.Blueprint)
	{
		const_cast<UBlueprint*>(PanelNew.Blueprint)->OnChanged().AddSP(this, &SBlueprintDiff::OnBlueprintChanged);
	}
	if (PanelOld.Blueprint)
	{
		const_cast<UBlueprint*>(PanelOld.Blueprint)->OnChanged().AddSP(this, &SBlueprintDiff::OnBlueprintChanged);
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

SBlueprintDiff::~SBlueprintDiff()
{
	if (AssetEditorCloseDelegate.IsValid())
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnAssetEditorRequestClose().Remove(AssetEditorCloseDelegate);
	}
}

void SBlueprintDiff::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (const TSharedPtr<IDiffControl> DiffControl = ModePanels[CurrentMode].DiffControl)
	{
		DiffControl->Tick();
	}
	if (PanelOld.DetailsView)
	{
		PanelOld.DetailsView->Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	}
	if (PanelNew.DetailsView)
	{
		PanelNew.DetailsView->Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	}
	if (GraphDetailDiff)
	{
		GraphDetailDiff->Tick();
	}
}

void SBlueprintDiff::OnCloseAssetEditor(UObject* Asset, EAssetEditorCloseReason CloseReason)
{
	if (PanelOld.Blueprint == Asset || PanelNew.Blueprint == Asset || CloseReason == EAssetEditorCloseReason::CloseAllAssetEditors)
	{
		// Tell our window to close and set our selves to collapsed to try and stop it from ticking
		SetVisibility(EVisibility::Collapsed);

		if (AssetEditorCloseDelegate.IsValid())
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnAssetEditorRequestClose().Remove(AssetEditorCloseDelegate);
		}

		if (WeakParentWindow.IsValid())
		{
			WeakParentWindow.Pin()->RequestDestroyWindow();
		}
	}
}

void SBlueprintDiff::CreateGraphEntry( UEdGraph* GraphOld, UEdGraph* GraphNew )
{
	Graphs.Add(MakeShared<FGraphToDiff>(this, GraphOld, GraphNew, PanelOld.RevisionInfo, PanelNew.RevisionInfo));
	Graphs.Last()->EnableComments(DifferencesTreeView.ToWeakPtr());
	
}

void SBlueprintDiff::OnGraphSelectionChanged(TSharedPtr<FGraphToDiff> Item, ESelectInfo::Type SelectionType)
{
	if (!Item.IsValid())
	{
		return;
	}

	FocusOnGraphRevisions(Item.Get());

}

void SBlueprintDiff::OnGraphChanged(FGraphToDiff* Diff)
{
	if (PanelNew.GraphEditor.IsValid() && PanelNew.GraphEditor.Pin()->GetCurrentGraph() == Diff->GetGraphNew())
	{
		FocusOnGraphRevisions(Diff);
	}
}

void SBlueprintDiff::OnBlueprintChanged(UBlueprint* InBlueprint)
{
	if (InBlueprint == PanelOld.Blueprint || InBlueprint == PanelNew.Blueprint)
	{
		// After a BP has changed significantly, we need to regenerate the UI and set back to initial UI to avoid crashes
		GenerateDifferencesList();
		SetCurrentMode(MyBlueprintMode);
	}
}

TSharedRef<SWidget> SBlueprintDiff::DefaultEmptyPanel()
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BlueprintDifGraphsToolTip", "Select Graph to Diff"))
		];
}

TSharedPtr<SWindow> SBlueprintDiff::CreateDiffWindow(FText WindowTitle, const UBlueprint* OldBlueprint, const UBlueprint* NewBlueprint, const FRevisionInfo& OldRevision, const FRevisionInfo& NewRevision)
{
	// sometimes we're comparing different revisions of one single asset (other 
	// times we're comparing two completely separate assets altogether)
	bool bIsSingleAsset = !OldBlueprint || !NewBlueprint || (NewBlueprint->GetName() == OldBlueprint->GetName());

	TSharedPtr<SWindow> Window = SNew(SWindow)
		.Title(WindowTitle)
		.ClientSize(FVector2D(1000, 800));

	Window->SetContent(SNew(SBlueprintDiff)
		.BlueprintOld(OldBlueprint)
		.BlueprintNew(NewBlueprint)
		.OldRevision(OldRevision)
		.NewRevision(NewRevision)
		.ShowAssetNames(!bIsSingleAsset)
		.ParentWindow(Window));

	// Make this window a child of the modal window if we've been spawned while one is active.
	const TSharedPtr<SWindow> ActiveModal = FSlateApplication::Get().GetActiveModalWindow();
	if (ActiveModal.IsValid())
	{
		FSlateApplication::Get().AddWindowAsNativeChild(Window.ToSharedRef(), ActiveModal.ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow(Window.ToSharedRef());
	}

	return Window;
}

TSharedPtr<SWindow> SBlueprintDiff::CreateDiffWindow(const UBlueprint* OldBlueprint, const UBlueprint* NewBlueprint,
                                                     const FRevisionInfo& OldRevision, const FRevisionInfo& NewRevision, const UClass* BlueprintClass)
{
	check(OldBlueprint || NewBlueprint);

	// sometimes we're comparing different revisions of one single asset (other 
	// times we're comparing two completely separate assets altogether)
	const bool bIsSingleAsset = !OldBlueprint || !NewBlueprint || (NewBlueprint->GetName() == OldBlueprint->GetName());

	FText WindowTitle = FText::Format(LOCTEXT("NamelessBlueprintDiff", "{0} Diff"), BlueprintClass->GetDisplayNameText());
	// if we're diffing one asset against itself 
	if (bIsSingleAsset)
	{
		// identify the assumed single asset in the window's title
		FString BPName;
		if (NewBlueprint)
		{
			BPName = NewBlueprint->GetName();
		}
		else if (OldBlueprint)
		{
			BPName = OldBlueprint->GetName();
		}
		WindowTitle = FText::Format(LOCTEXT("NamedBlueprintDiff", "{0} - {1} Diff"), FText::FromString(BPName), BlueprintClass->GetDisplayNameText());
	}

	return CreateDiffWindow(WindowTitle, OldBlueprint, NewBlueprint, OldRevision, NewRevision);
}

void SBlueprintDiff::NextDiff()
{
	DiffTreeView::HighlightNextDifference(DifferencesTreeView.ToSharedRef(), RealDifferences, PrimaryDifferencesList);
}

void SBlueprintDiff::PrevDiff()
{
	DiffTreeView::HighlightPrevDifference(DifferencesTreeView.ToSharedRef(), RealDifferences, PrimaryDifferencesList);
}

bool SBlueprintDiff::HasNextDiff() const
{
	return DiffTreeView::HasNextDifference(DifferencesTreeView.ToSharedRef(), RealDifferences);
}

bool SBlueprintDiff::HasPrevDiff() const
{
	return DiffTreeView::HasPrevDifference(DifferencesTreeView.ToSharedRef(), RealDifferences);
}

FGraphToDiff* SBlueprintDiff::FindGraphToDiffEntry(const FString& GraphPath)
{
	for (const TSharedPtr<FGraphToDiff>& Graph : Graphs)
	{
		FString SearchGraphPath = Graph->GetGraphOld() ? FGraphDiffControl::GetGraphPath(Graph->GetGraphOld()) : FGraphDiffControl::GetGraphPath(Graph->GetGraphNew());
		if (SearchGraphPath.Equals(GraphPath, ESearchCase::CaseSensitive))
		{
			return Graph.Get();
		}
	}
	return nullptr;
}

void SBlueprintDiff::FocusOnGraphRevisions( FGraphToDiff* Diff )
{
	HandleGraphChanged(Diff);

	ResetGraphEditors();
}

void SBlueprintDiff::OnDiffListSelectionChanged(TSharedPtr<FDiffResultItem> TheDiff )
{
	check( !TheDiff->Result.OwningObjectPath.IsEmpty() );
	FocusOnGraphRevisions( FindGraphToDiffEntry( TheDiff->Result.OwningObjectPath) );
	FDiffSingleResult Result = TheDiff->Result;

	const auto SafeClearSelection = []( TWeakPtr<SGraphEditor> GraphEditor )
	{
		TSharedPtr<SGraphEditor> GraphEditorPtr = GraphEditor.Pin();
		if (GraphEditorPtr.IsValid())
		{
			GraphEditorPtr->ClearSelectionSet();
		}
	};

	SafeClearSelection( PanelNew.GraphEditor );
	SafeClearSelection( PanelOld.GraphEditor );

	if (Result.Pin1)
	{
		GetDiffPanelForNode(*Result.Pin1->GetOwningNode()).FocusDiff(*Result.Pin1);
		if (Result.Pin2)
		{
			GetDiffPanelForNode(*Result.Pin2->GetOwningNode()).FocusDiff(*Result.Pin2);
		}
	}
	else if (Result.Node1)
	{
		GetDiffPanelForNode(*Result.Node1).FocusDiff(*Result.Node1);
		if (Result.Node2)
		{
			GetDiffPanelForNode(*Result.Node2).FocusDiff(*Result.Node2);
		}
	}
}

void SBlueprintDiff::OnToggleLockView()
{
	bLockViews = !bLockViews;
	ResetGraphEditors();
}

void SBlueprintDiff::OnToggleSplitViewMode()
{
	bVerticalSplitGraphMode = !bVerticalSplitGraphMode;

	if (SSplitter* DiffGraphSplitterPtr = DiffGraphSplitter.Get())
	{
		DiffGraphSplitterPtr->SetOrientation(bVerticalSplitGraphMode ? Orient_Horizontal : Orient_Vertical);
	}
}

FSlateIcon SBlueprintDiff::GetLockViewImage() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), bLockViews ? "Icons.Lock" : "Icons.Unlock");
}

FSlateIcon SBlueprintDiff::GetSplitViewModeImage() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), bVerticalSplitGraphMode ? "BlueprintDif.VerticalDiff.Small" : "BlueprintDif.HorizontalDiff.Small");
}

void SBlueprintDiff::ResetGraphEditors()
{
	if (PanelOld.GraphEditor.IsValid() && PanelNew.GraphEditor.IsValid())
	{
		if (bLockViews)
		{
			PanelOld.GraphEditor.Pin()->LockToGraphEditor(PanelNew.GraphEditor);
			PanelNew.GraphEditor.Pin()->LockToGraphEditor(PanelOld.GraphEditor);
		}
		else
		{
			PanelOld.GraphEditor.Pin()->UnlockFromGraphEditor(PanelNew.GraphEditor);
			PanelNew.GraphEditor.Pin()->UnlockFromGraphEditor(PanelOld.GraphEditor);
		}	
	}
}

void FDiffPanel::GeneratePanel(UEdGraph* NewGraph, UEdGraph* OldGraph )
{
	const TSharedPtr<TArray<FDiffSingleResult>> Diff = MakeShared<TArray<FDiffSingleResult>>();
	FGraphDiffControl::DiffGraphs(OldGraph, NewGraph, *Diff);
	GeneratePanel(NewGraph, Diff, {});
}

void FDiffPanel::GeneratePanel(UEdGraph* Graph, TSharedPtr<TArray<FDiffSingleResult>> DiffResults, TAttribute<int32> FocusedDiffResult)
{
	if (GraphEditor.IsValid() && GraphEditor.Pin()->GetCurrentGraph() == Graph)
	{
		return;
	}

	TSharedPtr<SWidget> Widget = SNew(SBorder)
								.HAlign(HAlign_Center)
								.VAlign(VAlign_Center)
								[
									SNew(STextBlock).Text( LOCTEXT("BPDifPanelNoGraphTip", "Graph does not exist in this revision"))
								];

	if (Graph)
	{
		SGraphEditor::FGraphEditorEvents InEvents;
		{
			const auto SelectionChangedHandler = [](const FGraphPanelSelectionSet& SelectionSet, TSharedPtr<SKismetInspector> Container)
			{
				Container->ShowDetailsForObjects(SelectionSet.Array());
			};

			const auto ContextMenuHandler = [](UEdGraph* CurrentGraph, const UEdGraphNode* InGraphNode, const UEdGraphPin* InGraphPin, FMenuBuilder* MenuBuilder, bool bIsDebugging)
			{
				MenuBuilder->AddMenuEntry(FGenericCommands::Get().Copy);
				return FActionMenuContent(MenuBuilder->MakeWidget());
			};

			InEvents.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateStatic(SelectionChangedHandler, DetailsView);
			InEvents.OnCreateNodeOrPinMenu = SGraphEditor::FOnCreateNodeOrPinMenu::CreateStatic(ContextMenuHandler);
		}

		if (!GraphEditorCommands.IsValid())
		{
			GraphEditorCommands = MakeShared<FUICommandList>();

			GraphEditorCommands->MapAction( FGenericCommands::Get().Copy,
				FExecuteAction::CreateRaw( this, &FDiffPanel::CopySelectedNodes ),
				FCanExecuteAction::CreateRaw( this, &FDiffPanel::CanCopyNodes )
				);
		}

		MyBlueprint->SetFocusedGraph(Graph);

		TSharedRef<SGraphEditor> Editor = SNew(SGraphEditor)
			.AdditionalCommands(GraphEditorCommands)
			.GraphToEdit(Graph)
			.GraphToDiff(nullptr)
			.DiffResults(DiffResults)
			.FocusedDiffResult(FocusedDiffResult)
			.IsEditable(false)
			.GraphEvents(InEvents);

		GraphEditor = Editor;
		Widget = Editor;
	}

	GraphEditorBox->SetContent(Widget.ToSharedRef());
}

TSharedRef<class SWidget> FDiffPanel::GenerateMyBlueprintWidget()
{
	if (Blueprint)
	{
		return SAssignNew(MyBlueprint, SMyBlueprint, TWeakPtr<FBlueprintEditor>(), Blueprint);
	}
	return SNew(STextBlock).Text(LOCTEXT("NullBlueprint","Missing Blueprint"));
}

FGraphPanelSelectionSet FDiffPanel::GetSelectedNodes() const
{
	FGraphPanelSelectionSet CurrentSelection;
	TSharedPtr<SGraphEditor> FocusedGraphEd = GraphEditor.Pin();
	if (FocusedGraphEd.IsValid())
	{
		CurrentSelection = FocusedGraphEd->GetSelectedNodes();
	}
	return CurrentSelection;
}

void FDiffPanel::CopySelectedNodes()
{
	// Export the selected nodes and place the text on the clipboard
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	FString ExportedText;
	FEdGraphUtilities::ExportNodesToText(SelectedNodes, /*out*/ ExportedText);
	FPlatformApplicationMisc::ClipboardCopy(*ExportedText);
}

bool FDiffPanel::CanCopyNodes() const
{
	// If any of the nodes can be duplicated then we should allow copying
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(SelectedNodes); SelectedIter; ++SelectedIter)
	{
		UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter);
		if ((Node != nullptr) && Node->CanDuplicateNode())
		{
			return true;
		}
	}
	return false;
}

void FDiffPanel::FocusDiff(UEdGraphPin& Pin)
{
	if (GraphEditor.IsValid())
	{
		GraphEditor.Pin()->JumpToPin(&Pin);
	}
}

void FDiffPanel::FocusDiff(UEdGraphNode& Node)
{
	if (GraphEditor.IsValid())
	{
		GraphEditor.Pin()->JumpToNode(&Node, false);
	}
}

TSharedRef<SWidget> FDiffPanel::GetMyBlueprintWidget() const
{
	// if this panel is displaying a null object, return an empty boarder instead of a MyBlueprint
	return MyBlueprint ? (TSharedRef<SWidget>)MyBlueprint.ToSharedRef() : TSharedRef<SWidget>(SNew(SBorder));
}

TSharedRef<SWidget> FDiffPanel::GetDetailsWidget() const
{
	// if this panel is displaying a null object, return an empty boarder instead of a details view
	return DetailsView ? (TSharedRef<SWidget>)DetailsView.ToSharedRef() : TSharedRef<SWidget>(SNew(SBorder));
}

FDiffPanel& SBlueprintDiff::GetDiffPanelForNode(UEdGraphNode& Node)
{
	TSharedPtr<SGraphEditor> OldGraphEditorPtr = PanelOld.GraphEditor.Pin();
	if (OldGraphEditorPtr.IsValid() && Node.GetGraph() == OldGraphEditorPtr->GetCurrentGraph())
	{
		return PanelOld;
	}
	TSharedPtr<SGraphEditor> NewGraphEditorPtr = PanelNew.GraphEditor.Pin();
	if (NewGraphEditorPtr.IsValid() && Node.GetGraph() == NewGraphEditorPtr->GetCurrentGraph())
	{
		return PanelNew;
	}
	ensureMsgf(false, TEXT("Looking for node %s but it cannot be found in provided panels"), *Node.GetName());
	static FDiffPanel Default;
	return Default;
}

void SBlueprintDiff::HandleGraphChanged(FGraphToDiff* Diff)
{
	check(Diff);
	SetCurrentMode(GraphMode);

	UEdGraph* GraphOld = Diff->GetGraphOld();
	UEdGraph* GraphNew = Diff->GetGraphNew();
	TSharedPtr<TArray<FDiffSingleResult>> DiffResults = Diff->FoundDiffs;
	int32 RealDifferencesStartIndex = Diff->RealDifferencesStartIndex;

	const TAttribute<int32> FocusedDiffResult = TAttribute<int32>::CreateLambda(
		[this, RealDifferencesStartIndex]()
		{
			int32 FocusedDiffResult = INDEX_NONE;
			if (RealDifferencesStartIndex != INDEX_NONE)
			{
				FocusedDiffResult = DiffTreeView::CurrentDifference(DifferencesTreeView.ToSharedRef(), RealDifferences) - RealDifferencesStartIndex;
			}

			// find selected index in all the graphs, and subtract the index of the first entry in this graph
			return FocusedDiffResult;
		});

	// only regenerate PanelOld if the old graph has changed
	if (!PanelOld.GraphEditor.IsValid() || GraphOld != PanelOld.GraphEditor.Pin()->GetCurrentGraph())
	{
		PanelOld.GeneratePanel(GraphOld, DiffResults, FocusedDiffResult);
	}

	// only regenerate PanelNew if the old graph has changed
	if (!PanelNew.GraphEditor.IsValid() || GraphNew != PanelNew.GraphEditor.Pin()->GetCurrentGraph())
	{
		PanelNew.GeneratePanel(GraphNew, DiffResults, FocusedDiffResult);
	}
}

void SBlueprintDiff::GenerateDifferencesList()
{
	PrimaryDifferencesList.Empty();
	RealDifferences.Empty();
	Graphs.Empty();
	ModePanels.Empty();

	// SMyBlueprint needs to be created *before* the KismetInspector or the diffs are generated, because the KismetInspector's customizations
	// need a reference to the SMyBlueprint widget that is controlling them...
	const auto CreateInspector = [](TSharedPtr<SMyBlueprint> InMyBlueprint, const TSharedPtr<SScrollBar>& Scrollbar, EHorizontalAlignment ScrollbarAlignment) {
		return SNew(SKismetInspector)
			.HideNameArea(true)
			.ViewIdentifier(FName("BlueprintInspector"))
			.MyBlueprintWidget(InMyBlueprint)
			.IsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateStatic([] { return false; }))
			.ShowLocalVariables(true)
			.ExternalScrollbar(Scrollbar)
			.ScrollbarAlignment(ScrollbarAlignment);
	};

	TArray<UEdGraph*> GraphsOld, GraphsNew;
	if (PanelOld.Blueprint)
	{
		PanelOld.GenerateMyBlueprintWidget();
		PanelOld.DetailScrollbar = SNew(SLinkableScrollBar);
		PanelOld.DetailsView = CreateInspector(PanelOld.MyBlueprint, PanelOld.DetailScrollbar, HAlign_Left);
		PanelOld.MyBlueprint->SetInspector(PanelOld.DetailsView);
		PanelOld.Blueprint->GetAllGraphs(GraphsOld);
	}
	if (PanelNew.Blueprint)
	{
		PanelNew.GenerateMyBlueprintWidget();
		PanelNew.DetailScrollbar = SNew(SLinkableScrollBar);
		PanelNew.DetailsView = CreateInspector(PanelNew.MyBlueprint, PanelNew.DetailScrollbar, HAlign_Right);
		PanelNew.MyBlueprint->SetInspector(PanelNew.DetailsView);
		PanelNew.Blueprint->GetAllGraphs(GraphsNew);
	}

	if (PanelOld.DetailsView && PanelNew.DetailsView)
	{
		GraphDetailDiff = MakeShared<FAsyncDetailViewDiff>(
			PanelOld.DetailsView->GetPropertyView().ToSharedRef(),
			PanelNew.DetailsView->GetPropertyView().ToSharedRef());

		SLinkableScrollBar::LinkScrollBars(PanelOld.DetailScrollbar.ToSharedRef(), PanelNew.DetailScrollbar.ToSharedRef(),
		TAttribute<TArray<FVector2f>>::CreateRaw(GraphDetailDiff.Get(), &FAsyncDetailViewDiff::GenerateScrollSyncRate));
		TAttribute<TArray<FVector2f>> ScrollRate = TAttribute<TArray<FVector2f>>::CreateRaw(GraphDetailDiff.Get(), &FAsyncDetailViewDiff::GenerateScrollSyncRate);
	}

	// match the graphs in the old blueprint with the graphs in the new blueprint so they can be diffed
	struct FGraphPair
	{
		TObjectPtr<UEdGraph> GraphOld;
		TObjectPtr<UEdGraph> GraphNew;
	};
	TMap<FString, FGraphPair> PathToGraphs;
	for (UEdGraph* Graph : GraphsOld)
	{
		FString Path;
		Graph->GetPathName(PanelOld.Blueprint, Path);
		PathToGraphs.FindOrAdd(Path).GraphOld = Graph;
	}
	for (UEdGraph* Graph : GraphsNew)
	{
		FString Path;
		Graph->GetPathName(PanelNew.Blueprint, Path);
		PathToGraphs.FindOrAdd(Path).GraphNew = Graph;
	}

	for (const auto& [Path, GraphPair] : PathToGraphs)
	{
		if (GraphPair.GraphOld ? IsGraphDiffNeeded(GraphPair.GraphOld) : IsGraphDiffNeeded(GraphPair.GraphNew))
		{
			CreateGraphEntry(GraphPair.GraphOld,GraphPair.GraphNew);
		}
	}

	bool bHasComponents = false;
	
	const auto BPClassFallback = [this]()->TSubclassOf<UObject>
	{
		if (PanelOld.Blueprint)
		{
			return PanelOld.Blueprint->GeneratedClass;
		}
		if (PanelNew.Blueprint)
		{
			return PanelNew.Blueprint->GeneratedClass;
		}
		check(false);
		return TSubclassOf<UObject>(nullptr); // this should never happen
	};
	
	const UClass* BlueprintClassOld = PanelOld.Blueprint ? PanelOld.Blueprint->GeneratedClass : BPClassFallback();
	const UClass* BlueprintClassNew = PanelNew.Blueprint ? PanelNew.Blueprint->GeneratedClass : BPClassFallback();
	const bool bIsOldClassActor = BlueprintClassOld && BlueprintClassOld->IsChildOf<AActor>();
	const bool bIsNewClassActor = BlueprintClassNew && BlueprintClassNew->IsChildOf<AActor>();
	if (bIsOldClassActor || bIsNewClassActor)
	{
		bHasComponents = true;
	}

	// If this isn't a normal blueprint type, add the type panel

	bool bIsSpecialized = false;
	if (PanelOld.Blueprint)
	{
		bIsSpecialized = (PanelOld.Blueprint->GetClass() != UBlueprint::StaticClass());
	}
	if (PanelNew.Blueprint)
	{
		bIsSpecialized |= (PanelNew.Blueprint->GetClass() != UBlueprint::StaticClass());
	}
	
	if (bIsSpecialized)
	{
		ModePanels.Add(BlueprintTypeMode, GenerateBlueprintTypePanel());
	}

	// Now that we have done the diffs, create the panel widgets
	ModePanels.Add(MyBlueprintMode, GenerateMyBlueprintPanel());
	ModePanels.Add(GraphMode, GenerateGraphPanel());
	if ((PanelOld.Blueprint && PanelOld.Blueprint->ParentClass) || (PanelNew.Blueprint && PanelNew.Blueprint->ParentClass))
	{
		ModePanels.Add(DefaultsMode, GenerateDefaultsPanel());
	}
	ModePanels.Add(ClassSettingsMode, GenerateClassSettingsPanel());
	if (bHasComponents)
	{
		ModePanels.Add(ComponentsMode, GenerateComponentsPanel());
	}

	for (const TSharedPtr<FGraphToDiff>& Graph : Graphs)
	{
		Graph->GenerateTreeEntries(PrimaryDifferencesList, RealDifferences);
	}
	
	ModePanels.Add(CommentsMode, GenerateGeneralFileCommentEntries());

	DifferencesTreeView->RebuildList();
}

SBlueprintDiff::FDiffControl SBlueprintDiff::GenerateBlueprintTypePanel()
{
	TSharedPtr<FBlueprintTypeDiffControl> NewDiffControl = MakeShared<FBlueprintTypeDiffControl>(PanelOld.Blueprint, PanelNew.Blueprint, FOnDiffEntryFocused::CreateRaw(this, &SBlueprintDiff::SetCurrentMode, BlueprintTypeMode));
	NewDiffControl->EnableComments(DifferencesTreeView.ToWeakPtr());
	NewDiffControl->GenerateTreeEntries(PrimaryDifferencesList, RealDifferences);

	SBlueprintDiff::FDiffControl Ret;
	//Splitter for left and right blueprint. Current convention is for the local (probably newer?) blueprint to be on the right:
	Ret.DiffControl = NewDiffControl;
	Ret.Widget = SNew(SSplitter)
		.PhysicalSplitterHandleSize(10.0f)
		+ SSplitter::Slot()
		.Value(0.5f)
		[
			SAssignNew(NewDiffControl->OldDetailsBox, SBox)
			.VAlign(VAlign_Fill)
			[
				DefaultEmptyPanel()
			]
		]
		+ SSplitter::Slot()
		.Value(0.5f)
		[
			SAssignNew(NewDiffControl->NewDetailsBox, SBox)
			.VAlign(VAlign_Fill)
			[
				DefaultEmptyPanel()
			]
		];

	return Ret;
}

SBlueprintDiff::FDiffControl SBlueprintDiff::GenerateMyBlueprintPanel()
{
	TSharedPtr<FMyBlueprintDiffControl> NewDiffControl = MakeShared<FMyBlueprintDiffControl>(PanelOld.Blueprint, PanelNew.Blueprint, FOnDiffEntryFocused::CreateRaw(this, &SBlueprintDiff::SetCurrentMode, MyBlueprintMode));
	NewDiffControl->EnableComments(DifferencesTreeView.ToWeakPtr());
	NewDiffControl->GenerateTreeEntries(PrimaryDifferencesList, RealDifferences);

	SBlueprintDiff::FDiffControl Ret;

	Ret.DiffControl = NewDiffControl;
	Ret.Widget = SNew(SVerticalBox)
	+ SVerticalBox::Slot()
	.FillHeight(1.f)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		[
			//diff window
			SNew(SSplitter)
			.Orientation(Orient_Vertical)
			+SSplitter::Slot()
			.Value(.8f)
			[
				SNew(SSplitter)
				.PhysicalSplitterHandleSize(10.0f)
				+ SSplitter::Slot()
				[
					PanelOld.GetMyBlueprintWidget()
				]
				+ SSplitter::Slot()
				[
					PanelNew.GetMyBlueprintWidget()
				]
			]
			+ SSplitter::Slot()
			.Value(.2f)
			[
				SNew(SSplitter)
				.PhysicalSplitterHandleSize(10.0f)
				+SSplitter::Slot()
				[
					PanelOld.GetDetailsWidget()
				]
				+ SSplitter::Slot()
				[
					PanelNew.GetDetailsWidget()
				]
			]
		]
	];

	return Ret;
}

SBlueprintDiff::FDiffControl SBlueprintDiff::GenerateGraphPanel()
{
	SBlueprintDiff::FDiffControl Ret;

	Ret.Widget = SNew(SVerticalBox)
	+ SVerticalBox::Slot()
	.FillHeight(1.f)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		[
			//diff window
			SNew(SSplitter)
			.Orientation(Orient_Vertical)
			+SSplitter::Slot()
			.Value(.8f)
			[
				SAssignNew(DiffGraphSplitter,SSplitter)
				.PhysicalSplitterHandleSize(10.0f)
				.Orientation(bVerticalSplitGraphMode ? Orient_Horizontal : Orient_Vertical)
				+ SSplitter::Slot() // Old revision graph slot
				[
					GenerateGraphWidgetForPanel(PanelOld)
				]
				+ SSplitter::Slot() // New revision graph slot
				[
					GenerateGraphWidgetForPanel(PanelNew)
				]
			]
			+ SSplitter::Slot()
			.Value(.2f)
			[
				SNew(SDetailsSplitter)
				+SDetailsSplitter::Slot()
					.Value(0.5f)
					.DetailsView(PanelOld.DetailsView ? PanelOld.DetailsView->GetPropertyView() : nullptr)
					.DifferencesWithRightPanel(GraphDetailDiff)
				+ SDetailsSplitter::Slot()
					.Value(0.5f)
					.DetailsView(PanelNew.DetailsView ? PanelNew.DetailsView->GetPropertyView() : nullptr)
			]
		]
	];

	return Ret;
}

SBlueprintDiff::FDiffControl SBlueprintDiff::GenerateDefaultsPanel()
{
	const UObject* A = DiffUtils::GetCDO(PanelOld.Blueprint);
	const UObject* B = DiffUtils::GetCDO(PanelNew.Blueprint);

	TSharedPtr<FCDODiffControl> NewDiffControl = MakeShared<FCDODiffControl>(A, B, FOnDiffEntryFocused::CreateRaw(this, &SBlueprintDiff::SetCurrentMode, DefaultsMode));
	NewDiffControl->EnableComments(DifferencesTreeView.ToWeakPtr());
	NewDiffControl->GenerateTreeEntries(PrimaryDifferencesList, RealDifferences);

	const TSharedRef<SDetailsSplitter> Splitter = SNew(SDetailsSplitter);
	if (A)
	{
		Splitter->AddSlot(
			SDetailsSplitter::Slot()
			.Value(0.5f)
			.DetailsView(NewDiffControl->GetDetailsWidget(A))
			.DifferencesWithRightPanel(NewDiffControl.ToSharedRef(), &FDetailsDiffControl::GetDifferencesWithRight, A)
		);
	}
	if (B)
	{
		Splitter->AddSlot(
			SDetailsSplitter::Slot()
			.Value(0.5f)
			.DetailsView(NewDiffControl->GetDetailsWidget(B))
			.DifferencesWithRightPanel(NewDiffControl.ToSharedRef(), &FDetailsDiffControl::GetDifferencesWithRight, B)
		);
	}

	SBlueprintDiff::FDiffControl Ret;
	Ret.DiffControl = NewDiffControl;
	Ret.Widget = Splitter;
	return Ret;
}

SBlueprintDiff::FDiffControl SBlueprintDiff::GenerateClassSettingsPanel()
{
	TSharedPtr<FClassSettingsDiffControl> NewDiffControl = MakeShared<FClassSettingsDiffControl>(PanelOld.Blueprint, PanelNew.Blueprint, FOnDiffEntryFocused::CreateRaw(this, &SBlueprintDiff::SetCurrentMode, ClassSettingsMode));
	NewDiffControl->EnableComments(DifferencesTreeView.ToWeakPtr());
	NewDiffControl->GenerateTreeEntries(PrimaryDifferencesList, RealDifferences);

	const TSharedRef<SDetailsSplitter> Splitter = SNew(SDetailsSplitter);
	if (PanelOld.Blueprint)
	{
		Splitter->AddSlot(
			SDetailsSplitter::Slot()
			.Value(0.5f)
			.DetailsView(NewDiffControl->GetDetailsWidget(PanelOld.Blueprint))
			.DifferencesWithRightPanel(NewDiffControl.ToSharedRef(), &FDetailsDiffControl::GetDifferencesWithRight, Cast<UObject>(PanelOld.Blueprint))
		);
	}
	if (PanelNew.Blueprint)
	{
		Splitter->AddSlot(
			SDetailsSplitter::Slot()
			.Value(0.5f)
			.DetailsView(NewDiffControl->GetDetailsWidget(PanelNew.Blueprint))
			.DifferencesWithRightPanel(NewDiffControl.ToSharedRef(), &FDetailsDiffControl::GetDifferencesWithRight, Cast<UObject>(PanelNew.Blueprint))
		);
	}
	
	SBlueprintDiff::FDiffControl Ret;
	Ret.DiffControl = NewDiffControl;
	Ret.Widget = Splitter;
	return Ret;
}

SBlueprintDiff::FDiffControl SBlueprintDiff::GenerateComponentsPanel()
{
	TSharedPtr<FSCSDiffControl> NewDiffControl = MakeShared<FSCSDiffControl>(PanelOld.Blueprint, PanelNew.Blueprint, FOnDiffEntryFocused::CreateRaw(this, &SBlueprintDiff::SetCurrentMode, ComponentsMode));
	NewDiffControl->EnableComments(DifferencesTreeView.ToWeakPtr());
	NewDiffControl->GenerateTreeEntries(PrimaryDifferencesList, RealDifferences);

	SBlueprintDiff::FDiffControl Ret;
	Ret.DiffControl = NewDiffControl;
	Ret.Widget = SNew(SSplitter)
		.PhysicalSplitterHandleSize(10.0f)
		+ SSplitter::Slot()
		.Value(0.5f)
		[
			NewDiffControl->OldTreeWidget()
		]
		+ SSplitter::Slot()
		.Value(0.5f)
		[
			NewDiffControl->NewTreeWidget()
		];

	return Ret;
}

SBlueprintDiff::FDiffControl SBlueprintDiff::GenerateGeneralFileCommentEntries()
{
	const UPackage* Package = PanelOld.Blueprint ? PanelOld.Blueprint->GetPackage() : PanelNew.Blueprint->GetPackage();
	const FPackagePath PackagePath = FPackagePath::FromPackageNameChecked(Package->GetName());
	const TSharedPtr<FReviewCommentsDiffControl> NewDiffControl = MakeShared<FReviewCommentsDiffControl>(PackagePath.GetLocalFullPath(), DifferencesTreeView.ToWeakPtr());
	NewDiffControl->GenerateTreeEntries(PrimaryDifferencesList, RealDifferences);

	SBlueprintDiff::FDiffControl Ret;
	Ret.DiffControl = NewDiffControl;

	return Ret;
}

TSharedRef<SOverlay> SBlueprintDiff::GenerateGraphWidgetForPanel(FDiffPanel& OutDiffPanel) const
{
	return SNew(SOverlay)
		+ SOverlay::Slot() // Graph slot
		[
			SAssignNew(OutDiffPanel.GraphEditorBox, SBox)
			.HAlign(HAlign_Fill)
			[
				DefaultEmptyPanel()
			]
		]
		+ SOverlay::Slot() // Revision info slot
		.VAlign(VAlign_Bottom)
		.HAlign(HAlign_Right)
		.Padding(FMargin(20.0f, 10.0f))
		[
			GenerateRevisionInfoWidgetForPanel(OutDiffPanel.OverlayGraphRevisionInfo,
			DiffViewUtils::GetPanelLabel(OutDiffPanel.Blueprint, OutDiffPanel.RevisionInfo, FText()))
		];
}

TSharedRef<SBox> SBlueprintDiff::GenerateRevisionInfoWidgetForPanel(TSharedPtr<SWidget>& OutGeneratedWidget,
	const FText& InRevisionText) const
{
	return SAssignNew(OutGeneratedWidget,SBox)
		.Padding(FMargin(4.0f, 10.0f))
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		[
			SNew(STextBlock)
			.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
			.Text(InRevisionText)
			.ShadowColorAndOpacity(FColor::Black)
			.ShadowOffset(FVector2D(1.4,1.4))
		];
}

void SBlueprintDiff::SetCurrentMode(FName NewMode)
{
	if (CurrentMode == NewMode)
	{
		return;
	}

	CurrentMode = NewMode;

	FDiffControl* FoundControl = ModePanels.Find(NewMode);

	if (FoundControl)
	{
		// Reset inspector view
		if (PanelOld.DetailsView)
		{
			PanelOld.DetailsView->ShowDetailsForObjects(TArray<UObject*>());
		}
		if (PanelNew.DetailsView)
		{
			PanelNew.DetailsView->ShowDetailsForObjects(TArray<UObject*>());
		}

		ModeContents->SetContent(FoundControl->Widget.ToSharedRef());
	}
	else
	{
		ensureMsgf(false, TEXT("Diff panel does not support mode %s"), *NewMode.ToString() );
	}

	OnModeChanged(NewMode);
}

void SBlueprintDiff::UpdateTopSectionVisibility(const FName& InNewViewMode) const
{
	SSplitter* GraphToolBarPtr = GraphToolBarWidget.Get();
	SSplitter* TopRevisionInfoWidgetPtr = TopRevisionInfoWidget.Get();

	if (!GraphToolBarPtr || !TopRevisionInfoWidgetPtr)
	{
		return;
	}
	
	if (InNewViewMode == GraphMode)
	{
		GraphToolBarPtr->SetVisibility(EVisibility::Visible);
		TopRevisionInfoWidgetPtr->SetVisibility(EVisibility::Collapsed);
	}
	else
	{
		GraphToolBarPtr->SetVisibility(EVisibility::Collapsed);
		TopRevisionInfoWidgetPtr->SetVisibility(EVisibility::HitTestInvisible);
	}
}

SBlueprintDiff::FScopedCategorySortChange::~FScopedCategorySortChange()
{
	if (Blueprint)
	{
		Blueprint->CategorySorting = Backup;
	}
}

void SBlueprintDiff::FScopedCategorySortChange::SetBlueprint(UBlueprint* InBlueprint)
{
	Blueprint = InBlueprint;
	Backup = Blueprint->CategorySorting;
}

void SBlueprintDiff::OnModeChanged(const FName& InNewViewMode) const
{
	UpdateTopSectionVisibility(InNewViewMode);
}

bool SBlueprintDiff::IsGraphDiffNeeded(UEdGraph* InGraph) const
{
	// Do not worry about graphs that are contained in MathExpression nodes, they are recreated each compile
	return !InGraph->GetOuter()->IsA<UK2Node_MathExpression>();
}

#undef LOCTEXT_NAMESPACE

