// Copyright Epic Games, Inc. All Rights Reserved.


#include "SBlueprintEditorToolbar.h"

#include "AssetToolsModule.h"
#include "BlueprintEditor.h"
#include "BlueprintEditorCommands.h"
#include "BlueprintEditorContext.h"
#include "DiffUtils.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Engine/Blueprint.h"
#include "Engine/LevelScriptBlueprint.h"
#include "FindInBlueprintManager.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/SlateDelegates.h"
#include "GraphEditorActions.h"
#include "HAL/PlatformMath.h"
#include "HAL/PlatformMisc.h"
#include "IAssetTools.h"
#include "IAssetTypeActions.h"
#include "ISourceCodeAccessModule.h"
#include "ISourceCodeAccessor.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "ISourceControlRevision.h"
#include "ISourceControlState.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/DebuggerCommands.h"
#include "Layout/Margin.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "SBlueprintEditorSelectedDebugObjectWidget.h"
#include "SBlueprintRevisionMenu.h"
#include "SourceControlHelpers.h"
#include "Styling/ISlateStyle.h"
#include "Templates/Casts.h"
#include "ToolMenu.h"
#include "ToolMenuDelegates.h"
#include "ToolMenuEntry.h"
#include "ToolMenuMisc.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectHash.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBorder.h"

class FUICommandList;
class SWidget;

#define LOCTEXT_NAMESPACE "KismetToolbar"

//////////////////////////////////////////////////////////////////////////
// SBlueprintModeSeparator

class SBlueprintModeSeparator : public SBorder
{
public:
	SLATE_BEGIN_ARGS(SBlueprintModeSeparator) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArg)
	{
		SBorder::Construct(
			SBorder::FArguments()
			.BorderImage(FAppStyle::GetBrush("BlueprintEditor.PipelineSeparator"))
			.Padding(0.0f)
			);
	}

	// SWidget interface
	virtual FVector2D ComputeDesiredSize(float) const override
	{
		const float Height = 20.0f;
		const float Thickness = 16.0f;
		return FVector2D(Thickness, Height);
	}
	// End of SWidget interface
};

//////////////////////////////////////////////////////////////////////////
// FKismet2Menu

void FKismet2Menu::FillFileMenuBlueprintSection(UToolMenu* InMenu)
{
	FToolMenuInsert InsertPosition("FileLoadAndSave", EToolMenuInsertType::After);

	{
		FToolMenuSection& Section = InMenu->AddSection("FileBlueprint", LOCTEXT("BlueprintHeading", "Blueprint"));
		Section.InsertPosition = InsertPosition;
		Section.AddMenuEntry( FBlueprintEditorCommands::Get().CompileBlueprint );
		Section.AddMenuEntry( FBlueprintEditorCommands::Get().RefreshAllNodes );
		Section.AddMenuEntry( FBlueprintEditorCommands::Get().ReparentBlueprint );
		Section.AddMenuEntry(FBlueprintEditorCommands::Get().BeginBlueprintMerge);
	}

	InMenu->AddDynamicSection("FileDeveloper", FNewToolMenuDelegate::CreateLambda([InsertPosition](UToolMenu* InMenu)
	{
		// Only show the developer menu on machines with the solution (assuming they can build it)
		ISourceCodeAccessModule* SourceCodeAccessModule = FModuleManager::GetModulePtr<ISourceCodeAccessModule>("SourceCodeAccess");
		if (SourceCodeAccessModule != nullptr && SourceCodeAccessModule->GetAccessor().CanAccessSourceCode())
		{
			FToolMenuSection& Section = InMenu->AddSection("FileDeveloper");
			Section.InsertPosition = InsertPosition;
			Section.AddSubMenu(
				"DeveloperMenu",
				LOCTEXT("DeveloperMenu", "Developer"),
				LOCTEXT("DeveloperMenu_ToolTip", "Open the developer menu"),
				FNewToolMenuDelegate::CreateStatic( &FKismet2Menu::FillDeveloperMenu ),
				false);
		}
	}));
}

void FKismet2Menu::FillDeveloperMenu(UToolMenu* InMenu)
{
	{
		FToolMenuSection& Section = InMenu->AddSection("FileDeveloperCompilerSettings", LOCTEXT("CompileOptionsHeading", "Compiler Settings"));
		Section.AddMenuEntry( FBlueprintEditorCommands::Get().SaveIntermediateBuildProducts );
	}

	if (FFindInBlueprintSearchManager::Get().ShouldEnableDeveloperMenuTools())
	{
		FToolMenuSection& Section = InMenu->AddSection("FileDeveloperSearchTools", LOCTEXT("SearchToolsHeading", "Search Tools"));
		Section.AddMenuEntry(FBlueprintEditorCommands::Get().GenerateSearchIndex);
		Section.AddMenuEntry(FBlueprintEditorCommands::Get().DumpCachedIndexData);
	}

	#if 0
	{
		{
			FToolMenuSection& Section = InMenu->AddSection("FileDeveloperFindReferences");
			Section.AddMenuEntry(FBlueprintEditorCommands::Get().FindReferencesFromClass);
			Section.AddMenuEntry(FBlueprintEditorCommands::Get().FindReferencesFromBlueprint);
			Section.AddMenuEntry(FBlueprintEditorCommands::Get().FindReferencesFromBlueprint);
		}
	}
	#endif

	{
		FToolMenuSection& Section = InMenu->AddSection("SchemaDeveloperSettings", LOCTEXT("SchemaDevUtilsHeading", "Schema Utilities"));
		Section.AddMenuEntry(FBlueprintEditorCommands::Get().ShowActionMenuItemSignatures);
	}
}

void FKismet2Menu::FillEditMenu(UToolMenu* InMenu)
{
	{
		FToolMenuSection& Section = InMenu->AddSection("EditSearch", LOCTEXT("EditMenu_SearchHeading", "Search"));
		Section.InsertPosition = FToolMenuInsert("EditHistory", EToolMenuInsertType::After);
		Section.AddMenuEntry( FBlueprintEditorCommands::Get().FindInBlueprint );
		Section.AddMenuEntry( FBlueprintEditorCommands::Get().FindInBlueprints );
		Section.AddMenuEntry( FBlueprintEditorCommands::Get().DeleteUnusedVariables );
	}
}

void FKismet2Menu::FillViewMenu(UToolMenu* InMenu)
{
	{
		FToolMenuSection& Section = InMenu->AddSection("ViewPinVisibility", LOCTEXT("ViewMenu_PinVisibilityHeading", "Pin Visibility"));
		Section.AddMenuEntry(FGraphEditorCommands::Get().ShowAllPins);
		Section.AddMenuEntry(FGraphEditorCommands::Get().HideNoConnectionNoDefaultPins);
		Section.AddMenuEntry(FGraphEditorCommands::Get().HideNoConnectionPins);
	}

	{
		FToolMenuSection& Section = InMenu->AddSection("ViewZoom", LOCTEXT("ViewMenu_ZoomHeading", "Zoom") );
		Section.AddMenuEntry( FBlueprintEditorCommands::Get().ZoomToWindow );
		Section.AddMenuEntry( FBlueprintEditorCommands::Get().ZoomToSelection );
	}
}

void FKismet2Menu::FillDebugMenu(UToolMenu* InMenu)
{
	{
		FToolMenuSection& Section = InMenu->AddSection("BlueprintDebugger", LOCTEXT("BlueprintDebugger_Heading", "Blueprint Debugger"));
		Section.AddMenuEntry(
			FBlueprintEditorCommands::Get().OpenBlueprintDebugger,
			/* InLabelOverride = */ LOCTEXT("BpDebuggerTitle", "Blueprint Debugger"),
			/* InTooltipOverride = */ LOCTEXT("BpDebuggerTooltip","Open the Blueprint Debugger."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintDebugger.TabIcon")
		);
	}
	
	{
		FToolMenuSection& Section = InMenu->AddSection("DebugBreakpoints", LOCTEXT("DebugMenu_BreakpointHeading", "Breakpoints"));
		Section.AddMenuEntry( FBlueprintEditorCommands::Get().DisableAllBreakpoints );
		Section.AddMenuEntry( FBlueprintEditorCommands::Get().EnableAllBreakpoints );
		Section.AddMenuEntry( FBlueprintEditorCommands::Get().ClearAllBreakpoints );
	}

	{
		FToolMenuSection& Section = InMenu->AddSection("DebugWatches", LOCTEXT("DebugMenu_WatchHeading", "Watches"));
		Section.AddMenuEntry( FBlueprintEditorCommands::Get().ClearAllWatches );
	}
}

void FKismet2Menu::SetupBlueprintEditorMenu(const FName MainMenuName)
{
	const FName ParentMenuName("MainFrame.MainMenu");

	{
		const FName FileMenuName = *(MainMenuName.ToString() + TEXT(".File"));
		if (!UToolMenus::Get()->IsMenuRegistered(FileMenuName))
		{
			FKismet2Menu::FillFileMenuBlueprintSection(UToolMenus::Get()->RegisterMenu(FileMenuName, *(ParentMenuName.ToString() + TEXT(".File"))));
		}
	}

	{
		const FName EditMenuName = *(MainMenuName.ToString() + TEXT(".Edit"));
		if (!UToolMenus::Get()->IsMenuRegistered(EditMenuName))
		{
			FKismet2Menu::FillEditMenu(UToolMenus::Get()->RegisterMenu(EditMenuName, *(ParentMenuName.ToString() + TEXT(".Edit"))));
		}
	}

	// Add additional blueprint editor menus
	{
		FToolMenuSection& Section = UToolMenus::Get()->ExtendMenu(MainMenuName)->FindOrAddSection(NAME_None);

		// View
		if (!Section.FindEntry("View"))
		{
			Section.AddSubMenu(
				"View",
				LOCTEXT("ViewMenu", "View"),
				LOCTEXT("ViewMenu_ToolTip", "Open the View menu"),
				FNewToolMenuDelegate::CreateStatic(&FKismet2Menu::FillViewMenu)
			).InsertPosition = FToolMenuInsert("Edit", EToolMenuInsertType::After);
		}

		// Debug
		if (!Section.FindEntry("Debug"))
		{
			Section.AddSubMenu(
				"Debug",
				LOCTEXT("DebugMenu", "Debug"),
				LOCTEXT("DebugMenu_ToolTip", "Open the debug menu"),
				FNewToolMenuDelegate::CreateStatic(&FKismet2Menu::FillDebugMenu)
			).InsertPosition = FToolMenuInsert("Edit", EToolMenuInsertType::After);
		}
	}
}


//////////////////////////////////////////////////////////////////////////
// FFullBlueprintEditorCommands

void FFullBlueprintEditorCommands::RegisterCommands() 
{
	UI_COMMAND(Compile, "Compile", "Compile the blueprint", EUserInterfaceActionType::Button, FInputChord());
	
	UI_COMMAND(SaveOnCompile_Never, "Never", "Sets the save-on-compile option to 'Never', meaning that your Blueprints will not be saved when they are compiled", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SaveOnCompile_SuccessOnly, "On Success Only", "Sets the save-on-compile option to 'Success Only', meaning that your Blueprints will be saved whenever they are successfully compiled", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SaveOnCompile_Always, "Always", "Sets the save-on-compile option to 'Always', meaning that your Blueprints will be saved whenever they are compiled (even if there were errors)", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(SwitchToScriptingMode, "Graph", "Switches to Graph Editing Mode", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SwitchToBlueprintDefaultsMode, "Defaults", "Switches to Class Defaults Mode", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SwitchToComponentsMode, "Components", "Switches to Components Mode", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(EditGlobalOptions, "Class Settings", "Edit Class Settings", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(EditClassDefaults, "Class Defaults", "Edit the initial values of your class.", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(JumpToErrorNode, "Jump to Error Node", "When enabled, then the Blueprint will snap focus to nodes producing an error during compilation", EUserInterfaceActionType::ToggleButton, FInputChord());
}

//////////////////////////////////////////////////////////////////////////
// Static FBlueprintEditorToolbar Helpers

namespace BlueprintEditorToolbarImpl
{
	static void GenerateCompileOptionsMenu(UToolMenu* InMenu);
	static void MakeSaveOnCompileSubMenu(UToolMenu* InMenu);
	static void MakeCompileDeveloperSubMenu(UToolMenu* InMenu);
};

static void BlueprintEditorToolbarImpl::GenerateCompileOptionsMenu(UToolMenu* InMenu)
{
	FToolMenuSection& Section = InMenu->AddSection("Section");
	const FFullBlueprintEditorCommands& Commands = FFullBlueprintEditorCommands::Get();

	// @TODO: disable the menu and change up the tooltip when all sub items are disabled
	Section.AddSubMenu(
		"SaveOnCompile",
		LOCTEXT("SaveOnCompileSubMenu", "Save on Compile"),
		LOCTEXT("SaveOnCompileSubMenu_ToolTip", "Determines how the Blueprint is saved whenever you compile it."),
		FNewToolMenuDelegate::CreateStatic(&BlueprintEditorToolbarImpl::MakeSaveOnCompileSubMenu));

	Section.AddMenuEntry(Commands.JumpToErrorNode);

// 	Section.AddSubMenu(
// 		"DevCompile",
// 		LOCTEXT("DevCompileSubMenu", "Developer"),
// 		LOCTEXT("DevCompileSubMenu_ToolTip", "Advanced settings that aid in devlopment/debugging of the Blueprint system as a whole."),
// 		FNewMenuDelegate::CreateStatic(&BlueprintEditorToolbarImpl::MakeCompileDeveloperSubMenu));

}

static void BlueprintEditorToolbarImpl::MakeSaveOnCompileSubMenu(UToolMenu* InMenu)
{
	FToolMenuSection& Section = InMenu->AddSection("Section");
	const FFullBlueprintEditorCommands& Commands = FFullBlueprintEditorCommands::Get();
	Section.AddMenuEntry(Commands.SaveOnCompile_Never);
	Section.AddMenuEntry(Commands.SaveOnCompile_SuccessOnly);
	Section.AddMenuEntry(Commands.SaveOnCompile_Always);
}

static void BlueprintEditorToolbarImpl::MakeCompileDeveloperSubMenu(UToolMenu* InMenu)
{
	FToolMenuSection& Section = InMenu->AddSection("Section");
	const FBlueprintEditorCommands& EditorCommands = FBlueprintEditorCommands::Get();
	Section.AddMenuEntry(EditorCommands.SaveIntermediateBuildProducts);
	Section.AddMenuEntry(EditorCommands.ShowActionMenuItemSignatures);
}


//////////////////////////////////////////////////////////////////////////
// FBlueprintEditorToolbar

void FBlueprintEditorToolbar::AddBlueprintGlobalOptionsToolbar(UToolMenu* InMenu, bool bRegisterViewport)
{
	FToolMenuSection& Section = InMenu->FindOrAddSection("Settings");
	Section.InsertPosition = FToolMenuInsert("Asset", EToolMenuInsertType::After);

	Section.AddDynamicEntry("BlueprintGlobalOptions", FNewToolMenuSectionDelegate::CreateLambda([bRegisterViewport](FToolMenuSection& InSection)
	{
		const UBlueprintEditorToolMenuContext* Context = InSection.FindContext<UBlueprintEditorToolMenuContext>();
		if (Context && Context->GetBlueprintObj())
		{
			const FFullBlueprintEditorCommands& Commands = FFullBlueprintEditorCommands::Get();
			InSection.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.EditGlobalOptions));
			InSection.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.EditClassDefaults));

			if (bRegisterViewport)
			{
				InSection.AddEntry(FToolMenuEntry::InitToolBarButton(FBlueprintEditorCommands::Get().EnableSimulation));
			}
		}
	}));
}

void FBlueprintEditorToolbar::AddCompileToolbar(UToolMenu* InMenu)
{
	FToolMenuSection& Section = InMenu->AddSection("Compile");
	Section.InsertPosition = FToolMenuInsert("Asset", EToolMenuInsertType::Before);

	Section.AddDynamicEntry("CompileCommands", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		const UBlueprintEditorToolMenuContext* Context = InSection.FindContext<UBlueprintEditorToolMenuContext>();
		if (Context && Context->BlueprintEditor.IsValid() && Context->GetBlueprintObj())
		{
			TSharedPtr<class FBlueprintEditorToolbar> BlueprintEditorToolbar = Context->BlueprintEditor.Pin()->GetToolbarBuilder();
			if (BlueprintEditorToolbar.IsValid())
			{
				const FFullBlueprintEditorCommands& Commands = FFullBlueprintEditorCommands::Get();

				FToolMenuEntry& CompileButton = InSection.AddEntry(FToolMenuEntry::InitToolBarButton(
					Commands.Compile,
					TAttribute<FText>(),
					TAttribute<FText>(BlueprintEditorToolbar.ToSharedRef(), &FBlueprintEditorToolbar::GetStatusTooltip),
					TAttribute<FSlateIcon>(BlueprintEditorToolbar.ToSharedRef(), &FBlueprintEditorToolbar::GetStatusImage),
					"CompileBlueprint"));
				CompileButton.StyleNameOverride = "CalloutToolbar";

				FToolMenuEntry& CompileOptions = InSection.AddEntry(FToolMenuEntry::InitComboButton(
					"CompileComboButton",
					FUIAction(),
					FNewToolMenuDelegate::CreateStatic(&BlueprintEditorToolbarImpl::GenerateCompileOptionsMenu),
					LOCTEXT("BlupeintCompileOptions_ToolbarTooltip", "Options to customize how Blueprints compile")
				));
				CompileOptions.StyleNameOverride = "CalloutToolbar";
				CompileOptions.ToolBarData.bSimpleComboBox = true;
			}
		}
	}));

	// We want the diff menu to be on any blueprint toolbar that also contains compile
	FToolMenuSection& DiffSection = InMenu->AddSection("SourceControl");
	DiffSection.InsertPosition = FToolMenuInsert("Asset", EToolMenuInsertType::After);

	DiffSection.AddDynamicEntry("SourceControlCommands", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			const UBlueprintEditorToolMenuContext* Context = InSection.FindContext<UBlueprintEditorToolMenuContext>();
			if (Context && Context->BlueprintEditor.IsValid() && Context->GetBlueprintObj())
			{
				TSharedPtr<class FBlueprintEditorToolbar> BlueprintEditorToolbar = Context->BlueprintEditor.Pin()->GetToolbarBuilder();
				if (BlueprintEditorToolbar.IsValid())
				{
					FToolMenuEntry& DiffEntry = InSection.AddEntry(FToolMenuEntry::InitComboButton(
						"Diff",
						FUIAction(),
						FOnGetContent::CreateStatic(&FBlueprintEditorToolbar::MakeDiffMenu, Context),
						LOCTEXT("Diff", "Diff"),
						LOCTEXT("BlueprintEditorDiffToolTip", "Diff against previous revisions"),
						FSlateIcon(FAppStyle::Get().GetStyleSetName(), "BlueprintDiff.ToolbarIcon")
					));
					DiffEntry.StyleNameOverride = "CalloutToolbar";
				}
			}
		}));
}

void FBlueprintEditorToolbar::AddNewToolbar(UToolMenu* InMenu)
{
	FToolMenuSection& Section = InMenu->AddSection("Add");
	Section.InsertPosition = FToolMenuInsert("MyBlueprint", EToolMenuInsertType::After);

	Section.AddDynamicEntry("AddCommands", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		UBlueprintEditorToolMenuContext* Context = InSection.FindContext<UBlueprintEditorToolMenuContext>();
		if (Context && Context->BlueprintEditor.IsValid() && Context->GetBlueprintObj())
		{
			const FBlueprintEditorCommands& Commands = FBlueprintEditorCommands::Get();
			InSection.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.AddNewVariable, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(), FName(TEXT("BPEAddNewVariable"))));
			InSection.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.AddNewFunction, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(), FName(TEXT("BPEAddNewFunction"))));
			InSection.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.AddNewMacroDeclaration, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(), FName(TEXT("BPEAddNewMacro"))));
			// Add New Animation Graph isn't supported right now.
			InSection.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.AddNewEventGraph, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(), FName(TEXT("BPEAddNewEventGraph"))));
			InSection.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.AddNewDelegate, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(), FName(TEXT("BPEAddNewDelegate"))));
		}
	}));
}

void FBlueprintEditorToolbar::AddScriptingToolbar(UToolMenu* InMenu)
{
	FToolMenuSection& Section = InMenu->AddSection("Script");
	Section.InsertPosition = FToolMenuInsert("Asset", EToolMenuInsertType::After);

	Section.AddDynamicEntry("ScriptCommands", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		UBlueprintEditorToolMenuContext* Context = InSection.FindContext<UBlueprintEditorToolMenuContext>();
		if (Context && Context->BlueprintEditor.IsValid() && Context->GetBlueprintObj())
		{
			InSection.AddEntry(FToolMenuEntry::InitToolBarButton(FBlueprintEditorCommands::Get().FindInBlueprint));

			InSection.AddEntry(FToolMenuEntry::InitToolBarButton(
				FBlueprintEditorCommands::Get().ToggleHideUnrelatedNodes,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.ToggleHideUnrelatedNodes")
			));

			InSection.AddEntry(FToolMenuEntry::InitComboButton(
				"HideUnrelatedNodesOptions",
				FUIAction(),
				FOnGetContent::CreateSP(Context->BlueprintEditor.Pin().Get(), &FBlueprintEditor::MakeHideUnrelatedNodesOptionsMenu),
				LOCTEXT("HideUnrelatedNodesOptions", "Hide Unrelated Nodes Options"),
				LOCTEXT("HideUnrelatedNodesOptionsMenu", "Hide Unrelated Nodes options menu"),
				TAttribute<FSlateIcon>(),
				true
			));
		}
	}));
}

void FBlueprintEditorToolbar::AddDebuggingToolbar(UToolMenu* InMenu)
{
	FToolMenuSection& Section = InMenu->AddSection("Debugging");
	Section.InsertPosition = FToolMenuInsert("Asset", EToolMenuInsertType::After);

	Section.AddDynamicEntry("DebuggingCommands", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		UBlueprintEditorToolMenuContext* Context = InSection.FindContext<UBlueprintEditorToolMenuContext>();
		if (Context && Context->BlueprintEditor.IsValid() && Context->GetBlueprintObj())
		{
			FPlayWorldCommands::BuildToolbar(InSection);

			if (Context->GetBlueprintObj()->BlueprintType != BPTYPE_MacroLibrary)
			{
				// Selected debug actor button
				InSection.AddEntry(FToolMenuEntry::InitWidget("SelectedDebugObjectWidget", SNew(SBlueprintEditorSelectedDebugObjectWidget, Context->BlueprintEditor.Pin()), FText::GetEmpty()));
			}
		}
	}));
}

FSlateIcon FBlueprintEditorToolbar::GetStatusImage() const
{
	UBlueprint* BlueprintObj = BlueprintEditor.Pin()->GetBlueprintObj();
	EBlueprintStatus Status = BlueprintObj->Status;

	// For macro types, always show as up-to-date, since we don't compile them
	if (BlueprintObj->BlueprintType == BPTYPE_MacroLibrary)
	{
		Status = BS_UpToDate;
	}

	
	static const FName CompileStatusBackground("Blueprint.CompileStatus.Background");
	static const FName CompileStatusUnknown("Blueprint.CompileStatus.Overlay.Unknown");
	static const FName CompileStatusError("Blueprint.CompileStatus.Overlay.Error");
	static const FName CompileStatusGood("Blueprint.CompileStatus.Overlay.Good");
	static const FName CompileStatusWarning("Blueprint.CompileStatus.Overlay.Warning");

	switch (Status)
	{
	default:
	case BS_Unknown:
	case BS_Dirty:
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusUnknown);
	case BS_Error:
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusError);
	case BS_UpToDate:
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusGood);
	case BS_UpToDateWithWarnings:
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusWarning);
	}
}

FText FBlueprintEditorToolbar::GetStatusTooltip() const
{
	UBlueprint* BlueprintObj = BlueprintEditor.Pin()->GetBlueprintObj();
	EBlueprintStatus Status = BlueprintObj->Status;

	// For macro types, always show as up-to-date, since we don't compile them
	if (BlueprintObj->BlueprintType == BPTYPE_MacroLibrary)
	{
		Status = BS_UpToDate;
	}

	switch (Status)
	{
	default:
	case BS_Unknown:
		return LOCTEXT("Recompile_Status", "Unknown status; should recompile");
	case BS_Dirty:
		return LOCTEXT("Dirty_Status", "Dirty; needs to be recompiled");
	case BS_Error:
		return LOCTEXT("CompileError_Status", "There was an error during compilation, see the log for details");
	case BS_UpToDate:
		return LOCTEXT("GoodToGo_Status", "Good to go");
	case BS_UpToDateWithWarnings:
		return LOCTEXT("GoodToGoWarning_Status", "There was a warning during compilation, see the log for details");
	}
}


/** Delegate called to diff a specific revision with the current */
static void OnDiffRevisionPicked(FRevisionInfo const& RevisionInfo, TWeakObjectPtr<UBlueprint> BlueprintObj)
{
	if (BlueprintObj.IsValid())
	{
		bool const bIsLevelScriptBlueprint = FBlueprintEditorUtils::IsLevelScriptBlueprint(BlueprintObj.Get());
		FString const Filename = SourceControlHelpers::PackageFilename(bIsLevelScriptBlueprint ? BlueprintObj.Get()->GetOuter()->GetPathName() : BlueprintObj.Get()->GetPathName());

		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

		// Get the SCC state
		FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(Filename, EStateCacheUsage::Use);
		if (SourceControlState.IsValid())
		{
			for (int32 HistoryIndex = 0; HistoryIndex < SourceControlState->GetHistorySize(); HistoryIndex++)
			{
				TSharedPtr<ISourceControlRevision, ESPMode::ThreadSafe> Revision = SourceControlState->GetHistoryItem(HistoryIndex);
				check(Revision.IsValid());
				if (Revision->GetRevision() == RevisionInfo.Revision)
				{
					// Get the revision of this package from source control
					if (UPackage* PreviousTempPkg = DiffUtils::LoadPackageForDiff(Revision))
					{
						UObject* PreviousAsset = nullptr;

						// If its a levelscript blueprint, find the previous levelscript blueprint in the map
						if (bIsLevelScriptBlueprint)
						{
							TArray<UObject*> ObjectsInOuter;
							GetObjectsWithOuter(PreviousTempPkg, ObjectsInOuter);

							// Look for the level script blueprint for this package
							for (int32 Index = 0; Index < ObjectsInOuter.Num(); Index++)
							{
								UObject* Obj = ObjectsInOuter[Index];
								if (ULevelScriptBlueprint* ObjAsBlueprint = Cast<ULevelScriptBlueprint>(Obj))
								{
									PreviousAsset = ObjAsBlueprint;
									break;
								}
							}
							if (!PreviousAsset)
							{
								UE_LOGF(LogSourceControl, Warning, "Revision %ls of %ls doesn't have a LevelScriptBlueprint", *Revision->GetRevision(), *Revision->GetFilename());
							}
						}
						// otherwise its a normal Blueprint
						else
						{
							FString PreviousAssetName = FPaths::GetBaseFilename(Filename, true);
							PreviousAsset = FindObject<UObject>(PreviousTempPkg, *PreviousAssetName);
						}

						if (PreviousAsset != nullptr)
						{
							FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
							FRevisionInfo OldRevision = { Revision->GetRevision(), Revision->GetCheckInIdentifier(), Revision->GetDate() };
							FRevisionInfo CurrentRevision = { TEXT(""), Revision->GetCheckInIdentifier(), Revision->GetDate() };
							AssetToolsModule.Get().DiffAssets(PreviousAsset, BlueprintObj.Get(), OldRevision, CurrentRevision);
						}
					}
					else
					{
						FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("SourceControl.HistoryWindow", "UnableToLoadAssets", "Unable to load assets to diff. Content may no longer be supported?"));
					}
					break;
				}
			}
		}
	}
}

TSharedRef<SWidget> FBlueprintEditorToolbar::MakeDiffMenu(const UBlueprintEditorToolMenuContext* InContext)
{
	if (ISourceControlModule::Get().IsEnabled() && ISourceControlModule::Get().GetProvider().IsAvailable())
	{
		UBlueprint* BlueprintObj =  InContext ? InContext->GetBlueprintObj() : nullptr;
		if (BlueprintObj)
		{
			TWeakObjectPtr<UBlueprint> BlueprintPtr = BlueprintObj;
			// Add our async SCC task widget
			return SNew(SBlueprintRevisionMenu, BlueprintObj)
				.OnRevisionSelected_Static(&OnDiffRevisionPicked, BlueprintPtr);
		}
		else
		{
			// if BlueprintObj is null then this means that multiple blueprints are selected
			FMenuBuilder MenuBuilder(true, NULL);
			MenuBuilder.AddMenuEntry(LOCTEXT("NoRevisionsForMultipleBlueprints", "Multiple blueprints selected"),
				FText(), FSlateIcon(), FUIAction());
			return MenuBuilder.MakeWidget();
		}
	}

	FMenuBuilder MenuBuilder(true, NULL);
	MenuBuilder.AddMenuEntry(LOCTEXT("SourceControlDisabled", "Revision control is disabled"),
		FText(), FSlateIcon(), FUIAction());
	return MenuBuilder.MakeWidget();
}


#undef LOCTEXT_NAMESPACE
