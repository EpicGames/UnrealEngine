// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBlueprintFavoritesPalette.h"

#include "BlueprintActionFilter.h"
#include "BlueprintActionMenuBuilder.h"
#include "BlueprintActionMenuUtils.h"
#include "BlueprintEditor.h"
#include "BlueprintPaletteFavorites.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphSchema.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformProcess.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Parse.h"
#include "SGraphActionMenu.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

#define LOCTEXT_NAMESPACE "BlueprintFavoritesPalette"

/*******************************************************************************
* Static File Helpers
*******************************************************************************/

/** 
 * Contains static helper methods (scoped inside this struct to avoid collisions 
 * during unified builds). 
 */
struct SBlueprintFavoritesPaletteUtils
{
	/** The definition of a delegate used to retrieve a set of palette actions */
	DECLARE_DELEGATE_OneParam(FPaletteActionGetter, TArray< TSharedPtr<FEdGraphSchemaAction> >&);

	/**
	 * Uses the provided ActionGetter to get a list of selected actions, and then 
	 * removes every one from the user's favorites.
	 * 
	 * @param  ActionGetter		A delegate to use for grabbing the palette's selected actions.
	 */
	static void RemoveSelectedFavorites(FPaletteActionGetter ActionGetter)
	{
		const UEditorPerProjectUserSettings* EditorPerProjectUserSettings = GetDefault<UEditorPerProjectUserSettings>();
		if (ActionGetter.IsBound() && (EditorPerProjectUserSettings->BlueprintFavorites != NULL))
		{
			TArray< TSharedPtr<FEdGraphSchemaAction> > SelectedActions;
			ActionGetter.Execute(SelectedActions);

			EditorPerProjectUserSettings->BlueprintFavorites->RemoveFavorites(SelectedActions);
		}
	}

	/**
	 * Removes every single favorite and sets the user's profile to a custom one
	 * (if it isn't already).
	 */
	static void ClearPaletteFavorites()
	{
		const UEditorPerProjectUserSettings* EditorPerProjectUserSettings = GetDefault<UEditorPerProjectUserSettings>();
		if (EditorPerProjectUserSettings->BlueprintFavorites != NULL)
		{
			EditorPerProjectUserSettings->BlueprintFavorites->ClearAllFavorites();
		}
	}

	/**
	 * A UI hook, used to determine if the specified profile can be loaded (or 
	 * is it what's currently set).
	 * 
	 * @param  ProfileName	The name of the profile you wish to check.
	 * @return True if the user's settings don't have that profile currently set, false if it is set or the user's setting are null.
	 */
	static bool CanLoadFavoritesProfile(FString const& ProfileName)
	{
		bool bIsLoaded = false;

		const UEditorPerProjectUserSettings* EditorPerProjectUserSettings = GetDefault<UEditorPerProjectUserSettings>();
		if (EditorPerProjectUserSettings->BlueprintFavorites != NULL)
		{
			bIsLoaded = (EditorPerProjectUserSettings->BlueprintFavorites->GetCurrentProfile() == ProfileName);
		}

		return !bIsLoaded;
	}

	/**
	 * A UI hook for loading a specific favorites-profile, which throws out all 
	 * current favorites and loads in ones for the specified profile.
	 * 
	 * @param  ProfileName	The name of the profile you wish to load.
	 */
	static void LoadFavoritesProfile(FString ProfileName)
	{
		const UEditorPerProjectUserSettings* EditorPerProjectUserSettings = GetDefault<UEditorPerProjectUserSettings>();
		if (EditorPerProjectUserSettings->BlueprintFavorites != NULL)
		{
			EditorPerProjectUserSettings->BlueprintFavorites->LoadProfile(ProfileName);
		}
	}

	/**
	 * Takes the provided menu builder and adds elements representing various 
	 * profiles that the user can choose from (default, tutorial, etc.).
	 * 
	 * @param  MenuBuilder	The builder to modify and add entries to.
	 */
	static void BuildProfilesSubMenu(FMenuBuilder& MenuBuilder)
	{
		TArray<FString> AvailableProfiles;
	
		const UEditorPerProjectUserSettings* EditorPerProjectUserSettings = GetDefault<UEditorPerProjectUserSettings>();
		if (EditorPerProjectUserSettings->BlueprintFavorites != NULL)
		{
			static FString const ProfilesConfigKey("Profiles");
			GConfig->GetArray(*SBlueprintFavoritesPaletteUtils::ConfigSection, *ProfilesConfigKey, AvailableProfiles, GEditorIni);	
		}

		struct LocalUtils
		{
			static bool CanExecute()               { return true; }
			static bool CannotExecute()            { return false; }
			static void NavigateToURL(FString URL) { FPlatformProcess::LaunchURL(*URL, NULL, NULL); }
		};

		if (AvailableProfiles.Num() > 0)
		{
			for (FString const& Profile : AvailableProfiles)
			{
				FString ProfileName;
				FParse::Value(*Profile, TEXT("Name="), ProfileName);
				FString FriendlyProfileName;
				FParse::Value(*Profile, TEXT("FriendlyName="), FriendlyProfileName);
				FString ProfileToolTip;
				FParse::Value(*Profile, TEXT("ToolTip="), ProfileToolTip);
				FString ProfileURL;
				FParse::Value(*Profile, TEXT("URL="), ProfileURL);
				FString ProfileURLName;
				FParse::Value(*Profile, TEXT("URLText="), ProfileURLName);

				// @TODO how to best localize this?
				FText ToolTipText = FText::FromString(ProfileToolTip);
				if (ProfileURLName.IsEmpty())
				{
					ProfileURLName = ProfileURL;
				}
				if (FriendlyProfileName.IsEmpty())
				{
					FriendlyProfileName = ProfileName;
				}
				
				FUIAction ProfileAction;
				if (SBlueprintFavoritesPaletteUtils::CanLoadFavoritesProfile(ProfileName))
				{
					if (ToolTipText.IsEmpty())
					{
						ToolTipText = FText::Format(LOCTEXT("ProfileAvailableFmt", "Loads {0} node favorites"), FText::FromString(FriendlyProfileName));
					}

					ProfileAction = FUIAction(
						FExecuteAction::CreateStatic(&SBlueprintFavoritesPaletteUtils::LoadFavoritesProfile, ProfileName),
						FCanExecuteAction::CreateStatic(&LocalUtils::CanExecute)
					);
				}
				else 
				{
					if (ToolTipText.IsEmpty())
					{
						ToolTipText = LOCTEXT("ProfileLoaded", "Current profile");
					}

					ProfileAction = FUIAction(
						FExecuteAction(),
						FCanExecuteAction::CreateStatic(&LocalUtils::CannotExecute)
					);
				}

				// build the text that goes in the sub-menu
				TSharedRef<STextBlock> MenuTextEntry = SNew(STextBlock)
					.TextStyle(MenuBuilder.GetStyleSet(), FAppStyle::Join("Menu", ".Label"))
					// @TODO how do we best localize this
					.Text(FText::FromString(FriendlyProfileName));

				FSlateFontInfo ToolTipFont = FCoreStyle::GetDefaultFontStyle("Regular", 8);

				TSharedPtr<SVerticalBox> ToolTipBox;
				// build the specialized tooltip
				TSharedRef<SToolTip> ToolTipWidget = SNew(SToolTip)
					[
						SAssignNew(ToolTipBox, SVerticalBox)
						+SVerticalBox::Slot()
						[
							SNew(STextBlock)
								.WrapTextAt(400)
								.Font(ToolTipFont)
								.Text(ToolTipText)
						]
					];

				// add the url if one was specified
				if (!ProfileURL.IsEmpty())
				{
					ToolTipBox->AddSlot()
						.AutoHeight()
						.HAlign(HAlign_Right)
					[
						SNew(SHyperlink)
							// @TODO how to best localize this?
							.Text(FText::FromString(ProfileURLName))
							.OnNavigate_Static(&LocalUtils::NavigateToURL, ProfileURL)
					];
				}

				// now build the actual menu widget
				TSharedRef<SWidget> MenuEntryWidget = SNew(SHorizontalBox)
						.ToolTip(ToolTipWidget)
						// so the tool tip shows up for the whole entry:
						.Visibility(EVisibility::Visible)
					+SHorizontalBox::Slot()
						// match the padding with normal sub-menu entries
						.Padding(FMargin(18.f, 0.f, 6.f, 0.f))
						.FillWidth(1.0f)
					[
						MenuTextEntry
					];

				MenuBuilder.AddMenuEntry(ProfileAction, MenuEntryWidget);
			}
		}
		else 
		{
			FUIAction NullProfileAction(FExecuteAction(), FCanExecuteAction::CreateStatic(&LocalUtils::CannotExecute));
			MenuBuilder.AddMenuEntry(LOCTEXT("NoProfiles", "No profiles available"), FText::GetEmpty(), FSlateIcon(), NullProfileAction);
		}
	}

	/** String constants shared between multiple SBlueprintLibraryPalette functions */
	static FString const ConfigSection;
	static FString const FavoritesCategoryName;
};

FString const SBlueprintFavoritesPaletteUtils::ConfigSection("BlueprintEditor.Favorites");
FString const SBlueprintFavoritesPaletteUtils::FavoritesCategoryName = LOCTEXT("FavoriteseCategory", "Favorites").ToString();

/*******************************************************************************
* FBlueprintFavoritesPaletteCommands
*******************************************************************************/

class FBlueprintFavoritesPaletteCommands : public TCommands<FBlueprintFavoritesPaletteCommands>
{
public:
	FBlueprintFavoritesPaletteCommands() : TCommands<FBlueprintFavoritesPaletteCommands>
		( "BlueprintFavoritesPalette"
		, LOCTEXT("FavoritesPaletteContext", "Favorites Palette")
		, NAME_None
		, FAppStyle::GetAppStyleSetName() )
	{
	}

	TSharedPtr<FUICommandInfo> RemoveSingleFavorite;
	TSharedPtr<FUICommandInfo> RemoveSubFavorites;
	TSharedPtr<FUICommandInfo> ClearFavorites;

	/** Registers context menu commands for the blueprint favorites palette. */
	virtual void RegisterCommands() override
	{
		UI_COMMAND(RemoveSingleFavorite, "Remove from Favorites",          "Removes this item from your favorites list.",                 EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(RemoveSubFavorites,   "Remove Category from Favorites", "Removes all the nodes in this category from your favorites.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(ClearFavorites,       "Clear All Favorites",			   "Clears out all of your favorited nodes.",                     EUserInterfaceActionType::Button, FInputChord());
	}
};

/*******************************************************************************
* SBlueprintFavoritesPalette Public Interface
*******************************************************************************/

//------------------------------------------------------------------------------
SBlueprintFavoritesPalette::~SBlueprintFavoritesPalette()
{
	const UEditorPerProjectUserSettings* EditorPerProjectUserSettings = GetDefault<UEditorPerProjectUserSettings>();
	if (EditorPerProjectUserSettings->BlueprintFavorites != NULL)
	{
		EditorPerProjectUserSettings->BlueprintFavorites->OnFavoritesUpdated.RemoveAll(this);
	}
}

//------------------------------------------------------------------------------
void SBlueprintFavoritesPalette::Construct(FArguments const& InArgs, TWeakPtr<FBlueprintEditor> InBlueprintEditor)
{
	SBlueprintSubPalette::FArguments SuperArgs;
	SuperArgs._Title       = LOCTEXT("PaletteTitle", "Favorites");
	SuperArgs._Icon        = FAppStyle::GetBrush("Icons.Star");
	SuperArgs._ToolTipText = LOCTEXT("PaletteToolTip", "A listing of your favorite and most used nodes.");

	static FString const ShowFreqUsedConfigKey("bShowFrequentlyUsed");
	// should be set before we call the super (so CollectAllActions() has the right value)
	bShowFrequentlyUsed = false;
	GConfig->GetBool(*SBlueprintFavoritesPaletteUtils::ConfigSection, *ShowFreqUsedConfigKey, bShowFrequentlyUsed, GEditorIni);	

	SBlueprintSubPalette::Construct(SuperArgs, InBlueprintEditor);

	const UEditorPerProjectUserSettings* EditorPerProjectUserSettings = GetDefault<UEditorPerProjectUserSettings>();
	if (EditorPerProjectUserSettings->BlueprintFavorites != NULL)
	{
		EditorPerProjectUserSettings->BlueprintFavorites->OnFavoritesUpdated.AddSP(this, &SBlueprintFavoritesPalette::RefreshActionsList, true);
	}
}

/*******************************************************************************
* SBlueprintFavoritesPalette Private Methods
*******************************************************************************/

//------------------------------------------------------------------------------
void SBlueprintFavoritesPalette::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	FBlueprintActionContext FilterContext;
	FilterContext.Blueprints.Add(GetBlueprint());
	FilterContext.EditorPtr = BlueprintEditorPtr;

	FBlueprintActionMenuBuilder FavoritesBuilder;
	FBlueprintActionMenuUtils::MakeFavoritesMenu(FilterContext, FavoritesBuilder);

	OutAllActions.Append(FavoritesBuilder);
}

//------------------------------------------------------------------------------
void SBlueprintFavoritesPalette::BindCommands(TSharedPtr<FUICommandList> CommandListIn) const
{
	SBlueprintSubPalette::BindCommands(CommandListIn);

	FBlueprintFavoritesPaletteCommands::Register();
	FBlueprintFavoritesPaletteCommands const& PaletteCommands = FBlueprintFavoritesPaletteCommands::Get();

	SBlueprintFavoritesPaletteUtils::FPaletteActionGetter ActionGetter = SBlueprintFavoritesPaletteUtils::FPaletteActionGetter::CreateRaw(GraphActionMenu.Get(), &SGraphActionMenu::GetSelectedActions);
	CommandListIn->MapAction(
		PaletteCommands.RemoveSingleFavorite,
		FExecuteAction::CreateStatic(&SBlueprintFavoritesPaletteUtils::RemoveSelectedFavorites, ActionGetter)
	);

	SBlueprintFavoritesPaletteUtils::FPaletteActionGetter CategoryGetter = SBlueprintFavoritesPaletteUtils::FPaletteActionGetter::CreateRaw(GraphActionMenu.Get(), &SGraphActionMenu::GetSelectedCategorySubActions);
	CommandListIn->MapAction(
		PaletteCommands.RemoveSubFavorites,
		FExecuteAction::CreateStatic(&SBlueprintFavoritesPaletteUtils::RemoveSelectedFavorites, CategoryGetter)
	);

	CommandListIn->MapAction(
		PaletteCommands.ClearFavorites,
		FExecuteAction::CreateStatic(&SBlueprintFavoritesPaletteUtils::ClearPaletteFavorites)
	);
}

//------------------------------------------------------------------------------
void SBlueprintFavoritesPalette::GenerateContextMenuEntries(FMenuBuilder& MenuBuilder) const
{
	FBlueprintFavoritesPaletteCommands const& PaletteCommands = FBlueprintFavoritesPaletteCommands::Get();

	MenuBuilder.BeginSection("FavoritedItem");
	{
		TSharedPtr<FEdGraphSchemaAction> SelectedAction = GetSelectedAction();
		// if we have a specific action selected
		if (SelectedAction.IsValid())
		{
			MenuBuilder.AddMenuEntry(PaletteCommands.RemoveSingleFavorite);
		}
		// if we have a category selected 
		{
			FString CategoryName = GraphActionMenu->GetSelectedCategoryName();
			// make sure it is an actual category and isn't the root (assume there's only one category with that name)
			if (!CategoryName.IsEmpty() && (CategoryName != SBlueprintFavoritesPaletteUtils::FavoritesCategoryName))
			{
				MenuBuilder.AddMenuEntry(PaletteCommands.RemoveSubFavorites);
			}
		}
	}
	MenuBuilder.EndSection();
	MenuBuilder.BeginSection("FavoritesList");
	{
		SBlueprintSubPalette::GenerateContextMenuEntries(MenuBuilder);

		MenuBuilder.AddSubMenu(	
			LOCTEXT("LoadProfile", "Load Profile"), 
			LOCTEXT("LoadProfileTooltip", "Replace your current favorites with ones from a pre-defined profile."),
			FNewMenuDelegate::CreateStatic(&SBlueprintFavoritesPaletteUtils::BuildProfilesSubMenu)
		);

		MenuBuilder.AddMenuEntry(PaletteCommands.ClearFavorites);
	}
	MenuBuilder.EndSection();
}

#undef LOCTEXT_NAMESPACE
