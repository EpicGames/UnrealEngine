// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SBlueprintSubPalette.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FBlueprintEditor;
class FMenuBuilder;
class FUICommandList;
struct FGraphActionListBuilderBase;

/*******************************************************************************
* SBlueprintFavoritesPalette
*******************************************************************************/

class SBlueprintFavoritesPalette : public SBlueprintSubPalette
{
public:
	SLATE_BEGIN_ARGS(SBlueprintFavoritesPalette) {}
	SLATE_END_ARGS()

	/** Unsubscribes this from events before it is destroyed */
	virtual ~SBlueprintFavoritesPalette();

	/**
	 * Creates a sub-palette widget for the blueprint palette UI (this 
	 * contains a subset of the library palette, specifically the user's 
	 * favorites and most used nodes)
	 * 
	 * @param  InArgs				A set of slate arguments, defined above.
	 * @param  InBlueprintEditor	A pointer to the blueprint editor that this palette belongs to.
	 */
	void Construct(const FArguments& InArgs, TWeakPtr<FBlueprintEditor> InBlueprintEditor);

private:
	// SGraphPalette Interface
	virtual void CollectAllActions(FGraphActionListBuilderBase& OutAllActions) override;
	// End SGraphPalette Interface

	// SBlueprintSubPalette Interface
	virtual void BindCommands(TSharedPtr<FUICommandList> CommandListIn) const override;
	virtual void GenerateContextMenuEntries(FMenuBuilder& MenuBuilder) const override;
	// End SBlueprintSubPalette Interface

	/** Flags weather we should add the "frequently used" list to the user's favorites */
	bool bShowFrequentlyUsed;
};
