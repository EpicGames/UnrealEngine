// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBlueprintLibraryPalette.h"

#include "BlueprintActionFilter.h"
#include "BlueprintActionMenuBuilder.h"
#include "BlueprintActionMenuUtils.h"
#include "BlueprintEditor.h"
#include "BlueprintPaletteFavorites.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Engine/Blueprint.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Visibility.h"
#include "Modules/ModuleManager.h"
#include "SGraphActionMenu.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "BlueprintLibraryPalette"

/*******************************************************************************
* Static File Helpers
*******************************************************************************/

/** 
 * Contains static helper methods (scoped inside this struct to avoid collisions 
 * during unified builds). 
 */
struct SBlueprintLibraryPaletteUtils
{
	/** The definition of a delegate used to retrieve a set of palette actions */
	DECLARE_DELEGATE_OneParam(FPaletteActionGetter, TArray< TSharedPtr<FEdGraphSchemaAction> >&);

	/**
	 * Uses the provided ActionGetter to get a list of selected actions, and then 
	 * adds every one from the user's favorites.
	 * 
	 * @param  ActionGetter		A delegate to use for grabbing the palette's selected actions.
	 */
	static void AddSelectedToFavorites(FPaletteActionGetter ActionGetter)
	{
		const UEditorPerProjectUserSettings* EditorPerProjectUserSettings = GetDefault<UEditorPerProjectUserSettings>();
		if (ActionGetter.IsBound() && (EditorPerProjectUserSettings->BlueprintFavorites != NULL))
		{
			TArray< TSharedPtr<FEdGraphSchemaAction> > SelectedActions;
			ActionGetter.Execute(SelectedActions);

			EditorPerProjectUserSettings->BlueprintFavorites->AddFavorites(SelectedActions);
		}
	}

	/**
	 * Uses the provided ActionGetter to get a list of selected actions, and then 
	 * removes every one from the user's favorites.
	 * 
	 * @param  ActionGetter		A delegate to use for grabbing the palette's selected actions.
	 */
	static void RemoveSelectedFavorites(FPaletteActionGetter ActionGetter)
	{
		const UEditorPerProjectUserSettings* EditorPerProjectUserSettings = GetDefault<UEditorPerProjectUserSettings>();
		if (ActionGetter.IsBound() && (EditorPerProjectUserSettings->BlueprintFavorites != NULL))
		{
			TArray< TSharedPtr<FEdGraphSchemaAction> > SelectedActions;
			ActionGetter.Execute(SelectedActions);

			EditorPerProjectUserSettings->BlueprintFavorites->RemoveFavorites(SelectedActions);
		}
	}

	/**
	 * Utility function used to check if any of the selected actions (returned 
	 * by the supplied ActionGetter) are candidates for adding to the user's
	 * favorites.
	 * 
	 * @param  ActionGetter		A delegate that'll retrieve the list of actions that you want tested.
	 * @return True if at least one action (returned by ActionGetter) can be added as a favorite, false if not.
	 */
	static bool IsAnyActionFavoritable(FPaletteActionGetter ActionGetter)
	{
		bool bCanAnyBeFavorited = false;

		const UEditorPerProjectUserSettings* EditorPerProjectUserSettings = GetDefault<UEditorPerProjectUserSettings>();
		if (ActionGetter.IsBound() && (EditorPerProjectUserSettings->BlueprintFavorites != NULL))
		{
			TArray< TSharedPtr<FEdGraphSchemaAction> > SelectedActions;
			ActionGetter.Execute(SelectedActions);

			for (TSharedPtr<FEdGraphSchemaAction> Action : SelectedActions)
			{
				if (EditorPerProjectUserSettings->BlueprintFavorites->CanBeFavorited(Action) && !EditorPerProjectUserSettings->BlueprintFavorites->IsFavorited(Action))
				{
					bCanAnyBeFavorited = true;
					break;
				}
			}
		}

		return bCanAnyBeFavorited;
	}

	/**
	 * Utility function used to check if any of the selected actions (returned 
	 * by the supplied ActionGetter) are currently one of the user's favorites.
	 * 
	 * @param  ActionGetter		A delegate that'll retrieve the list of actions that you want tested.
	 * @return True if at least one action (returned by ActionGetter) can be removed from the user's favorites, false if not.
	 */
	static bool IsAnyActionRemovable(FPaletteActionGetter ActionGetter)
	{
		bool bCanAnyBeRemoved = false;

		const UEditorPerProjectUserSettings* EditorPerProjectUserSettings = GetDefault<UEditorPerProjectUserSettings>();
		if (ActionGetter.IsBound() && (EditorPerProjectUserSettings->BlueprintFavorites != NULL))
		{
			TArray< TSharedPtr<FEdGraphSchemaAction> > SelectedActions;
			ActionGetter.Execute(SelectedActions);

			for (TSharedPtr<FEdGraphSchemaAction> Action : SelectedActions)
			{
				if (EditorPerProjectUserSettings->BlueprintFavorites->IsFavorited(Action))
				{
					bCanAnyBeRemoved = true;
					break;
				}
			}
		}

		return bCanAnyBeRemoved;
	}

	/** String constants shared between multiple SBlueprintLibraryPalette functions */
	static FString const LibraryCategoryName;
};

FString const SBlueprintLibraryPaletteUtils::LibraryCategoryName = LOCTEXT("PaletteRootCategory", "Library").ToString();

/*******************************************************************************
* FBlueprintLibraryPaletteCommands
*******************************************************************************/

class FBlueprintLibraryPaletteCommands : public TCommands<FBlueprintLibraryPaletteCommands>
{
public:
	FBlueprintLibraryPaletteCommands() : TCommands<FBlueprintLibraryPaletteCommands>
		( "BlueprintLibraryPalette"
		, LOCTEXT("LibraryPaletteContext", "Library Palette")
		, NAME_None
		, FAppStyle::GetAppStyleSetName() )
	{
	}

	TSharedPtr<FUICommandInfo> AddSingleFavorite;
	TSharedPtr<FUICommandInfo> AddSubFavorites;
	TSharedPtr<FUICommandInfo> RemoveSingleFavorite;
	TSharedPtr<FUICommandInfo> RemoveSubFavorites;

	/** Registers context menu commands for the blueprint library palette. */
	virtual void RegisterCommands() override
	{
		UI_COMMAND(AddSingleFavorite,    "Add to Favorites",               "Adds this item to your favorites list.",                      EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(AddSubFavorites,      "Add Category to Favorites",      "Adds all the nodes in this category to your favorites.",      EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(RemoveSingleFavorite, "Remove from Favorites",          "Removes this item from your favorites list.",                 EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(RemoveSubFavorites,   "Remove Category from Favorites", "Removes all the nodes in this category from your favorites.", EUserInterfaceActionType::Button, FInputChord());
	}
};
/*******************************************************************************
* FPaletteClassFilter
*******************************************************************************/

/** Filter to only show classes with blueprint accessible members */
class FPaletteClassFilter : public IClassViewerFilter
{
public:
	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs ) override
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		return K2Schema->ClassHasBlueprintAccessibleMembers(InClass);
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		// @TODO: One day would be nice to see functions on unloaded classes...
		return false;
	}
};

/*******************************************************************************
* SBlueprintLibraryPalette Public Interface
*******************************************************************************/

//------------------------------------------------------------------------------
void SBlueprintLibraryPalette::Construct(FArguments const& InArgs, TWeakPtr<FBlueprintEditor> InBlueprintEditor)
{
	SBlueprintSubPalette::FArguments SuperArgs;
	SuperArgs._Title       = LOCTEXT("PaletteTitle", "Find a Node");
	SuperArgs._Icon        = FAppStyle::GetBrush("Icons.Search");
	SuperArgs._ToolTipText = LOCTEXT("PaletteToolTip", "An all encompassing list of every node that is available for this blueprint.");
	SuperArgs._ShowFavoriteToggles = true;

	bUseLegacyLayout = InArgs._UseLegacyLayout.Get();

	SBlueprintSubPalette::Construct(SuperArgs, InBlueprintEditor);
}

/*******************************************************************************
* SBlueprintLibraryPalette Private Methods
*******************************************************************************/

//------------------------------------------------------------------------------
void SBlueprintLibraryPalette::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	FString RootCategory = SBlueprintLibraryPaletteUtils::LibraryCategoryName;
	if (bUseLegacyLayout)
	{
		RootCategory = TEXT("");
	}
	
	FBlueprintActionContext FilterContext;
	FilterContext.Blueprints.Add(GetBlueprint());
	FilterContext.EditorPtr = BlueprintEditorPtr;
	
	UClass* ClassFilter = nullptr;
	if (FilterClass.IsValid())
	{
		ClassFilter = FilterClass.Get();
	}
	
	FBlueprintActionMenuBuilder PaletteBuilder;
	FBlueprintActionMenuUtils::MakePaletteMenu(FilterContext, ClassFilter, PaletteBuilder);
	OutAllActions.Append(PaletteBuilder);
}

//------------------------------------------------------------------------------
TSharedRef<SVerticalBox> SBlueprintLibraryPalette::ConstructHeadingWidget(FSlateBrush const* const Icon, FText const& TitleText, FText const& InToolTip)
{
	TSharedRef<SVerticalBox> SuperHeading = SBlueprintSubPalette::ConstructHeadingWidget(Icon, TitleText, InToolTip);

	TSharedPtr<SToolTip> ClassPickerToolTip;
	SAssignNew(ClassPickerToolTip, SToolTip).Text(LOCTEXT("ClassFilter", "Filter the available nodes by class."));

	if (bUseLegacyLayout)
	{
		SuperHeading = SNew(SVerticalBox).ToolTipText(InToolTip);
	}

	SuperHeading->AddSlot()
		.AutoHeight()
		.Padding(0.f, 0.f, 0.f, 2.f)
		[
			SNew(SHorizontalBox)
				.ToolTip(ClassPickerToolTip)
				// so we still get tooltip text for the empty parts of the SHorizontalBox
				.Visibility(EVisibility::Visible) 

			+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
			[
				SNew(STextBlock).Text(LOCTEXT("Class", "Class: "))
			]

			+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
			[
				SAssignNew(FilterComboButton, SComboButton)
					.OnGetMenuContent(this, &SBlueprintLibraryPalette::ConstructClassFilterDropdownContent)
					.ButtonContent()
				[
					SNew(STextBlock).Text(this, &SBlueprintLibraryPalette::GetFilterClassName)
				]
			]
		];

	return SuperHeading;
}

//------------------------------------------------------------------------------
void SBlueprintLibraryPalette::BindCommands(TSharedPtr<FUICommandList> CommandListIn) const
{
	SBlueprintSubPalette::BindCommands(CommandListIn);

	FBlueprintLibraryPaletteCommands::Register();
	FBlueprintLibraryPaletteCommands const& PaletteCommands = FBlueprintLibraryPaletteCommands::Get();

	struct FActionVisibilityUtils
	{
		static bool CanNotRemoveAny(SBlueprintLibraryPaletteUtils::FPaletteActionGetter ActionGetter)
		{
			return !SBlueprintLibraryPaletteUtils::IsAnyActionRemovable(ActionGetter);
		}
	};

	SBlueprintLibraryPaletteUtils::FPaletteActionGetter ActionGetter = SBlueprintLibraryPaletteUtils::FPaletteActionGetter::CreateRaw(GraphActionMenu.Get(), &SGraphActionMenu::GetSelectedActions);
	CommandListIn->MapAction(
		PaletteCommands.AddSingleFavorite,
		FExecuteAction::CreateStatic(&SBlueprintLibraryPaletteUtils::AddSelectedToFavorites, ActionGetter),
		FCanExecuteAction::CreateStatic(&SBlueprintLibraryPaletteUtils::IsAnyActionFavoritable, ActionGetter),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateStatic(&FActionVisibilityUtils::CanNotRemoveAny, ActionGetter)
	);

	SBlueprintLibraryPaletteUtils::FPaletteActionGetter CategoryGetter = SBlueprintLibraryPaletteUtils::FPaletteActionGetter::CreateRaw(GraphActionMenu.Get(), &SGraphActionMenu::GetSelectedCategorySubActions);
	CommandListIn->MapAction(
		PaletteCommands.AddSubFavorites,
		FExecuteAction::CreateStatic(&SBlueprintLibraryPaletteUtils::AddSelectedToFavorites, CategoryGetter),
		FCanExecuteAction::CreateStatic(&SBlueprintLibraryPaletteUtils::IsAnyActionFavoritable, CategoryGetter),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateStatic(&SBlueprintLibraryPaletteUtils::IsAnyActionFavoritable, CategoryGetter)
	);

	CommandListIn->MapAction(
		PaletteCommands.RemoveSingleFavorite,
		FExecuteAction::CreateStatic(&SBlueprintLibraryPaletteUtils::RemoveSelectedFavorites, ActionGetter),
		FCanExecuteAction(), FIsActionChecked(),
		FIsActionButtonVisible::CreateStatic(&SBlueprintLibraryPaletteUtils::IsAnyActionRemovable, ActionGetter)
	);

	CommandListIn->MapAction(
		PaletteCommands.RemoveSubFavorites,
		FExecuteAction::CreateStatic(&SBlueprintLibraryPaletteUtils::RemoveSelectedFavorites, CategoryGetter),
		FCanExecuteAction(), FIsActionChecked(),
		FIsActionButtonVisible::CreateStatic(&SBlueprintLibraryPaletteUtils::IsAnyActionRemovable, CategoryGetter)
	);
}

//------------------------------------------------------------------------------
void SBlueprintLibraryPalette::GenerateContextMenuEntries(FMenuBuilder& MenuBuilder) const
{
	if (!bUseLegacyLayout)
	{
		FBlueprintLibraryPaletteCommands const& PaletteCommands = FBlueprintLibraryPaletteCommands::Get();

		MenuBuilder.BeginSection("Favorites");
		{
			TSharedPtr<FEdGraphSchemaAction> SelectedAction = GetSelectedAction();
			// if we have a specific action selected
			if (SelectedAction.IsValid())
			{
				MenuBuilder.AddMenuEntry(PaletteCommands.AddSingleFavorite);
				MenuBuilder.AddMenuEntry(PaletteCommands.RemoveSingleFavorite);
			}
			// if we have a category selected 
			{
				FString CategoryName = GraphActionMenu->GetSelectedCategoryName();
				// make sure it is an actual category and isn't the root (assume there's only one category with that name)
				if (!CategoryName.IsEmpty() && (CategoryName != SBlueprintLibraryPaletteUtils::LibraryCategoryName))
				{
					MenuBuilder.AddMenuEntry(PaletteCommands.AddSubFavorites);
					MenuBuilder.AddMenuEntry(PaletteCommands.RemoveSubFavorites);
				}
			}
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("ListActions");
		SBlueprintSubPalette::GenerateContextMenuEntries(MenuBuilder);
		MenuBuilder.EndSection();
	}
}

//------------------------------------------------------------------------------
TSharedRef<SWidget> SBlueprintLibraryPalette::ConstructClassFilterDropdownContent()
{
	FClassViewerInitializationOptions Options;
	Options.Mode        = EClassViewerMode::ClassPicker;
	Options.DisplayMode = EClassViewerDisplayMode::TreeView;
	Options.ClassFilters.Add(MakeShareable(new FPaletteClassFilter));
	//  create a class picker for the drop-down
	TSharedRef<SWidget> ClassPickerWidget = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").CreateClassViewer(Options, FOnClassPicked::CreateSP(this, &SBlueprintLibraryPalette::OnClassPicked));

	TSharedPtr<SToolTip> ClearFilterToolTip;
	SAssignNew(ClearFilterToolTip, SToolTip).Text(LOCTEXT("ClearFilter", "Clears the class filter so you can see all available nodes for placement."));

	return SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		[
			// achieving fixed width by nesting items within a fixed width box.
			SNew(SBox)
				.WidthOverride(350.0f)
			[
				SNew(SVerticalBox)
				
				// 'All' button
				+SVerticalBox::Slot()
					.Padding(2.f, 0.f, 2.f, 2.f)
				[
					SNew(SButton)
						.OnClicked(this, &SBlueprintLibraryPalette::ClearClassFilter)
						.ToolTip(ClearFilterToolTip)
					[
						SNew(STextBlock).Text(LOCTEXT("All", "All"))
					]
				]

				// Class picker
				+SVerticalBox::Slot()
					.MaxHeight(400.0f)
					.AutoHeight()
				[
					ClassPickerWidget
				]
			]
		];
}

//------------------------------------------------------------------------------
FText SBlueprintLibraryPalette::GetFilterClassName() const
{
	FText FilterDisplayString = LOCTEXT("All", "All");
	if (FilterClass != NULL)
	{
		UBlueprint* Blueprint = UBlueprint::GetBlueprintFromClass(FilterClass.Get());
		FilterDisplayString = FText::FromString((Blueprint != NULL) ? Blueprint->GetName() : FilterClass->GetName());
	}

	return FilterDisplayString;
}

//------------------------------------------------------------------------------
FReply SBlueprintLibraryPalette::ClearClassFilter()
{
	FilterComboButton->SetIsOpen(false);
	if (FilterClass.IsValid())
	{
		FilterClass = NULL;
		RefreshActionsList(true);
	}
	return FReply::Handled();
}

//------------------------------------------------------------------------------
void SBlueprintLibraryPalette::OnClassPicked(UClass* PickedClass)
{
	FilterClass = PickedClass;
	FilterComboButton->SetIsOpen(false);
	RefreshActionsList(true);
}

#undef LOCTEXT_NAMESPACE
