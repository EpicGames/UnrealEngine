// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "SBlueprintSubPalette.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FBlueprintEditor;
class FMenuBuilder;
class FUICommandList;
class SComboButton;
class SVerticalBox;
class SWidget;
class UClass;
struct FGraphActionListBuilderBase;
struct FSlateBrush;

/*******************************************************************************
* SBlueprintLibraryPalette
*******************************************************************************/

class SBlueprintLibraryPalette : public SBlueprintSubPalette
{
public:
	SLATE_BEGIN_ARGS( SBlueprintLibraryPalette )
		: _UseLegacyLayout(false) {}

		SLATE_ATTRIBUTE(bool, UseLegacyLayout)
	SLATE_END_ARGS()

	/**
	 * Creates a sub-palette widget for the blueprint palette UI (this 
	 * contains a subset of the library palette, specifically the user's 
	 * favorites and most used nodes)
	 * 
	 * @param  InArgs				A set of slate arguments, defined above.
	 * @param  InBlueprintEditor	A pointer to the blueprint editor that this palette belongs to.
	 */
	void Construct(const FArguments& InArgs, TWeakPtr<FBlueprintEditor> InBlueprintEditor);

private:
	// SGraphPalette Interface
	virtual void CollectAllActions(FGraphActionListBuilderBase& OutAllActions) override;
	// End SGraphPalette Interface

	// SBlueprintSubPalette Interface
	virtual TSharedRef<SVerticalBox> ConstructHeadingWidget(FSlateBrush const* const Icon, FText const& TitleText, FText const& ToolTip) override;
	virtual void BindCommands(TSharedPtr<FUICommandList> CommandListIn) const override;
	virtual void GenerateContextMenuEntries(FMenuBuilder& MenuBuilder) const override;
	// End SBlueprintSubPalette Interface

	/**
	 * Constructs a slate widget for the class drop-down menu, which is used to
	 * select a filter class.
	 * 
	 * @return A pointer to the newly created menu widget.
	 */
	TSharedRef<SWidget> ConstructClassFilterDropdownContent();

	/**
	 * Retrieves the name of the currently selected class filter (or "All" if
	 * no class filter has been selected).
	 * 
	 * @return A string representing the selected class filter.
	 */
	FText GetFilterClassName() const;

	/**
	 * Clears the current class filter (if one is set), and refreshes the 
	 * displayed action list.
	 * 
	 * @return FReply::Handled()
	 */
	FReply ClearClassFilter();

	/**
	 * Resets the current class filter and refreshes the displayed action list.
	 * 
	 * @param  PickedClass	The new class to filter the library by.
	 */
	void OnClassPicked(UClass* PickedClass);

	/** Used to help ease the transition for users who like the old format */
	bool bUseLegacyLayout;

	/** Class we want to see content of */
	TWeakObjectPtr<UClass> FilterClass;

	/** Combo button used to choose class to filter */
	TSharedPtr<SComboButton> FilterComboButton;
};

