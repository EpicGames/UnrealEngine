// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBlueprintNamespaceEntry.h"

#include "Algo/Sort.h"
#include "BlueprintNamespacePathTree.h"
#include "BlueprintNamespaceRegistry.h"
#include "Containers/StringFwd.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Views/ITypedTableView.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/Char.h"
#include "Misc/StringBuilder.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "SlotBase.h"
#include "Styling/SlateColor.h"
#include "Types/SlateStructs.h"
#include "UObject/NameTypes.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SSuggestionTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"

class ITableRow;
class SWidget;

#define LOCTEXT_NAMESPACE "SBlueprintNamespaceEntry"

float SBlueprintNamespaceEntry::NamespaceListBorderPadding = 1.0f;
float SBlueprintNamespaceEntry::NamespaceListMinDesiredWidth = 350.0f;

void SBlueprintNamespaceEntry::Construct(const FArguments& InArgs)
{
	CurrentNamespace = InArgs._CurrentNamespace;
	OnNamespaceSelected = InArgs._OnNamespaceSelected;
	OnGetNamespacesToExclude = InArgs._OnGetNamespacesToExclude;
	ExcludedNamespaceTooltipText = InArgs._ExcludedNamespaceTooltipText;

	ChildSlot
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			SAssignNew(TextBox, SSuggestionTextBox)
			.Font(InArgs._Font)
			.ForegroundColor(FSlateColor::UseForeground())
			.Visibility(InArgs._AllowTextEntry ? EVisibility::Visible : EVisibility::Collapsed)
			.Text(this, &SBlueprintNamespaceEntry::GetCurrentNamespaceText)
			.OnTextChanged(this, &SBlueprintNamespaceEntry::OnTextChanged)
			.OnTextCommitted(this, &SBlueprintNamespaceEntry::OnTextCommitted)
			.OnShowingSuggestions(this, &SBlueprintNamespaceEntry::OnShowingSuggestions)
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SAssignNew(ComboButton, SComboButton)
			.CollapseMenuOnParentFocus(true)
			.OnGetMenuContent(this, &SBlueprintNamespaceEntry::OnGetNamespaceTreeMenuContent)
			.ButtonContent()
			[
				InArgs._ButtonContent.Widget
			]
		]
	];
}

void SBlueprintNamespaceEntry::SetCurrentNamespace(const FString& InNamespace)
{
	// Pass through the text box in order to validate the string before committing it to the current value.
	if (TextBox.IsValid())
	{
		TextBox->SetText(FText::FromString(InNamespace));
	}
}

void SBlueprintNamespaceEntry::OnTextChanged(const FText& InText)
{
	// Note: Empty string is valid (i.e. global namespace).
	bool bIsValidString = true;

	// Only allow alphanumeric characters, '.' and '_'.
	FString NewString = InText.ToString();
	for (const TCHAR& NewChar : NewString)
	{
		if (!FChar::IsAlnum(NewChar) && NewChar != TEXT('_') && NewChar != TEXT('.'))
		{
			bIsValidString = false;
			break;
		}
	}

	FString ErrorText;
	if (bIsValidString)
	{
		// Keep the current namespace in sync with the last-known valid text box value.
		CurrentNamespace = MoveTemp(NewString);
	}
	else
	{
		ErrorText = LOCTEXT("InvalidNamespaceIdentifierStringError", "Invalid namespace identifier string.").ToString();
	}

	// Set the error text regardless of whether or not the path is valid; this will clear the error state if the string is valid.
	if (TextBox.IsValid())
	{
		TextBox->SetError(ErrorText);
	}
}

void SBlueprintNamespaceEntry::OnTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit)
{
	// Ensure that case correctly matches up with any registered path name(s).
	FString CaseCorrectedPath;
	TArray<FString> PathSegments;
	CurrentNamespace.ParseIntoArray(PathSegments, FBlueprintNamespacePathTree::PathSeparator);
	for (FString& PathSegment : PathSegments)
	{
		TArray<FName> ActualNames;
		FBlueprintNamespaceRegistry::Get().GetNamesUnderPath(CaseCorrectedPath, ActualNames);
		for (const FName& ActualName : ActualNames)
		{
			FString ActualNameAsString = ActualName.ToString();
			if (PathSegment.Equals(ActualNameAsString, ESearchCase::IgnoreCase))
			{
				PathSegment = MoveTemp(ActualNameAsString);
				break;
			}
		}

		if (!CaseCorrectedPath.IsEmpty())
		{
			CaseCorrectedPath += FBlueprintNamespacePathTree::PathSeparator;
		}

		CaseCorrectedPath += PathSegment;
	}

	// Update the current namespace string.
	CurrentNamespace = MoveTemp(CaseCorrectedPath);

	// Not using the current textbox value here because it might be invalid, and we want to revert to the last-known valid namespace string on commit.
	SelectNamespace(CurrentNamespace);
}

void SBlueprintNamespaceEntry::OnShowingSuggestions(const FString& InputText, TArray<FString>& OutSuggestions)
{
	int32 PathEnd;
	FString CurrentPath;
	FString CurrentName;
	if (InputText.FindLastChar(FBlueprintNamespacePathTree::PathSeparator[0], PathEnd))
	{
		CurrentPath = InputText.LeftChop(InputText.Len() - PathEnd);
		CurrentName = InputText.RightChop(PathEnd + 1);
	}
	else
	{
		CurrentName = InputText;
	}

	// Find all names (path segments) that fall under the current path prefix.
	TArray<FName> SuggestedNames;
	FBlueprintNamespaceRegistry::Get().GetNamesUnderPath(CurrentPath, SuggestedNames);

	// Sort the list alphabetically.
	Algo::Sort(SuggestedNames, FNameLexicalLess());

	// Allow the owner to exclude one or more paths.
	TSet<FString> ExcludedPaths;
	OnGetNamespacesToExclude.ExecuteIfBound(ExcludedPaths);

	// Build the suggestion set based on the set of matching names we found above.
	TStringBuilder<128> PathBuilder;
	for (FName SuggestedName : SuggestedNames)
	{
		FString SuggestedNameAsString = SuggestedName.ToString();
		if (CurrentName.IsEmpty() || SuggestedNameAsString.StartsWith(CurrentName))
		{
			if (CurrentPath.Len() > 0)
			{
				PathBuilder += CurrentPath;
				PathBuilder += FBlueprintNamespacePathTree::PathSeparator;
			}

			PathBuilder += SuggestedNameAsString;

			FString SuggestedNamespace = PathBuilder.ToString();
			if (!ExcludedPaths.Contains(SuggestedNamespace))
			{
				OutSuggestions.Add(MoveTemp(SuggestedNamespace));
			}

			PathBuilder.Reset();
		}
	}
}

TSharedRef<SWidget> SBlueprintNamespaceEntry::OnGetNamespaceTreeMenuContent()
{
	// Gather the full set of registered namespace paths.
	AllRegisteredPaths.Reset();
	FBlueprintNamespaceRegistry::Get().GetAllRegisteredPaths(AllRegisteredPaths);

	// Allow external owners to filter the list.
	ExcludedTreeViewPaths.Reset();
	OnGetNamespacesToExclude.ExecuteIfBound(ExcludedTreeViewPaths);
	if (!ExcludedTreeViewPaths.IsEmpty())
	{
		AllRegisteredPaths.RemoveAllSwap([&ExcludedTreeViewPaths = this->ExcludedTreeViewPaths](const FString& Path)
		{
			return ExcludedTreeViewPaths.Contains(Path);
		});
	}

	// Sort the list alphabetically.
	AllRegisteredPaths.Sort();

	// Reset the search box so we don't reapply a previous menu's filter.
	SearchBox.Reset();

	// Build the namespace item tree from the filtered list of registered paths.
	PopulateNamespaceTree();

	// Construct the tree view widget that we'll use for the menu content.
	SAssignNew(TreeView, STreeView<TSharedPtr<FPathTreeNodeItem>>)
		.SelectionMode(ESelectionMode::Single)
		.TreeItemsSource(&RootNodes)
		.OnGenerateRow(this, &SBlueprintNamespaceEntry::OnGenerateRowForNamespaceTreeItem)
		.OnGetChildren(this, &SBlueprintNamespaceEntry::OnGetChildrenForNamespaceTreeItem)
		.OnSelectionChanged(this, &SBlueprintNamespaceEntry::OnNamespaceTreeSelectionChanged)
		.OnIsSelectableOrNavigable(this, &SBlueprintNamespaceEntry::OnIsNamespaceTreeItemSelectable);

	// All tree view items are always expanded by default.
	ExpandAllTreeViewItems();

	// If we are allowing manual entry and the current namespace is non-empty, look for a matching item in the set.
	if (!CurrentNamespace.IsEmpty() && TextBox.IsValid() && TextBox->GetVisibility().IsVisible())
	{
		// If we found a match, make it the initial selection.
		if (const TSharedPtr<FPathTreeNodeItem>* CurrentItemPtr = FindTreeViewNode(CurrentNamespace))
		{
			TreeView->SetSelection(*CurrentItemPtr);
		}
	}

	return SNew(SBorder)
		.Padding(NamespaceListBorderPadding)
		[
			SNew(SBox)
			.MinDesiredWidth(NamespaceListMinDesiredWidth)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(SearchBox, SSearchBox)
					.OnTextChanged(this, &SBlueprintNamespaceEntry::OnNamespaceTreeFilterTextChanged)
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SSeparator)
				]
				+SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					TreeView.ToSharedRef()
				]
			]
		];
}

TSharedRef<ITableRow> SBlueprintNamespaceEntry::OnGenerateRowForNamespaceTreeItem(TSharedPtr<FPathTreeNodeItem> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	check(Item.IsValid());

	// Check for an empty tree and add a single (disabled) item if found.
	bool bIsEnabled = true;
	FText ItemText;
	if (Item->NodePath.IsEmpty() && RootNodes.Num() == 1 && RootNodes[0] == Item)
	{
		bIsEnabled = false;
		ItemText = LOCTEXT("BlueprintNamespaceList_NoItems", "No Matching Items");
	}
	else
	{
		ItemText = FText::FromString(*Item->NodePath);
	}

	FText ToolTipText = FText::GetEmpty();
	if (!Item->bIsSelectable && ExcludedNamespaceTooltipText.IsSet())
	{
		ToolTipText = ExcludedNamespaceTooltipText.Get();
	}

	// Construct a new row widget, highlighting any text that matches the search filter.
	return SNew(STableRow<TSharedPtr<FPathTreeNodeItem>>, OwnerTable)
	.IsEnabled(bIsEnabled)
	.ShowSelection(Item->bIsSelectable)
	[
		SNew(STextBlock)
		.Text(ItemText)
		.ToolTipText(ToolTipText)
		.IsEnabled(Item->bIsSelectable)
		.HighlightText(bIsEnabled && SearchBox.IsValid() ? SearchBox->GetText() : FText::GetEmpty())
	];
}

void SBlueprintNamespaceEntry::OnGetChildrenForNamespaceTreeItem(TSharedPtr<FPathTreeNodeItem> Item, TArray<TSharedPtr<FPathTreeNodeItem>>& OutChildren)
{
	check(Item.IsValid());
	OutChildren.Append(Item->ChildNodes);
}

void SBlueprintNamespaceEntry::OnNamespaceTreeFilterTextChanged(const FText& InText)
{
	// Gather/filter all registered paths.
	PopulateNamespaceTree();

	// Refresh the namespace item tree view.
	if (TreeView.IsValid())
	{
		ExpandAllTreeViewItems();

		TreeView->RequestTreeRefresh();
	}
}

void SBlueprintNamespaceEntry::OnNamespaceTreeSelectionChanged(TSharedPtr<FPathTreeNodeItem> Item, ESelectInfo::Type SelectInfo)
{
	// These actions should not trigger a selection.
	if (SelectInfo == ESelectInfo::OnNavigation || SelectInfo == ESelectInfo::Direct)
	{
		return;
	}

	if (Item.IsValid())
	{
		// Disallow selection of inclusive nodes that are filtered out.
		if (!Item->bIsSelectable)
		{
			return;
		}

		SelectNamespace(*Item->NodePath);
	}

	// Clear the search filter text.
	if (SearchBox.IsValid())
	{
		SearchBox->SetText(FText::GetEmpty());
	}

	// Close the combo button menu after a selection.
	if (ComboButton.IsValid())
	{
		ComboButton->SetIsOpen(false);
	}

	// Switch focus back to the text box if present and visible.
	if (TextBox.IsValid() && TextBox->GetVisibility() == EVisibility::Visible)
	{
		FSlateApplication::Get().SetKeyboardFocus(TextBox);
		FSlateApplication::Get().SetUserFocus(0, TextBox);
	}
}

bool SBlueprintNamespaceEntry::OnIsNamespaceTreeItemSelectable(TSharedPtr<FPathTreeNodeItem> Item) const
{
	return Item.IsValid() && Item->bIsSelectable;
}

FText SBlueprintNamespaceEntry::GetCurrentNamespaceText() const
{
	return FText::FromString(CurrentNamespace);
}

void SBlueprintNamespaceEntry::PopulateNamespaceTree()
{
	// Clear the current list.
	RootNodes.Reset();

	// Set up an expression evaluator to further trim the list according to the search filter.
	FTextFilterExpressionEvaluator SearchFilter(ETextFilterExpressionEvaluatorMode::BasicString);
	SearchFilter.SetFilterText(SearchBox.IsValid() ? SearchBox->GetText() : FText::GetEmpty());

	// Build the source for the tree view widget.
	for (const FString& Path : AllRegisteredPaths)
	{
		// Only include items that match the current search filter text.
		if (SearchFilter.TestTextFilter(FBasicStringFilterExpressionContext(Path)))
		{
			FString CurrentNodePath;
			TArray<TSharedPtr<FPathTreeNodeItem>>* NodeList = &RootNodes;

			TArray<FString> PathSegments;
			Path.ParseIntoArray(PathSegments, FBlueprintNamespacePathTree::PathSeparator);
			for (const FString& PathSegment : PathSegments)
			{
				CurrentNodePath += PathSegment;

				TSharedPtr<FPathTreeNodeItem>* NodePtr = NodeList->FindByPredicate([&CurrentNodePath](const TSharedPtr<FPathTreeNodeItem>& Value)
				{
					return CurrentNodePath.Equals(Value->NodePath, ESearchCase::IgnoreCase);
				});

				// Add a new node to the current list if an existing one was not found.
				if (!NodePtr)
				{
					NodePtr = &NodeList->Add_GetRef(MakeShared<FPathTreeNodeItem>());

					// Note: We're intentionally using the full path name here rather than just the segment name.
					(*NodePtr)->NodePath = CurrentNodePath;

					// Treat inclusive nodes that are filtered out by the owner as non-selectable.
					// 
					// Example: Consider that "MyProject.MyNamespace" is a registered namespace. When building the tree view for the
					// drop-down menu, we will first include "MyProject" as the subtree root, and add "MyProject.MyNamespace" as a
					// child node. If the owner excludes "MyProject" as a namespace (e.g. because it's already imported as an inclusive
					// namespace), we could also exclude it from the drop-down, but this would mask the hierarchical relationship. So
					// rather than exclude it from the tree, we make it a non-selectable node.
					(*NodePtr)->bIsSelectable = !ExcludedTreeViewPaths.Contains(CurrentNodePath);
				}

				NodeList = &(*NodePtr)->ChildNodes;

				CurrentNodePath += FBlueprintNamespacePathTree::PathSeparator;
			}
		}
	}

	// If no items were added, we signal this by adding a single blank entry.
	if (RootNodes.Num() == 0)
	{
		RootNodes.Add(MakeShared<FPathTreeNodeItem>());
	}
}

void SBlueprintNamespaceEntry::SelectNamespace(const FString& InNamespace)
{
	if (TextBox.IsValid())
	{
		// Update the textbox to reflect the selected value. Note that this should also clear any error state via OnTextChanged().
		TextBox->SetText(FText::FromString(InNamespace));
	}

	// Invoke the delegate in response to the new selection.
	OnNamespaceSelected.ExecuteIfBound(InNamespace);
}

void SBlueprintNamespaceEntry::ExpandAllTreeViewItems(const TArray<TSharedPtr<FPathTreeNodeItem>>* NodeListPtr)
{
	if (!TreeView.IsValid())
	{
		return;
	}

	if (NodeListPtr == nullptr)
	{
		NodeListPtr = &RootNodes;
	}

	for (const TSharedPtr<FPathTreeNodeItem>& NodeItem : *NodeListPtr)
	{
		check(NodeItem.IsValid());
		TreeView->SetItemExpansion(NodeItem, true);

		ExpandAllTreeViewItems(&NodeItem->ChildNodes);
	}
}

const TSharedPtr<SBlueprintNamespaceEntry::FPathTreeNodeItem>* SBlueprintNamespaceEntry::FindTreeViewNode(const FString& NodePath, const TArray<TSharedPtr<FPathTreeNodeItem>>* NodeListPtr) const
{
	const TSharedPtr<SBlueprintNamespaceEntry::FPathTreeNodeItem>* Result = nullptr;

	if (NodeListPtr == nullptr)
	{
		NodeListPtr = &RootNodes;
	}

	for (const TSharedPtr<FPathTreeNodeItem>& NodeItem : *NodeListPtr)
	{
		check(NodeItem.IsValid());
		if (NodeItem->NodePath.Equals(NodePath, ESearchCase::IgnoreCase))
		{
			Result = &NodeItem;
		}
		else
		{
			Result = FindTreeViewNode(NodePath, &NodeItem->ChildNodes);
		}

		if (Result)
		{
			break;
		}
	}

	return Result;
}

#undef LOCTEXT_NAMESPACE