// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBlueprintPalette.h"

#include "AnimStateConduitNode.h"
#include "AnimationBlendSpaceSampleGraph.h"
#include "AnimationGraph.h"
#include "AnimationStateGraph.h"
#include "AnimationStateMachineGraph.h"
#include "AnimationCustomTransitionGraph.h"
#include "AnimationStateMachineSchema.h"
#include "AnimationTransitionGraph.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetToolsModule.h"
#include "BlendSpaceGraph.h"
#include "BlueprintActionMenuItem.h"
#include "BlueprintActionMenuUtils.h"
#include "BlueprintDragDropMenuItem.h"
#include "BlueprintEditor.h"
#include "BlueprintEditorSettings.h"
#include "BlueprintNamespaceUtilities.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintPaletteFavorites.h"
#include "Components/ActorComponent.h"
#include "Components/TimelineComponent.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "Dialogs/Dialogs.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphNode_Comment.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphSchema_K2_Actions.h"
#include "Engine/Blueprint.h"
#include "Engine/MemberReference.h"
#include "Engine/SCS_Node.h"
#include "Engine/SimpleConstructionScript.h"
#include "FieldNotifyToggle.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/SlateDelegates.h"
#include "GenericPlatform/GenericApplication.h"
#include "HAL/Platform.h"
#include "IAssetTools.h"
#include "IDocumentation.h"
#include "Internationalization/Culture.h"
#include "Internationalization/Internationalization.h"
#include "INotifyFieldValueChanged.h"
#include "K2Node.h"
#include "K2Node_CallFunction.h"
#include "K2Node_Variable.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Layout/Children.h"
#include "Layout/ChildrenBase.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "SBlueprintFavoritesPalette.h"
#include "SBlueprintLibraryPalette.h"
#include "SGraphActionMenu.h"
#include "SMyBlueprint.h"
#include "SPinTypeSelector.h"
#include "ScopedTransaction.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleDefaults.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Textures/SlateIcon.h"
#include "TutorialMetaData.h"
#include "Types/ISlateMetaData.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/Script.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakFieldPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "WidgetBlueprintEditorUtils.h"
#include "Widgets/IToolTip.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SToolTip.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"

class FDragDropEvent;
struct FGeometry;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "BlueprintPalette"

/*******************************************************************************
* Static File Helpers
*******************************************************************************/

/** namespace'd to avoid collisions during unified builds */
namespace BlueprintPalette
{
	static FString const ConfigSection("BlueprintEditor.Palette");
	static FString const FavoritesHeightConfigKey("FavoritesHeightRatio");
	static FString const LibraryHeightConfigKey("LibraryHeightRatio");
}

/**
 * A helper method intended for constructing tooltips on palette items 
 * associated with specific blueprint variables (gets a string representing the 
 * specified variable's type)
 * 
 * @param  VarScope	The struct that owns the variable in question.
 * @param  VarName	The name of the variable you want the type of.
 * @param  Detailed	If true the returned string includes SubCategoryObject
 * @return A string representing the variable's type (empty if the variable couldn't be found).
 */
static FString GetVarType(UStruct* VarScope, FName VarName, bool bUseObjToolTip, bool bDetailed = false)
{
	FString VarDesc;

	if (VarScope)
	{
		if (FProperty* Property = FindFProperty<FProperty>(VarScope, VarName))
		{
			// If it is an object property, see if we can get a nice class description instead of just the name
			FObjectProperty* ObjProp = CastField<FObjectProperty>(Property);
			if (bUseObjToolTip && ObjProp && ObjProp->PropertyClass)
			{
				VarDesc = ObjProp->PropertyClass->GetToolTipText(GetDefault<UBlueprintEditorSettings>()->bShowShortTooltips).ToString();
			}

			// Name of type
			if (VarDesc.Len() == 0)
			{
				const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

				FEdGraphPinType PinType;
				if (K2Schema->ConvertPropertyToPinType(Property, PinType)) // use schema to get the color
				{
					VarDesc = UEdGraphSchema_K2::TypeToText(PinType).ToString();
				}
			}
		}
	}

	return VarDesc;
}

/**
 * Util function that helps construct a tooltip for a specific variable action 
 * (attempts to grab the variable's "tooltip" metadata).
 * 
 * @param  InBlueprint	The blueprint that the palette is associated.
 * @param  VarClass		The class that owns the variable.
 * @param  VarName		The variable you want a tooltip for.
 * @return A string from the variable's "tooltip" metadata (empty if the variable wasn't found, or it didn't have the metadata).
 */
static FString GetVarTooltip(UBlueprint* InBlueprint, UClass* VarClass, FName VarName)
{
	FString ResultTooltip;
	if (VarClass)
	{
	
		if (FProperty* Property = FindFProperty<FProperty>(VarClass, VarName))
		{
			// discover if the variable property is a non blueprint user variable
			UClass* SourceClass = Property->GetOwnerClass();
			if( SourceClass && SourceClass->ClassGeneratedBy == nullptr )
			{
				ResultTooltip = Property->GetToolTipText().ToString();
			}
			else
			{
				const UBlueprint* SourceBlueprint = SourceClass ? Cast<UBlueprint>(SourceClass->ClassGeneratedBy) : nullptr;
				FBlueprintEditorUtils::GetBlueprintVariableMetaData(SourceBlueprint ? SourceBlueprint : InBlueprint, VarName, nullptr, TEXT("tooltip"), ResultTooltip);
			}
		}
	}

	return ResultTooltip;
}

/**
 * A utility function intended to aid the construction of a specific blueprint 
 * palette item (specifically FEdGraphSchemaAction_K2Graph palette items). Based 
 * off of the sub-graph's type, this gets an icon representing said sub-graph.
 * 
 * @param  ActionIn		The FEdGraphSchemaAction_K2Graph action that the palette item represents.
 * @param  BlueprintIn	The blueprint currently being edited (that the action is for).
 * @param  IconOut		An icon denoting the sub-graph's type.
 * @param  ColorOut		An output color, further denoting the specified action.
 * @param  ToolTipOut	The tooltip to display when the icon is hovered over (describing the sub-graph type).
 */
static void GetSubGraphIcon(FEdGraphSchemaAction_K2Graph const* const ActionIn, UBlueprint const* BlueprintIn, FSlateBrush const*& IconOut, FSlateColor& ColorOut, FText& ToolTipOut)
{
	check(BlueprintIn);

	switch (ActionIn->GraphType)
	{
	case EEdGraphSchemaAction_K2Graph::Graph:
		{
			if (ActionIn->EdGraph)
			{
				IconOut = FBlueprintEditor::GetGlyphForGraph(ActionIn->EdGraph);
			}
			else
			{
				IconOut = FAppStyle::GetBrush(TEXT("GraphEditor.EventGraph_16x"));
			}

			ToolTipOut = LOCTEXT("EventGraph_ToolTip", "Event Graph");
		}
		break;
	case EEdGraphSchemaAction_K2Graph::Subgraph:
		{
			if (Cast<UAnimationStateMachineGraph>(ActionIn->EdGraph))
			{
				IconOut = FAppStyle::GetBrush(TEXT("GraphEditor.StateMachine_16x") );
				ToolTipOut = LOCTEXT("AnimationStateMachineGraph_ToolTip", "Animation State Machine");
			}
			else if (Cast<UAnimationStateGraph>(ActionIn->EdGraph))
			{
				IconOut = FAppStyle::GetBrush(TEXT("GraphEditor.State_16x") );
				ToolTipOut = LOCTEXT("AnimationState_ToolTip", "Animation State");
			}
			else if (Cast<UAnimationTransitionGraph>(ActionIn->EdGraph))
			{
				UAnimStateConduitNode* EdGraphOuter = Cast<UAnimStateConduitNode>(ActionIn->EdGraph->GetOuter());
				if (EdGraphOuter)
				{
					IconOut = FAppStyle::GetBrush(TEXT("GraphEditor.Conduit_16x"));
					ToolTipOut = LOCTEXT("ConduitGraph_ToolTip", "Conduit");
				}
				else
				{
					IconOut = FAppStyle::GetBrush(TEXT("GraphEditor.Rule_16x"));
					ToolTipOut = LOCTEXT("AnimationTransitionGraph_ToolTip", "Animation Transition Rule");
				}
			}
			else if (Cast<UBlendSpaceGraph>(ActionIn->EdGraph))
			{
				IconOut = FAppStyle::GetBrush(TEXT("BlendSpace.Graph") );
				ToolTipOut = LOCTEXT("BlendSpace_ToolTip", "BlendSpace");
			}
			else if (Cast<UAnimationBlendSpaceSampleGraph>(ActionIn->EdGraph))
			{
				IconOut = FAppStyle::GetBrush(TEXT("BlendSpace.SampleGraph") );
				ToolTipOut = LOCTEXT("BlendSpaceSample_ToolTip", "BlendSpace Sample");
			}
			else if (Cast<UAnimationCustomTransitionGraph>(ActionIn->EdGraph))
			{
				IconOut = FAppStyle::GetBrush(TEXT("GraphEditor.SubGraph_16x"));
				ToolTipOut = LOCTEXT("CustomBlendGraph_ToolTip", "Custom Blend");
			}
			else
			{
				IconOut = FAppStyle::GetBrush(TEXT("GraphEditor.SubGraph_16x") );
				ToolTipOut = LOCTEXT("EventSubgraph_ToolTip", "Event Subgraph");
			}
		}
		break;
	case EEdGraphSchemaAction_K2Graph::Macro:
		{
			IconOut = FAppStyle::GetBrush(TEXT("GraphEditor.Macro_16x"));
			if ( ActionIn->EdGraph == nullptr )
			{
				ToolTipOut = LOCTEXT("PotentialOverride_Tooltip", "Potential Override");	
			}
			else
			{
				// Need to see if this is a function overriding something in the parent, or 
				;
				if (UFunction* OverrideFunc = FindUField<UFunction>(BlueprintIn->ParentClass, ActionIn->FuncName))
				{
					ToolTipOut = LOCTEXT("Override_Tooltip", "Override");
				}
				else 
				{
					ToolTipOut = LOCTEXT("Macro_Tooltip", "Macro");
				}
			}
		}
		break;
	case EEdGraphSchemaAction_K2Graph::Interface:
		{
			IconOut = FAppStyle::GetBrush(TEXT("GraphEditor.InterfaceFunction_16x"));

			FFormatNamedArguments Args;
			Args.Add(TEXT("InterfaceName"), FText::FromName(ActionIn->FuncName));
			ToolTipOut = FText::Format(LOCTEXT("FunctionFromInterface_Tooltip", "Function (from Interface '{InterfaceName}')"), Args);
			if (UFunction* OverrideFunc = FindUField<UFunction>(BlueprintIn->SkeletonGeneratedClass, ActionIn->FuncName))
			{
				if (UEdGraphSchema_K2::FunctionCanBePlacedAsEvent(OverrideFunc))
				{
					ToolTipOut = FText::Format(LOCTEXT("EventFromInterface_Tooltip", "Event (from Interface '{InterfaceName}')"), Args);
					ColorOut = FLinearColor::Yellow;
				}
			}
			if (const UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(BlueprintIn))
			{
				IAnimClassInterface* AnimClassInterface = IAnimClassInterface::GetFromClass(AnimBlueprint->GetAnimBlueprintGeneratedClass());
				if (const FAnimBlueprintFunction* Function = IAnimClassInterface::FindAnimBlueprintFunction(AnimClassInterface, ActionIn->FuncName))
				{
					if (Function->bImplemented)
					{
						ToolTipOut = FText::Format(LOCTEXT("FunctionFromInterfaceImplemented_Tooltip", "Function (from Interface '{InterfaceName}') with implementation"), Args);
						ColorOut = FLinearColor(0.0f, 1.0f, 1.0f);
					}
				}
			}
		}
		break;
	case EEdGraphSchemaAction_K2Graph::Function:
		{
			if (ActionIn->EdGraph == nullptr)
			{
				IconOut = FAppStyle::GetBrush(TEXT("GraphEditor.PotentialOverrideFunction_16x"));
				ToolTipOut = LOCTEXT("PotentialOverride_Tooltip", "Potential Override");	
			}
			else
			{
				if (ActionIn->EdGraph->IsA(UAnimationGraph::StaticClass()))
				{
					IconOut = FAppStyle::GetBrush(TEXT("GraphEditor.Animation_16x"));
				}
				else if (UFunction* OverrideFunc = FindUField<UFunction>(BlueprintIn->ParentClass, ActionIn->FuncName))
				{
					const bool bIsPureFunction = OverrideFunc && OverrideFunc->HasAnyFunctionFlags(FUNC_BlueprintPure);
					IconOut = FAppStyle::GetBrush(bIsPureFunction ? TEXT("GraphEditor.OverridePureFunction_16x") : TEXT("GraphEditor.OverrideFunction_16x"));
					ToolTipOut = LOCTEXT("Override_Tooltip", "Override");
				}
				else
				{
					UFunction* Function = FindUField<UFunction>(BlueprintIn->SkeletonGeneratedClass, ActionIn->FuncName);
					const bool bIsPureFunction = Function && Function->HasAnyFunctionFlags(FUNC_BlueprintPure);

					IconOut = FAppStyle::GetBrush(bIsPureFunction ? TEXT("GraphEditor.PureFunction_16x") : TEXT("GraphEditor.Function_16x"));
					if (ActionIn->EdGraph->IsA(UAnimationGraph::StaticClass()))
					{
						ToolTipOut = LOCTEXT("AnimationGraph_Tooltip", "Animation Graph");
					}
					else
					{
						ToolTipOut = LOCTEXT("Function_Tooltip", "Function");
					}
				}
			}
		}
		break;
	}
}

/**
 * A utility function intended to aid the construction of a specific blueprint 
 * palette item. This looks at the item's associated action, and based off its  
 * type, retrieves an icon, color and tooltip for the slate widget.
 * 
 * @param  ActionIn		The action associated with the palette item you want an icon for.
 * @param  BlueprintIn	The blueprint currently being edited (that the action is for).
 * @param  BrushOut		An output of the icon, best representing the specified action.
 * @param  ColorOut		An output color, further denoting the specified action.
 * @param  ToolTipOut	An output tooltip, best describing the specified action type.
 */
static void GetPaletteItemIcon(TSharedPtr<FEdGraphSchemaAction> ActionIn, UBlueprint const* BlueprintIn, FSlateBrush const*& BrushOut, FSlateColor& ColorOut, FText& ToolTipOut, FString& DocLinkOut, FString& DocExcerptOut, FSlateBrush const*& SecondaryBrushOut, FSlateColor& SecondaryColorOut)
{
	// Default to tooltip based on action supplied
	ToolTipOut = ActionIn->GetTooltipDescription().IsEmpty() ? ActionIn->GetMenuDescription() : ActionIn->GetTooltipDescription();

	if (ActionIn->GetTypeId() == FBlueprintActionMenuItem::StaticGetTypeId())
	{
		FBlueprintActionMenuItem* NodeSpawnerAction = (FBlueprintActionMenuItem*)ActionIn.Get();
		BrushOut = NodeSpawnerAction->GetMenuIcon(ColorOut);

		TSubclassOf<UEdGraphNode> VarNodeClass = NodeSpawnerAction->GetRawAction()->NodeClass;
		// if the node is a variable getter or setter, use the variable icon instead, because maps need two brushes
		if (*VarNodeClass && VarNodeClass->IsChildOf(UK2Node_Variable::StaticClass()))
		{
			const UK2Node_Variable* TemplateNode = Cast<UK2Node_Variable>(NodeSpawnerAction->GetRawAction()->GetTemplateNode());
			FProperty* Property = TemplateNode->GetPropertyForVariable();
			BrushOut = FBlueprintEditor::GetVarIconAndColorFromProperty(Property, ColorOut, SecondaryBrushOut, SecondaryColorOut);
		}
	}
	else if (ActionIn->GetTypeId() == FBlueprintDragDropMenuItem::StaticGetTypeId())
	{
		FBlueprintDragDropMenuItem* DragDropAction = (FBlueprintDragDropMenuItem*)ActionIn.Get();
		BrushOut = DragDropAction->GetMenuIcon(ColorOut);
	}
	// for backwards compatibility:
	else if (UK2Node const* const NodeTemplate = FBlueprintActionMenuUtils::ExtractNodeTemplateFromAction(ActionIn))
	{
		// If the node wants to create tooltip text, use that instead, because its probably more detailed
		FText NodeToolTipText = NodeTemplate->GetTooltipText();
		if (!NodeToolTipText.IsEmpty())
		{
			ToolTipOut = NodeToolTipText;
		}

		// Ask node for a palette icon
		FLinearColor IconLinearColor = FLinearColor::White;
		BrushOut = NodeTemplate->GetIconAndTint(IconLinearColor).GetOptionalIcon();
		ColorOut = IconLinearColor;
	}
	// for MyBlueprint tab specific actions:
	else if (ActionIn->GetTypeId() == FEdGraphSchemaAction_K2Graph::StaticGetTypeId())
	{
		FEdGraphSchemaAction_K2Graph const* GraphAction = (FEdGraphSchemaAction_K2Graph const*)ActionIn.Get();
		GetSubGraphIcon(GraphAction, BlueprintIn, BrushOut, ColorOut, ToolTipOut);
	}
	else if (ActionIn->GetTypeId() == FEdGraphSchemaAction_K2Delegate::StaticGetTypeId())
	{
		FEdGraphSchemaAction_K2Delegate* DelegateAction = (FEdGraphSchemaAction_K2Delegate*)ActionIn.Get();

		BrushOut = FAppStyle::GetBrush(TEXT("GraphEditor.Delegate_16x"));
		FFormatNamedArguments Args;
		Args.Add(TEXT("EventDispatcherName"), FText::FromName(DelegateAction->GetDelegateName()));
		ToolTipOut = FText::Format(LOCTEXT("Delegate_Tooltip", "Event Dispatcher '{EventDispatcherName}'"), Args);
	}
	else if (ActionIn->GetTypeId() == FEdGraphSchemaAction_K2Var::StaticGetTypeId())
	{
		FEdGraphSchemaAction_K2Var* VarAction = (FEdGraphSchemaAction_K2Var*)ActionIn.Get();

		UClass* VarClass = VarAction->GetVariableClass();
		BrushOut = FBlueprintEditor::GetVarIconAndColor(VarClass, VarAction->GetVariableName(), ColorOut, SecondaryBrushOut, SecondaryColorOut);
		ToolTipOut = FText::FromString(GetVarType(VarClass, VarAction->GetVariableName(), true, true));

		DocLinkOut = TEXT("Shared/Editor/Blueprint/VariableTypes");
		DocExcerptOut = GetVarType(VarClass, VarAction->GetVariableName(), false, false);
	}
	else if (ActionIn->GetTypeId() == FEdGraphSchemaAction_K2LocalVar::StaticGetTypeId())
	{
		FEdGraphSchemaAction_K2LocalVar* LocalVarAction = (FEdGraphSchemaAction_K2LocalVar*)ActionIn.Get();

		UStruct* VarScope = CastChecked<UStruct>(LocalVarAction->GetVariableScope());
		BrushOut = FBlueprintEditor::GetVarIconAndColor(VarScope, LocalVarAction->GetVariableName(), ColorOut, SecondaryBrushOut, SecondaryColorOut);
		ToolTipOut = FText::FromString(GetVarType(VarScope, LocalVarAction->GetVariableName(), true));

		DocLinkOut = TEXT("Shared/Editor/Blueprint/VariableTypes");
		DocExcerptOut = GetVarType(VarScope, LocalVarAction->GetVariableName(), false);
	}
	else if (ActionIn->IsA(FEdGraphSchemaAction_BlueprintVariableBase::StaticGetTypeId()))
	{
		FEdGraphSchemaAction_BlueprintVariableBase* BPVarAction = (FEdGraphSchemaAction_BlueprintVariableBase*)(ActionIn.Get());
		const FEdGraphPinType PinType = BPVarAction->GetPinType();
		
		BrushOut = FBlueprintEditor::GetVarIconAndColorFromPinType(PinType, ColorOut, SecondaryBrushOut, SecondaryColorOut);
		ToolTipOut = FText::FromString(UEdGraphSchema_K2::TypeToText(PinType).ToString());

		DocLinkOut = TEXT("Shared/Editor/Blueprint/VariableTypes");
		DocExcerptOut = UEdGraphSchema_K2::TypeToText(PinType).ToString();
	}
	else if (ActionIn->GetTypeId() == FEdGraphSchemaAction_K2Enum::StaticGetTypeId())
	{
		BrushOut = FAppStyle::GetBrush(TEXT("GraphEditor.EnumGlyph"));
		ToolTipOut = LOCTEXT("Enum_Tooltip", "Enum Asset");
	}
	else if (ActionIn->GetTypeId() == FEdGraphSchemaAction_K2Struct::StaticGetTypeId())
	{
		BrushOut = FAppStyle::GetBrush(TEXT("GraphEditor.StructGlyph"));
		ToolTipOut = LOCTEXT("Struct_Tooltip", "Struct Asset");
	}
	else
	{
		BrushOut = ActionIn->GetPaletteIcon();
		const FText ActionToolTip = ActionIn->GetPaletteToolTip();
		if(!ActionToolTip.IsEmpty())
		{
			ToolTipOut = ActionToolTip;
		}
	}
}

/**
 * Takes the existing tooltip and concats a path id (for the specified action) 
 * to the end.
 * 
 * @param  ActionIn		The action you want to show the path for.
 * @param  OldToolTip	The tooltip that you're replacing (we fold it into the new one)/
 * @return The newly created tooltip (now with the action's path tacked on the bottom).
 */
static TSharedRef<IToolTip> ConstructToolTipWithActionPath(TSharedPtr<FEdGraphSchemaAction> ActionIn, TSharedPtr<IToolTip> OldToolTip)
{
	TSharedRef<IToolTip> NewToolTip = OldToolTip.ToSharedRef();
	if (!ActionIn)
	{
		return NewToolTip;
	}

	FFavoritedBlueprintPaletteItem ActionItem(*ActionIn);
	if (ActionItem.IsValid())
	{
		static FTextBlockStyle PathStyle = FTextBlockStyle()
			.SetFont(FCoreStyle::GetDefaultFontStyle("Regular", 8))
			.SetColorAndOpacity(FLinearColor(0.4f, 0.4f, 0.4f));

		NewToolTip = SNew(SToolTip)

		// Emulate text-only tool-tip styling that SToolTip uses when no custom content is supplied.  We want node tool-tips to 
		// be styled just like text-only tool-tips
		.BorderImage( FCoreStyle::Get().GetBrush("ToolTip.BrightBackground") )
		.TextMargin(FMargin(11.0f))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			[
				OldToolTip->GetContentWidget()
			]

			+ SVerticalBox::Slot()
			.HAlign(EHorizontalAlignment::HAlign_Right)
			[
				SNew(STextBlock)
				.TextStyle( FAppStyle::Get(), "Documentation.SDocumentationTooltip")
				.Text(FText::FromString(ActionItem.ToString()))
				//.TextStyle(&PathStyle)
			]
		];
	}

	return NewToolTip;
}

/*******************************************************************************
* FBlueprintPaletteItemRenameUtils
*******************************************************************************/

/** A set of utilities to aid SBlueprintPaletteItem when the user attempts to rename one. */
struct FBlueprintPaletteItemRenameUtils
{
private:
	static bool VerifyNewAssetName(UObject* Object, const FText& InNewText, FText& OutErrorMessage)
	{
		if (!Object)
		{
			return false;
		}

		if (Object->GetName() == InNewText.ToString())
		{
			return true;
		}

		TArray<FAssetData> AssetData;
		FAssetRegistryModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		AssetToolsModule.Get().GetAssetsByPath(FName(*FPaths::GetPath(Object->GetOutermost()->GetPathName())), AssetData);

		if(!FFileHelper::IsFilenameValidForSaving(InNewText.ToString(), OutErrorMessage) || !FName(*InNewText.ToString()).IsValidObjectName( OutErrorMessage ))
		{
			return false;
		}
		else if( InNewText.ToString().Len() >= NAME_SIZE )
		{
			OutErrorMessage = FText::Format(LOCTEXT("RenameFailed_NameTooLong", "Names must have fewer than {0} characters!"), NAME_SIZE);
		}
		else
		{
			// Check to see if the name conflicts
			for ( const FAssetData& AssetInfo : AssetData)
			{
				if(AssetInfo.AssetName.ToString() == InNewText.ToString())
				{
					OutErrorMessage = LOCTEXT("RenameFailed_AlreadyInUse", "Asset name already in use!");
					return false;
				}
			}
		}

		return true;
	}

	static void CommitNewAssetName(UObject* Object, FBlueprintEditor* BlueprintEditor, const FText& NewText)
	{
		if (Object && BlueprintEditor)
		{
			if(Object->GetName() != NewText.ToString())
			{
				TArray<FAssetRenameData> AssetsAndNames;
				const FString PackagePath = FPackageName::GetLongPackagePath(Object->GetOutermost()->GetName());
				new(AssetsAndNames) FAssetRenameData(Object, PackagePath, NewText.ToString());

				FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
				AssetToolsModule.Get().RenameAssetsWithDialog(AssetsAndNames);
			}

			TSharedPtr<SMyBlueprint> MyBlueprint = BlueprintEditor->GetMyBlueprintWidget();
			if (MyBlueprint.IsValid())
			{
				MyBlueprint->SelectItemByName(FName(*Object->GetPathName()));
			}	
		}
	}

public:
	/**
	 * Determines whether the enum node, associated with the selected action, 
	 * can be renamed with the specified text.
	 * 
	 * @param  InNewText		The text you want to verify.
	 * @param  OutErrorMessage	Text explaining why the associated node couldn't be renamed (if the return value is false).
	 * @param  ActionPtr		The selected action that the calling palette item represents.
	 * @return True if it is ok to rename the associated node with the given string (false if not).
	 */
	static bool VerifyNewEnumName(const FText& InNewText, FText& OutErrorMessage, TWeakPtr<FEdGraphSchemaAction> ActionPtr)
	{
		// Should never make it here with anything but an enum action
		check(ActionPtr.Pin()->GetTypeId() == FEdGraphSchemaAction_K2Enum::StaticGetTypeId());

		FEdGraphSchemaAction_K2Enum* EnumAction = (FEdGraphSchemaAction_K2Enum*)ActionPtr.Pin().Get();
		
		return VerifyNewAssetName(EnumAction ? EnumAction->Enum : nullptr, InNewText, OutErrorMessage);
	}

	/**
	 * Take the verified text and renames the enum node associated with the 
	 * selected action.
	 * 
	 * @param  NewText				The new (verified) text to rename the node with.
	 * @param  InTextCommit			A value denoting how the text was entered.
	 * @param  ActionPtr			The selected action that the calling palette item represents.
	 * @param  BlueprintEditorPtr	A pointer to the blueprint editor that the palette belongs to.
	 */
	static void CommitNewEnumName(const FText& NewText, ETextCommit::Type InTextCommit, TWeakPtr<FEdGraphSchemaAction> ActionPtr, TWeakPtr<FBlueprintEditor> BlueprintEditorPtr)
	{
		// Should never make it here with anything but an enum action
		check(ActionPtr.Pin()->GetTypeId() == FEdGraphSchemaAction_K2Enum::StaticGetTypeId());

		FEdGraphSchemaAction_K2Enum* EnumAction = (FEdGraphSchemaAction_K2Enum*)ActionPtr.Pin().Get();

		if(EnumAction->Enum->GetName() != NewText.ToString())
		{
			FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
			TArray<FAssetRenameData> AssetsAndNames;
			const FString PackagePath = FPackageName::GetLongPackagePath(EnumAction->Enum->GetOutermost()->GetName());
			new(AssetsAndNames) FAssetRenameData(EnumAction->Enum, PackagePath, NewText.ToString());

			BlueprintEditorPtr.Pin()->GetMyBlueprintWidget()->SelectItemByName(FName("ConstructionScript"));

			AssetToolsModule.Get().RenameAssetsWithDialog(AssetsAndNames);
		}

		BlueprintEditorPtr.Pin()->GetMyBlueprintWidget()->SelectItemByName(FName(*EnumAction->Enum->GetPathName()));
	}

	/**
	* Determines whether the struct node, associated with the selected action,
	* can be renamed with the specified text.
	*
	* @param  InNewText		The text you want to verify.
	* @param  OutErrorMessage	Text explaining why the associated node couldn't be renamed (if the return value is false).
	* @param  ActionPtr		The selected action that the calling palette item represents.
	* @return True if it is ok to rename the associated node with the given string (false if not).
	*/
	static bool VerifyNewStructName(const FText& InNewText, FText& OutErrorMessage, TWeakPtr<FEdGraphSchemaAction> ActionPtr)
	{
		// Should never make it here with anything but a struct action
		check(ActionPtr.Pin()->GetTypeId() == FEdGraphSchemaAction_K2Struct::StaticGetTypeId());

		FEdGraphSchemaAction_K2Struct* Action = (FEdGraphSchemaAction_K2Struct*)ActionPtr.Pin().Get();

		return VerifyNewAssetName(Action ? Action->Struct : nullptr, InNewText, OutErrorMessage);
	}

	/**
	 * Determines whether the event node, associated with the selected action, 
	 * can be renamed with the specified text.
	 * 
	 * @param  InNewText		The text you want to verify.
	 * @param  OutErrorMessage	Text explaining why the associated node couldn't be renamed (if the return value is false).
	 * @param  ActionPtr		The selected action that the calling palette item represents.
	 * @return True if it is ok to rename the associated node with the given string (false if not).
	 */
	static bool VerifyNewEventName(const FText& InNewText, FText& OutErrorMessage, TWeakPtr<FEdGraphSchemaAction> ActionPtr)
	{
		bool bIsNameValid = false;
		OutErrorMessage = LOCTEXT("RenameFailed_NodeRename", "Cannot rename associated node!");

		check(ActionPtr.Pin()->GetTypeId() == FEdGraphSchemaAction_K2Event::StaticGetTypeId());
		FEdGraphSchemaAction_K2Event* EventAction = (FEdGraphSchemaAction_K2Event*)ActionPtr.Pin().Get();

		UK2Node* AssociatedNode = EventAction->NodeTemplate;
		if (AssociatedNode && AssociatedNode->GetCanRenameNode())
		{
			TSharedPtr<INameValidatorInterface> NodeNameValidator = FNameValidatorFactory::MakeValidator(AssociatedNode);
			bIsNameValid = (NodeNameValidator->IsValid(InNewText.ToString(), true) == EValidatorResult::Ok);
		}
		return bIsNameValid;
	}

	/**
	 * Take the verified text and renames the struct node associated with the 
	 * selected action.
	 * 
	 * @param  NewText				The new (verified) text to rename the node with.
	 * @param  InTextCommit			A value denoting how the text was entered.
	 * @param  ActionPtr			The selected action that the calling palette item represents.
	 * @param  BlueprintEditorPtr	A pointer to the blueprint editor that the palette belongs to.
	 */
	static void CommitNewStructName(const FText& NewText, ETextCommit::Type InTextCommit, TWeakPtr<FEdGraphSchemaAction> ActionPtr, TWeakPtr<FBlueprintEditor> BlueprintEditorPtr)
	{
		// Should never make it here with anything but a struct action
		check(ActionPtr.Pin()->GetTypeId() == FEdGraphSchemaAction_K2Struct::StaticGetTypeId());

		FEdGraphSchemaAction_K2Struct* Action = (FEdGraphSchemaAction_K2Struct*)ActionPtr.Pin().Get();

		CommitNewAssetName(Action ? Action->Struct : nullptr, BlueprintEditorPtr.Pin().Get(), NewText);
	}

	/**
	 * Take the verified text and renames the event node associated with the 
	 * selected action.
	 * 
	 * @param  NewText		The new (verified) text to rename the node with.
	 * @param  InTextCommit	A value denoting how the text was entered.
	 * @param  ActionPtr	The selected action that the calling palette item represents.
	 */
	static void CommitNewEventName(const FText& NewText, ETextCommit::Type InTextCommit, TWeakPtr<FEdGraphSchemaAction> ActionPtr)
	{
		check(ActionPtr.Pin()->GetTypeId() == FEdGraphSchemaAction_K2Event::StaticGetTypeId());

		FEdGraphSchemaAction_K2Event* EventAction = (FEdGraphSchemaAction_K2Event*)ActionPtr.Pin().Get();
		if (EventAction->NodeTemplate)
		{
			EventAction->NodeTemplate->OnRenameNode(NewText.ToString());
		}
	}

	/**
	 * Determines whether the target node, associated with the selected action, 
	 * can be renamed with the specified text.
	 * 
	 * @param  InNewText		The text you want to verify.
	 * @param  OutErrorMessage	Text explaining why the associated node couldn't be renamed (if the return value is false).
	 * @param  ActionPtr		The selected action that the calling palette item represents.
	 * @return True if it is ok to rename the associated node with the given string (false if not).
	 */
	static bool VerifyNewTargetNodeName(const FText& InNewText, FText& OutErrorMessage, TWeakPtr<FEdGraphSchemaAction> ActionPtr)
	{

		bool bIsNameValid = false;
		OutErrorMessage = LOCTEXT("RenameFailed_NodeRename", "Cannot rename associated node!");

		check(ActionPtr.Pin()->GetTypeId() == FEdGraphSchemaAction_K2TargetNode::StaticGetTypeId());
		FEdGraphSchemaAction_K2TargetNode* TargetNodeAction = (FEdGraphSchemaAction_K2TargetNode*)ActionPtr.Pin().Get();

		UK2Node* AssociatedNode = TargetNodeAction->NodeTemplate;
		if (AssociatedNode && AssociatedNode->GetCanRenameNode())
		{
			TSharedPtr<INameValidatorInterface> NodeNameValidator = FNameValidatorFactory::MakeValidator(AssociatedNode);
			bIsNameValid = (NodeNameValidator->IsValid(InNewText.ToString(), true) == EValidatorResult::Ok);
		}
		return bIsNameValid;
	}

	/**
	 * Take the verified text and renames the target node associated with the 
	 * selected action.
	 * 
	 * @param  NewText		The new (verified) text to rename the node with.
	 * @param  InTextCommit	A value denoting how the text was entered.
	 * @param  ActionPtr	The selected action that the calling palette item represents.
	 */
	static void CommitNewTargetNodeName(const FText& NewText, ETextCommit::Type InTextCommit, TWeakPtr<FEdGraphSchemaAction> ActionPtr)
	{
		check(ActionPtr.Pin()->GetTypeId() == FEdGraphSchemaAction_K2TargetNode::StaticGetTypeId());

		FEdGraphSchemaAction_K2TargetNode* TargetNodeAction = (FEdGraphSchemaAction_K2TargetNode*)ActionPtr.Pin().Get();
		if (TargetNodeAction->NodeTemplate)
		{
			TargetNodeAction->NodeTemplate->OnRenameNode(NewText.ToString());
		}
	}
};

/*******************************************************************************
* SPinTypeSelectorHelper
*******************************************************************************/
DECLARE_DELEGATE_OneParam(FOnPinTypeChanged, const FEdGraphPinType&)

class SPinTypeSelectorHelper : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SPinTypeSelectorHelper ) {}
		SLATE_ATTRIBUTE(bool, ReadOnly)
		SLATE_EVENT(FOnPinTypeChanged, OnTypeChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TWeakPtr<FEdGraphSchemaAction_BlueprintVariableBase> InAction, UBlueprint* InBlueprint, TWeakPtr<FBlueprintEditor> InBlueprintEditor)
	{
		BlueprintObj = InBlueprint;
		BlueprintEditorPtr = InBlueprintEditor;
		ActionPtr = InAction;
		VariableProperty = nullptr;
		if (ActionPtr.IsValid())
		{
			VariableProperty = ActionPtr.Pin()->GetProperty();
		}

		ConstructInternal(InArgs);
	}

private:

	void ConstructInternal(const FArguments& InArgs)
	{
		OnTypeChanged = InArgs._OnTypeChanged;
		
		TArray<TSharedPtr<IPinTypeSelectorFilter>> CustomPinTypeFilters;
		if (BlueprintEditorPtr.IsValid())
		{
			BlueprintEditorPtr.Pin()->GetPinTypeSelectorFilters(CustomPinTypeFilters);
		}

		const UEdGraphSchema* Schema = GetDefault<UEdGraphSchema_K2>();
		if (BlueprintEditorPtr.IsValid())
		{
			if (BlueprintEditorPtr.Pin()->GetFocusedGraph())
			{
				Schema = BlueprintEditorPtr.Pin()->GetFocusedGraph()->GetSchema();
			}
			else
			{
				Schema = BlueprintEditorPtr.Pin()->GetDefaultSchema().GetDefaultObject();
			}
		}
		
		const bool bIsDelegate = ActionPtr.IsValid() && ActionPtr.Pin()->GetTypeId() == FEdGraphSchemaAction_K2Delegate::StaticGetTypeId();

		// You cannot change the type of multicast delegates in blueprints
		if(!bIsDelegate)
		{
			this->ChildSlot
			[
				SNew(SPinTypeSelector, FGetPinTypeTree::CreateUObject(GetDefault<UEdGraphSchema_K2>(), &UEdGraphSchema_K2::GetVariableTypeTree))
				.ReadOnly(InArgs._ReadOnly)
				.Schema(Schema)
				.SchemaAction(ActionPtr)
				.TargetPinType(this, &SPinTypeSelectorHelper::OnGetVarType)
				.OnPinTypeChanged(this, &SPinTypeSelectorHelper::OnVarTypeChanged)
				.TypeTreeFilter(ETypeTreeFilter::None)
				.SelectorType(BlueprintEditorPtr.IsValid() ? SPinTypeSelector::ESelectorType::Partial : SPinTypeSelector::ESelectorType::None)
				.CustomFilters(CustomPinTypeFilters)
			];	
		}
	}
	
	FEdGraphPinType OnGetVarType() const
	{
		if (FProperty* VarProp = const_cast<FProperty*>(VariableProperty.Get()))
		{
			const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
			FEdGraphPinType Type;
			K2Schema->ConvertPropertyToPinType(VarProp, Type);
			return Type;
		}
		else if(ActionPtr.IsValid())
		{
			return ActionPtr.Pin()->GetPinType();
		}
		return FEdGraphPinType();
	}

	void OnVarTypeChanged(const FEdGraphPinType& InNewPinType)
	{
		if (FBlueprintEditorUtils::IsPinTypeValid(InNewPinType))
		{
			TSharedPtr<FBlueprintEditor> BlueprintEditor = BlueprintEditorPtr.Pin();

			if (FProperty* VarProp = VariableProperty.Get())
			{
				FName VarName = VarProp->GetFName();

				if (VarName != NAME_None)
				{
					if (BlueprintEditor.IsValid())
					{
						// Set the MyBP tab's last pin type used as this, for adding lots of variables of the same type
						BlueprintEditor->GetMyBlueprintWidget()->GetLastPinTypeUsed() = InNewPinType;
					}

					if (UFunction* LocalVariableScope = VarProp->GetOwner<UFunction>())
					{
						FBlueprintEditorUtils::ChangeLocalVariableType(BlueprintObj, LocalVariableScope, VarName, InNewPinType);
					}
					else
					{
						FBlueprintEditorUtils::ChangeMemberVariableType(BlueprintObj, VarName, InNewPinType);
					}
				}
			}
			else if(ActionPtr.IsValid())
			{
				if (BlueprintEditor.IsValid())
				{
					// Set the MyBP tab's last pin type used as this, for adding lots of variables of the same type
					BlueprintEditor->GetMyBlueprintWidget()->GetLastPinTypeUsed() = InNewPinType;
				}
				
				ActionPtr.Pin()->ChangeVariableType(InNewPinType);
			}

			// Auto-import the underlying type object's default namespace set into the current editor context.
			const UObject* PinSubCategoryObject = InNewPinType.PinSubCategoryObject.Get();
			if (PinSubCategoryObject && BlueprintEditor.IsValid())
			{
				FBlueprintEditor::FImportNamespaceExParameters Params;
				FBlueprintNamespaceUtilities::GetDefaultImportsForObject(PinSubCategoryObject, Params.NamespacesToImport);
				BlueprintEditor->ImportNamespaceEx(Params);
			}
		}

		if (OnTypeChanged.IsBound())
		{
			OnTypeChanged.Execute(InNewPinType);
		}
	}

private:
	/** The action that the owning palette entry represents */
	TWeakPtr<FEdGraphSchemaAction_BlueprintVariableBase> ActionPtr;

	/** Pointer back to the blueprint that is being displayed: */
	UBlueprint* BlueprintObj;

	/** Pointer back to the blueprint editor that owns this, optional because of diff and merge views: */
	TWeakPtr<FBlueprintEditor>     BlueprintEditorPtr;

	/** Variable Property to change the type of */
	TWeakFieldPtr<FProperty> VariableProperty;

	/** Event when type has changed */
	FOnPinTypeChanged OnTypeChanged;
};

/*******************************************************************************
* SPaletteItemVisibilityToggle
*******************************************************************************/

class SPaletteItemVisibilityToggle : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SPaletteItemVisibilityToggle ) {}
	SLATE_END_ARGS()

	/**
	 * Constructs a visibility-toggle widget (for variable actions only, so that 
	 * the user can modify the variable's "edit-on-instance" state).
	 * 
	 * @param  InArgs			A set of slate arguments, defined above.
	 * @param  ActionPtrIn		The FEdGraphSchemaAction that the parent item represents.
	 * @param  BlueprintEdPtrIn	A pointer to the blueprint editor that the palette belongs to.
	 */
	void Construct(const FArguments& InArgs, TWeakPtr<FEdGraphSchemaAction> ActionPtrIn, TWeakPtr<FBlueprintEditor> InBlueprintEditor, UBlueprint* InBlueprint)
	{
		ActionPtr = ActionPtrIn;
		BlueprintEditorPtr = InBlueprintEditor;
		BlueprintObj = InBlueprint;
		TSharedPtr<FEdGraphSchemaAction> PaletteAction = ActionPtrIn.Pin();

		bool bShouldHaveAVisibilityToggle = false;
		if (PaletteAction->GetTypeId() == FEdGraphSchemaAction_K2Var::StaticGetTypeId())
		{
			FProperty* VariableProp = StaticCastSharedPtr<FEdGraphSchemaAction_K2Var>(PaletteAction)->GetProperty();
			FObjectProperty* VariableObjProp = CastField<FObjectProperty>(VariableProp);

			UStruct* VarSourceScope = (VariableProp ? CastChecked<UStruct>(VariableProp->GetOwner<UObject>()) : nullptr);
			const bool bIsBlueprintVariable = (VarSourceScope == BlueprintObj->SkeletonGeneratedClass);
			const bool bIsComponentVar = (VariableObjProp && VariableObjProp->PropertyClass && VariableObjProp->PropertyClass->IsChildOf(UActorComponent::StaticClass()));
			bShouldHaveAVisibilityToggle = bIsBlueprintVariable && (!bIsComponentVar || FBlueprintEditorUtils::IsVariableCreatedByBlueprint(BlueprintObj, VariableObjProp));
		}

		this->ChildSlot
		[
			SNew(SBorder)
			.Padding(0.0f)
			.BorderImage(FStyleDefaults::GetNoBrush())
			.Visibility(bShouldHaveAVisibilityToggle ? EVisibility::Visible : EVisibility::Collapsed)
			//.ForegroundColor(this, &SPaletteItemVisibilityToggle::GetVisibilityToggleColor)
			[
				SNew(SCheckBox)
				.ToolTipText(this, &SPaletteItemVisibilityToggle::GetVisibilityToggleToolTip)
				.OnCheckStateChanged(this, &SPaletteItemVisibilityToggle::OnVisibilityToggleFlipped)
				.IsChecked(this, &SPaletteItemVisibilityToggle::GetVisibilityToggleState)
				.Style(FAppStyle::Get(), "TransparentCheckBox")
				[
					SNew(SImage)
					.Image(this, &SPaletteItemVisibilityToggle::GetVisibilityIcon)
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		];
	}

private:
	/**
	 * Used by this visibility-toggle widget to see if the property represented 
	 * by this item is visible outside of Kismet.
	 * 
	 * @return ECheckBoxState::Checked if the property is visible, false if not (or if the property wasn't found)
	 */
	ECheckBoxState GetVisibilityToggleState() const
	{
		TSharedPtr<FEdGraphSchemaAction> PaletteAction = ActionPtr.Pin();
		if ( PaletteAction->GetTypeId() == FEdGraphSchemaAction_K2Var::StaticGetTypeId() )
		{
			TSharedPtr<FEdGraphSchemaAction_K2Var> VarAction = StaticCastSharedPtr<FEdGraphSchemaAction_K2Var>(PaletteAction);
			if (FProperty* VariableProperty = VarAction->GetProperty())
			{
				return VariableProperty->HasAnyPropertyFlags(CPF_DisableEditOnInstance) ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
			}
		}

		return ECheckBoxState::Unchecked;
	}

	/**
	 * Used by this visibility-toggle widget when the user makes a change to the
	 * checkbox (modifies the property represented by this item by flipping its
	 * edit-on-instance flag).
	 * 
	 * @param  InNewState	The new state that the user set the checkbox to.
	 */
	void OnVisibilityToggleFlipped(ECheckBoxState InNewState)
	{
		if( !BlueprintEditorPtr.IsValid() )
		{
			return;
		}

		TSharedPtr<FEdGraphSchemaAction> PaletteAction = ActionPtr.Pin();
		if ( PaletteAction->GetTypeId() == FEdGraphSchemaAction_K2Var::StaticGetTypeId() )
		{
			TSharedPtr<FEdGraphSchemaAction_K2Var> VarAction = StaticCastSharedPtr<FEdGraphSchemaAction_K2Var>(PaletteAction);

			// Toggle the flag on the blueprint's version of the variable description, based on state
			const bool bVariableIsExposed = ( InNewState == ECheckBoxState::Checked );

			FBlueprintEditorUtils::SetBlueprintOnlyEditableFlag(BlueprintObj, VarAction->GetVariableName(), !bVariableIsExposed);
		}
	}

	/**
	 * Used by this visibility-toggle widget to convey the visibility of the 
	 * property represented by this item.
	 * 
	 * @return A image representing the variable's "edit-on-instance" state.
	 */
	const FSlateBrush* GetVisibilityIcon() const
	{
		return GetVisibilityToggleState() == ECheckBoxState::Checked ?
			FAppStyle::GetBrush( "Kismet.VariableList.ExposeForInstance" ) :
			FAppStyle::GetBrush( "Kismet.VariableList.HideForInstance" );
	}

	/**
	 * Used by this visibility-toggle widget to convey the visibility of the 
	 * property represented by this item (as well as the status of the 
	 * variable's tooltip).
	 * 
	 * @return A color denoting the item's visibility and tootip status.
	 */
	FSlateColor GetVisibilityToggleColor() const 
	{
		if ( GetVisibilityToggleState() != ECheckBoxState::Checked )
		{
			return FSlateColor::UseForeground();
		}
		else
		{
			TSharedPtr<FEdGraphSchemaAction_K2Var> VarAction = StaticCastSharedPtr<FEdGraphSchemaAction_K2Var>(ActionPtr.Pin());

			FString Result;
			FBlueprintEditorUtils::GetBlueprintVariableMetaData(BlueprintObj, VarAction->GetVariableName(), nullptr, TEXT("tooltip"), Result);

			if ( !Result.IsEmpty() )
			{
				static const FName TooltipExistsColor("Colors.AccentGreen");
				return FAppStyle::Get().GetSlateColor(TooltipExistsColor);
			}
			else
			{
				static const FName TooltipDoesntExistColor("Colors.AccentYellow");
				return FAppStyle::Get().GetSlateColor(TooltipDoesntExistColor);
			}
		}
	}

	/**
	 * Used by this visibility-toggle widget to supply the toggle with a tooltip
	 * representing the "edit-on-instance" state of the variable represented by 
	 * this item.
	 * 
	 * @return Tooltip text for this toggle.
	 */
	FText GetVisibilityToggleToolTip() const
	{
		FText ToolTipText = FText::GetEmpty();
		if ( GetVisibilityToggleState() != ECheckBoxState::Checked )
		{
			ToolTipText = LOCTEXT("VariablePrivacy_not_public_Tooltip", "Variable is not public and will not be editable on an instance of this Blueprint.");
		}
		else
		{
			ToolTipText = LOCTEXT("VariablePrivacy_is_public_Tooltip", "Variable is public and is editable on each instance of this Blueprint.");
		}
		return ToolTipText;
	}

private:
	/** The action that the owning palette entry represents */
	TWeakPtr<FEdGraphSchemaAction> ActionPtr;

	/** Pointer back to the blueprint editor that owns this, optional because of diff and merge views: */
	TWeakPtr<FBlueprintEditor>     BlueprintEditorPtr;

	/** Pointer back to the blueprint that is being diplayed: */
	UBlueprint* BlueprintObj;
};

/*******************************************************************************
* SBlueprintPaletteItem Public Interface
*******************************************************************************/

//------------------------------------------------------------------------------
BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SBlueprintPaletteItem::Construct(const FArguments& InArgs, FCreateWidgetForActionData* const InCreateData, TWeakPtr<FBlueprintEditor> InBlueprintEditor)
{
	Construct(InArgs, InCreateData, InBlueprintEditor.Pin()->GetBlueprintObj(), InBlueprintEditor);
}

void SBlueprintPaletteItem::Construct(const FArguments& InArgs, FCreateWidgetForActionData* const InCreateData, UBlueprint* InBlueprint)
{
	Construct( InArgs, InCreateData, InBlueprint, TWeakPtr<FBlueprintEditor>() );
}

void SBlueprintPaletteItem::Construct(const FArguments& InArgs, FCreateWidgetForActionData* const InCreateData, UBlueprint* InBlueprint, TWeakPtr<FBlueprintEditor> InBlueprintEditor)
{
	check(InCreateData->Action.IsValid());
	check(InBlueprint);

	Blueprint = InBlueprint;

	bShowClassInTooltip = InArgs._ShowClassInTooltip;	

	TSharedPtr<FEdGraphSchemaAction> GraphAction = InCreateData->Action;
	ActionPtr = InCreateData->Action;
	BlueprintEditorPtr = InBlueprintEditor;

	const bool bIsFullyReadOnly = !InBlueprintEditor.IsValid() || InCreateData->bIsReadOnly;
	
	TWeakPtr<FEdGraphSchemaAction> WeakGraphAction = GraphAction;
	auto IsReadOnlyLambda = [WeakGraphAction, InBlueprintEditor, bIsFullyReadOnly]()
	{ 
		if(WeakGraphAction.IsValid() && InBlueprintEditor.IsValid())
		{
			return bIsFullyReadOnly || FBlueprintEditorUtils::IsPaletteActionReadOnly(WeakGraphAction.Pin(), InBlueprintEditor.Pin());
		}

		return bIsFullyReadOnly;
	};
	
	// We differentiate enabled/read-only state here to not dim icons out unnecessarily, which in some situations
	// (like the right-click palette menu) is confusing to users.
	auto IsEditingEnabledLambda = [InBlueprintEditor]()
	{ 
		if(InBlueprintEditor.IsValid())
		{
			return InBlueprintEditor.Pin()->InEditingMode();
		}

		return true;
	};

	TAttribute<bool> bIsReadOnly = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(IsReadOnlyLambda));
	TAttribute<bool> bIsEditingEnabled = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(IsEditingEnabledLambda));

	// construct the icon widget
	FSlateBrush const* IconBrush   = FAppStyle::GetBrush(TEXT("NoBrush"));
	FSlateBrush const* SecondaryBrush = FAppStyle::GetBrush(TEXT("NoBrush"));
	FSlateColor        IconColor   = FSlateColor::UseForeground();
	FSlateColor        SecondaryIconColor   = FSlateColor::UseForeground();
	FText			   IconToolTip = GraphAction->GetTooltipDescription();
	FString			   IconDocLink, IconDocExcerpt;
	GetPaletteItemIcon(GraphAction, Blueprint, IconBrush, IconColor, IconToolTip, IconDocLink, IconDocExcerpt, SecondaryBrush, SecondaryIconColor);
	TSharedRef<SWidget> IconWidget = CreateIconWidget(IconToolTip, IconBrush, IconColor, IconDocLink, IconDocExcerpt, SecondaryBrush, SecondaryIconColor);
	IconWidget->SetEnabled(bIsEditingEnabled);

	UBlueprintEditorSettings* Settings = GetMutableDefault<UBlueprintEditorSettings>();

	// Enum representing the access specifier of this function or variable
	enum class EAccessSpecifier : uint8
	{
		None		= 0,
		Private		= 1,
		Protected	= 2,
		Public		= 3
	};

	// We should only bother checking for access if the setting is on and this is not an animation graph
	const bool bShouldCheckForAccessSpec = Settings->bShowAccessSpecifier;

	EAccessSpecifier ActionAccessSpecifier = EAccessSpecifier::None;	

	// Setup a meta tag for this node
	FTutorialMetaData TagMeta("PaletteItem"); 
	if( ActionPtr.IsValid() )
	{
		TagMeta.Tag = *FString::Printf(TEXT("PaletteItem,%s,%d"), *GraphAction->GetMenuDescription().ToString(), GraphAction->GetSectionID());
		TagMeta.FriendlyName = GraphAction->GetMenuDescription().ToString();
	}
	// construct the text widget
	TSharedRef<SWidget> NameSlotWidget = CreateTextSlotWidget(InCreateData, bIsReadOnly );
	
	// Will set the icon of this property to be a Pin Type selector. 
	auto GenerateVariableSettings = [&](TSharedPtr<FEdGraphSchemaAction_BlueprintVariableBase> Action)
	{
		FProperty* VariableProp = nullptr;
		if (Action.IsValid())
		{
			VariableProp = Action->GetProperty();
		}
		if (VariableProp)
		{
			if (bShouldCheckForAccessSpec)
			{
				if (VariableProp->GetBoolMetaData(FBlueprintMetadata::MD_Private))
				{
					ActionAccessSpecifier = EAccessSpecifier::Private;
				}
				else if (VariableProp->GetBoolMetaData(FBlueprintMetadata::MD_Protected))
				{
					ActionAccessSpecifier = EAccessSpecifier::Protected;
				}
				else
				{
					ActionAccessSpecifier = EAccessSpecifier::Public;
				}
			}

			if (FBlueprintEditorUtils::IsVariableCreatedByBlueprint(Blueprint, VariableProp) || VariableProp->GetOwner<UFunction>())
			{
				const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
				IconWidget = SNew(SPinTypeSelectorHelper, Action, Blueprint, BlueprintEditorPtr)
					.IsEnabled(bIsEditingEnabled)
					.ReadOnly_Lambda([this]() {return !IsHovered(); });
			}
		}
	};

	// For Variables and Local Variables, we will convert the icon widget into a pin type selector.
	if (GraphAction->GetTypeId() == FEdGraphSchemaAction_K2Var::StaticGetTypeId())
	{	
		GenerateVariableSettings(StaticCastSharedPtr<FEdGraphSchemaAction_K2Var>(GraphAction));
	}
	else if (GraphAction->GetTypeId() == FEdGraphSchemaAction_K2LocalVar::StaticGetTypeId())
	{
		GenerateVariableSettings(StaticCastSharedPtr<FEdGraphSchemaAction_K2LocalVar>(GraphAction));
	}
	else if (GraphAction->IsA(FEdGraphSchemaAction_BlueprintVariableBase::StaticGetTypeId()))
	{
		ActionAccessSpecifier = EAccessSpecifier::Private;
		const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
		IconWidget = SNew(SPinTypeSelectorHelper, StaticCastSharedPtr<FEdGraphSchemaAction_BlueprintVariableBase>(GraphAction), Blueprint, BlueprintEditorPtr)
			.IsEnabled(bIsEditingEnabled)
			.ReadOnly_Lambda([this]() {return !IsHovered(); })
			.OnTypeChanged_Lambda([this](const FEdGraphPinType& NewPinType) { BlueprintEditorPtr.Pin()->GetMyBlueprintWidget()->Refresh(); });
	}
	
	// Determine the access level of this action if it is a function graph or for interface events
	else if (bShouldCheckForAccessSpec && GraphAction->GetTypeId() == FEdGraphSchemaAction_K2Graph::StaticGetTypeId())
	{
		UFunction* FunctionToCheck = nullptr;

		if (FEdGraphSchemaAction_K2Graph* FuncGraphAction = (FEdGraphSchemaAction_K2Graph*)(GraphAction.Get()))
		{
			FunctionToCheck = FindUField<UFunction>(Blueprint->SkeletonGeneratedClass, FuncGraphAction->FuncName);

			// Handle override/interface functions
			if(!FunctionToCheck)
			{
				FBlueprintEditorUtils::GetOverrideFunctionClass(Blueprint, FuncGraphAction->FuncName, &FunctionToCheck);			
			}
		}

		// If we have found a function that matches this action name, then grab it's access specifier
		if (FunctionToCheck)
		{
			if (FunctionToCheck->HasAnyFunctionFlags(FUNC_Protected))
			{
				ActionAccessSpecifier = EAccessSpecifier::Protected;
			}
			else if (FunctionToCheck->HasAnyFunctionFlags(FUNC_Private))
			{
				ActionAccessSpecifier = EAccessSpecifier::Private;
			}
			else
			{
				ActionAccessSpecifier = EAccessSpecifier::Public;
			}
		}
	}

	FText AccessModifierText = FText::GetEmpty();

	switch (ActionAccessSpecifier)
	{
		case EAccessSpecifier::Public:
		{
			AccessModifierText = LOCTEXT("AccessModifierPublic", "public");
		}
		break;
		case EAccessSpecifier::Protected:
		{
			AccessModifierText = LOCTEXT("AccessModifierProtected", "protected");
		}
		break;
		case EAccessSpecifier::Private:
		{
			AccessModifierText = LOCTEXT("AccessModifierPrivate", "private");
		}
		break;
	}

	// Calculate a color so that the text gets brighter the more accessible the action is
	const bool AccessSpecifierEnabled = (ActionAccessSpecifier != EAccessSpecifier::None) && bShouldCheckForAccessSpec;

	// Create the widget with an icon
	TSharedRef<SHorizontalBox> ActionBox = SNew(SHorizontalBox)		
		.AddMetaData<FTutorialMetaData>(TagMeta);


	auto CreateAccessSpecifierLambda = [&ActionBox, &AccessSpecifierEnabled, &AccessModifierText, &ActionAccessSpecifier]() {

		ActionBox.Get().AddSlot()
			.MaxWidth(50.f)
			.FillWidth(AccessSpecifierEnabled ? 0.4f : 0.0f)
			.Padding(FMargin(/* horizontal */ AccessSpecifierEnabled ? 6.0f : 0.0f, /* vertical */ 0.0f))
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				// Will only display text if we have a modifier level
			.IsEnabled(AccessSpecifierEnabled)
			.Text(AccessModifierText)
			.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			// Bold if public
			.TextStyle(FAppStyle::Get(), ActionAccessSpecifier == EAccessSpecifier::Public ? "BlueprintEditor.AccessModifier.Public" : "BlueprintEditor.AccessModifier.Default")
			];
	};

	
	if (GraphAction->IsA(FEdGraphSchemaAction_BlueprintVariableBase::StaticGetTypeId()))
	{


		if (ActionAccessSpecifier != EAccessSpecifier::None && GraphAction->GetTypeId() != FEdGraphSchemaAction_K2LocalVar::StaticGetTypeId())
		{
			CreateAccessSpecifierLambda();
		}

		ActionBox.Get().AddSlot()
			.FillWidth(0.6f)
			.VAlign(VAlign_Center)
			.Padding(3.0f, 0.0f)
			[
				NameSlotWidget
			];

		ActionBox.Get().AddSlot()
			.FillWidth(0.4f)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				IconWidget
			];

		if (InBlueprint && FBlueprintEditorUtils::ImplementsInterface(InBlueprint, true, UNotifyFieldValueChanged::StaticClass()) && GraphAction->GetTypeId() == FEdGraphSchemaAction_K2Var::StaticGetTypeId())
		{
			ActionBox.Get().AddSlot()
				.AutoWidth()
				.Padding(FMargin(6.0f, 0.0f, 3.0f, 0.0f))
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					SNew(SPaletteItemVarFieldNotifyToggle, ActionPtr, InBlueprintEditor, InBlueprint)
					.IsEnabled(bIsEditingEnabled)
				];
		}

		ActionBox.Get().AddSlot()
			.AutoWidth()
			.Padding(FMargin(6.0f, 0.0f, 3.0f, 0.0f))
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(SPaletteItemVisibilityToggle, ActionPtr, InBlueprintEditor, InBlueprint)
				.IsEnabled(bIsEditingEnabled)
			];


		if (TSharedPtr<FEdGraphSchemaAction> Action = ActionPtr.Pin())
		{
			if (GraphAction->GetTypeId() == FEdGraphSchemaAction_K2Var::StaticGetTypeId())
			{
				if (const FProperty* Property = StaticCastSharedPtr<FEdGraphSchemaAction_K2Var>(GraphAction)->GetProperty())
				{
					bool bIsOptional = false;
					if (FWidgetBlueprintEditorUtils::IsBindWidgetProperty(Property, bIsOptional))
					{
						ActionBox.Get().AddSlot()
							.AutoWidth()
							.Padding(FMargin(6.0f, 0.0f, 3.0f, 0.0f))
							.HAlign(HAlign_Right)
							.VAlign(VAlign_Center)
							[
								SNew(SImage)
									.Image(FAppStyle::Get().GetBrush("MainFrame.AddCodeToProject"))
									.IsEnabled(!bIsOptional) // make grey to differentiate between optional and required
									.ToolTipText(bIsOptional
										? LOCTEXT("CppBindWidgetOptionalTooltip", "Widget marked with BindWidgetOptional in native.")
										: LOCTEXT("CppBindWidgetTooltip", "Widget marked with BindWidget in native."))
							];
					}
				}
			}
		}
	}
	else
	{
		ActionBox.Get().AddSlot()
			.AutoWidth()
			.Padding(FMargin(0.0f, 0.0f, 3.0f, 0.0f))
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(SPaletteItemVisibilityToggle, ActionPtr, InBlueprintEditor, InBlueprint)
				.IsEnabled(bIsEditingEnabled)
			];


		ActionBox.Get().AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				IconWidget
			];

		// Only add an access specifier if we have one
		if (ActionAccessSpecifier != EAccessSpecifier::None)
		{
			CreateAccessSpecifierLambda();
		}

		ActionBox.Get().AddSlot()
			.FillWidth(1.f)
			.VAlign(VAlign_Center)
			.Padding(/* horizontal */ 3.0f, /* vertical */ 3.0f)
			[
				NameSlotWidget
			];

		if (TSharedPtr<FEdGraphSchemaAction> Action = ActionPtr.Pin())
		{
			if (GraphAction->GetTypeId() == FEdGraphSchemaAction_K2Graph::StaticGetTypeId())
			{
				if (TSharedPtr<FEdGraphSchemaAction_K2Graph> ActionK2Graph = StaticCastSharedPtr<FEdGraphSchemaAction_K2Graph>(Action))
				{
					if (InBlueprint && FBlueprintEditorUtils::ImplementsInterface(InBlueprint, true, UNotifyFieldValueChanged::StaticClass()) && ActionK2Graph->GraphType == EEdGraphSchemaAction_K2Graph::Function)
					{
						ActionBox.Get().AddSlot()
							.AutoWidth()
							.Padding(FMargin(6.0f, 0.0f, 3.0f, 0.0f))
							.HAlign(HAlign_Right)
							.VAlign(VAlign_Center)
							[
								SNew(SPaletteItemFunctionFieldNotifyToggle, ActionPtr, InBlueprintEditor, InBlueprint)
								.IsEnabled(bIsEditingEnabled)
							];
					}
				}
			}
		}
	}

	// Now, create the actual widget
	ChildSlot
	[
		ActionBox
	];
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SBlueprintPaletteItem::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (BlueprintEditorPtr.IsValid())
	{
		SGraphPaletteItem::OnDragEnter(MyGeometry, DragDropEvent);
	}
}

/*******************************************************************************
* SBlueprintPaletteItem Private Methods
*******************************************************************************/

//------------------------------------------------------------------------------
TSharedRef<SWidget> SBlueprintPaletteItem::CreateTextSlotWidget(FCreateWidgetForActionData* const InCreateData, TAttribute<bool> bIsReadOnlyIn)
{
	FName const ActionTypeId = InCreateData->Action->GetTypeId();

	FOnVerifyTextChanged OnVerifyTextChanged;
	FOnTextCommitted     OnTextCommitted;
		
	// enums have different rules for renaming that exist outside the bounds of other items.
	if (ActionTypeId == FEdGraphSchemaAction_K2Enum::StaticGetTypeId())
	{
		OnVerifyTextChanged.BindStatic(&FBlueprintPaletteItemRenameUtils::VerifyNewEnumName, ActionPtr);
		OnTextCommitted.BindStatic(&FBlueprintPaletteItemRenameUtils::CommitNewEnumName, ActionPtr, BlueprintEditorPtr);
	}
	else if (ActionTypeId == FEdGraphSchemaAction_K2Struct::StaticGetTypeId())
	{
		OnVerifyTextChanged.BindStatic(&FBlueprintPaletteItemRenameUtils::VerifyNewStructName, ActionPtr );
		OnTextCommitted.BindStatic(&FBlueprintPaletteItemRenameUtils::CommitNewStructName, ActionPtr, BlueprintEditorPtr);
	}
	else if (ActionTypeId == FEdGraphSchemaAction_K2Event::StaticGetTypeId())
	{
		OnVerifyTextChanged.BindStatic(&FBlueprintPaletteItemRenameUtils::VerifyNewEventName, ActionPtr);
		OnTextCommitted.BindStatic(&FBlueprintPaletteItemRenameUtils::CommitNewEventName, ActionPtr);
	}
	else if (ActionTypeId == FEdGraphSchemaAction_K2TargetNode::StaticGetTypeId())
	{
		OnVerifyTextChanged.BindStatic(&FBlueprintPaletteItemRenameUtils::VerifyNewTargetNodeName, ActionPtr);
		OnTextCommitted.BindStatic(&FBlueprintPaletteItemRenameUtils::CommitNewTargetNodeName, ActionPtr);
	}
	else
	{
		// default to our own rename methods
		OnVerifyTextChanged.BindSP(this, &SBlueprintPaletteItem::OnNameTextVerifyChanged);
		OnTextCommitted.BindSP(this, &SBlueprintPaletteItem::OnNameTextCommitted);
	}

	// Copy the mouse delegate binding if we want it
	if( InCreateData->bHandleMouseButtonDown )
	{
		MouseButtonDownDelegate = InCreateData->MouseButtonDownDelegate;
	}

	TSharedPtr<SToolTip> ToolTipWidget = ConstructToolTipWidget();

	TSharedPtr<SOverlay> DisplayWidget;
	TSharedPtr<SInlineEditableTextBlock> EditableTextElement;
	const FInlineEditableTextBlockStyle* Style = ActionTypeId == FEdGraphSchemaAction_Dummy::StaticGetTypeId()
		? &FAppStyle::Get().GetWidgetStyle<FInlineEditableTextBlockStyle>("InlineEditableTextBlockHintStyle")
		: SInlineEditableTextBlock::FArguments()._Style;
	SAssignNew(DisplayWidget, SOverlay)
		+SOverlay::Slot()
		[
			SAssignNew(EditableTextElement, SInlineEditableTextBlock)
				.Style(Style)
				.Text(this, &SBlueprintPaletteItem::GetDisplayText)
				.HighlightText(InCreateData->HighlightText)
				.ToolTip(ToolTipWidget)
				.OnVerifyTextChanged(OnVerifyTextChanged)
				.OnTextCommitted(OnTextCommitted)
				.IsSelected(InCreateData->IsRowSelectedDelegate)
				.IsReadOnly(bIsReadOnlyIn)
		];
	InlineRenameWidget = EditableTextElement.ToSharedRef();

	InCreateData->OnRenameRequest->BindSP(InlineRenameWidget.Get(), &SInlineEditableTextBlock::EnterEditingMode);

	if (GetDefault<UBlueprintEditorSettings>()->bShowActionMenuItemSignatures && ActionPtr.IsValid())
	{
		check(InlineRenameWidget.IsValid());
		TSharedPtr<IToolTip> ExistingToolTip = InlineRenameWidget->GetToolTip();

		DisplayWidget->AddSlot(0)
			[
				SNew(SHorizontalBox)
				.Visibility(EVisibility::Visible)
				.ToolTip(ConstructToolTipWithActionPath(ActionPtr.Pin(), ExistingToolTip))
			];
	}

	return DisplayWidget.ToSharedRef();
}

//------------------------------------------------------------------------------
FText SBlueprintPaletteItem::GetDisplayText() const
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	if (MenuDescriptionCache.IsOutOfDate(K2Schema))
	{
		TSharedPtr< FEdGraphSchemaAction > GraphAction = ActionPtr.Pin();
		if (GraphAction->GetTypeId() == FEdGraphSchemaAction_K2Enum::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2Enum* EnumAction = (FEdGraphSchemaAction_K2Enum*)GraphAction.Get();
			FText DisplayText = FText::FromString(EnumAction->Enum->GetName());
			MenuDescriptionCache.SetCachedText(DisplayText, K2Schema);
		}
		else if (GraphAction->GetTypeId() == FEdGraphSchemaAction_K2Struct::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2Struct* StructAction = (FEdGraphSchemaAction_K2Struct*)GraphAction.Get();
			FText DisplayText = StructAction->Struct ? FText::FromString(StructAction->Struct->GetName()) : FText::FromString(TEXT("None"));
			MenuDescriptionCache.SetCachedText(DisplayText, K2Schema);
		}
		else
		{
			MenuDescriptionCache.SetCachedText(ActionPtr.Pin()->GetMenuDescription(), K2Schema);
		}
	}

	return MenuDescriptionCache;
}

//------------------------------------------------------------------------------
bool SBlueprintPaletteItem::OnNameTextVerifyChanged(const FText& InNewText, FText& OutErrorMessage)
{
	FString TextAsString = InNewText.ToString();

	FName OriginalName;

	UStruct* ValidationScope = nullptr;

	const UEdGraphSchema* Schema = nullptr;

	// Check if certain action names are unchanged.
	if (ActionPtr.Pin()->GetTypeId() == FEdGraphSchemaAction_K2Var::StaticGetTypeId())
	{
		FEdGraphSchemaAction_K2Var* VarAction = (FEdGraphSchemaAction_K2Var*)ActionPtr.Pin().Get();
		OriginalName = (VarAction->GetVariableName());

		UClass* VarClass = VarAction->GetVariableClass();
		if (VarClass)
		{
			UBlueprint* BlueprintObj = UBlueprint::GetBlueprintFromClass(VarClass);
			TArray<UEdGraph*> Graphs;
			BlueprintObj->GetAllGraphs(Graphs);
			if (Graphs.Num() > 0)
			{
				Schema = Graphs[0]->GetSchema();
			}
		}
	}
	else if (ActionPtr.Pin()->GetTypeId() == FEdGraphSchemaAction_K2LocalVar::StaticGetTypeId())
	{
		FEdGraphSchemaAction_K2LocalVar* LocalVarAction = (FEdGraphSchemaAction_K2LocalVar*)ActionPtr.Pin().Get();
		OriginalName = (LocalVarAction->GetVariableName());
		
		ValidationScope = CastChecked<UStruct>(LocalVarAction->GetVariableScope());

		UClass* VarClass = LocalVarAction->GetVariableClass();
		if (VarClass)
		{
			UBlueprint* BlueprintObj = UBlueprint::GetBlueprintFromClass(VarClass);
			TArray<UEdGraph*> Graphs;
			BlueprintObj->GetAllGraphs(Graphs);
			if (Graphs.Num() > 0)
			{
				Schema = Graphs[0]->GetSchema();
			}
		}
	}
	else
	{
		UEdGraph* Graph = nullptr;

		if(ActionPtr.Pin()->GetTypeId() == FEdGraphSchemaAction_K2Graph::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2Graph* GraphAction = (FEdGraphSchemaAction_K2Graph*)ActionPtr.Pin().Get();
			Graph = GraphAction->EdGraph;
		}
		else if(ActionPtr.Pin()->GetTypeId() == FEdGraphSchemaAction_K2Delegate::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2Delegate* DelegateAction = (FEdGraphSchemaAction_K2Delegate*)ActionPtr.Pin().Get();
			Graph = DelegateAction->EdGraph;
		}

		if (Graph)
		{
			OriginalName = Graph->GetFName();
			Schema = Graph->GetSchema();
		}
	}

	UBlueprint* BlueprintObj = BlueprintEditorPtr.Pin()->GetBlueprintObj();
	check(BlueprintObj);

	if (BlueprintObj->SimpleConstructionScript)
	{
		for (USCS_Node* Node : BlueprintObj->SimpleConstructionScript->GetAllNodes())
		{
			if (Node && Node->GetVariableName() == OriginalName && !FComponentEditorUtils::IsValidVariableNameString(Node->ComponentTemplate, InNewText.ToString()))
			{
				OutErrorMessage = LOCTEXT("RenameFailed_NotValid", "This name is reserved for engine use.");
				return false;
			}
		}
	}

	if (OriginalName.IsNone() && ActionPtr.Pin()->IsA(FEdGraphSchemaAction_BlueprintVariableBase::StaticGetTypeId()))
	{
		FEdGraphSchemaAction_BlueprintVariableBase* BPVar = (FEdGraphSchemaAction_BlueprintVariableBase*)ActionPtr.Pin().Get();
		return BPVar->IsValidName(FName(TextAsString), OutErrorMessage);
	}
	else
	{
		TSharedPtr<INameValidatorInterface> NameValidator = nullptr;
		if (Schema)
		{
			NameValidator = Schema->GetNameValidator(BlueprintObj, OriginalName, ValidationScope, ActionPtr.Pin()->GetTypeId());	
		}
		
		if (NameValidator.IsValid())
		{
			EValidatorResult ValidatorResult = NameValidator->IsValid(TextAsString);
			switch (ValidatorResult)
			{
			case EValidatorResult::Ok:
			case EValidatorResult::ExistingName:
				// These are fine, don't need to surface to the user, the rename can 'proceed' even if the name is the existing one
				break;
			default:
				OutErrorMessage = INameValidatorInterface::GetErrorText(TextAsString, ValidatorResult);
				break;
			}
		}
	}
	
	return OutErrorMessage.IsEmpty();
}

//------------------------------------------------------------------------------
void SBlueprintPaletteItem::OnNameTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit)
{
	const FString NewNameString = NewText.ToString();
	const FName NewName = *NewNameString;

	if(ActionPtr.Pin()->GetTypeId() == FEdGraphSchemaAction_K2Graph::StaticGetTypeId())
	{
		FEdGraphSchemaAction_K2Graph* GraphAction = (FEdGraphSchemaAction_K2Graph*)ActionPtr.Pin().Get();

		UEdGraph* Graph = GraphAction->EdGraph;
		if (Graph && (Graph->bAllowDeletion || Graph->bAllowRenaming))
		{
			if (GraphAction->EdGraph)
			{
				if (const UEdGraphSchema* GraphSchema = GraphAction->EdGraph->GetSchema())
				{
					FGraphDisplayInfo DisplayInfo;
					GraphSchema->GetGraphDisplayInformation(*GraphAction->EdGraph, DisplayInfo);

					// Check if the name is unchanged
					if (NewText.EqualTo(DisplayInfo.PlainName))
					{
						return;
					}

					if (GraphSchema->TryRenameGraph(Graph, *NewText.ToString()))
					{
						return;
					}
				}
			}

			FBPVariableDescription* RepNotifyVar = FBlueprintEditorUtils::GetVariableFromOnRepFunction(Blueprint, GraphAction->FuncName);
			if (RepNotifyVar)
			{
				FSuppressableWarningDialog::FSetupInfo Info(LOCTEXT("RenameRepNotifyMessage", "This function is linked to a RepNotify variable. Renaming it will still allow the variable to be replicated, but this function will not be called. Do you wish to proceed?"),
					LOCTEXT("RenameRepNotifyTitle", "Rename RepNotify Function?"), TEXT("RenameRepNotifyWarning"));
				Info.ConfirmText = LOCTEXT("Confirm", "Confirm");
				Info.CancelText = LOCTEXT("Cancel", "Cancel");

				if (FSuppressableWarningDialog(Info).ShowModal() == FSuppressableWarningDialog::Cancel)
				{
					return;
				}
			}

			// Make sure we aren't renaming the graph into something that already exists
			UEdGraph* ExistingGraph = FindObject<UEdGraph>(Graph->GetOuter(), *NewNameString );
			if (ExistingGraph == nullptr || ExistingGraph == Graph)
			{
				const FScopedTransaction Transaction( LOCTEXT( "Rename Function", "Rename Function" ) );

				if (RepNotifyVar)
				{
					if (const UEdGraphSchema_K2* K2Schema = Cast<UEdGraphSchema_K2>(Graph->GetSchema()))
					{
						Graph->Modify();
						// make function editable
						// reconfigure the graph as if it were user created
						int32 ExtraFunctionFlags = (FUNC_BlueprintCallable | FUNC_BlueprintEvent | FUNC_Public);
						if (BPTYPE_FunctionLibrary == Blueprint->BlueprintType)
						{
							ExtraFunctionFlags |= FUNC_Static;
						}
						if (BPTYPE_Const == Blueprint->BlueprintType)
						{
							ExtraFunctionFlags |= FUNC_Const;
						}
						K2Schema->MarkFunctionEntryAsEditable(Graph, true);
						K2Schema->AddExtraFunctionFlags(Graph, ExtraFunctionFlags);

						Blueprint->Modify();
						RepNotifyVar->RepNotifyFunc = NAME_None;
					}
				}

				FBlueprintEditorUtils::RenameGraph(Graph, NewNameString );
			}
		}
	}
	else if(ActionPtr.Pin()->GetTypeId() == FEdGraphSchemaAction_K2Delegate::StaticGetTypeId())
	{
		FEdGraphSchemaAction_K2Delegate* DelegateAction = (FEdGraphSchemaAction_K2Delegate*)ActionPtr.Pin().Get();

		UEdGraph* Graph = DelegateAction->EdGraph;
		if (Graph && (Graph->bAllowDeletion || Graph->bAllowRenaming))
		{
			if (const UEdGraphSchema* GraphSchema = Graph->GetSchema())
			{
				FGraphDisplayInfo DisplayInfo;
				GraphSchema->GetGraphDisplayInformation(*Graph, DisplayInfo);

				// Check if the name is unchanged
				if (NewText.EqualTo(DisplayInfo.PlainName))
				{
					return;
				}
			}

			// Make sure we aren't renaming the graph into something that already exists
			UEdGraph* ExistingGraph = FindObject<UEdGraph>(Graph->GetOuter(), *NewNameString );
			if (ExistingGraph == nullptr || ExistingGraph == Graph)
			{
				const FScopedTransaction Transaction( LOCTEXT( "Rename Delegate", "Rename Event Dispatcher" ) );
				const FName OldName =  Graph->GetFName();

				UBlueprint* BlueprintObj = BlueprintEditorPtr.Pin()->GetBlueprintObj();
				FBlueprintEditorUtils::RenameMemberVariable(BlueprintObj, OldName, NewName);
			}
		}
	}
	else if (ActionPtr.Pin()->GetTypeId() == FEdGraphSchemaAction_K2Var::StaticGetTypeId())
	{
		FEdGraphSchemaAction_K2Var* VarAction = (FEdGraphSchemaAction_K2Var*)ActionPtr.Pin().Get();

		// Check if the name is unchanged
		if (NewName.IsEqual(VarAction->GetVariableName(), ENameCase::CaseSensitive))
		{
			return;
		}

		const FScopedTransaction Transaction( LOCTEXT( "RenameVariable", "Rename Variable" ) );

		BlueprintEditorPtr.Pin()->GetBlueprintObj()->Modify();

		// Double check we're not renaming a timeline disguised as a variable
		bool bIsTimeline = false;
		if (FProperty* VariableProperty = VarAction->GetProperty())
		{
			// Don't allow removal of timeline properties - you need to remove the timeline node for that
			FObjectProperty* ObjProperty = CastField<FObjectProperty>(VariableProperty);
			if (ObjProperty && ObjProperty->PropertyClass == UTimelineComponent::StaticClass())
			{
				bIsTimeline = true;
			}
		}

		// Rename as a timeline if required
		if (bIsTimeline)
		{
			FBlueprintEditorUtils::RenameTimeline(BlueprintEditorPtr.Pin()->GetBlueprintObj(), VarAction->GetVariableName(), NewName);
		}
		else
		{
			FBlueprintEditorUtils::RenameMemberVariable(BlueprintEditorPtr.Pin()->GetBlueprintObj(), VarAction->GetVariableName(), NewName);
		}
	}
	else if (ActionPtr.Pin()->GetTypeId() == FEdGraphSchemaAction_K2LocalVar::StaticGetTypeId())
	{
		FEdGraphSchemaAction_K2LocalVar* LocalVarAction = (FEdGraphSchemaAction_K2LocalVar*)ActionPtr.Pin().Get();

		// Check if the name is unchanged
		if (NewName.IsEqual(LocalVarAction->GetVariableName(), ENameCase::CaseSensitive))
		{
			return;
		}

		const FScopedTransaction Transaction( LOCTEXT( "RenameVariable", "Rename Variable" ) );

		BlueprintEditorPtr.Pin()->GetBlueprintObj()->Modify();

		FBlueprintEditorUtils::RenameLocalVariable(BlueprintEditorPtr.Pin()->GetBlueprintObj(), CastChecked<UStruct>(LocalVarAction->GetVariableScope()), LocalVarAction->GetVariableName(), NewName);
	}
	else if (ActionPtr.Pin()->IsA(FEdGraphSchemaAction_BlueprintVariableBase::StaticGetTypeId()))
	{
		FEdGraphSchemaAction_BlueprintVariableBase* BPVarAction = (FEdGraphSchemaAction_BlueprintVariableBase*)ActionPtr.Pin().Get();

		// Check if the name is unchanged
		if (NewName.IsEqual(BPVarAction->GetVariableName(), ENameCase::CaseSensitive))
		{
			return;
		}

		const FScopedTransaction Transaction( LOCTEXT( "RenameVariable", "Rename Variable" ) );
		BlueprintEditorPtr.Pin()->GetBlueprintObj()->Modify();

		BPVarAction->RenameVariable(NewName);
		BlueprintEditorPtr.Pin()->GetMyBlueprintWidget()->Refresh();
	}
	BlueprintEditorPtr.Pin()->GetMyBlueprintWidget()->SelectItemByName(NewName, ESelectInfo::OnMouseClick);
}

//------------------------------------------------------------------------------
FText SBlueprintPaletteItem::GetToolTipText() const
{
	TSharedPtr<FEdGraphSchemaAction> PaletteAction = ActionPtr.Pin();

	FText ToolTipText;
	FText ClassDisplayName;

	if (PaletteAction.IsValid())
	{
		// Default tooltip is taken from the action
		ToolTipText = PaletteAction->GetTooltipDescription().IsEmpty() ? PaletteAction->GetMenuDescription() : PaletteAction->GetTooltipDescription();

		if(PaletteAction->GetTypeId() == FEdGraphSchemaAction_K2AddComponent::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2AddComponent* AddCompAction = (FEdGraphSchemaAction_K2AddComponent*)PaletteAction.Get();
			// Show component-specific tooltip
			UClass* ComponentClass = *(AddCompAction->ComponentClass);
			if (ComponentClass)
			{
				ToolTipText = ComponentClass->GetToolTipText();
			}
		}
		else if (UK2Node const* const NodeTemplate = FBlueprintActionMenuUtils::ExtractNodeTemplateFromAction(*PaletteAction))
		{
			// If the node wants to create tooltip text, use that instead, because its probably more detailed
			FText NodeToolTipText = NodeTemplate->GetTooltipText();
			if (!NodeToolTipText.IsEmpty())
			{
				ToolTipText = NodeToolTipText;
			}

			if (UK2Node_CallFunction const* CallFuncNode = Cast<UK2Node_CallFunction const>(NodeTemplate))
			{			
				if(UClass* ParentClass = CallFuncNode->FunctionReference.GetMemberParentClass(CallFuncNode->GetBlueprintClassFromNode()))
				{
					UBlueprint* BlueprintObj = UBlueprint::GetBlueprintFromClass(ParentClass);
					if (BlueprintObj == nullptr)
					{
						ClassDisplayName = ParentClass->GetDisplayNameText();
					}
					else if (!BlueprintObj->HasAnyFlags(RF_Transient))
					{
						ClassDisplayName = FText::FromName(BlueprintObj->GetFName());
					}					
				}
			}
		}
		else if (PaletteAction->GetTypeId() == FEdGraphSchemaAction_K2Graph::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2Graph* GraphAction = (FEdGraphSchemaAction_K2Graph*)PaletteAction.Get();
			if (GraphAction->EdGraph)
			{
				if (const UEdGraphSchema* GraphSchema = GraphAction->EdGraph->GetSchema())
				{
					FGraphDisplayInfo DisplayInfo;
					GraphSchema->GetGraphDisplayInformation(*(GraphAction->EdGraph), DisplayInfo);
					ToolTipText = DisplayInfo.Tooltip;
				}
			}
		}
		else if (PaletteAction->GetTypeId() == FEdGraphSchemaAction_K2Var::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2Var* VarAction = (FEdGraphSchemaAction_K2Var*)PaletteAction.Get();
			UClass* VarClass = VarAction->GetVariableClass();
			if (bShowClassInTooltip && VarClass)
			{
				UBlueprint* BlueprintObj = UBlueprint::GetBlueprintFromClass(VarClass);
				ClassDisplayName = (BlueprintObj ? FText::FromName(BlueprintObj->GetFName()) : VarClass->GetDisplayNameText());
			}
			else
			{
				// Use the native display name metadata if we can
				const FProperty* Property = FindFProperty<FProperty>(VarClass, VarAction->GetVariableName());
				if (Property && Property->IsNative())
				{
					ToolTipText = Property->GetDisplayNameText();
				}
				else
				{	
					FString Result = GetVarTooltip(Blueprint, VarClass, VarAction->GetVariableName());
					// Only use the variable tooltip if it has been filled out.
					ToolTipText = FText::FromString( !Result.IsEmpty() ? Result : GetVarType(VarClass, VarAction->GetVariableName(), true, true) );	
				}
			}
		}
		else if (PaletteAction->GetTypeId() == FEdGraphSchemaAction_K2LocalVar::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2LocalVar* LocalVarAction = (FEdGraphSchemaAction_K2LocalVar*)PaletteAction.Get();
			// The variable scope can not be found in intermediate graphs
			UStruct* LocalVarScope = Cast<UStruct>(LocalVarAction->GetVariableScope());
			if(LocalVarScope)
			{
				UClass* VarClass = CastChecked<UClass>(LocalVarAction->GetVariableScope()->GetOuter());
				if (bShowClassInTooltip && (VarClass != nullptr))
				{
					UBlueprint* BlueprintObj = UBlueprint::GetBlueprintFromClass(VarClass);
					ClassDisplayName = (BlueprintObj ? FText::FromName(BlueprintObj->GetFName()) : VarClass->GetDisplayNameText());
				}
				else
				{
					FString Result;
					FBlueprintEditorUtils::GetBlueprintVariableMetaData(Blueprint, LocalVarAction->GetVariableName(), LocalVarScope, TEXT("tooltip"), Result);
					// Only use the variable tooltip if it has been filled out.
					ToolTipText = FText::FromString( !Result.IsEmpty() ? Result : GetVarType(LocalVarScope, LocalVarAction->GetVariableName(), true, true) );
				}
			}
		}
		else if (PaletteAction->GetTypeId() == FEdGraphSchemaAction_K2Delegate::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2Delegate* DelegateAction = (FEdGraphSchemaAction_K2Delegate*)PaletteAction.Get();
			
			FString Result = GetVarTooltip(Blueprint, DelegateAction->GetDelegateClass(), DelegateAction->GetDelegateName());
			ToolTipText = !Result.IsEmpty() ? FText::FromString(Result) : FText::FromName(DelegateAction->GetDelegateName());
		}
		else if (PaletteAction->GetTypeId() == FEdGraphSchemaAction_K2Enum::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2Enum* EnumAction = (FEdGraphSchemaAction_K2Enum*)PaletteAction.Get();
			if (EnumAction->Enum)
			{
				ToolTipText = FText::FromName(EnumAction->Enum->GetFName());
			}
		}
		else if (PaletteAction->GetTypeId() == FEdGraphSchemaAction_K2TargetNode::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2TargetNode* TargetNodeAction = (FEdGraphSchemaAction_K2TargetNode*)PaletteAction.Get();
			if (TargetNodeAction->NodeTemplate)
			{
				ToolTipText = TargetNodeAction->NodeTemplate->GetTooltipText();
			}
		}
	}

	if (bShowClassInTooltip && !ClassDisplayName.IsEmpty())
	{
		ToolTipText = FText::Format(LOCTEXT("BlueprintItemClassTooltip", "{0}\nClass: {1}"), ToolTipText, ClassDisplayName);
	}

	return ToolTipText;
}

TSharedPtr<SToolTip> SBlueprintPaletteItem::ConstructToolTipWidget() const
{
	TSharedPtr<FEdGraphSchemaAction> PaletteAction = ActionPtr.Pin();
	UEdGraphNode const* const NodeTemplate = FBlueprintActionMenuUtils::ExtractNodeTemplateFromAction(PaletteAction);

	FBlueprintActionMenuItem::FDocExcerptRef DocExcerptRef;

	if (PaletteAction.IsValid())
	{
		if (NodeTemplate != nullptr)
		{
			// Take rich tooltip from node
			DocExcerptRef.DocLink = NodeTemplate->GetDocumentationLink();
			DocExcerptRef.DocExcerptName = NodeTemplate->GetDocumentationExcerptName();

			// sometimes, with FBlueprintActionMenuItem's, the NodeTemplate 
			// doesn't always reflect the node that will be spawned (some things 
			// we don't want to be executed until spawn time, like adding of 
			// component templates)... in that case, the 
			// FBlueprintActionMenuItem's may have a more specific documentation 
			// link of its own (most of the time, it will reflect the NodeTemplate's)
			if ( !DocExcerptRef.IsValid() && (PaletteAction->GetTypeId() == FBlueprintActionMenuItem::StaticGetTypeId()) )
			{
				FBlueprintActionMenuItem* NodeSpawnerAction = (FBlueprintActionMenuItem*)PaletteAction.Get();
				DocExcerptRef = NodeSpawnerAction->GetDocumentationExcerpt();
			}
		}
		else if (PaletteAction->GetTypeId() == FEdGraphSchemaAction_K2Graph::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2Graph* GraphAction = (FEdGraphSchemaAction_K2Graph*)PaletteAction.Get();
			if (GraphAction->EdGraph)
			{
				FGraphDisplayInfo DisplayInfo;
				if (const UEdGraphSchema* GraphSchema = GraphAction->EdGraph->GetSchema())
				{
					GraphSchema->GetGraphDisplayInformation(*(GraphAction->EdGraph), DisplayInfo);
				}

				DocExcerptRef.DocLink = DisplayInfo.DocLink;
				DocExcerptRef.DocExcerptName = DisplayInfo.DocExcerptName;
			}
		}
		else if (PaletteAction->GetTypeId() == FEdGraphSchemaAction_K2Var::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2Var* VarAction = (FEdGraphSchemaAction_K2Var*)PaletteAction.Get();
			UClass* VarClass = VarAction->GetVariableClass();
			if (!bShowClassInTooltip || VarClass == nullptr)
			{
				// Don't show big tooltip if we are showing class as well (means we are not in MyBlueprint)
				DocExcerptRef.DocLink = TEXT("Shared/Editors/BlueprintEditor/GraphTypes");
				DocExcerptRef.DocExcerptName = TEXT("Variable");
			}
		}
		else if (PaletteAction->GetTypeId() == FEdGraphSchemaAction_K2Event::StaticGetTypeId())
		{
			DocExcerptRef.DocLink = TEXT("Shared/Editors/BlueprintEditor/GraphTypes");
			DocExcerptRef.DocExcerptName = TEXT("Event");
		}
		else if (PaletteAction->GetTypeId() == FEdGraphSchemaAction_K2AddComment::StaticGetTypeId() ||
			PaletteAction->GetTypeId() == FEdGraphSchemaAction_NewStateComment::StaticGetTypeId())
		{
			// Taking tooltip from action is fine
			const UEdGraphNode_Comment* DefaultComment = GetDefault<UEdGraphNode_Comment>();
			DocExcerptRef.DocLink = DefaultComment->GetDocumentationLink();
			DocExcerptRef.DocExcerptName = DefaultComment->GetDocumentationExcerptName();
		}
		else if (PaletteAction->GetTypeId() == FEdGraphSchemaAction_K2LocalVar::StaticGetTypeId())
		{
			// Don't show big tooltip if we are showing class as well (means we are not in MyBlueprint)
			DocExcerptRef.DocLink = TEXT("Shared/Editors/BlueprintEditor/GraphTypes");
			DocExcerptRef.DocExcerptName = TEXT("LocalVariable");
		}
	}

	// Setup the attribute for dynamically pulling the tooltip
	TAttribute<FText> TextAttribute;
	TextAttribute.Bind(this, &SBlueprintPaletteItem::GetToolTipText);

	TSharedRef< SToolTip > TooltipWidget = IDocumentation::Get()->CreateToolTip(TextAttribute, nullptr, DocExcerptRef.DocLink, DocExcerptRef.DocExcerptName);

	// English speakers have no real need to know this exists.
	if ( (NodeTemplate != nullptr) && (FInternationalization::Get().GetCurrentCulture()->GetTwoLetterISOLanguageName() != TEXT("en")) )
	{
		FText NativeNodeName = FText::FromString(NodeTemplate->GetNodeTitle(ENodeTitleType::ListView).BuildSourceString());
		const FTextBlockStyle& SubduedTextStyle = FAppStyle::GetWidgetStyle<FTextBlockStyle>("Documentation.SDocumentationTooltipSubdued");

		TSharedPtr<SToolTip> InternationalTooltip;
		TSharedPtr<SVerticalBox> TooltipBody;

		SAssignNew(InternationalTooltip, SToolTip)
			// Emulate text-only tool-tip styling that SToolTip uses 
			// when no custom content is supplied.  We want node tool-
			// tips to be styled just like text-only tool-tips
			.BorderImage( FCoreStyle::Get().GetBrush("ToolTip.BrightBackground") )
			.TextMargin(FMargin(11.0f))
		[
			SAssignNew(TooltipBody, SVerticalBox)
		];

		if (!DocExcerptRef.IsValid())
		{
			auto GetNativeNamePromptVisibility = []()->EVisibility
			{
				FModifierKeysState KeyState = FSlateApplication::Get().GetModifierKeys();
				return KeyState.IsAltDown() ? EVisibility::Collapsed : EVisibility::Visible;
			};

			TooltipBody->AddSlot()
			[
				SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "Documentation.SDocumentationTooltip")
					.Text(NativeNodeName)
					.Visibility_Lambda([GetNativeNamePromptVisibility]()->EVisibility
					{
						return (GetNativeNamePromptVisibility() == EVisibility::Visible) ? EVisibility::Collapsed : EVisibility::Visible;
					})
			];

			TooltipBody->AddSlot()
			[
				SNew(SHorizontalBox)
					.Visibility_Lambda(GetNativeNamePromptVisibility)
				+SHorizontalBox::Slot()
				[
					TooltipWidget->GetContentWidget()
				]
			];

			TooltipBody->AddSlot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				.Padding(0.f, 8.f, 0.f, 0.f)
			[

				SNew(STextBlock)
					.Text( LOCTEXT("NativeNodeName", "hold (Alt) for native node name") )
					.TextStyle(&SubduedTextStyle)
					.Visibility_Lambda(GetNativeNamePromptVisibility)
			];
		}
		else
		{
			auto GetNativeNodeNameVisibility = []()->EVisibility
			{
				FModifierKeysState KeyState = FSlateApplication::Get().GetModifierKeys();
				return KeyState.IsAltDown() && KeyState.IsControlDown() ? EVisibility::Visible : EVisibility::Collapsed;
			};

			// give the "advanced" tooltip a header
			TooltipBody->AddSlot()
				.AutoHeight()
				.HAlign(HAlign_Right)
				.Padding(0.f, 0.f, 0.f, 8.f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
					.AutoWidth()
				[
					SNew(STextBlock)
					.TextStyle(&SubduedTextStyle)
						.Text(LOCTEXT("NativeNodeNameLabel", "Native Node Name: "))
						.Visibility_Lambda(GetNativeNodeNameVisibility)
				]
				+SHorizontalBox::Slot()
					.AutoWidth()
				[
					SNew(STextBlock)
						.TextStyle(&SubduedTextStyle)
						.Text(NativeNodeName)
						.Visibility_Lambda(GetNativeNodeNameVisibility)
				]
			];

			TooltipBody->AddSlot()
			[
				TooltipWidget->GetContentWidget()
			];
		}

		return InternationalTooltip;
	}
	return TooltipWidget;
}

/*******************************************************************************
* SBlueprintPalette
*******************************************************************************/

//------------------------------------------------------------------------------
void SBlueprintPalette::Construct(const FArguments& InArgs, TWeakPtr<FBlueprintEditor> InBlueprintEditor)
{
	const float NumProgressFrames = 2.0f;
	const float SecondsToWaitBeforeShowingProgressDialog = 0.25f;

	FScopedSlowTask SlowTask(NumProgressFrames, LOCTEXT("ConstructingPaletteTabContent", "Initializing Palette..."));
	SlowTask.MakeDialogDelayed(SecondsToWaitBeforeShowingProgressDialog);

	float FavoritesHeightRatio = 0.33f;
	GConfig->GetFloat(*BlueprintPalette::ConfigSection, *BlueprintPalette::FavoritesHeightConfigKey, FavoritesHeightRatio, GEditorPerProjectIni);
	float LibraryHeightRatio = 1.f - FavoritesHeightRatio;
	GConfig->GetFloat(*BlueprintPalette::ConfigSection, *BlueprintPalette::LibraryHeightConfigKey, LibraryHeightRatio, GEditorPerProjectIni);

	bool bUseLegacyLayout = false;
	GConfig->GetBool(*BlueprintPalette::ConfigSection, TEXT("bUseLegacyLayout"), bUseLegacyLayout, GEditorIni);

	SlowTask.EnterProgressFrame();
	TSharedRef<SWidget> FavoritesContent = SNew(SBlueprintFavoritesPalette, InBlueprintEditor);

	SlowTask.EnterProgressFrame();
	TSharedRef<SWidget> LibraryContent = SNew(SBlueprintLibraryPalette, InBlueprintEditor)
		.UseLegacyLayout(bUseLegacyLayout);

	if (bUseLegacyLayout)
	{
		LibraryWrapper = LibraryContent;

		this->ChildSlot
		[
			LibraryContent
		];
	}
	else 
	{
		LibraryContent->AddMetadata<FTagMetaData>(MakeShared<FTagMetaData>(TEXT("BlueprintPaletteLibrary")));
		FavoritesContent->AddMetadata<FTagMetaData>(MakeShared<FTagMetaData>(TEXT("BlueprintPaletteFavorites")));

		this->ChildSlot
		[
			SAssignNew(PaletteSplitter, SSplitter)
				.Orientation(Orient_Vertical)
				.OnSplitterFinishedResizing(this, &SBlueprintPalette::OnSplitterResized)
				.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("FullBlueprintPalette")))

			+ SSplitter::Slot()
			.Value(FavoritesHeightRatio)
			[
				FavoritesContent
			]

			+ SSplitter::Slot()
			.Value(LibraryHeightRatio)
			[
				LibraryContent
			]
		];
	}	
}

//------------------------------------------------------------------------------
void SBlueprintPalette::OnSplitterResized() const
{
	FChildren const* const SplitterChildren = PaletteSplitter->GetChildren();
	for (int32 SlotIndex = 0; SlotIndex < SplitterChildren->Num(); ++SlotIndex)
	{
		SSplitter::FSlot const& SplitterSlot = PaletteSplitter->SlotAt(SlotIndex);

		if (SplitterSlot.GetWidget() == FavoritesWrapper)
		{
			GConfig->SetFloat(*BlueprintPalette::ConfigSection, *BlueprintPalette::FavoritesHeightConfigKey, SplitterSlot.GetSizeValue(), GEditorPerProjectIni);
		}
		else if (SplitterSlot.GetWidget() == LibraryWrapper)
		{
			GConfig->SetFloat(*BlueprintPalette::ConfigSection, *BlueprintPalette::LibraryHeightConfigKey, SplitterSlot.GetSizeValue(), GEditorPerProjectIni);
		}

	}
}

#undef LOCTEXT_NAMESPACE
