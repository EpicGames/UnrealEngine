// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "EdGraph/EdGraphNodeUtils.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "SGraphPalette.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FBlueprintEditor;
class FDragDropEvent;
class SSplitter;
class SToolTip;
class SWidget;
class UBlueprint;
struct FCreateWidgetForActionData;
struct FGeometry;

/*******************************************************************************
* SBlueprintPaletteItem
*******************************************************************************/

/** Widget for displaying a single item  */
class SBlueprintPaletteItem : public SGraphPaletteItem
{
public:
	SLATE_BEGIN_ARGS( SBlueprintPaletteItem )
			: _ShowClassInTooltip(false)
		{}

		SLATE_ARGUMENT(bool, ShowClassInTooltip)
	SLATE_END_ARGS()

	/**
	 * Creates the slate widget to be place in a palette.
	 * 
	 * @param  InArgs				A set of slate arguments, defined above.
	 * @param  InCreateData			A set of data associated with a FEdGraphSchemaAction that this item represents.
	 * @param  InBlueprintEditor	A pointer to the blueprint editor that the palette belongs to.
	 */
	void Construct(const FArguments& InArgs, FCreateWidgetForActionData* const InCreateData, TWeakPtr<FBlueprintEditor> InBlueprintEditor);
	void Construct(const FArguments& InArgs, FCreateWidgetForActionData* const InCreateData, UBlueprint* InBlueprint);

private:
	void Construct(const FArguments& InArgs, FCreateWidgetForActionData* const InCreateData, UBlueprint* InBlueprint, TWeakPtr<FBlueprintEditor> InBlueprintEditor);

	// SWidget Interface
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	// End of SWidget Interface

	// SGraphPaletteItem Interface
	virtual TSharedRef<SWidget> CreateTextSlotWidget(FCreateWidgetForActionData* const InCreateData, TAttribute<bool> bIsReadOnly ) override;
	virtual FText GetDisplayText() const override;
	virtual bool OnNameTextVerifyChanged(const FText& InNewText, FText& OutErrorMessage) override;
	virtual void OnNameTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit) override;
	// End of SGraphPaletteItem Interface

	/**
	 * Creates a tooltip widget based off the specified action (attempts to 
	 * mirror the tool-tip that would be found on the node once it's placed).
	 * 
	 * @return A new slate widget to be used as the tool tip for this item's text element.
	 */
	TSharedPtr<SToolTip> ConstructToolTipWidget() const;

	/** Returns the up-to-date tooltip for the item */
	FText GetToolTipText() const;
private:
	/** True if the class should be displayed in the tooltip */
	bool bShowClassInTooltip;

	UBlueprint* Blueprint;

	/** Pointer back to the blueprint editor that owns this */
	TWeakPtr<FBlueprintEditor> BlueprintEditorPtr;

	/** Cache for the MenuDescription to be displayed for this item */
	FNodeTextCache MenuDescriptionCache;
};


/*******************************************************************************
* SBlueprintPalette
*******************************************************************************/

class SBlueprintPalette : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SBlueprintPalette ) {};
	SLATE_END_ARGS()

	/**
	 * Creates the slate widget that represents a list of available actions for 
	 * the specified blueprint.
	 * 
	 * @param  InArgs				A set of slate arguments, defined above.
	 * @param  InBlueprintEditor	A pointer to the blueprint editor that this palette belongs to.
	 */
	void Construct(const FArguments& InArgs, TWeakPtr<FBlueprintEditor> InBlueprintEditor);

private:
	/**
	 * Saves off the user's new sub-palette configuration (so as to not annoy 
	 * them by reseting it every time they open the blueprint editor). 
	 */
	void OnSplitterResized() const;

	TSharedPtr<SWidget> FavoritesWrapper;
	TSharedPtr<SSplitter> PaletteSplitter;
	TSharedPtr<SWidget> LibraryWrapper;
};

