// Copyright Epic Games, Inc. All Rights Reserved.


#include "SBlueprintRevisionMenu.h"

#include "Engine/Blueprint.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformMath.h"
#include "IAssetTypeActions.h"
#include "ISourceControlModule.h"
#include "ISourceControlRevision.h"
#include "ISourceControlState.h"
#include "Internationalization/Text.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Layout/Children.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "SlotBase.h"
#include "SourceControlHelpers.h"
#include "SourceControlOperations.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateEnums.h"
#include "UObject/Object.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SBlueprintRevisionMenu"

/**  */
namespace ESourceControlQueryState
{
	enum Type
	{
		NotQueried,
		QueryInProgress,
		Queried,
	};
}

//------------------------------------------------------------------------------
SBlueprintRevisionMenu::~SBlueprintRevisionMenu()
{
	// cancel any operation if this widget is destroyed while in progress
	if (SourceControlQueryState == ESourceControlQueryState::QueryInProgress)
	{
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
		if (SourceControlQueryOp.IsValid() && SourceControlProvider.CanCancelOperation(SourceControlQueryOp.ToSharedRef()))
		{
			SourceControlProvider.CancelOperation(SourceControlQueryOp.ToSharedRef());
		}
	}
}

//------------------------------------------------------------------------------
void SBlueprintRevisionMenu::Construct(const FArguments& InArgs, UBlueprint const* Blueprint)
{
	bIncludeLocalRevision = InArgs._bIncludeLocalRevision;
	OnRevisionSelected = InArgs._OnRevisionSelected;

	SourceControlQueryState = ESourceControlQueryState::NotQueried;

	ChildSlot	
	[
		SAssignNew(MenuBox, SVerticalBox)
		+SVerticalBox::Slot()
		[
			SNew(SBorder)
			.Visibility(this, &SBlueprintRevisionMenu::GetInProgressVisibility)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
			.Content()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SThrobber)
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(2.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DiffMenuOperationInProgress", "Updating history..."))
				]
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.Visibility(this, &SBlueprintRevisionMenu::GetCancelButtonVisibility)
					.OnClicked(this, &SBlueprintRevisionMenu::OnCancelButtonClicked)
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					.Content()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("DiffMenuCancelButton", "Cancel"))
					]
				]
			]
		]
	];

	if (Blueprint != nullptr)
	{
		bool const bIsLevelScriptBlueprint = FBlueprintEditorUtils::IsLevelScriptBlueprint(Blueprint);
		Filename = SourceControlHelpers::PackageFilename(bIsLevelScriptBlueprint ? Blueprint->GetOuter()->GetPathName() : Blueprint->GetPathName());

		// make sure the history info is up to date
		SourceControlQueryOp = ISourceControlOperation::Create<FUpdateStatus>();
		SourceControlQueryOp->SetUpdateHistory(true);
		ISourceControlModule::Get().GetProvider().Execute(SourceControlQueryOp.ToSharedRef(), Filename, EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateSP(this, &SBlueprintRevisionMenu::OnSourceControlQueryComplete));

		SourceControlQueryState = ESourceControlQueryState::QueryInProgress;
	}
}

//------------------------------------------------------------------------------
EVisibility SBlueprintRevisionMenu::GetInProgressVisibility() const
{
	return (SourceControlQueryState == ESourceControlQueryState::QueryInProgress) ? EVisibility::Visible : EVisibility::Collapsed;
}

//------------------------------------------------------------------------------
EVisibility SBlueprintRevisionMenu::GetCancelButtonVisibility() const
{
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	return SourceControlQueryOp.IsValid() && SourceControlProvider.CanCancelOperation(SourceControlQueryOp.ToSharedRef()) ? EVisibility::Visible : EVisibility::Collapsed;
}

//------------------------------------------------------------------------------
FReply SBlueprintRevisionMenu::OnCancelButtonClicked() const
{
	if (SourceControlQueryOp.IsValid())
	{
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
		SourceControlProvider.CancelOperation(SourceControlQueryOp.ToSharedRef());
	}

	return FReply::Handled();
}

//------------------------------------------------------------------------------
void SBlueprintRevisionMenu::OnSourceControlQueryComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	check(SourceControlQueryOp == InOperation);


	// Add pop-out menu for each revision
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection =*/true, /*InCommandList =*/NULL);

	MenuBuilder.BeginSection("AddDiffRevision", LOCTEXT("Revisions", "Revisions"));
	if (bIncludeLocalRevision)
	{
		FText const ToolTipText = LOCTEXT("LocalRevisionToolTip", "The current copy you have saved to disk (locally)");

		FOnRevisionSelected OnRevisionSelectedDelegate = OnRevisionSelected;
		auto OnMenuItemSelected = [OnRevisionSelectedDelegate]()
		{
			OnRevisionSelectedDelegate.ExecuteIfBound(FRevisionInfo::InvalidRevision());
		};

		MenuBuilder.AddMenuEntry(LOCTEXT("LocalRevision", "Local"), ToolTipText, FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda(OnMenuItemSelected)));
	}

	if (InResult == ECommandResult::Succeeded)
	{
		// get the cached state
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
		FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(Filename, EStateCacheUsage::Use);

		if (SourceControlState.IsValid() && SourceControlState->GetHistorySize() > 0)
		{
			// Figure out the highest revision # (so we can label it "Depot")
			int32 LatestRevision = 0;
			for (int32 HistoryIndex = 0; HistoryIndex < SourceControlState->GetHistorySize(); HistoryIndex++)
			{
				TSharedPtr<ISourceControlRevision, ESPMode::ThreadSafe> Revision = SourceControlState->GetHistoryItem(HistoryIndex);
				if (Revision.IsValid() && Revision->GetRevisionNumber() > LatestRevision)
				{
					LatestRevision = Revision->GetRevisionNumber();
				}
			}

			for (int32 HistoryIndex = 0; HistoryIndex < SourceControlState->GetHistorySize(); HistoryIndex++)
			{
				TSharedPtr<ISourceControlRevision, ESPMode::ThreadSafe> Revision = SourceControlState->GetHistoryItem(HistoryIndex);
				if (Revision.IsValid())
				{
					FInternationalization& I18N = FInternationalization::Get();

					FText Label = FText::Format(LOCTEXT("RevisionNumber", "Revision {0}"), FText::AsNumber(Revision->GetRevisionNumber(), NULL, I18N.GetInvariantCulture()));

					FFormatNamedArguments Args;
					Args.Add(TEXT("CheckInNumber"), FText::AsNumber(Revision->GetCheckInIdentifier(), NULL, I18N.GetInvariantCulture()));
					Args.Add(TEXT("Revision"), FText::FromString(Revision->GetRevision()));
					Args.Add(TEXT("UserName"), FText::FromString(Revision->GetUserName()));
					Args.Add(TEXT("DateTime"), FText::AsDate(Revision->GetDate()));
					Args.Add(TEXT("ChanglistDescription"), FText::FromString(Revision->GetDescription()));
					FText ToolTipText;
					if (ISourceControlModule::Get().GetProvider().UsesChangelists())
					{
						ToolTipText = FText::Format(LOCTEXT("ChangelistToolTip", "CL #{CheckInNumber} {UserName} \n{DateTime} \n{ChanglistDescription}"), Args);
					}
					else
					{
						ToolTipText = FText::Format(LOCTEXT("RevisionToolTip", "{Revision} {UserName} \n{DateTime} \n{ChanglistDescription}"), Args);
					}

					if (LatestRevision == Revision->GetRevisionNumber())
					{
						Label = LOCTEXT("Depo", "Depot");
					}

					FRevisionInfo RevisionInfo = { 
						Revision->GetRevision(), 
						Revision->GetCheckInIdentifier(), 
						Revision->GetDate() 
					};
					FOnRevisionSelected OnRevisionSelectedDelegate = OnRevisionSelected;
					auto OnMenuItemSelected = [RevisionInfo, OnRevisionSelectedDelegate]()
					{
						OnRevisionSelectedDelegate.ExecuteIfBound(RevisionInfo);
					};
					MenuBuilder.AddMenuEntry( TAttribute<FText>(Label), ToolTipText, FSlateIcon(),
						FUIAction(FExecuteAction::CreateLambda(OnMenuItemSelected)) );
				}
			}
		}
		else if (!bIncludeLocalRevision)
		{
			// Show 'empty' item in toolbar
			MenuBuilder.AddMenuEntry(LOCTEXT("NoRevisonHistory", "No revisions found"),
				FText(), FSlateIcon(), FUIAction());
		}
	}
	else if (!bIncludeLocalRevision)
	{
		// Show 'empty' item in toolbar
		MenuBuilder.AddMenuEntry(LOCTEXT("NoRevisonHistory", "No revisions found"),
			FText(), FSlateIcon(), FUIAction());
	}

	MenuBuilder.EndSection();
	MenuBox->AddSlot() 
	[
		MenuBuilder.MakeWidget(nullptr, 500)
	];

	SourceControlQueryOp.Reset();
	SourceControlQueryState = ESourceControlQueryState::Queried;
}

#undef LOCTEXT_NAMESPACE
