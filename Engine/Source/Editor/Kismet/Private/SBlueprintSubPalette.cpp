// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBlueprintSubPalette.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "BPDelegateDragDropAction.h"
#include "BPFunctionDragDropAction.h"
#include "BPVariableDragDropAction.h"
#include "BlueprintActionDatabase.h"
#include "BlueprintActionMenuUtils.h"
#include "BlueprintDragDropMenuItem.h"
#include "BlueprintEditor.h"
#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphSchema_K2_Actions.h"
#include "Engine/Blueprint.h"
#include "Engine/LevelScriptBlueprint.h"
#include "Engine/MemberReference.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "K2Node.h"
#include "K2Node_CallFunction.h"
#include "K2Node_Event.h"
#include "K2Node_SpawnActor.h"
#include "K2Node_SpawnActorFromClass.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "SBlueprintActionMenu.h"
#include "SBlueprintPalette.h"
#include "SGraphActionMenu.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SExpanderArrow.h"

class FDragDropOperation;
class SWidget;
struct FPointerEvent;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "BlueprintSubPalette"

/*******************************************************************************
* Static File Helpers
*******************************************************************************/

/**
 * An analytics hook, for tracking when a node was spawned from the palette 
 * (updates the "node creation stats" with a palette drag-placement flag).
 * 
 * @param  BlueprintEditorPtr	A pointer to the blueprint editor currently being worked in.
 */
static void OnNodePlacement(TWeakPtr<FBlueprintEditor> BlueprintEditorPtr)
{
	if( BlueprintEditorPtr.IsValid() )
	{
		BlueprintEditorPtr.Pin()->UpdateNodeCreationStats( ENodeCreateAction::PaletteDragPlacement );
	}
}

/**
 * Checks to see if the user can drop the currently dragged action to place its
 * associated node in the graph.
 * 
 * @param  DropActionIn			The action that will be executed when the user drops the dragged item.
 * @param  HoveredGraphIn		A pointer to the graph that the user currently has the item dragged over.
 * @param  ImpededReasonOut		If this returns false, this will be filled out with a reason to present the user with.
 * @return True is the dragged palette item can be dropped where it currently is, false if not.
 */
static bool CanPaletteItemBePlaced(TSharedPtr<FEdGraphSchemaAction> DropActionIn, UEdGraph* HoveredGraphIn, FText& ImpededReasonOut)
{
	bool bCanBePlaced = true;
	if (!DropActionIn.IsValid())
	{
		bCanBePlaced = false;
		ImpededReasonOut = LOCTEXT("InvalidDropAction", "Invalid action for placement");
	}
	else if (HoveredGraphIn == NULL)
	{
		bCanBePlaced = false;
		ImpededReasonOut = LOCTEXT("DropOnlyInGraph", "Nodes can only be placed inside the blueprint graph");
	}
	else if (UK2Node const* NodeToBePlaced = FBlueprintActionMenuUtils::ExtractNodeTemplateFromAction(DropActionIn))
	{
		UEdGraphSchema const* const GraphSchema = HoveredGraphIn->GetSchema();
		check(GraphSchema != nullptr);

		bool bIsFunctionGraph = (GraphSchema->GetGraphType(HoveredGraphIn) == EGraphType::GT_Function);

		if (UK2Node_CallFunction const* CallFuncNode = Cast<UK2Node_CallFunction const>(NodeToBePlaced))
		{
			FName const FuncName = CallFuncNode->FunctionReference.GetMemberName();
			check(FuncName != NAME_None);
			UClass const* const FuncOwner = CallFuncNode->FunctionReference.GetMemberParentClass(CallFuncNode->GetBlueprintClassFromNode());
			check(FuncOwner != nullptr);

			UFunction* const Function = FindUField<UFunction>(FuncOwner, FuncName);
			UEdGraphSchema_K2 const* const K2Schema = Cast<UEdGraphSchema_K2 const>(GraphSchema);

			if (Function == nullptr)
			{
				bCanBePlaced = false;
				ImpededReasonOut = LOCTEXT("InvalidFuncAction", "Invalid function for placement");
			}
			else if (K2Schema == nullptr)
			{
				bCanBePlaced = false;
				ImpededReasonOut = LOCTEXT("CannotCreateInThisSchema", "Cannot call functions in this type of graph");
			}
			else
			{
				// Note: We only check function context for UK2Node_CallFunction types specifically; derivatives are typically bound to specific functions that should be placeable but may not be explicitly callable (e.g. InternalUseOnly).
				// @TODO - Consolidate this as a call to UK2Node::IsActionFilteredOut() here instead? Would need to add the ImpededReason as an 'out' param to that API first.
				//  	   We could then also skip the additonal 'CanPasteHere' check below in that case as it would be redundant for CallFunction node types specifically.
				if(NodeToBePlaced->GetClass() == UK2Node_CallFunction::StaticClass())
				{ 
					uint32 AllowedFunctionTypes = UEdGraphSchema_K2::EFunctionType::FT_Pure | UEdGraphSchema_K2::EFunctionType::FT_Const | UEdGraphSchema_K2::EFunctionType::FT_Protected;
					if(K2Schema->DoesGraphSupportImpureFunctions(HoveredGraphIn))
					{
						AllowedFunctionTypes |= UEdGraphSchema_K2::EFunctionType::FT_Imperative;
					}

					const UClass* GeneratedClass = FBlueprintEditorUtils::FindBlueprintForGraphChecked(HoveredGraphIn)->GeneratedClass;
					bCanBePlaced = K2Schema->CanFunctionBeUsedInGraph(GeneratedClass, Function, HoveredGraphIn, AllowedFunctionTypes, false, &ImpededReasonOut);
				}
			}
		}
		else if (UK2Node_Event const* EventNode = Cast<UK2Node_Event const>(NodeToBePlaced))
		{
			// function graphs cannot have more than one entry point
			if (bIsFunctionGraph)
			{
				bCanBePlaced = false;
				ImpededReasonOut = LOCTEXT("NoSecondEntryPoint", "Function graphs can only have one entry point");
			}
			else if (GraphSchema->GetGraphType(HoveredGraphIn) != EGraphType::GT_Ubergraph)
			{
				bCanBePlaced = false;
				ImpededReasonOut = LOCTEXT("NoEventsOnlyInUberGraphs", "Events can only be placed in event graphs");
			}
		}
		else if (Cast<UK2Node_SpawnActor const>(NodeToBePlaced) || Cast<UK2Node_SpawnActorFromClass const>(NodeToBePlaced))
		{
			UEdGraphSchema_K2 const* const K2Schema = Cast<UEdGraphSchema_K2 const>(GraphSchema);
			if (K2Schema && K2Schema->IsConstructionScript(HoveredGraphIn))
			{
				bCanBePlaced = false;
				ImpededReasonOut = LOCTEXT("NoSpawnActorInConstruction", "Cannot spawn actors from a construction script");
			}
		}

		bool bWillFocusOnExistingNode = (DropActionIn->GetTypeId() == FEdGraphSchemaAction_K2TargetNode::StaticGetTypeId());
		if (!bWillFocusOnExistingNode && DropActionIn->GetTypeId() == FEdGraphSchemaAction_K2AddEvent::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2AddEvent* AddEventAction = (FEdGraphSchemaAction_K2AddEvent*)DropActionIn.Get();
			bWillFocusOnExistingNode = AddEventAction->EventHasAlreadyBeenPlaced(FBlueprintEditorUtils::FindBlueprintForGraph(HoveredGraphIn));
		}
		
		// if this will instead focus on an existing node, reverse any previous decision... it is ok to drop!
		if (bWillFocusOnExistingNode)
		{
			bCanBePlaced = true;
			ImpededReasonOut = FText::GetEmpty();
		}
		// as a general catch-all, if a node cannot be pasted or placed in the graph, it probably can't be created there.
		// Some nodes allow themselves to be pasted where they are generally not allowed, if either does not want the 
		// node placed, it should not be placeable
		else if (bCanBePlaced && (!NodeToBePlaced->CanPasteHere(HoveredGraphIn) || !NodeToBePlaced->IsCompatibleWithGraph(HoveredGraphIn)) && !bWillFocusOnExistingNode)
		{
			bCanBePlaced = false;
			ImpededReasonOut = LOCTEXT("CannotPaste", "Cannot place this node in this type of graph");
		}
	}

	return bCanBePlaced;
}

/*******************************************************************************
* FBlueprintPaletteCommands
*******************************************************************************/

class FBlueprintPaletteCommands : public TCommands<FBlueprintPaletteCommands>
{
public:
	FBlueprintPaletteCommands() : TCommands<FBlueprintPaletteCommands>
		( "BlueprintPalette"
		, LOCTEXT("PaletteContext", "Palette")
		, NAME_None
		, FAppStyle::GetAppStyleSetName() )
	{
	}

	TSharedPtr<FUICommandInfo> RefreshPalette;

	/**
	 * Registers context menu commands for the blueprint palette.
	 */
	virtual void RegisterCommands() override
	{
		UI_COMMAND(RefreshPalette, "Refresh List", "Refreshes the list of nodes.", EUserInterfaceActionType::Button, FInputChord());
	}
};

/*******************************************************************************
* SBlueprintSubPalette Public Interface
*******************************************************************************/

//------------------------------------------------------------------------------
SBlueprintSubPalette::~SBlueprintSubPalette()
{
	FBlueprintActionDatabase& ActionDatabase = FBlueprintActionDatabase::Get();
	ActionDatabase.OnEntryRemoved().RemoveAll(this);
	ActionDatabase.OnEntryUpdated().RemoveAll(this);
}

//------------------------------------------------------------------------------
void SBlueprintSubPalette::Construct(FArguments const& InArgs, TWeakPtr<FBlueprintEditor> InBlueprintEditor)
{
	bIsActiveTimerRegistered = false;
	BlueprintEditorPtr = InBlueprintEditor;

	struct LocalUtils
	{
		static TSharedRef<SExpanderArrow> CreateCustomExpander(const FCustomExpanderData& ActionMenuData, bool bShowFavoriteToggle)
		{
			TSharedPtr<SExpanderArrow> CustomExpander;
			if (bShowFavoriteToggle)
			{
				SAssignNew(CustomExpander, SBlueprintActionMenuExpander, ActionMenuData);
			}
			else
			{
				SAssignNew(CustomExpander, SExpanderArrow, ActionMenuData.TableRow);
			}
			return CustomExpander.ToSharedRef();
		}
	};

	ChildSlot
 	[
		SNew(SBorder)
		.Padding(2.f)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew( SVerticalBox )
			+SVerticalBox::Slot()
				.AutoHeight()
				.Padding( 0.f, 2.f, 0.f, 0.f )
			[
				ConstructHeadingWidget(InArgs._Icon.Get(), InArgs._Title.Get(), InArgs._ToolTipText.Get())
			]

			+SVerticalBox::Slot()
			[
				SNew(SOverlay)
				+SOverlay::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
				[
					SAssignNew(GraphActionMenu, SGraphActionMenu)
						.OnCreateWidgetForAction(this, &SBlueprintSubPalette::OnCreateWidgetForAction)
						.OnActionDragged(this, &SBlueprintSubPalette::OnActionDragged)
						.OnCollectAllActions(this, &SBlueprintSubPalette::CollectAllActions)
						.OnContextMenuOpening(this, &SBlueprintSubPalette::ConstructContextMenuWidget)
						.OnCreateCustomRowExpander_Static(&LocalUtils::CreateCustomExpander, InArgs._ShowFavoriteToggles.Get())
				]
			]
		]
	];

	CommandList = MakeShareable(new FUICommandList);
	// has to come after GraphActionMenu has been set
	BindCommands(CommandList);

	FBlueprintActionDatabase& ActionDatabase = FBlueprintActionDatabase::Get();
	ActionDatabase.OnEntryRemoved().AddSP(this, &SBlueprintSubPalette::OnDatabaseActionsRemoved);
	ActionDatabase.OnEntryUpdated().AddSP(this, &SBlueprintSubPalette::OnDatabaseActionsUpdated);
}

//------------------------------------------------------------------------------
UBlueprint* SBlueprintSubPalette::GetBlueprint() const
{
	UBlueprint* BlueprintBeingEdited = NULL;
	if (BlueprintEditorPtr.IsValid())
	{
		BlueprintBeingEdited = BlueprintEditorPtr.Pin()->GetBlueprintObj();
	}
	return BlueprintBeingEdited;
}

//------------------------------------------------------------------------------
TSharedPtr<FEdGraphSchemaAction> SBlueprintSubPalette::GetSelectedAction() const
{
	TArray< TSharedPtr<FEdGraphSchemaAction> > SelectedActions;
	GraphActionMenu->GetSelectedActions(SelectedActions);

	return TSharedPtr<FEdGraphSchemaAction>( (SelectedActions.Num() > 0) ? SelectedActions[0] : NULL );
}

/*******************************************************************************
* Protected SBlueprintSubPalette Methods
*******************************************************************************/

//------------------------------------------------------------------------------
void SBlueprintSubPalette::RefreshActionsList(bool bPreserveExpansion)
{
	// Prevent refreshing the palette if we're in PIE
	if( !GIsPlayInEditorWorld )
	{
		SGraphPalette::RefreshActionsList(bPreserveExpansion);
	}
}

//------------------------------------------------------------------------------
TSharedRef<SWidget> SBlueprintSubPalette::OnCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData)
{
	return SNew(SBlueprintPaletteItem, InCreateData, BlueprintEditorPtr.Pin());
}

//------------------------------------------------------------------------------
FReply SBlueprintSubPalette::OnActionDragged( const TArray< TSharedPtr<FEdGraphSchemaAction> >& InActions, const FPointerEvent& MouseEvent )
{
	if( InActions.Num() > 0 && InActions[0].IsValid() )
	{
		TSharedPtr<FEdGraphSchemaAction> InAction = InActions[0];
		auto AnalyticsDelegate = FNodeCreationAnalytic::CreateStatic(&OnNodePlacement, BlueprintEditorPtr);

		auto CanNodeBePlacedDelegate = FKismetDragDropAction::FCanBeDroppedDelegate::CreateStatic(&CanPaletteItemBePlaced);

		if(InAction->GetTypeId() == FEdGraphSchemaAction_K2NewNode::StaticGetTypeId())
		{
			return FReply::Handled().BeginDragDrop(FKismetDragDropAction::New(InAction, AnalyticsDelegate, CanNodeBePlacedDelegate));
		}
		else if(InAction->GetTypeId() == FEdGraphSchemaAction_K2Var::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2Var* VarAction = (FEdGraphSchemaAction_K2Var*)InAction.Get();
			if (UClass* VarClass = VarAction->GetVariableClass())
			{
				return FReply::Handled().BeginDragDrop(FKismetVariableDragDropAction::New(InAction, VarAction->GetVariableName(), VarClass, AnalyticsDelegate));
			}
		}
		else if(InAction->GetTypeId() == FEdGraphSchemaAction_K2Delegate::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2Delegate* DelegateAction = (FEdGraphSchemaAction_K2Delegate*)InAction.Get();
			if (UClass* VarClass = DelegateAction->GetDelegateClass())
			{
				return FReply::Handled().BeginDragDrop(FKismetDelegateDragDropAction::New(InAction, DelegateAction->GetDelegateName(), VarClass, AnalyticsDelegate));
			}
		}
		else if (InAction->GetTypeId() == FBlueprintDragDropMenuItem::StaticGetTypeId())
		{
			FBlueprintDragDropMenuItem* BlueprintAction = (FBlueprintDragDropMenuItem*)InAction.Get();

			TSharedPtr<FDragDropOperation> DragDropOp = BlueprintAction->OnDragged(AnalyticsDelegate);
			if (DragDropOp.IsValid())
			{
				return FReply::Handled().BeginDragDrop(DragDropOp.ToSharedRef());
			}
			else
			{
				return FReply::Handled().BeginDragDrop(FKismetDragDropAction::New(InAction, AnalyticsDelegate, CanNodeBePlacedDelegate));
			}
		}
		else
		{
			return FReply::Handled().BeginDragDrop(FKismetDragDropAction::New(InAction, AnalyticsDelegate, CanNodeBePlacedDelegate));
		}
	}

	return FReply::Unhandled();
}

//------------------------------------------------------------------------------
void SBlueprintSubPalette::BindCommands(TSharedPtr<FUICommandList> CommandListIn) const
{
	FBlueprintPaletteCommands::Register();
	FBlueprintPaletteCommands const& PaletteCommands = FBlueprintPaletteCommands::Get();

	CommandListIn->MapAction(
		PaletteCommands.RefreshPalette,
		FExecuteAction::CreateSP(const_cast<SBlueprintSubPalette*>(this), &SBlueprintSubPalette::RefreshActionsList, /*bPreserveExpansion =*/true)
	);
}

//------------------------------------------------------------------------------
TSharedPtr<SWidget> SBlueprintSubPalette::ConstructContextMenuWidget() const
{
	FMenuBuilder MenuBuilder(/* bInShouldCloseWindowAfterMenuSelection =*/true, CommandList);
	GenerateContextMenuEntries(MenuBuilder);
	return MenuBuilder.MakeWidget();
}

//------------------------------------------------------------------------------
void SBlueprintSubPalette::GenerateContextMenuEntries(FMenuBuilder& MenuBuilder) const
{
	FBlueprintPaletteCommands const& PaletteCommands = FBlueprintPaletteCommands::Get();
	MenuBuilder.AddMenuEntry(PaletteCommands.RefreshPalette);
}

//------------------------------------------------------------------------------
void SBlueprintSubPalette::RequestRefreshActionsList()
{
	bIsActiveTimerRegistered = true;
}

//------------------------------------------------------------------------------
void SBlueprintSubPalette::OnDatabaseActionsUpdated(UObject* /*ActionsKey*/)
{
	RequestRefreshActionsList();
}

//------------------------------------------------------------------------------
void SBlueprintSubPalette::OnDatabaseActionsRemoved(UObject* ActionsKey)
{
	ULevelScriptBlueprint* RemovedLevelScript = Cast<ULevelScriptBlueprint>(ActionsKey);
	bool const bAssumeDestroyingWorld = (RemovedLevelScript != nullptr);

	if (bAssumeDestroyingWorld)
	{
		// have to update the action list immediatly (cannot wait until Tick(), 
		// because we have to handle level switching, which expects all references 
		// to be cleared immediately)
		ForceRefreshActionList();
	}
	else
	{
		RequestRefreshActionsList();
	}
}

void SBlueprintSubPalette::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SGraphPalette::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if(bIsActiveTimerRegistered)
	{
		static FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		if (!AssetRegistryModule.Get().IsLoadingAssets() && !AssetRegistryModule.Get().IsGathering() && !IsAsyncLoading())
		{
			bIsActiveTimerRegistered = false;
			RefreshActionsList(true);
		}
	}
}

/*******************************************************************************
* Private SBlueprintSubPalette Methods
*******************************************************************************/

//------------------------------------------------------------------------------
void SBlueprintSubPalette::ForceRefreshActionList()
{
	RefreshActionsList(/*bPreserveExpansion =*/true);
}

//------------------------------------------------------------------------------
TSharedRef<SVerticalBox> SBlueprintSubPalette::ConstructHeadingWidget(FSlateBrush const* const Icon, FText const& TitleText, FText const& ToolTipText)
{
	TSharedPtr<SToolTip> ToolTipWidget;
	SAssignNew(ToolTipWidget, SToolTip).Text(ToolTipText);

	static FTextBlockStyle TitleStyle = FTextBlockStyle()
		.SetFont(FCoreStyle::GetDefaultFontStyle("Bold", 10))
		.SetColorAndOpacity(FLinearColor(0.4f, 0.4f, 0.4f));

	return SNew(SVerticalBox)
		.ToolTip(ToolTipWidget)
		// so we still get tooltip text for an empty SHorizontalBox
		.Visibility(EVisibility::Visible) 
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.f, 2.f)
			[
				SNew(SImage).Image(Icon)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.f, 2.f)
			[
				SNew(STextBlock)
				.Text(TitleText)
				.TextStyle(&TitleStyle)
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.f, 2.f, 0.f, 5.f)
		[
			SNew(SBorder)
			// use the border's padding to actually create the horizontal line
			.Padding(1.f)
			.BorderImage(FAppStyle::GetBrush(TEXT("Menu.Separator")))
		];	
}

#undef LOCTEXT_NAMESPACE
