// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "SGraphPalette.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FBlueprintEditor;
class FMenuBuilder;
class FUICommandList;
class SVerticalBox;
class SWidget;
class UBlueprint;
class UObject;
struct FCreateWidgetForActionData;
struct FEdGraphSchemaAction;
struct FPointerEvent;
struct FSlateBrush;

/*******************************************************************************
* SBlueprintSubPalette
*******************************************************************************/

class SBlueprintSubPalette : public SGraphPalette
{
public:
	SLATE_BEGIN_ARGS( SBlueprintSubPalette )
		: _Title()
		, _Icon(FCoreStyle::Get().GetDefaultBrush())
		, _ShowFavoriteToggles(false)
		{}

		SLATE_ATTRIBUTE(FText, Title)
		SLATE_ATTRIBUTE(FSlateBrush const*, Icon)
		SLATE_ATTRIBUTE(bool, ShowFavoriteToggles)
	SLATE_END_ARGS()

	/** Unsubscribes this from events before it is destroyed */
	virtual ~SBlueprintSubPalette();

	/**
	 * Creates a sub-palette widget for the blueprint palette UI (this serves as 
	 * a base class for more specialized sub-palettes).
	 * 
	 * @param  InArgs				A set of slate arguments, defined above.
	 * @param  InBlueprintEditor	A pointer to the blueprint editor that this palette belongs to.
	 */
	void Construct(const FArguments& InArgs, TWeakPtr<FBlueprintEditor> InBlueprintEditor);

	/**
	 * Retrieves, from the owning blueprint-editor, the blueprint currently  
	 * being worked on.
	 * 
	 * @return The blueprint currently being edited by this palette's blueprint-editor.
	 */
	UBlueprint* GetBlueprint() const;

	/**
	 * Retrieves the palette menu item currently selected by the user.
	 * 
	 * @return A pointer to the palette's currently selected action (NULL if a category is selected, or nothing at all)
	 */
	TSharedPtr<FEdGraphSchemaAction> GetSelectedAction() const;

protected:
	// SGraphPalette Interface
	virtual void RefreshActionsList(bool bPreserveExpansion) override;
	virtual TSharedRef<SWidget> OnCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData) override;
	virtual FReply OnActionDragged(const TArray< TSharedPtr<FEdGraphSchemaAction> >& InActions, const FPointerEvent& MouseEvent) override;
	// End of SGraphPalette Interface

	/**
	 * A place to bind all context menu actions for this sub-palette. Sub-classes
	 * can override this to bind their own specialized commands.
	 * 
	 * @param  CommandListIn	The command list to map your actions with.
	 */
	virtual void BindCommands(TSharedPtr<FUICommandList> CommandListIn) const;

	/**
	 * Constructs the slate header for the sub-palette. Inherited classes can 
	 * override this to tack on their own headers.
	 * 
	 * @param  Icon			The icon, identifying this specific sub-palette.
	 * @param  TitleText	A title identifying this sub-palette.
	 * @param  ToolTip		A tooltip you want displayed when the user hovers over the heading.
	 * @return A reference to the newly created vertical slate box containing the header.
	 */
	virtual TSharedRef<SVerticalBox> ConstructHeadingWidget(FSlateBrush const* const Icon, FText const& TitleText, FText const& ToolTip);

	/**
	 * An overridable method, that fills out the provided menu-builder with 
	 * actions for this sub-palette's right-click context menu (sub-classes can
	 * provide their own).
	 * 
	 * @param  MenuBuilder	The menu builder you want the sub-palette's actions added to.
	 */
	virtual void GenerateContextMenuEntries(FMenuBuilder& MenuBuilder) const;

	/** Delegate to call to request a refresh */
	void RequestRefreshActionsList();
	/** Delegate handlers for when the blueprint database is updated (so we can release references and refresh the list) */
	void OnDatabaseActionsUpdated(UObject* ActionsKey);
	void OnDatabaseActionsRemoved(UObject* ActionsKey);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/** Pointer back to the blueprint editor that owns us */
	TWeakPtr<FBlueprintEditor> BlueprintEditorPtr;

private:
	/** Immediately calls RefreshActionsList(), doesn't not defer until Tick() like RequestRefreshActionsList() does. */
	void ForceRefreshActionList();

	/**
	 * Constructs a slate widget for the right-click context menu in this 
	 * palette. While this isn't virtual, sub-classes can override GenerateContextMenuEntries()
	 * to provide their own specialized entries.
	 * 
	 * @return A pointer to the newly created menu widget.
	 */
	TSharedPtr<SWidget> ConstructContextMenuWidget() const;

private:
	/** Pointer to the command list created for this (so multiple sub-palettes can have their own bindings)*/
	TSharedPtr<FUICommandList> CommandList;

	/** Whether the active timer to refresh the actions list is currently registered */
	bool bIsActiveTimerRegistered;
};

