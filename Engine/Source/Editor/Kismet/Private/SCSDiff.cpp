// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCSDiff.h"

#include "Components/ActorComponent.h"
#include "Delegates/Delegate.h"
#include "Engine/Blueprint.h"
#include "Engine/SimpleConstructionScript.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "IDetailsView.h"
#include "Internationalization/Text.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyEditorDelegates.h"
#include "PropertyPath.h"
#include "SKismetInspector.h"
#include "SSubobjectBlueprintEditor.h"
#include "SSubobjectEditor.h"
#include "SlotBase.h"
#include "SubobjectData.h"
#include "SubobjectDataHandle.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Types/SlateEnums.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSplitter.h"

class SWidget;

FSCSDiff::FSCSDiff(const UBlueprint* InBlueprint)
{
	// Need to pull off const because the ICH access functions theoretically could modify it
	Blueprint = const_cast<UBlueprint*>(InBlueprint);

	if (!FBlueprintEditorUtils::SupportsConstructionScript(InBlueprint) || InBlueprint->SimpleConstructionScript == NULL)
	{
		ContainerWidget = SNew(SBox);
		return;
	}

	Inspector = SNew(SKismetInspector)
		.HideNameArea(true)
		.ViewIdentifier(FName("BlueprintInspector"))
		.IsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateStatic([] { return false; }));

	ContainerWidget = SNew(SSplitter)
		.Orientation(Orient_Vertical)
		+ SSplitter::Slot()
		[
			SAssignNew(SubobjectEditor, SSubobjectBlueprintEditor)
				.ObjectContext(InBlueprint->GeneratedClass->GetDefaultObject<AActor>())
				.AllowEditing(false)
				.HideComponentClassCombo(true)
				.OnSelectionUpdated(SSubobjectEditor::FOnSelectionUpdated::CreateRaw(this, &FSCSDiff::OnSCSEditorUpdateSelectionFromNodes))
				.OnHighlightPropertyInDetailsView(SSubobjectBlueprintEditor::FOnHighlightPropertyInDetailsView::CreateRaw(this, &FSCSDiff::OnSCSEditorHighlightPropertyInDetailsView))
		]
		+ SSplitter::Slot()
		[
			Inspector.ToSharedRef()
		];
}

void FSCSDiff::HighlightProperty(FName VarName, FPropertySoftPath Property)
{
	if (SubobjectEditor.IsValid())
	{
		check(VarName != FName());
		SubobjectEditor->HighlightTreeNode(VarName, FPropertyPath());
	}
}

TSharedRef< SWidget > FSCSDiff::TreeWidget()
{
	return ContainerWidget.ToSharedRef();
}

void GetDisplayedHierarchyRecursive(UBlueprint* Blueprint, TArray<int32>& TreeAddress, const FSubobjectDataHandle& Node, TArray<FSCSResolvedIdentifier>& OutResult)
{
	FSubobjectData* NodeData = Node.GetData();
	if (!NodeData)
	{
		return;
	}

	FSCSIdentifier Identifier = { NodeData->GetVariableName(), TreeAddress };
	FSCSResolvedIdentifier ResolvedIdentifier = { Identifier, NodeData->GetObjectForBlueprint<UActorComponent>(Blueprint) };
	OutResult.Push(ResolvedIdentifier);
	
	const TArray<FSubobjectDataHandle>& Children = NodeData->GetChildrenHandles();
	for (int32 Iter = 0; Iter != Children.Num(); ++Iter)
	{
		TreeAddress.Push(Iter);
		GetDisplayedHierarchyRecursive(Blueprint, TreeAddress, Children[Iter], OutResult);
		TreeAddress.Pop();
	}
}

TArray<FSCSResolvedIdentifier> FSCSDiff::GetDisplayedHierarchy() const
{
	TArray< FSCSResolvedIdentifier > Ret;

	if(SubobjectEditor.IsValid() && SubobjectEditor->GetRootNodes().Num() > 0)
	{
		const TArray<FSubobjectEditorTreeNodePtrType>& RootNodes = SubobjectEditor->GetRootNodes();
		for (int32 Iter = 0; Iter != RootNodes.Num(); ++Iter)
		{
			TArray< int32 > TreeAddress;
			TreeAddress.Push(Iter);
			GetDisplayedHierarchyRecursive(Blueprint, TreeAddress, RootNodes[Iter]->GetDataHandle(), Ret);
		}
	}

	return Ret;
}

void FSCSDiff::OnSCSEditorUpdateSelectionFromNodes(const TArray<FSubobjectEditorTreeNodePtrType>& SelectedNodes)
{
	FText InspectorTitle = FText::GetEmpty();
	TArray<UObject*> InspectorObjects;
	InspectorObjects.Empty(SelectedNodes.Num());
	for (auto NodeIt = SelectedNodes.CreateConstIterator(); NodeIt; ++NodeIt)
	{
		auto NodePtr = *NodeIt;
		if(NodePtr.IsValid() && NodePtr->GetDataSource()->CanEdit())
		{
			InspectorTitle = FText::FromString(NodePtr->GetDisplayString());
			InspectorObjects.Add(const_cast<UObject*>(NodePtr->GetDataSource()->GetObjectForBlueprint(Blueprint)));
		}
	}

	if( Inspector.IsValid() )
	{
		SKismetInspector::FShowDetailsOptions Options(InspectorTitle, true);
		Inspector->ShowDetailsForObjects(InspectorObjects, Options);
	}
}

void FSCSDiff::OnSCSEditorHighlightPropertyInDetailsView(const FPropertyPath& InPropertyPath)
{
	if( Inspector.IsValid() )
	{
		Inspector->GetPropertyView()->HighlightProperty(InPropertyPath);
	}
}
