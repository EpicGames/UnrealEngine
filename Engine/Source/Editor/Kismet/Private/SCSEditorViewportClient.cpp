// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCSEditorViewportClient.h"

#include "BlueprintEditor.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "ComponentVisualizer.h"
#include "ComponentVisualizerManager.h"
#include "Components/ActorComponent.h"
#include "Components/ChildActorComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Components/SceneComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorViewportSelectability.h"
#include "EditorComponents.h"
#include "EditorViewportSelectabilityBridge.h"
#include "Engine/Blueprint.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "Engine/SimpleConstructionScript.h"
#include "Engine/StaticMesh.h"
#include "Engine/World.h"
#include "EngineDefines.h"
#include "EngineUtils.h"
#include "GameFramework/Actor.h"
#include "HitProxies.h"
#include "ISCSEditorCustomization.h"
#include "Internationalization/Text.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Logging/LogMacros.h"
#include "Materials/Material.h"
#include "Math/Color.h"
#include "Math/IntPoint.h"
#include "Math/Plane.h"
#include "Math/QuatRotationTranslationMatrix.h"
#include "Math/Transform.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "ObjectTools.h"
#include "PhysicsEngine/ConstraintInstance.h"
#include "PhysicsEngine/PhysicsConstraintComponent.h"
#include "PreviewScene.h"
#include "SEditorViewport.h"
#include "SSCSEditorViewport.h"
#include "SSubobjectEditor.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneView.h"
#include "ScopedTransaction.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "ShowFlags.h"
#include "SnappingUtils.h"
#include "StaticMeshResources.h"
#include "SubobjectData.h"
#include "SubobjectDataSubsystem.h"
#include "Templates/Casts.h"
#include "Templates/UnrealTemplate.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectHash.h"
#include "UnrealClient.h"
#include "UnrealEdGlobals.h"
#include "UnrealWidget.h"
#include "Widgets/SWidget.h"

struct FSubobjectDataHandle;

DEFINE_LOG_CATEGORY_STATIC(LogSCSEditorViewport, Log, All);

namespace
{
	/** Automatic translation applied to the camera in the default editor viewport logic when orbit mode is enabled. */
	const float AutoViewportOrbitCameraTranslate = 256.0f;

	void DrawAngles(FCanvas* Canvas, int32 XPos, int32 YPos, EAxisList::Type ManipAxis, UE::Widget::EWidgetMode MoveMode, const FRotator& Rotation, const FVector& Translation)
	{
		FString OutputString(TEXT(""));
		if(MoveMode == UE::Widget::WM_Rotate && Rotation.IsZero() == false)
		{
			//Only one value moves at a time
			const FVector EulerAngles = Rotation.Euler();
			if(ManipAxis == EAxisList::X)
			{
				OutputString += FString::Printf(TEXT("Roll: %0.2f"), EulerAngles.X);
			}
			else if(ManipAxis == EAxisList::Y)
			{
				OutputString += FString::Printf(TEXT("Pitch: %0.2f"), EulerAngles.Y);
			}
			else if(ManipAxis == EAxisList::Z)
			{
				OutputString += FString::Printf(TEXT("Yaw: %0.2f"), EulerAngles.Z);
			}
		}
		else if(MoveMode == UE::Widget::WM_Translate && Translation.IsZero() == false)
		{
			//Only one value moves at a time
			if(ManipAxis == EAxisList::X)
			{
				OutputString += FString::Printf(TEXT(" %0.2f"), Translation.X);
			}
			else if(ManipAxis == EAxisList::Y)
			{
				OutputString += FString::Printf(TEXT(" %0.2f"), Translation.Y);
			}
			else if(ManipAxis == EAxisList::Z)
			{
				OutputString += FString::Printf(TEXT(" %0.2f"), Translation.Z);
			}
		}

		if(OutputString.Len() > 0)
		{
			FCanvasTextItem TextItem( FVector2D(XPos, YPos), FText::FromString( OutputString ), GEngine->GetSmallFont(), FLinearColor::White );
			Canvas->DrawItem( TextItem );
		} 
	}

	// Determine whether or not the given node has a parent node that is not the root node, is movable and is selected
	bool IsMovableParentNodeSelected(const FSubobjectEditorTreeNodePtrType& NodePtr, const TArray<FSubobjectEditorTreeNodePtrType>& SelectedNodes)
	{
		if(NodePtr.IsValid())
		{
			// Check for a valid parent node
			FSubobjectEditorTreeNodePtrType ParentNodePtr = NodePtr->GetParent();
			const FSubobjectData* ParentData = ParentNodePtr.IsValid()  ? ParentNodePtr->GetDataSource() : nullptr;
			if(ParentData && !ParentData->IsRootComponent())
			{
				if(SelectedNodes.Contains(ParentNodePtr))
				{
					// The parent node is not the root node and is also selected; success
					return true;
				}
				else
				{
					// Recursively search for any other parent nodes farther up the tree that might be selected
					return IsMovableParentNodeSelected(ParentNodePtr, SelectedNodes);
				}
			}
		}

		return false;
	}
}

/////////////////////////////////////////////////////////////////////////
// FSCSEditorViewportClient

FSCSEditorViewportClient::FSCSEditorViewportClient(TWeakPtr<FBlueprintEditor>& InBlueprintEditorPtr, FPreviewScene* InPreviewScene, const TSharedRef<SSCSEditorViewport>& InSCSEditorViewport, FEditorModeTools* InModeTools)
	: FEditorViewportClient(InModeTools, InPreviewScene, StaticCastSharedRef<SEditorViewport>(InSCSEditorViewport))
	, BlueprintEditorPtr(InBlueprintEditorPtr)
	, PreviewActorBounds(ForceInitToZero)
	, bIsManipulating(false)
	, ScopedTransaction(NULL)
	, bIsSimulateEnabled(false)
{
	WidgetMode = UE::Widget::WM_Translate;
	WidgetCoordSystem = COORD_Local;
	EngineShowFlags.DisableAdvancedFeatures();

	check(Widget);
	Widget->SetSnapEnabled(true);

	// Selectively set particular show flags that we need
	EngineShowFlags.SetSelectionOutline(GetDefault<ULevelEditorViewportSettings>()->bUseSelectionOutline);

	// Set if the grid will be drawn
	DrawHelper.bDrawGrid = GetDefault<UEditorPerProjectUserSettings>()->bSCSEditorShowGrid;

	// now add floor
	EditorFloorComp = NewObject<UStaticMeshComponent>(GetTransientPackage(), TEXT("EditorFloorComp"));

	UStaticMesh* FloorMesh = LoadObject<UStaticMesh>(NULL, TEXT("/Engine/EditorMeshes/PhAT_FloorBox.PhAT_FloorBox"), NULL, LOAD_None, NULL);
	if (ensure(FloorMesh))
	{
		EditorFloorComp->SetStaticMesh(FloorMesh);
	}

	UMaterial* Material = LoadObject<UMaterial>(NULL, TEXT("/Engine/EditorMaterials/PersonaFloorMat.PersonaFloorMat"), NULL, LOAD_None, NULL);
	if (ensure(Material))
	{
		EditorFloorComp->SetMaterial(0, Material);
	}

	EditorFloorComp->SetRelativeScale3D(FVector(3.f, 3.f, 1.f));
	EditorFloorComp->SetVisibility(GetDefault<UEditorPerProjectUserSettings>()->bSCSEditorShowFloor);
	EditorFloorComp->SetCollisionEnabled(GetDefault<UEditorPerProjectUserSettings>()->bSCSEditorShowFloor? ECollisionEnabled::QueryAndPhysics : ECollisionEnabled::NoCollision);
	PreviewScene->AddComponent(EditorFloorComp, FTransform::Identity);

	// Turn off so that actors added to the world do not have a lifespan (so they will not auto-destroy themselves).
	PreviewScene->GetWorld()->SetBegunPlay(false);

	PreviewScene->SetSkyCubemap(GUnrealEd->GetThumbnailManager()->AmbientCubemap);
}

FSCSEditorViewportClient::~FSCSEditorViewportClient()
{
	// Ensure that an in-progress transaction is ended
	EndTransaction();
}

void FSCSEditorViewportClient::Tick(float DeltaSeconds)
{
	FEditorViewportClient::Tick(DeltaSeconds);

	// Register the selection override delegate for the preview actor's components
	TSharedPtr<SSubobjectEditor> SubobjectEditor = BlueprintEditorPtr.Pin()->GetSubobjectEditor();
	AActor* PreviewActor = GetPreviewActor();
	if (PreviewActor != nullptr)
	{
		for (UActorComponent* Component : PreviewActor->GetComponents())
		{
			if (UPrimitiveComponent* PrimComponent = Cast<UPrimitiveComponent>(Component))
			{
				if (!PrimComponent->SelectionOverrideDelegate.IsBound())
				{
					SubobjectEditor->SetSelectionOverride(PrimComponent);
				}
			}
		}
	}
	else
	{
		InvalidatePreview(false);
	}

	// Tick the preview scene world.
	if (!GIntraFrameDebuggingGameThread)
	{
		// Ensure that the preview actor instance is up-to-date for component editing (e.g. after compiling the Blueprint, the actor may be reinstanced outside of this class)
		if(PreviewActor != BlueprintEditorPtr.Pin()->GetBlueprintObj()->SimpleConstructionScript->GetComponentEditorActorInstance())
		{
			BlueprintEditorPtr.Pin()->GetBlueprintObj()->SimpleConstructionScript->SetComponentEditorActorInstance(PreviewActor);
		}

		// Allow full tick only if preview simulation is enabled and we're not currently in an active SIE or PIE session
		if(bIsSimulateEnabled && GEditor->PlayWorld == NULL && !GEditor->bIsSimulatingInEditor)
		{
			PreviewScene->GetWorld()->Tick(IsRealtime() ? LEVELTICK_All : LEVELTICK_TimeOnly, DeltaSeconds);
		}
		else
		{
			PreviewScene->GetWorld()->Tick(IsRealtime() ? LEVELTICK_ViewportsOnly : LEVELTICK_TimeOnly, DeltaSeconds);
		}
	}
}

void FSCSEditorViewportClient::Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	FEditorViewportClient::Draw(View, PDI);

	bool bHitTesting = PDI->IsHitTesting();
	AActor* PreviewActor = GetPreviewActor();
	if(PreviewActor)
	{
		if(GUnrealEd != nullptr)
		{
			TArray<FSubobjectEditorTreeNodePtrType> SelectedNodes = BlueprintEditorPtr.Pin()->GetSelectedSubobjectEditorTreeNodes();
			for(FSubobjectEditorTreeNodePtrType SelectedNode : SelectedNodes)
			{
				const FSubobjectData* Data = SelectedNode->GetDataSource();
				const UActorComponent* Comp = Data ? Data->FindComponentInstanceInActor(PreviewActor) : nullptr;
				if(Comp != nullptr && Comp->IsRegistered())
				{
					// Try and find a visualizer
					TSharedPtr<FComponentVisualizer> Visualizer = GUnrealEd->FindComponentVisualizer(Comp->GetClass());
					if (Visualizer.IsValid())
					{
						Visualizer->DrawVisualization(Comp, View, PDI);
					}
				}
			}
		}
	}
}

void FSCSEditorViewportClient::DrawCanvas( FViewport& InViewport, FSceneView& View, FCanvas& Canvas )
{
	AActor* PreviewActor = GetPreviewActor();
	if(PreviewActor)
	{
		if (GUnrealEd != nullptr)
		{
			TArray<FSubobjectEditorTreeNodePtrType> SelectedNodes = BlueprintEditorPtr.Pin()->GetSelectedSubobjectEditorTreeNodes();
			for(FSubobjectEditorTreeNodePtrType SelectedNode : SelectedNodes)
			{
				const FSubobjectData* Data = SelectedNode->GetDataSource();
				const UActorComponent* Comp = Data ? Data->FindComponentInstanceInActor(PreviewActor) : nullptr;
								
				if (Comp != nullptr && Comp->IsRegistered())
				{
					// Try and find a visualizer
					TSharedPtr<FComponentVisualizer> Visualizer = GUnrealEd->FindComponentVisualizer(Comp->GetClass());
					if (Visualizer.IsValid())
					{
						Visualizer->DrawVisualizationHUD(Comp, &InViewport, &View, &Canvas);
					}
				}
			}
		}

		TGuardValue<bool> AutoRestore(GAllowActorScriptExecutionInEditor, true);

		const int32 HalfX = Viewport->GetSizeXY().X / 2;
		const int32 HalfY = Viewport->GetSizeXY().Y / 2;

		TArray<FSubobjectEditorTreeNodePtrType> SelectedNodes = BlueprintEditorPtr.Pin()->GetSelectedSubobjectEditorTreeNodes();

		if(bIsManipulating && SelectedNodes.Num() > 0)
		{
			const FSubobjectData* Data = SelectedNodes[0]->GetDataSource();
			const USceneComponent* SceneComp = Data ? Cast<USceneComponent>(Data->FindComponentInstanceInActor(PreviewActor)) : nullptr;
			if(SceneComp)
			{
				const FVector WidgetLocation = GetWidgetLocation();
				const FPlane Proj = View.Project(WidgetLocation);
				if(Proj.W > 0.0f)
				{
					const int32 XPos = static_cast<int32>(HalfX + (HalfX * Proj.X));
					const int32 YPos = static_cast<int32>(HalfY + (HalfY * Proj.Y * -1.0));
					DrawAngles(&Canvas, XPos, YPos, GetCurrentWidgetAxis(), GetWidgetMode(), GetWidgetCoordSystem().Rotator(), WidgetLocation);
				}
			}
		}
	}

	// If enabled, draw viewport selection limited text
	const TSharedPtr<FBlueprintEditor> BlueprintEditor = BlueprintEditorPtr.Pin();
	if (BlueprintEditor.IsValid())
	{
		if (FEditorViewportSelectabilityBridge* SelectabilityBridge = BlueprintEditor->GetViewportSelectabilityBridge())
		{
			if (SelectabilityBridge->IsViewportSelectionLimited())
			{
				const FText SelectionLimitedText = SelectabilityBridge->GetViewportSelectionLimitedText();
				FEditorViewportSelectability::DrawEnabledTextNotice(&Canvas, SelectionLimitedText);
			}
		}
	}
}

bool FSCSEditorViewportClient::InputKey(const FInputKeyEventArgs& EventArgs)
{
	bool bHandled = GUnrealEd->ComponentVisManager.HandleInputKey(this, EventArgs.Viewport, EventArgs.Key, EventArgs.Event);

	if(!bHandled)
	{
		bHandled = FEditorViewportClient::InputKey(EventArgs);
	}

	return bHandled;
}

void FSCSEditorViewportClient::MouseMove(FViewport* InViewport, int32 InX, int32 InY)
{
	HHitProxy* const HitResult = InViewport ? InViewport->GetHitProxy(InX, InY) : nullptr;
	UpdateHoverFromHitProxy(HitResult);

	return FEditorViewportClient::MouseMove(InViewport, InX, InY);
}

EMouseCursor::Type FSCSEditorViewportClient::GetCursor(FViewport* InViewport, int32 InX, int32 InY)
{
	if (MouseCursor.IsSet())
	{
		return MouseCursor.GetValue();
	}

	return FEditorViewportClient::GetCursor(InViewport, InX, InY);
}

void FSCSEditorViewportClient::ProcessClick(class FSceneView& View, class HHitProxy* HitProxy, FKey Key, EInputEvent Event, uint32 HitX, uint32 HitY)
{
	const FViewportClick Click(&View, this, Key, Event, HitX, HitY);

	if (!HitProxy)
	{
		if (const TSharedPtr<FBlueprintEditor> BlueprintEditor = BlueprintEditorPtr.Pin())
		{
			if (const TSharedPtr<SSubobjectEditor> SubobjectEditor = BlueprintEditor->GetSubobjectEditor())
			{
				// If the user clicked on empty space, clear the selection
				SubobjectEditor->ClearSelection();
			}
		}
		return;
	}

	if (HitProxy->IsA(HInstancedStaticMeshInstance::StaticGetType()))
	{
		HInstancedStaticMeshInstance* InstancedStaticMeshInstanceProxy = ((HInstancedStaticMeshInstance*)HitProxy);

		TSharedPtr<ISCSEditorCustomization> Customization = BlueprintEditorPtr.Pin()->CustomizeSubobjectEditor(InstancedStaticMeshInstanceProxy->Component);
		if (Customization.IsValid() && Customization->HandleViewportClick(AsShared(), View, HitProxy, Key, Event, HitX, HitY))
		{
			Invalidate();
		}

		return;
	}

	if (HitProxy->IsA(HWidgetAxis::StaticGetType()))
	{
		const bool bOldModeWidgets1 = EngineShowFlags.ModeWidgets;
		const bool bOldModeWidgets2 = View.Family->EngineShowFlags.ModeWidgets;

		EngineShowFlags.SetModeWidgets(false);
		FSceneViewFamily* SceneViewFamily = const_cast<FSceneViewFamily*>(View.Family);
		SceneViewFamily->EngineShowFlags.SetModeWidgets(false);
		bool bWasWidgetDragging = Widget->IsDragging();
		Widget->SetDragging(false);

		// Invalidate the hit proxy map so it will be rendered out again when GetHitProxy
		// is called
		Viewport->InvalidateHitProxy();

		// This will actually re-render the viewport's hit proxies!
		HHitProxy* HitProxyWithoutAxisWidgets = Viewport->GetHitProxy(HitX, HitY);
		if (HitProxyWithoutAxisWidgets != NULL && !HitProxyWithoutAxisWidgets->IsA(HWidgetAxis::StaticGetType()))
		{
			// Try this again, but without the widget this time!
			ProcessClick(View, HitProxyWithoutAxisWidgets, Key, Event, HitX, HitY);
		}

		// Undo the evil
		EngineShowFlags.SetModeWidgets(bOldModeWidgets1);
		SceneViewFamily->EngineShowFlags.SetModeWidgets(bOldModeWidgets2);

		Widget->SetDragging(bWasWidgetDragging);

		// Invalidate the hit proxy map again so that it'll be refreshed with the original
		// scene contents if we need it again later.
		Viewport->InvalidateHitProxy();
		return;
	}

	if(GUnrealEd->ComponentVisManager.HandleClick(this, HitProxy, Click))
	{
		// Component Vis Manager handled this click, no need to do anything
		return;
	}

	if (HitProxy->IsA(HActor::StaticGetType()))
	{
		HActor* ActorProxy = (HActor*)HitProxy;
		AActor* PreviewActor = GetPreviewActor();
		if (ActorProxy && ActorProxy->Actor && ActorProxy->PrimComponent)
		{
			const USceneComponent* SelectedCompInstance = nullptr;

			if (ActorProxy->Actor == PreviewActor)
			{
				const UPrimitiveComponent* TestComponent = ActorProxy->PrimComponent;
				if (ActorProxy->Actor->GetComponents().Contains(TestComponent)
					&& IsObjectSelectableInViewport(const_cast<UPrimitiveComponent*>(TestComponent)))
				{
					SelectedCompInstance = TestComponent;
				}
			}
			else if (ActorProxy->Actor->IsChildActor())
			{
				AActor* TestActor = ActorProxy->Actor;
				while (TestActor->GetParentActor()->IsChildActor())
				{
					TestActor = TestActor->GetParentActor();
				}

				if (TestActor->GetParentActor() == PreviewActor)
				{
					SelectedCompInstance = TestActor->GetParentComponent();
				}
			}

			if (SelectedCompInstance)
			{
				TSharedPtr<ISCSEditorCustomization> Customization = BlueprintEditorPtr.Pin()->CustomizeSubobjectEditor(SelectedCompInstance);
				if (!(Customization.IsValid() && Customization->HandleViewportClick(AsShared(), View, HitProxy, Key, Event, HitX, HitY)))
				{
					const bool bIsCtrlKeyDown = Viewport->KeyState(EKeys::LeftControl) || Viewport->KeyState(EKeys::RightControl);
					if (BlueprintEditorPtr.IsValid())
					{
						// Note: This will find and select any node associated with the component instance that's attached to the proxy (including visualizers)
						BlueprintEditorPtr.Pin()->FindAndSelectSubobjectEditorTreeNode(SelectedCompInstance, bIsCtrlKeyDown);
					}
				}
			}
		}

		Invalidate();
	}
}

struct FTemplateComponentMoved
{
	int32 SelectedNodeIndex;
	USceneComponent* SceneComp;
	USceneComponent* SelectedTemplate;
	FVector OldRelativeLocation;
	FRotator OldRelativeRotation;
	FVector OldRelativeScale3D;
};

bool FSCSEditorViewportClient::InputWidgetDelta( FViewport* InViewport, EAxisList::Type CurrentAxis, FVector& Drag, FRotator& Rot, FVector& Scale )
{
	bool bHandled = false;
	if(bIsManipulating && CurrentAxis != EAxisList::None)
	{
		bHandled = true;
		AActor* PreviewActor = GetPreviewActor();
		TSharedPtr<FBlueprintEditor> BlueprintEditor = BlueprintEditorPtr.Pin();
		if (PreviewActor && BlueprintEditor.IsValid())
		{
			TArray<FSubobjectEditorTreeNodePtrType> SelectedNodes = BlueprintEditorPtr.Pin()->GetSelectedSubobjectEditorTreeNodes();
			if(SelectedNodes.Num() > 0)
			{
				FVector ModifiedScale = Scale;

				// (mirrored from Level Editor VPC) - we don't scale components when we only have a very small scale change
				if (!Scale.IsNearlyZero())
				{
					FSnappingUtils::AdjustPercentageBasedScale(ModifiedScale);
				}
				else
				{
					ModifiedScale = FVector::ZeroVector;
				}
				
				// RerunConstructionScripts only needs to be called once, after all selected components have been moved (it's potentially quite 
				// expensive).  After that, some post-move work needs to happen per component, so we keep track of the components that were moved.
				bool bNeedsRerunConstructionScripts = false;

				TArray<FTemplateComponentMoved> TemplateComponentsMoved;
				TArray<UActorComponent*> ActorComponentsMoved;
				TemplateComponentsMoved.Reserve(SelectedNodes.Num());

				for (int32 SelectedNodeIndex = 0; SelectedNodeIndex < SelectedNodes.Num(); SelectedNodeIndex++)
				{
					const FSubobjectEditorTreeNodePtrType& SelectedNodePtr = SelectedNodes[SelectedNodeIndex];
					const FSubobjectData* Data = SelectedNodePtr->GetDataSource();
					// Don't allow editing of a root node, inherited SCS node or child node that also has a movable (non-root) parent node selected
					const bool bCanEdit = GUnrealEd->ComponentVisManager.IsActive() ||
						(Data && !Data->IsRootComponent() && !IsMovableParentNodeSelected(SelectedNodePtr, SelectedNodes));

					if(bCanEdit)
					{
						if (GUnrealEd->ComponentVisManager.HandleInputDelta(this, InViewport, Drag, Rot, Scale))
						{
							GUnrealEd->RedrawLevelEditingViewports();
							Invalidate();
							return true;
						}

						// #TODO_BH Clean up const casts
						USceneComponent* SceneComp = const_cast<USceneComponent*>(Cast<USceneComponent>(Data->FindComponentInstanceInActor(PreviewActor)));
						USceneComponent* SelectedTemplate = const_cast<USceneComponent*>(Cast<USceneComponent>(Data->GetObjectForBlueprint(BlueprintEditor->GetBlueprintObj())));
						if(SceneComp && SelectedTemplate)
						{
							// Cache the current default values for propagation
							FVector OldRelativeLocation = SelectedTemplate->GetRelativeLocation();
							FRotator OldRelativeRotation = SelectedTemplate->GetRelativeRotation();
							FVector OldRelativeScale3D = SelectedTemplate->GetRelativeScale3D();

							// Adjust the deltas as necessary
							FComponentEditorUtils::AdjustComponentDelta(SceneComp, Drag, Rot);

							TSharedPtr<ISCSEditorCustomization> Customization = BlueprintEditor->CustomizeSubobjectEditor(SceneComp);
							if(Customization.IsValid() && Customization->HandleViewportDrag(SceneComp, SelectedTemplate, Drag, Rot, ModifiedScale, GetWidgetLocation()))
							{
								// Handled by SCS Editor customization
							}
							else
							{
								// Apply delta to the template component object 
								// (the preview scene component will be set in one of the ArchetypeInstances loops below... to keep the two in sync) 
								GEditor->ApplyDeltaToComponent(
										SelectedTemplate,
										true,
										&Drag,
										&Rot,
										&ModifiedScale,
										SelectedTemplate->GetRelativeLocation());
							}

							UBlueprint* PreviewBlueprint = UBlueprint::GetBlueprintFromClass(PreviewActor->GetClass());
							if(PreviewBlueprint != nullptr)
							{
								// Like PostEditMove(), but we only need to re-run construction scripts
								if(PreviewBlueprint && PreviewBlueprint->bRunConstructionScriptOnDrag)
								{
									bNeedsRerunConstructionScripts = true;
								}

								FTemplateComponentMoved& ComponentMoved = TemplateComponentsMoved[TemplateComponentsMoved.AddUninitialized()];
								ComponentMoved.SelectedNodeIndex = SelectedNodeIndex;
								ComponentMoved.SceneComp = SceneComp;
								ComponentMoved.SelectedTemplate = SelectedTemplate;
								ComponentMoved.OldRelativeLocation = OldRelativeLocation;
								ComponentMoved.OldRelativeRotation = OldRelativeRotation;
								ComponentMoved.OldRelativeScale3D = OldRelativeScale3D;

								// Track static meshes for bulk re-registration
								UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(SceneComp);
								if (StaticMeshComponent)
								{
									ActorComponentsMoved.Add(StaticMeshComponent);
								}
							}
						}
					}
				}

				{
					// This bulk re-register context forces Add/RemovePrimitive and debug physics update commands to be sent to the
					// render thread in batches, significantly improving performance.
					FStaticMeshComponentBulkReregisterContext ReregisterContext(GetScene(), ActorComponentsMoved);

					// Get the SCS if present
					UBlueprintGeneratedClass* PreviewBlueprint = Cast<UBlueprintGeneratedClass>(PreviewActor->GetClass());
					USimpleConstructionScript* SCS = nullptr;
					if (PreviewBlueprint && PreviewBlueprint->SimpleConstructionScript)
					{
						SCS = PreviewBlueprint->SimpleConstructionScript;

						// Tell the reregister context about the simple construction script, which allows the SCS to batch render commands
						// for newly created components generated during construction.
						ReregisterContext.AddSimpleConstructionScript(SCS);

						// Optimize calls to GetArchetypeInstances by generating the SCS node map for the blueprint class.
						SCS->CreateNameToSCSNodeMap();
					}

					if (bNeedsRerunConstructionScripts)
					{
						PreviewActor->RerunConstructionScripts();

						// The construction scripts will have recreated the selected components, and the ones in the TemplateComponentsMoved
						// array now point to the deleted version.  Update to the newly created version.
						for (FTemplateComponentMoved& Moved : TemplateComponentsMoved)
						{
							const FSubobjectEditorTreeNodePtrType& SelectedNodePtr = SelectedNodes[Moved.SelectedNodeIndex];
							const FSubobjectData* Data = SelectedNodePtr->GetDataSource();

							Moved.SceneComp = const_cast<USceneComponent*>(Cast<USceneComponent>(Data->FindComponentInstanceInActor(PreviewActor)));
						}
					}

					// Array corresponds to TemplateComponentsMoved, with objects we need to search for archetypes
					TArray<UObject*> ArchetypeSearchObjects;
					ArchetypeSearchObjects.Reserve(TemplateComponentsMoved.Num());

					for (FTemplateComponentMoved& Moved : TemplateComponentsMoved)
					{
						Moved.SceneComp->PostEditComponentMove(true); // @TODO HACK passing 'finished' every frame...

						// If a constraint, copy back updated constraint frames to template
						UPhysicsConstraintComponent* ConstraintComp = Cast<UPhysicsConstraintComponent>(Moved.SceneComp);
						UPhysicsConstraintComponent* TemplateComp = Cast<UPhysicsConstraintComponent>(Moved.SelectedTemplate);
						if (ConstraintComp && TemplateComp)
						{
							TemplateComp->ConstraintInstance.CopyConstraintGeometryFrom(&ConstraintComp->ConstraintInstance);
						}

						// Add to the list of objects we need to search for archetypes
						if (Moved.SelectedTemplate->HasAnyFlags(RF_ArchetypeObject))
						{
							// Searching the item itself
							ArchetypeSearchObjects.Add(Moved.SelectedTemplate);
						}
						else if (UObject* Outer = Moved.SelectedTemplate->GetOuter())
						{
							// Searching the outer
							ArchetypeSearchObjects.Add(Outer);
						}
						else
						{
							// Searching nothing -- place a dummy item to preserve array ordering
							ArchetypeSearchObjects.Add(nullptr);
						}
					}

					// Get the list of active archetype instances for each moved object, in bulk for efficiency
					TArray<TArray<UObject*>> ArchetypeInstancesList;
					ObjectTools::BatchGetArchetypeInstances(ArchetypeSearchObjects, ArchetypeInstancesList);

					// Propagate the change(s) to the matching component instance
					for (int32 ObjectIndex = 0; ObjectIndex < TemplateComponentsMoved.Num(); ObjectIndex++)
					{
						// Did the search find any instances?
						TArray<UObject*>& ArchetypeInstances = ArchetypeInstancesList[ObjectIndex];
						if (ArchetypeInstances.Num())
						{
							FTemplateComponentMoved& Moved = TemplateComponentsMoved[ObjectIndex];
							if (Moved.SelectedTemplate->HasAnyFlags(RF_ArchetypeObject))
							{
								// Object itself was archetype
								for (int32 InstanceIndex = 0; InstanceIndex < ArchetypeInstances.Num(); ++InstanceIndex)
								{
									USceneComponent* SceneComp = Cast<USceneComponent>(ArchetypeInstances[InstanceIndex]);
									if (SceneComp != nullptr)
									{
										FComponentEditorUtils::ApplyDefaultValueChange(SceneComp, SceneComp->GetRelativeLocation_DirectMutable(), Moved.OldRelativeLocation, Moved.SelectedTemplate->GetRelativeLocation());
										FComponentEditorUtils::ApplyDefaultValueChange(SceneComp, SceneComp->GetRelativeRotation_DirectMutable(), Moved.OldRelativeRotation, Moved.SelectedTemplate->GetRelativeRotation());
										FComponentEditorUtils::ApplyDefaultValueChange(SceneComp, SceneComp->GetRelativeScale3D_DirectMutable(), Moved.OldRelativeScale3D, Moved.SelectedTemplate->GetRelativeScale3D());
									}
								}
							}
							else
							{
								// Outer was archetype
								for (int32 InstanceIndex = 0; InstanceIndex < ArchetypeInstances.Num(); ++InstanceIndex)
								{
									USceneComponent* SceneComp = static_cast<USceneComponent*>(FindObjectWithOuter(ArchetypeInstances[InstanceIndex], Moved.SelectedTemplate->GetClass(), Moved.SelectedTemplate->GetFName()));
									if (SceneComp)
									{
										FComponentEditorUtils::ApplyDefaultValueChange(SceneComp, SceneComp->GetRelativeLocation_DirectMutable(), Moved.OldRelativeLocation, Moved.SelectedTemplate->GetRelativeLocation());
										FComponentEditorUtils::ApplyDefaultValueChange(SceneComp, SceneComp->GetRelativeRotation_DirectMutable(), Moved.OldRelativeRotation, Moved.SelectedTemplate->GetRelativeRotation());
										FComponentEditorUtils::ApplyDefaultValueChange(SceneComp, SceneComp->GetRelativeScale3D_DirectMutable(), Moved.OldRelativeScale3D, Moved.SelectedTemplate->GetRelativeScale3D());
									}
								}
							}
						}
					}

					if (SCS)
					{
						SCS->RemoveNameToSCSNodeMap();
					}
				}

				GUnrealEd->RedrawLevelEditingViewports();
			}
		}

		Invalidate();
	}

	return bHandled;
}

void FSCSEditorViewportClient::TrackingStarted( const struct FInputEventState& InInputState, bool bIsDraggingWidget, bool bNudge )
{
	if( !bIsManipulating && bIsDraggingWidget )
	{
		HandleBeginTransform();
	}
}

void FSCSEditorViewportClient::TrackingStopped() 
{
	if( bIsManipulating )
	{
		HandleEndTransform();
	}
}

bool FSCSEditorViewportClient::BeginTransform(const FGizmoState& InState)
{
	return HandleBeginTransform();
}

bool FSCSEditorViewportClient::EndTransform(const FGizmoState& InState)
{
	return HandleEndTransform();
}

bool FSCSEditorViewportClient::HandleBeginTransform()
{
	if (!bIsManipulating)
	{
		// Suspend component modification during each delta step to avoid recording unnecessary overhead into the transaction buffer
		GEditor->DisableDeltaModification(true);

		// Begin transaction
		BeginTransaction( NSLOCTEXT("UnrealEd", "ModifyComponents", "Modify Component(s)") );
		bIsManipulating = true;
		return true;
	}
	
	return false;
}

bool FSCSEditorViewportClient::HandleEndTransform()
{
	if (bIsManipulating)
	{
		// Re-run construction scripts if we haven't done so yet (so that the components in the preview actor can update their transforms)
		AActor* PreviewActor = GetPreviewActor();
		if(PreviewActor != nullptr)
		{
			UBlueprint* PreviewBlueprint = UBlueprint::GetBlueprintFromClass(PreviewActor->GetClass());
			if(PreviewBlueprint != nullptr && !PreviewBlueprint->bRunConstructionScriptOnDrag)
			{
				PreviewActor->RerunConstructionScripts();
			}
		}

		// End transaction
		bIsManipulating = false;
		EndTransaction();

		// Restore component delta modification
		GEditor->DisableDeltaModification(false);

		return true;
	}

	return false;
}

UE::Widget::EWidgetMode FSCSEditorViewportClient::GetWidgetMode() const
{
	// Default to not drawing the widget
	UE::Widget::EWidgetMode ReturnWidgetMode = UE::Widget::WM_None;

	AActor* PreviewActor = GetPreviewActor();
	if(!bIsSimulateEnabled && PreviewActor)
	{
		const TSharedPtr<FBlueprintEditor> BluePrintEditor = BlueprintEditorPtr.Pin();
		if (BluePrintEditor.IsValid())
		{
			TArray<FSubobjectEditorTreeNodePtrType> SelectedNodes = BlueprintEditorPtr.Pin()->GetSelectedSubobjectEditorTreeNodes();

			if (BluePrintEditor->GetSubobjectEditor()->GetSceneRootNode().IsValid())
			{
				TArray<FSubobjectEditorTreeNodePtrType> RootNodes = BluePrintEditor->GetSubobjectEditor()->GetRootNodes();

				if (GUnrealEd->ComponentVisManager.IsActive() &&
					GUnrealEd->ComponentVisManager.IsVisualizingArchetype())
				{
					// Component visualizer is active and editing the archetype
					ReturnWidgetMode = WidgetMode;
				}
				else
				{
					// if the selected nodes array is empty, or only contains entries from the
					// root nodes array, or isn't visible in the preview actor, then don't display a transform widget
					for(FSubobjectEditorTreeNodePtrType CurrentNodePtr : SelectedNodes)
					{
						if (CurrentNodePtr.IsValid())
						{
							FSubobjectData* Data = CurrentNodePtr->GetDataSource();
							if (Data && Data->CanEdit())
							{
								const bool bIsNotRootComponent = !RootNodes.Contains(CurrentNodePtr) && !Data->IsRootComponent();
								const bool bIsISM = 
									Data->GetObject<UInstancedStaticMeshComponent>() && 
									CastChecked<UInstancedStaticMeshComponent>(Data->FindComponentInstanceInActor(GetPreviewActor()))->SelectedInstances.Contains(true);
								const bool bHasInstanceInActor = Data->FindComponentInstanceInActor(PreviewActor) != nullptr;

								if ((bIsNotRootComponent || bIsISM) && bHasInstanceInActor)
								{
									// a non-NULL, non-root item is selected, draw the widget
									ReturnWidgetMode = WidgetMode;
									break;
								}
							}
						}
					}
				}
			}
		}
	}

	return ReturnWidgetMode;
}


void FSCSEditorViewportClient::SetWidgetMode( UE::Widget::EWidgetMode NewMode )
{
	WidgetMode = NewMode;
}

void FSCSEditorViewportClient::SetWidgetCoordSystemSpace( ECoordSystem NewCoordSystem ) 
{
	WidgetCoordSystem = NewCoordSystem;
}

FVector FSCSEditorViewportClient::GetWidgetLocation() const
{
	FVector ComponentVisWidgetLocation;
	if (GUnrealEd->ComponentVisManager.IsVisualizingArchetype() &&
		GUnrealEd->ComponentVisManager.GetWidgetLocation(this, ComponentVisWidgetLocation))
	{
		return ComponentVisWidgetLocation;
	}

	FVector Location = FVector::ZeroVector;

	AActor* PreviewActor = GetPreviewActor();
	if(PreviewActor)
	{
		TArray<FSubobjectEditorTreeNodePtrType> SelectedNodes = BlueprintEditorPtr.Pin()->GetSelectedSubobjectEditorTreeNodes();
		if(SelectedNodes.Num() > 0)
		{
			// Use the last selected item for the widget location
			const FSubobjectData* Data = SelectedNodes[0]->GetDataSource();

			const USceneComponent* SceneComp = Data ? Cast<USceneComponent>(Data->FindComponentInstanceInActor(PreviewActor)) : nullptr;
			if(SceneComp)
			{
				TSharedPtr<ISCSEditorCustomization> Customization = BlueprintEditorPtr.Pin()->CustomizeSubobjectEditor(SceneComp);
				FVector CustomLocation;
				if(Customization.IsValid() && Customization->HandleGetWidgetLocation(const_cast<USceneComponent*>(SceneComp), CustomLocation))
				{
					Location = CustomLocation;
				}
				else
				{
					Location = SceneComp->GetComponentLocation();
				}
			}
		}
	}

	return Location;
}

FMatrix FSCSEditorViewportClient::GetWidgetCoordSystem() const
{
	FMatrix ComponentVisWidgetCoordSystem;
	if (GUnrealEd->ComponentVisManager.IsVisualizingArchetype() &&
		GUnrealEd->ComponentVisManager.GetCustomInputCoordinateSystem(this, ComponentVisWidgetCoordSystem))
	{
		return ComponentVisWidgetCoordSystem;
	}

	FMatrix Matrix = FMatrix::Identity;
	if( GetWidgetCoordSystemSpace() == COORD_Local )
	{
		AActor* PreviewActor = GetPreviewActor();
		TSharedPtr<FBlueprintEditor> BlueprintEditor = BlueprintEditorPtr.Pin();
		if (PreviewActor && BlueprintEditor.IsValid())
		{
			TArray<FSubobjectEditorTreeNodePtrType> SelectedNodes = BlueprintEditorPtr.Pin()->GetSelectedSubobjectEditorTreeNodes();
			if(SelectedNodes.Num() > 0)
			{
				const FSubobjectEditorTreeNodePtrType SelectedNode = SelectedNodes.Last();
				const FSubobjectData* Data = SelectedNode->GetDataSource();
				const USceneComponent* SceneComp = Data ? Cast<USceneComponent>(Data->FindComponentInstanceInActor(PreviewActor)) : nullptr;
				if(SceneComp)
				{
					TSharedPtr<ISCSEditorCustomization> Customization = BlueprintEditor->CustomizeSubobjectEditor(SceneComp);
					FMatrix CustomTransform;
					if(Customization.IsValid() && Customization->HandleGetWidgetTransform(SceneComp, CustomTransform))
					{
						Matrix = CustomTransform;
					}					
					else
					{
						Matrix = FQuatRotationMatrix( SceneComp->GetComponentQuat() );
					}
				}
			}
		}
	}

	if(!Matrix.Equals(FMatrix::Identity))
	{
		Matrix.RemoveScaling();
	}

	return Matrix;
}

const FEditorViewportCameraSpeedSettings& FSCSEditorViewportClient::GetCameraSpeedSettings() const
{
	return GetDefault<UEditorPerProjectUserSettings>()->SCSViewportCameraSpeedSettings;
}

void FSCSEditorViewportClient::SetCameraSpeedSettings(const FEditorViewportCameraSpeedSettings& NewCameraSpeedSettings)
{
	GetMutableDefault<UEditorPerProjectUserSettings>()->SCSViewportCameraSpeedSettings = NewCameraSpeedSettings;
}

void FSCSEditorViewportClient::InvalidatePreview(bool bResetCamera)
{
	// Ensure that the editor is valid before continuing
	if(!BlueprintEditorPtr.IsValid())
	{
		return;
	}

	UBlueprint* Blueprint = BlueprintEditorPtr.Pin()->GetBlueprintObj();
	check(Blueprint);

	const bool bIsPreviewActorValid = GetPreviewActor() != nullptr;

	// Create or update the Blueprint actor instance in the preview scene
	BlueprintEditorPtr.Pin()->UpdatePreviewActor(Blueprint, !bIsPreviewActorValid);

	Invalidate();
	RefreshPreviewBounds();
	
	if( bResetCamera )
	{
		ResetCamera();
	}
}

void FSCSEditorViewportClient::ResetCamera()
{
	UBlueprint* Blueprint = BlueprintEditorPtr.Pin()->GetBlueprintObj();

	// For now, loosely base default camera positioning on thumbnail preview settings
	USceneThumbnailInfo* ThumbnailInfo = Cast<USceneThumbnailInfo>(Blueprint->ThumbnailInfo);
	if(ThumbnailInfo == nullptr)
	{
		ThumbnailInfo = USceneThumbnailInfo::StaticClass()->GetDefaultObject<USceneThumbnailInfo>();
	}

	// Clamp zoom to the actor's bounding sphere radius
	double OrbitZoom = ThumbnailInfo->OrbitZoom;
	if (PreviewActorBounds.SphereRadius + OrbitZoom < 0.0)
	{
		OrbitZoom = -PreviewActorBounds.SphereRadius;
	}

	ToggleOrbitCamera(true);
	{
		double TargetDistance = PreviewActorBounds.SphereRadius;
		if(TargetDistance <= 0.0f)
		{
			TargetDistance = AutoViewportOrbitCameraTranslate;
		}

		FRotator ThumbnailAngle(ThumbnailInfo->OrbitPitch, ThumbnailInfo->OrbitYaw, 0.0f);

		SetViewLocationForOrbiting(PreviewActorBounds.Origin);
		SetViewLocation( GetViewLocation() + FVector(0.0f, TargetDistance * 1.5f + OrbitZoom - AutoViewportOrbitCameraTranslate, 0.0f) );
		SetViewRotation( ThumbnailAngle );
	}

	Invalidate();
}

void FSCSEditorViewportClient::ToggleRealtimePreview()
{
	SetRealtime(!IsRealtime());

	Invalidate();
}

AActor* FSCSEditorViewportClient::GetPreviewActor() const
{
	return BlueprintEditorPtr.Pin()->GetPreviewActor();
}

void FSCSEditorViewportClient::FocusViewportToSelection()
{
	AActor* PreviewActor = GetPreviewActor();
	if(PreviewActor)
	{
		TArray<FSubobjectEditorTreeNodePtrType> SelectedNodes = BlueprintEditorPtr.Pin()->GetSelectedSubobjectEditorTreeNodes();
		if(SelectedNodes.Num() > 0)
		{
			const FSubobjectData* Data = SelectedNodes[0]->GetDataSource();
			// Use the last selected item for the widget location
			const USceneComponent* SceneComp = Data ? Cast<USceneComponent>(Data->FindComponentInstanceInActor(PreviewActor)) : nullptr;
			if(SceneComp)
			{
				FocusViewportOnBox(SceneComp->Bounds.GetBox());
			}
		}
		else
		{
			FocusViewportOnBox(PreviewActor->GetComponentsBoundingBox(true));
		}
	}
}

bool FSCSEditorViewportClient::GetIsSimulateEnabled() 
{ 
	return bIsSimulateEnabled;
}

void FSCSEditorViewportClient::ToggleIsSimulateEnabled() 
{
	// Must destroy existing actors before we toggle the world state
	BlueprintEditorPtr.Pin()->DestroyPreview();

	bIsSimulateEnabled = !bIsSimulateEnabled;
	PreviewScene->GetWorld()->SetBegunPlay(bIsSimulateEnabled);
	PreviewScene->GetWorld()->bShouldSimulatePhysics = bIsSimulateEnabled;

	TSharedPtr<SWidget> SubobjectEditor = BlueprintEditorPtr.Pin()->GetSubobjectEditor();
	TSharedRef<SWidget> Inspector = BlueprintEditorPtr.Pin()->GetInspector();

	// When simulate is enabled, we don't want to allow the user to modify the components
	BlueprintEditorPtr.Pin()->UpdatePreviewActor(BlueprintEditorPtr.Pin()->GetBlueprintObj(), true);

	SubobjectEditor->SetEnabled(!bIsSimulateEnabled);
	Inspector->SetEnabled(!bIsSimulateEnabled);

	if(!IsRealtime())
	{
		ToggleRealtimePreview();
	}
}

bool FSCSEditorViewportClient::GetShowFloor() 
{
	return GetDefault<UEditorPerProjectUserSettings>()->bSCSEditorShowFloor;
}

void FSCSEditorViewportClient::ToggleShowFloor() 
{
	UEditorPerProjectUserSettings* Settings = GetMutableDefault<UEditorPerProjectUserSettings>();

	bool bShowFloor = Settings->bSCSEditorShowFloor;
	bShowFloor = !bShowFloor;
	
	EditorFloorComp->SetVisibility(bShowFloor);
	EditorFloorComp->SetCollisionEnabled(bShowFloor? ECollisionEnabled::QueryAndPhysics : ECollisionEnabled::NoCollision);

	Settings->bSCSEditorShowFloor = bShowFloor;
	Settings->PostEditChange();

	Invalidate();
}

bool FSCSEditorViewportClient::GetShowGrid() 
{
	return GetDefault<UEditorPerProjectUserSettings>()->bSCSEditorShowGrid;
}

void FSCSEditorViewportClient::ToggleShowGrid() 
{
	UEditorPerProjectUserSettings* Settings = GetMutableDefault<UEditorPerProjectUserSettings>();

	bool bShowGrid = Settings->bSCSEditorShowGrid;
	bShowGrid = !bShowGrid;

	DrawHelper.bDrawGrid = bShowGrid;

	Settings->bSCSEditorShowGrid = bShowGrid;
	Settings->PostEditChange();
	
	Invalidate();
}

void FSCSEditorViewportClient::BeginTransaction(const FText& Description)
{
	//UE_LOGF(LogSCSEditorViewport, Log, "FSCSEditorViewportClient::BeginTransaction() pre: %ls %08x", SessionName, *((uint32*)&ScopedTransaction));

	if(!ScopedTransaction)
	
	{
		if(USubobjectDataSubsystem* System = USubobjectDataSubsystem::Get())
		{
			TArray<FSubobjectDataHandle> SelectedNodes = BlueprintEditorPtr.Pin()->GetSubobjectEditor()->GetSelectedHandles();
			TSharedPtr<FBlueprintEditor> BlueprintEditor = BlueprintEditorPtr.Pin();
			ScopedTransaction = System->BeginTransaction(SelectedNodes, Description, BlueprintEditor.IsValid() ? BlueprintEditor->GetBlueprintObj() : nullptr);
		}
	}

	//UE_LOGF(LogSCSEditorViewport, Log, "FSCSEditorViewportClient::BeginTransaction() post: %ls %08x", SessionName, *((uint32*)&ScopedTransaction));
}

void FSCSEditorViewportClient::EndTransaction()
{
	//UE_LOGF(LogSCSEditorViewport, Log, "FSCSEditorViewportClient::EndTransaction(): %08x", *((uint32*)&ScopedTransaction));

	if(ScopedTransaction)
	{
		delete ScopedTransaction;
		ScopedTransaction = nullptr;
	}
}

void FSCSEditorViewportClient::RefreshPreviewBounds()
{
	AActor* PreviewActor = GetPreviewActor();

	if(PreviewActor)
	{
		// Compute actor bounds as the sum of its visible parts
		FBoxSphereBounds::Builder BoundsBuilder;
		for (UActorComponent* Component : PreviewActor->GetComponents())
		{
			// Aggregate primitive components that either have collision enabled or are otherwise visible components in-game
			if (UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(Component))
			{
				if (PrimComp->IsRegistered() && (!PrimComp->bHiddenInGame || PrimComp->IsCollisionEnabled()) && PrimComp->Bounds.SphereRadius < HALF_WORLD_MAX)
				{
					BoundsBuilder += PrimComp->Bounds;
				}
			}
		}
		PreviewActorBounds = BoundsBuilder;
	}
}

bool FSCSEditorViewportClient::IsViewportSelectionLimited() const
{
	if (const TSharedPtr<FBlueprintEditor> BlueprintEditor = BlueprintEditorPtr.Pin())
	{
		if (FEditorViewportSelectabilityBridge* SelectabilityBridge = BlueprintEditor->GetViewportSelectabilityBridge())
		{
			return SelectabilityBridge->IsViewportSelectionLimited();
		}
	}
	return false;
}

bool FSCSEditorViewportClient::IsObjectSelectableInViewport(UObject* const InObject) const
{
	if (const TSharedPtr<FBlueprintEditor> BlueprintEditor = BlueprintEditorPtr.Pin())
	{
		if (FEditorViewportSelectabilityBridge* SelectabilityBridge = BlueprintEditor->GetViewportSelectabilityBridge())
		{
			return SelectabilityBridge->IsObjectSelectableInViewport(InObject);
		}
	}
	return true;
}

void FSCSEditorViewportClient::UpdateHoverFromHitProxy(HHitProxy* const InHitProxy)
{
	const bool bIsViewportSelectionLimited = IsViewportSelectionLimited();

	UPrimitiveComponent* PrimitiveComponent = nullptr;
	bool bIsGizmoHit = false;
	bool bIsActorHit = false;

	if (InHitProxy)
	{
		if (InHitProxy->IsA(HWidgetAxis::StaticGetType()))
		{
			if (bIsViewportSelectionLimited)
			{
				bIsGizmoHit = true;
			}
		}
		else if (InHitProxy->IsA(HActor::StaticGetType()))
		{
			const HActor* const ActorHitProxy = static_cast<HActor*>(InHitProxy);
			if (ActorHitProxy && IsValid(ActorHitProxy->Actor))
			{
				if (bIsViewportSelectionLimited)
				{
					bIsActorHit = true;
				}
				PrimitiveComponent = const_cast<UPrimitiveComponent*>(ActorHitProxy->PrimComponent.Get());
			}
		}
	}

	FEditorViewportSelectability::UpdateHoveredPrimitive(bIsViewportSelectionLimited
		, PrimitiveComponent
		, HoveredPrimitiveComponents
		, [this](UObject* const InObject) -> bool
		{
			return IsObjectSelectableInViewport(InObject);
		});

	// Set mouse cursor after hovered primitive component list has been updated
	if (bIsGizmoHit)
	{
		MouseCursor = EMouseCursor::CardinalCross;
	}
	else if (bIsActorHit)
	{
		MouseCursor = HoveredPrimitiveComponents.IsEmpty() ? EMouseCursor::SlashedCircle : EMouseCursor::Crosshairs;
	}
	else if (bIsViewportSelectionLimited)
	{
		MouseCursor = EMouseCursor::SlashedCircle;
	}
	else
	{
		MouseCursor.Reset();
	}
}
