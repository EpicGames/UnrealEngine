// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorViewportClient.h"
#include "EditorViewportSelectability.h"
#include "Engine/EngineBaseTypes.h"
#include "InputCoreTypes.h"
#include "Math/Axis.h"
#include "Math/BoxSphereBounds.h"
#include "Math/Matrix.h"
#include "Math/Rotator.h"
#include "Math/UnrealMathSSE.h"
#include "Templates/SharedPointer.h"
#include "UnrealWidgetFwd.h"

class AActor;
class FCanvas;
class FEditorViewportSelectability;
class FPreviewScene;
class FPrimitiveDrawInterface;
class FSceneView;
class FScopedTransaction;
class FText;
class FViewport;
class SSCSEditorViewport;
class UStaticMeshComponent;
struct FInputKeyEventArgs;
struct FGizmoState;

/**
 * An editor viewport client subclass for the SCS editor viewport.
 */
class FSCSEditorViewportClient : public FEditorViewportClient, public TSharedFromThis<FSCSEditorViewportClient>
{
public:
	/**
	 * Constructor.
	 *
	 * @param InBlueprintEditorPtr A weak reference to the Blueprint Editor context.
	 * @param InPreviewScene The preview scene to use.
	 * @param InModeTools Optional pre-configured mode tools (if nullptr, a new one will be created).
	 */
	FSCSEditorViewportClient(TWeakPtr<class FBlueprintEditor>& InBlueprintEditorPtr, FPreviewScene* InPreviewScene, const TSharedRef<SSCSEditorViewport>& InSCSEditorViewport, FEditorModeTools* InModeTools = nullptr);

	/**
	 * Destructor.
	 */
	virtual ~FSCSEditorViewportClient();

	// FEditorViewportClient interface
	virtual void Tick(float DeltaSeconds) override;
	virtual void Draw(const FSceneView* View,FPrimitiveDrawInterface* PDI) override;
	virtual void DrawCanvas( FViewport& InViewport, FSceneView& View, FCanvas& Canvas ) override;
	virtual bool InputKey(const FInputKeyEventArgs& EventArgs) override;
	virtual void MouseMove(FViewport* InViewport, int32 InX, int32 InY) override;
	virtual EMouseCursor::Type GetCursor(FViewport* InViewport, int32 InX, int32 InY) override;
	virtual void ProcessClick(class FSceneView& View, class HHitProxy* HitProxy, FKey Key, EInputEvent Event, uint32 HitX, uint32 HitY) override;
	virtual bool InputWidgetDelta( FViewport* Viewport, EAxisList::Type CurrentAxis, FVector& Drag, FRotator& Rot, FVector& Scale ) override;
	virtual void TrackingStarted( const struct FInputEventState& InInputState, bool bIsDragging, bool bNudge ) override;
	virtual void TrackingStopped() override;
	virtual bool BeginTransform(const FGizmoState& InState) override;
	virtual bool EndTransform(const FGizmoState& InState) override;
	virtual UE::Widget::EWidgetMode GetWidgetMode() const override;
	virtual void SetWidgetMode( UE::Widget::EWidgetMode NewMode ) override;
	virtual void SetWidgetCoordSystemSpace( ECoordSystem NewCoordSystem ) override;
	virtual FVector GetWidgetLocation() const override;
	virtual FMatrix GetWidgetCoordSystem() const override;
	virtual ECoordSystem GetWidgetCoordSystemSpace() const override { return WidgetCoordSystem; }
	virtual const FEditorViewportCameraSpeedSettings& GetCameraSpeedSettings() const override;
	virtual void SetCameraSpeedSettings(const FEditorViewportCameraSpeedSettings& NewCameraSpeedSettings) override;


	/** 
	 * Recreates the preview scene and invalidates the owning viewport.
	 *
	 * @param bResetCamera Whether or not to reset the camera after recreating the preview scene.
	 */
	void InvalidatePreview(bool bResetCamera = true);

	/**
	 * Resets the camera position
	 */
	void ResetCamera();

	/**
	 * Determines whether or not realtime preview is enabled.
	 * 
	 * @return true if realtime preview is enabled, false otherwise.
	 */
	bool GetRealtimePreview() const
	{
		return IsRealtime();
	}

	/**
	 * Toggles realtime preview on/off.
	 */
	void ToggleRealtimePreview();

	/**
	 * Focuses the viewport on the selected components
	 */
	void FocusViewportToSelection();

	/**
	 * Returns true if simulate is enabled in the viewport
	 */
	bool GetIsSimulateEnabled();

	/**
	 * Will toggle the simulation mode of the viewport
	 */
	void ToggleIsSimulateEnabled();

	/**
	 * Returns true if the floor is currently visible in the viewport
	 */
	bool GetShowFloor();

	/**
	 * Will toggle the floor's visibility in the viewport
	 */
	void ToggleShowFloor();

	/**
	 * Returns true if the grid is currently visible in the viewport
	 */
	bool GetShowGrid();

	/**
	 * Will toggle the grid's visibility in the viewport
	 */
	void ToggleShowGrid();

	/**
	 * Gets the current preview actor instance.
	 */
	AActor* GetPreviewActor() const;

	bool IsViewportSelectionLimited() const;

	/** @return True if the specified object is selectable in the viewport and not made unselectable by the Sequencer selection limiting. */
	bool IsObjectSelectableInViewport(UObject* const InObject) const;

protected:
	/**
	 * Initiates a transaction.
	 */
	void BeginTransaction(const FText& Description);

	/**
	 * Ends the current transaction, if one exists.
	 */
	void EndTransaction();

	/**
	 * Updates preview bounds and floor positioning
	 */
	void RefreshPreviewBounds();

	void UpdateHoverFromHitProxy(HHitProxy* const InHitProxy);

private:

	UE::Widget::EWidgetMode WidgetMode;
	ECoordSystem WidgetCoordSystem;

	/** Weak reference to the editor hosting the viewport */
	TWeakPtr<class FBlueprintEditor> BlueprintEditorPtr;

	/** The full bounds of the preview scene (encompasses all visible components) */
	FBoxSphereBounds PreviewActorBounds;

	/** If true then we are manipulating a specific property or component */
	bool bIsManipulating;

	/** The current transaction for undo/redo */
	FScopedTransaction* ScopedTransaction;

	/** Floor static mesh component */
	UStaticMeshComponent* EditorFloorComp;

	/** If true, the physics simulation gets ticked */
	bool bIsSimulateEnabled;

	bool HandleBeginTransform();
	bool HandleEndTransform();

	/** Hovered primitives and their last overlay color before we apply the hover overlay */
	TMap<TWeakObjectPtr<UPrimitiveComponent>, TOptional<FColor>> HoveredPrimitiveComponents;

	TOptional<EMouseCursor::Type> MouseCursor;
};
