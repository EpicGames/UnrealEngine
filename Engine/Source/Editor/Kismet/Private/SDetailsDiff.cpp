// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDetailsDiff.h"
#include "AsyncDetailViewDiff.h"
#include "Editor.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SOverlay.h"
#include "SlateOptMacros.h"
#include "Widgets/Layout/SSpacer.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/AppStyle.h"
#include "K2Node_MathExpression.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "DetailsDiff.h"
#include "DetailTreeNode.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Framework/Application/SlateApplication.h"
#include "SBlueprintDiff.h"
#include "DiffControl.h"
#include "IDetailsView.h"
#include "SDetailsSplitter.h"

#include "Subsystems/AssetEditorSubsystem.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#define LOCTEXT_NAMESPACE "SDetailsDif"

typedef TMap< FName, const FProperty* > FNamePropertyMap;

static const FName DetailsMode = FName(TEXT("DetailsMode"));

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SDetailsDiff::Construct( const FArguments& InArgs)
{
	check(InArgs._OldAsset || InArgs._NewAsset);
	PanelOld.Object = InArgs._OldAsset;
	PanelNew.Object = InArgs._NewAsset;
	PanelOld.RevisionInfo = InArgs._OldRevision;
	PanelNew.RevisionInfo = InArgs._NewRevision;

	// sometimes we want to clearly identify the assets being diffed (when it's
	// not the same asset in each panel)
	PanelOld.bShowAssetName = InArgs._ShowAssetNames;
	PanelNew.bShowAssetName = InArgs._ShowAssetNames;

	OnCustomizeDetailsWidget = InArgs._OnCustomizeDetailsWidget;
	OnGenerateCustomDiffEntries = InArgs._OnGenerateCustomDiffEntries;
	OnGenerateCustomDiffEntryWidget = InArgs._OnGenerateCustomDiffEntryWidget;
	OnOrganizeDiffEntries = InArgs._OnOrganizeDiffEntries;
	ShouldHighlightRow = InArgs._ShouldHighlightRow;
	RowHighlightColor = InArgs._RowHighlightColor;

	if (InArgs._ParentWindow.IsValid())
	{
		WeakParentWindow = InArgs._ParentWindow;

		AssetEditorCloseDelegate = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnAssetEditorRequestClose().AddSP(this, &SDetailsDiff::OnCloseAssetEditor);
	}
	FCoreUObjectDelegates::OnObjectsReplaced.AddSP(this, &SDetailsDiff::OnObjectReplaced);

	FToolBarBuilder NavToolBarBuilder(TSharedPtr< const FUICommandList >(), FMultiBoxCustomization::None);
	NavToolBarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateSP(this, &SDetailsDiff::PrevDiff),
			FCanExecuteAction::CreateSP( this, &SDetailsDiff::HasPrevDiff)
		)
		, NAME_None
		, LOCTEXT("PrevDiffLabel", "Prev")
		, LOCTEXT("PrevDiffTooltip", "Go to previous difference")
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintDif.PrevDiff")
	);
	NavToolBarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateSP(this, &SDetailsDiff::NextDiff),
			FCanExecuteAction::CreateSP(this, &SDetailsDiff::HasNextDiff)
		)
		, NAME_None
		, LOCTEXT("NextDiffLabel", "Next")
		, LOCTEXT("NextDiffTooltip", "Go to next difference")
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintDif.NextDiff")
	);

	DifferencesTreeView = DiffTreeView::CreateTreeView(&PrimaryDifferencesList);

	GenerateDifferencesList();

	const auto TextBlock = [](FText Text) -> TSharedRef<SWidget>
	{
		return SNew(SBox)
		.Padding(FMargin(4.0f,10.0f))
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		[
			SNew(STextBlock)
			.Visibility(EVisibility::HitTestInvisible)
			.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
			.Text(Text)
		];
	};

	TopRevisionInfoWidget =
		SNew(SSplitter)
		.Visibility(EVisibility::HitTestInvisible)
		+ SSplitter::Slot()
		.Value(.2f)
		[
			SNew(SBox)
		]
		+ SSplitter::Slot()
		.Value(.8f)
		[
			SNew(SSplitter)
			.PhysicalSplitterHandleSize(10.0f)
			+ SSplitter::Slot()
			.Value(.5f)
			[
				TextBlock(DiffViewUtils::GetPanelLabel(PanelOld.Object, PanelOld.RevisionInfo, FText()))
			]
			+ SSplitter::Slot()
			.Value(.5f)
			[
				TextBlock(DiffViewUtils::GetPanelLabel(PanelNew.Object, PanelNew.RevisionInfo, FText()))
			]
		];

	this->ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush( "Docking.Tab", ".ContentAreaBrush" ))
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				.VAlign(VAlign_Top)
				[
					TopRevisionInfoWidget.ToSharedRef()		
				]
				+ SOverlay::Slot()
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 2.0f, 0.0f, 2.0f)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.Padding(4.f)
						.AutoWidth()
						[
							NavToolBarBuilder.MakeWidget()
						]
						+ SHorizontalBox::Slot()
						[
							SNew(SSpacer)
						]
					]
					+ SVerticalBox::Slot()
					[
						SNew(SSplitter)
						+ SSplitter::Slot()
						.Value(.2f)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							[
								DifferencesTreeView.ToSharedRef()
							]
						]
						+ SSplitter::Slot()
						.Value(.8f)
						[
							SAssignNew(ModeContents, SBox)
						]
					]
				]
			]
		];

	SetCurrentMode(DetailsMode);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

SDetailsDiff::~SDetailsDiff()
{
	if (AssetEditorCloseDelegate.IsValid())
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnAssetEditorRequestClose().Remove(AssetEditorCloseDelegate);
	}
}

void SDetailsDiff::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	ModePanels[CurrentMode].DiffControl->Tick();
}

void SDetailsDiff::OnCloseAssetEditor(UObject* Asset, EAssetEditorCloseReason CloseReason)
{
	if (PanelOld.Object == Asset || PanelNew.Object == Asset || CloseReason == EAssetEditorCloseReason::CloseAllAssetEditors)
	{
		// Tell our window to close and set our selves to collapsed to try and stop it from ticking
		SetVisibility(EVisibility::Collapsed);

		if (AssetEditorCloseDelegate.IsValid())
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnAssetEditorRequestClose().Remove(AssetEditorCloseDelegate);
		}

		if (WeakParentWindow.IsValid())
		{
			WeakParentWindow.Pin()->RequestDestroyWindow();
		}
	}
}

void SDetailsDiff::OnObjectReplaced(const FCoreUObjectDelegates::FReplacementObjectMap& Replacements)
{
	bool bNeedsRegenerate = false;

	auto Refresh = [](const UObject* Obj, const UObject* From, const UObject* To)
	{
		bool bNeedsRegenerate = false;
		if (Obj == From)
		{
			Obj = To;
			bNeedsRegenerate = true;
		}
		if (Obj == From->GetClass()->ClassGeneratedBy)
		{
			bNeedsRegenerate = true;
		}
		return bNeedsRegenerate;
	};
	
	// if any of the objects being displayed were reinstanced, refresh them
	for (const auto &[From, To] : Replacements)
	{
		bNeedsRegenerate |= Refresh(OutputObject, From, To);
		bNeedsRegenerate |= Refresh(PanelNew.Object, From, To);
		bNeedsRegenerate |= Refresh(PanelOld.Object, From, To);
	}
	
	if (bNeedsRegenerate)
	{
		GenerateDifferencesList();
		RefreshCurrentModePanel();
	}
}

TSharedRef<SWidget> SDetailsDiff::DefaultEmptyPanel()
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BlueprintDifGraphsToolTip", "Select Graph to Diff"))
		];
}

TSharedRef<SDetailsDiff> SDetailsDiff::CreateDiffWindow(FText WindowTitle, const UObject* OldObject, const UObject* NewObject, const FRevisionInfo& OldRevision, const FRevisionInfo& NewRevision)
{
	// sometimes we're comparing different revisions of one single asset (other 
	// times we're comparing two completely separate assets altogether)
	const bool bIsSingleAsset = !NewObject || !OldObject || (NewObject->GetName() == OldObject->GetName());

	TSharedPtr<SWindow> Window = SNew(SWindow)
		.Title(WindowTitle)
		.ClientSize(FVector2D(1000.f, 800.f));

	TSharedRef<SDetailsDiff> DetailsDiff = SNew(SDetailsDiff)
		.OldAsset(OldObject)
		.NewAsset(NewObject)
		.OldRevision(OldRevision)
		.NewRevision(NewRevision)
		.ShowAssetNames(!bIsSingleAsset)
		.ParentWindow(Window);
	
	Window->SetContent(DetailsDiff);

	// Make this window a child of the modal window if we've been spawned while one is active.
	const TSharedPtr<SWindow> ActiveModal = FSlateApplication::Get().GetActiveTopLevelWindow();
	if (ActiveModal.IsValid())
	{
		FSlateApplication::Get().AddWindowAsNativeChild(Window.ToSharedRef(), ActiveModal.ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow(Window.ToSharedRef());
	}

	TWeakPtr<SDetailsDiff> SelfWeak = DetailsDiff.ToWeakPtr();
	Window->SetOnWindowClosed(::FOnWindowClosed::CreateLambda([SelfWeak](const TSharedRef<SWindow>&)
	{
		if (const TSharedPtr<SDetailsDiff> Self = SelfWeak.Pin())
		{
			Self->OnWindowClosedEvent.Broadcast(Self.ToSharedRef());
		}
	}));

	return DetailsDiff;
}

TSharedRef<SDetailsDiff> SDetailsDiff::CreateDiffWindow(const UObject* OldObject, const UObject* NewObject,
                                                   const FRevisionInfo& OldRevision, const FRevisionInfo& NewRevision, const UClass* ObjectClass)
{
	check(OldObject || NewObject);

	// sometimes we're comparing different revisions of one single asset (other 
	// times we're comparing two completely separate assets altogether)
	const bool bIsSingleAsset = !OldObject || !NewObject || (NewObject->GetName() == OldObject->GetName());

	FText WindowTitle = FText::Format(LOCTEXT("NamelessBlueprintDiff", "{0} Diff"), ObjectClass->GetDisplayNameText());
	// if we're diffing one asset against itself 
	if (bIsSingleAsset)
	{
		// identify the assumed single asset in the window's title
		const FString BPName = NewObject? NewObject->GetName() : OldObject->GetName();
		WindowTitle = FText::Format(LOCTEXT("NamedBlueprintDiff", "{0} - {1} Diff"), FText::FromString(BPName), ObjectClass->GetDisplayNameText());
	}

	return CreateDiffWindow(WindowTitle, OldObject, NewObject, OldRevision, NewRevision);
}

void SDetailsDiff::SetOutputObject(UObject* InOutputObject)
{
	OutputObject = InOutputObject;
	OnOutputObjectSetEvent.Broadcast();
}

UObject* SDetailsDiff::GetOutputObject() const
{
	return OutputObject;
}

bool SDetailsDiff::IsOutputEnabled() const
{
	return OutputObject != nullptr;
}

void SDetailsDiff::ReportMergeConflicts(const TMap<FString, TMap<FPropertySoftPath, ETreeDiffResult>>& Conflicts)
{
	MergeConflicts = Conflicts;
}

void SDetailsDiff::NextDiff()
{
	DiffTreeView::HighlightNextDifference(DifferencesTreeView.ToSharedRef(), RealDifferences, PrimaryDifferencesList);
}

void SDetailsDiff::PrevDiff()
{
	DiffTreeView::HighlightPrevDifference(DifferencesTreeView.ToSharedRef(), RealDifferences, PrimaryDifferencesList);
}

bool SDetailsDiff::HasNextDiff() const
{
	return DiffTreeView::HasNextDifference(DifferencesTreeView.ToSharedRef(), RealDifferences);
}

bool SDetailsDiff::HasPrevDiff() const
{
	return DiffTreeView::HasPrevDifference(DifferencesTreeView.ToSharedRef(), RealDifferences);
}

void SDetailsDiff::OnDiffListSelectionChanged(TSharedPtr<FDiffResultItem> TheDiff )
{
	check( !TheDiff->Result.OwningObjectPath.IsEmpty() );
	// TODO: What do I put here?
}

void SDetailsDiff::GenerateDifferencesList()
{
	PrimaryDifferencesList.Empty();
	RealDifferences.Empty();
	ModePanels.Empty();
	OnOutputObjectSetEvent.Clear(); // will be repopulated by ModePanel generation methods

	// Now that we have done the diffs, create the panel widgets
	// (we're currently only generating the details panel but we can add more as needed)

	const auto GetBlueprintCDO = [](const UObject* Object)->const UObject*
	{
		return CastChecked<UBlueprint>(Object)->GeneratedClass->GetDefaultObject();
	};
	
	TFunction<const UObject*(const UObject*)> Redirector;
	if ((!PanelOld.Object || PanelOld.Object->IsA<UBlueprint>()) && (!PanelNew.Object || PanelNew.Object->IsA<UBlueprint>()))
	{
		// Blueprints diff their GeneratedClass CDO in the details panel instead
		Redirector = GetBlueprintCDO;
	}
	
	ModePanels.Add(DetailsMode, GenerateDetailsPanel(Redirector));

	DifferencesTreeView->RebuildList();
}


SDetailsDiff::FDiffControl SDetailsDiff::GenerateDetailsPanel(const TFunction<const UObject*(const UObject*)>& Redirector)
{
	const UObject* OldObject = Redirector ? Redirector(PanelOld.Object) : PanelOld.Object;
	const UObject* NewObject = Redirector ? Redirector(PanelNew.Object) : PanelNew.Object;

	const TSharedPtr<FDetailsDiffControl> NewDiffControl = MakeShared<FDetailsDiffControl>(OldObject, NewObject, FOnDiffEntryFocused::CreateRaw(this, &SDetailsDiff::SetCurrentMode, DetailsMode), true);
	NewDiffControl->EnableComments(DifferencesTreeView.ToWeakPtr());
	NewDiffControl->GenerateTreeEntries(PrimaryDifferencesList, RealDifferences);
	NewDiffControl->GenerateCustomEntriesCallback = OnGenerateCustomDiffEntries;
	NewDiffControl->GenerateCustomEntryWidgetCallback = OnGenerateCustomDiffEntryWidget;
	NewDiffControl->OrganizeEntriesCallback = OnOrganizeDiffEntries;
	if (OnCustomizeDetailsWidget.IsBound())
	{
		OnCustomizeDetailsWidget.Execute(NewDiffControl->GetDetailsWidget(OldObject));
		OnCustomizeDetailsWidget.Execute(NewDiffControl->GetDetailsWidget(NewObject));
	}

	const TSharedRef<SDetailsSplitter> Splitter = SNew(SDetailsSplitter)
		.ShouldHighlightRow(ShouldHighlightRow)
		.RowHighlightColor(RowHighlightColor);

	if (OldObject)
	{
		Splitter->AddSlot(
			SDetailsSplitter::Slot()
			.Value(0.5f)
			.DetailsView(NewDiffControl->GetDetailsWidget(OldObject))
			.DifferencesWithRightPanel(NewDiffControl.ToSharedRef(), &FDetailsDiffControl::GetDifferencesWithRight, Cast<UObject>(OldObject))
		);
	}
	if (NewObject)
	{
		Splitter->AddSlot(
			SDetailsSplitter::Slot()
			.Value(0.5f)
			.DetailsView(NewDiffControl->GetDetailsWidget(NewObject))
			.DifferencesWithRightPanel(NewDiffControl.ToSharedRef(), &FDetailsDiffControl::GetDifferencesWithRight, Cast<UObject>(NewObject))
		);
	}

	
	const TWeakPtr<SDetailsSplitter> WeakSplitter = Splitter;
	const TWeakPtr<FDetailsDiffControl> WeakDiffControl = NewDiffControl;
	OnOutputObjectSetEvent.AddLambda([WeakSplitter, WeakDiffControl, this, Redirector]()
	{
		const UObject* OutputDetailObject = Redirector ? Redirector(OutputObject) : OutputObject;
		
		const TSharedPtr<SDetailsSplitter> Splitter = WeakSplitter.Pin();
		const TSharedPtr<FDetailsDiffControl> DiffControl = WeakDiffControl.Pin();
		if (Splitter && DiffControl)
		{
			// if output object is already in panel, don't insert a new one
			TSharedPtr<IDetailsView> DetailsView = DiffControl->TryGetDetailsWidget(OutputDetailObject);
			if (DetailsView)
			{
				// update readonly status in splitter so that property merge buttons appear
				const int32 Index = DiffControl->IndexOfObject(OutputDetailObject);
				Splitter->GetPanel(Index).IsReadonly = false;
			}
			else
			{
				DetailsView = DiffControl->InsertObject(OutputDetailObject, false, 1);
				// insert the output object as a central panel
				Splitter->AddSlot(
					SDetailsSplitter::Slot()
						.DetailsView(DetailsView)
						.Value(0.5f)
						.IsReadonly(false)
						.DifferencesWithRightPanel(DiffControl.ToSharedRef(), &FDetailsDiffControl::GetDifferencesWithRight, (const UObject*)OutputDetailObject),
					1 // insert between left and right panel (index 1)
				);
			}

			// allow user to edit the output panel
			DetailsView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateStatic([]{return true; }));
			
			// highlight merge conflicts
			TMap<FString, TMap<FPropertySoftPath, FLinearColor>> Highlights;
			for (auto& [ObjectPath, Properties] : MergeConflicts)
			{
				for (auto& [propertyPath, Diff] : Properties)
				{
					switch(Diff)
					{
					case ETreeDiffResult::MissingFromTree1: // fall through
					case ETreeDiffResult::MissingFromTree2: // fall through
					case ETreeDiffResult::DifferentValues:
						// color is intentionally using values greater than 1 so that it stays very saturated
						Highlights.FindOrAdd(ObjectPath).Add(propertyPath, FLinearColor(1.5f, 0.3f, 0.3f));
						break;
					
					default:; // ignore identical and invalid
					}
				}
			}

			Splitter->HighlightFromMergeResults(MergeConflicts);
		}
	});
	if (OutputObject)
	{
		OnOutputObjectSetEvent.Broadcast();
	}

	SDetailsDiff::FDiffControl Ret;
	Ret.DiffControl = NewDiffControl;
	Ret.Widget = Splitter;
	return Ret;
}

TSharedRef<SBox> SDetailsDiff::GenerateRevisionInfoWidgetForPanel(TSharedPtr<SWidget>& OutGeneratedWidget, const FText& InRevisionText) const
{
	return SAssignNew(OutGeneratedWidget,SBox)
		.Padding(FMargin(4.0f, 10.0f))
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		[
			SNew(STextBlock)
			.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
			.Text(InRevisionText)
			.ShadowColorAndOpacity(FColor::Black)
			.ShadowOffset(FVector2D(1.4,1.4))
		];
}

void SDetailsDiff::SetCurrentMode(FName NewMode)
{
	if (CurrentMode == NewMode)
	{
		return;
	}

	CurrentMode = NewMode;

	FDiffControl* FoundControl = ModePanels.Find(NewMode);

	if (FoundControl)
	{
		ModeContents->SetContent(FoundControl->Widget.ToSharedRef());
	}
	else
	{
		ensureMsgf(false, TEXT("Diff panel does not support mode %s"), *NewMode.ToString() );
	}

	OnModeChanged(NewMode);
}

void SDetailsDiff::RefreshCurrentModePanel()
{

	FDiffControl* FoundControl = ModePanels.Find(CurrentMode);

	if (FoundControl)
	{
		ModeContents->SetContent(FoundControl->Widget.ToSharedRef());
	}
	else
	{
		ensureMsgf(false, TEXT("Diff panel does not support mode %s"), *CurrentMode.ToString() );
	}

	OnModeChanged(CurrentMode);
}

void SDetailsDiff::UpdateTopSectionVisibility(const FName& InNewViewMode) const
{
	SSplitter* TopRevisionInfoWidgetPtr = TopRevisionInfoWidget.Get();

	if (!TopRevisionInfoWidgetPtr)
	{
		return;
	}
	
	TopRevisionInfoWidgetPtr->SetVisibility(EVisibility::HitTestInvisible);
}

void SDetailsDiff::OnModeChanged(const FName& InNewViewMode) const
{
	UpdateTopSectionVisibility(InNewViewMode);
}

#undef LOCTEXT_NAMESPACE

