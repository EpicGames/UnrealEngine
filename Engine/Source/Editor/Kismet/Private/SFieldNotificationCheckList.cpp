// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFieldNotificationCheckList.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "INotifyFieldValueChanged.h"
#include "Layout/Children.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_VariableSet.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ScriptInterface.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SFieldNotificationCheckList"
namespace UE::FieldNotification
{

void SFieldNotificationCheckList::Construct(const FArguments& InArgs)
{
	BlueprintPtr = InArgs._BlueprintPtr;
	FieldName = InArgs._FieldName;

	SAssignNew(ComboListView, SComboListType)
			.ListItemsSource(&FieldNotificationIdsSource)
			.SelectionMode(ESelectionMode::None)
			.OnGenerateRow(this, &SFieldNotificationCheckList::GenerateMenuItemRow);

	TSharedPtr<SWidget> ButtonContent = SNew(SImage)
		.Image(FAppStyle::Get().GetBrush("Kismet.VariableList.FieldNotify"))
		.Visibility(this, &SFieldNotificationCheckList::GetFieldNotifyIconVisibility);

	ChildSlot
	[
		SNew(SComboButton)
		.ToolTipText(LOCTEXT("FieldNotifyListTooltip", "Select which field in this blueprint class will be notified when the current property changes."))
		.ContentPadding(FMargin(2.0f, 2.0f))
		.OnMenuOpenChanged(this, &SFieldNotificationCheckList::OnMenuOpenChanged)
		.IsFocusable(true)
		.MenuContent()
		[
			ComboListView.ToSharedRef()
		]
		.ButtonContent()
		[
			ButtonContent.ToSharedRef()
		]
	];
}

TSharedRef<ITableRow> SFieldNotificationCheckList::GenerateMenuItemRow(TSharedPtr<FFieldNotificationId> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	// Each row is a checkbox. Depending on whether we selected a function or a variable, we bind different delegates for the state of the checkbox.
	TSharedPtr<SWidget> RowCheckBox =
		SNew(SCheckBox)
		.IsChecked(this, &SFieldNotificationCheckList::OnVariableCheckboxState, InItem->GetFieldName())
		.OnCheckStateChanged(this, &SFieldNotificationCheckList::OnVariableCheckBoxChanged, InItem->GetFieldName())
		.IsEnabled(this, &SFieldNotificationCheckList::IsCheckBoxEnabled, InItem->GetFieldName())
		.Content()
		[
			SNew(STextBlock)
			.Text(FText::FromName(InItem->GetFieldName()))
		];

	return SNew(SComboRow<TSharedPtr<FFieldNotificationId>>, OwnerTable)
			[
				SNew(SBox)
				.Padding(10.0f, 5.0f)
				[
					RowCheckBox.ToSharedRef()
				]
			];
}

bool SFieldNotificationCheckList::IsCheckBoxEnabled(FName OtherName) const
{
	return OtherName != FieldName;
}

ECheckBoxState SFieldNotificationCheckList::OnVariableCheckboxState(FName OtherName) const
{
	if (OtherName == FieldName)
	{
		return ECheckBoxState::Checked;
	}
	UBlueprint* BlueprintObj = BlueprintPtr.Get();
	if (BlueprintObj && !OtherName.IsNone())
	{
		const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(BlueprintObj, FieldName);
		if (VarIndex != INDEX_NONE)
		{
			// Parse the metadata of the variable by delimiter | and check if OtherName is in there.
			const FBPVariableDescription& VariableDescription = BlueprintObj->NewVariables[VarIndex];
			FString FieldNotifyValues = VariableDescription.HasMetaData(FBlueprintMetadata::MD_FieldNotify) ? VariableDescription.GetMetaData(FBlueprintMetadata::MD_FieldNotify) : FString();
			TArray<FString> ListOfFieldNotifies;
			const TCHAR* Delimiter = TEXT("|");

			FieldNotifyValues.ParseIntoArray(ListOfFieldNotifies, Delimiter);
			return ListOfFieldNotifies.Contains(OtherName.ToString()) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
	}
	return ECheckBoxState::Unchecked;
}

void SFieldNotificationCheckList::OnVariableCheckBoxChanged(ECheckBoxState InNewState, FName OtherName)
{
	UBlueprint* BlueprintObj = BlueprintPtr.Get();
	if (BlueprintObj && !OtherName.IsNone())
	{
		const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(BlueprintObj, FieldName);
		if (VarIndex != INDEX_NONE)
		{
			const FBPVariableDescription& VariableDescription = BlueprintObj->NewVariables[VarIndex];
			FString FieldNotifyValues = VariableDescription.HasMetaData(FBlueprintMetadata::MD_FieldNotify) ? VariableDescription.GetMetaData(FBlueprintMetadata::MD_FieldNotify) : FString();
			if (InNewState == ECheckBoxState::Checked)
			{
				const TCHAR* Delimiter = TEXT("|");
				FieldNotifyValues = FieldNotifyValues.Len() > 0 ? FieldNotifyValues + Delimiter + OtherName.ToString() : OtherName.ToString();
				BlueprintObj->NewVariables[VarIndex].SetMetaData(FBlueprintMetadata::MD_FieldNotify, FieldNotifyValues);
			}
			else
			{
				TArray<FString> ListOfFieldNotifies;
				const TCHAR* Delimiter = TEXT("|");
				FieldNotifyValues.ParseIntoArray(ListOfFieldNotifies, Delimiter);
				ListOfFieldNotifies.Remove(OtherName.ToString());
				if (ListOfFieldNotifies.Num() > 0)
				{
					BlueprintObj->NewVariables[VarIndex].SetMetaData(FBlueprintMetadata::MD_FieldNotify, FString::Join(ListOfFieldNotifies, Delimiter));
				}
				else
				{
					BlueprintObj->NewVariables[VarIndex].SetMetaData(FBlueprintMetadata::MD_FieldNotify, TEXT(""));
				}
			}

			// Brute force mark all dependent blueprints modified if they have setter nodes for this member
			// Use asset registry to find all referencing blueprints
			TArray<FAssetIdentifier> Referencers;
			const FAssetIdentifier Source(BlueprintObj->GetPackage()->GetFName(), NAME_None);
			IAssetRegistry::Get()->GetReferencers(Source, Referencers);

			TArray<UBlueprint*> OtherBlueprints;
			for (const FAssetIdentifier& AssetIdentifier : Referencers)
			{
				TArray<FAssetData> Assets;
				IAssetRegistry::Get()->GetAssetsByPackageName(AssetIdentifier.PackageName, Assets);

				for (const FAssetData& AssetData : Assets)
				{
					if (AssetData.GetClass()->IsChildOf(UBlueprint::StaticClass()))
					{
						OtherBlueprints.Add(Cast<UBlueprint>(AssetData.GetAsset()));
					}
				}
			}

			// Mark all the blueprints modified if they have broadcasting setter nodes that refer to a variable with this ones name. 
			for (UBlueprint* OtherBlueprint : OtherBlueprints)
			{
				TArray<UK2Node_VariableSet*> SetNodes;
				FBlueprintEditorUtils::GetAllNodesOfClass(OtherBlueprint, SetNodes);

				for (const UK2Node_VariableSet* SetNode : SetNodes)
				{
					if (SetNode->HasFieldNotificationBroadcast() && SetNode->VariableReference.GetMemberName() == FieldName)
					{
						FBlueprintEditorUtils::MarkBlueprintAsModified(OtherBlueprint);
						break;
					}
				}
			}
		}
	}
}

TArray<UK2Node_FunctionEntry*> SFieldNotificationCheckList::GetFieldNotifyFunctionEntryNodesInBlueprint()
{
	UBlueprint* BlueprintObj = BlueprintPtr.Get();
	TArray<UK2Node_FunctionEntry*> FunctionEntryNodes = TArray<UK2Node_FunctionEntry*>();
	if (BlueprintObj)
	{
		for (int32 i = 0; i < BlueprintObj->FunctionGraphs.Num(); ++i)
		{
			TObjectPtr<UEdGraph>& FunctionGraph = BlueprintObj->FunctionGraphs[i];
			for (UEdGraphNode* Node : FunctionGraph->Nodes)
			{
				if (UK2Node_FunctionEntry* NodeFunctionEntry = Cast<UK2Node_FunctionEntry>(Node))
				{
					FunctionEntryNodes.Add(NodeFunctionEntry);
					break;
				}
			}
		}
	}
	return FunctionEntryNodes;
}

void SFieldNotificationCheckList::OnMenuOpenChanged(bool bOpen)
{
	FieldNotificationIdsSource.Reset();
	UBlueprint* BlueprintObj = BlueprintPtr.Get();

	if (BlueprintObj)
	{
		if (UClass* Class = BlueprintObj->GeneratedClass)
		{
			// Populate the source array when the menu opens.
			// First, add all the fields from the Descriptor to the source array
			TScriptInterface<INotifyFieldValueChanged> ScriptObject = Class->GetDefaultObject();
			if (ScriptObject.GetInterface() && ScriptObject.GetObject())
			{
				SFieldNotificationCheckList* Self = this;
				const UE::FieldNotification::IClassDescriptor& Descriptor = ScriptObject->GetFieldNotificationDescriptor();
				Descriptor.ForEachField(Class, [Self, Class](const FFieldId& Id) ->bool
					{
						Self->FieldNotificationIdsSource.Add(MakeShared<FFieldNotificationId>(Id.GetName()));
						return true;
					});
			}

			// The class descriptor is updated on compile, so we may have some outdated properties that are not field notify anymore
			// or new properties that are field notify. So we check with the blueprint variables to get the latest data.
			for (int32 i = 0; i < BlueprintObj->NewVariables.Num(); ++i)
			{
				FBPVariableDescription& Variable = BlueprintObj->NewVariables[i];
				bool bIsValidFieldNotify = Variable.HasMetaData(FBlueprintMetadata::MD_FieldNotify);
				// Try to locate this variable name in the source array.
				int32 FoundValueIndex = FieldNotificationIdsSource.IndexOfByPredicate([Variable](const TSharedPtr<FFieldNotificationId>& Other)
					{
						return Other->GetFieldName() == Variable.VarName;
					});

				// If this variable is not found in the source array but it is field notify, add it.
				if (FoundValueIndex == INDEX_NONE && bIsValidFieldNotify)
				{
					FieldNotificationIdsSource.Add(MakeShared<FFieldNotificationId>(FName(Variable.VarName)));
				}
				// If we found this variable in the source array but it's not field notify anymore, remove it.
				else if (FoundValueIndex != INDEX_NONE && !bIsValidFieldNotify)
				{
					FieldNotificationIdsSource.RemoveAtSwap(FoundValueIndex);
				}
			}

			// Repeat the same logic above for functions. 
			TArray<UK2Node_FunctionEntry*> NewFunctions = GetFieldNotifyFunctionEntryNodesInBlueprint();
			for (int32 i = 0; i < NewFunctions.Num(); ++i)
			{
				UK2Node_FunctionEntry* EntryNode = NewFunctions[i];
				bool bIsValidFieldNotify = EntryNode->MetaData.HasMetaData(FBlueprintMetadata::MD_FieldNotify);

				int32 FoundValueIndex = FieldNotificationIdsSource.IndexOfByPredicate([EntryNode](const TSharedPtr<FFieldNotificationId>& Other)
					{
						return Other->GetFieldName() == EntryNode->FunctionReference.GetMemberName();
					});
				if (FoundValueIndex == INDEX_NONE && bIsValidFieldNotify)
				{
					FieldNotificationIdsSource.Add(MakeShared<FFieldNotificationId>(EntryNode->FunctionReference.GetMemberName()));
				}
				else if (FoundValueIndex != INDEX_NONE && !bIsValidFieldNotify)
				{
					FieldNotificationIdsSource.RemoveAtSwap(FoundValueIndex);
				}
			}
			
			// Get the list of field notifies from the last time we compiled.
			// Detect if we have deleted any functions and variables since the last compile and update the source array.
			TArray<FFieldNotificationId> LastFieldNotifyList = CastChecked<UBlueprintGeneratedClass>(Class)->FieldNotifies;

			for (int32 i = 0; i < LastFieldNotifyList.Num(); ++i)
			{
				FFieldNotificationId Id = LastFieldNotifyList[i];
				int32 FoundIndexInBlueprint = BlueprintObj->NewVariables.IndexOfByPredicate([Id](const FBPVariableDescription& Other)
					{
						return Other.VarName == Id.GetFieldName();
					});
				if (FoundIndexInBlueprint == INDEX_NONE)
				{
					FoundIndexInBlueprint = BlueprintObj->FunctionGraphs.IndexOfByPredicate([Id](const TObjectPtr<UEdGraph>& Other)
						{
							return Other->GetFName() == Id.GetFieldName();
						});

					if (FoundIndexInBlueprint == INDEX_NONE)
					{
						int32 FoundValueIndex = FieldNotificationIdsSource.IndexOfByPredicate([Id](const TSharedPtr<FFieldNotificationId>& Other)
							{
								return Other->GetFieldName() == Id.GetFieldName();
							});

						if (FoundValueIndex != INDEX_NONE)
						{
							FieldNotificationIdsSource.RemoveAtSwap(FoundValueIndex);
						}
					}
				}
			}

			// Sort the source array alphabetically.
			if (FieldNotificationIdsSource.Num())
			{
				FieldNotificationIdsSource.Sort([](const TSharedPtr<FFieldNotificationId>& A, const TSharedPtr<FFieldNotificationId>& B)
				{
					return A->GetFieldName().LexicalLess(B->GetFieldName());
				});
			}
		}
	}
}

EVisibility SFieldNotificationCheckList::GetFieldNotifyIconVisibility() const
{
	UBlueprint* BlueprintObj = BlueprintPtr.Get();
	if (BlueprintObj)
	{
		const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(BlueprintObj, FieldName);
		if (VarIndex != INDEX_NONE)
		{
			const FBPVariableDescription& VariableDescription = BlueprintObj->NewVariables[VarIndex];
			FString FieldNotifyValues = VariableDescription.HasMetaData(FBlueprintMetadata::MD_FieldNotify) ? VariableDescription.GetMetaData(FBlueprintMetadata::MD_FieldNotify) : FString();
			return FieldNotifyValues.Len() > 0 ? EVisibility::Visible : EVisibility::Collapsed;
		}
	}
	return EVisibility::Collapsed;
}

}
#undef LOCTEXT_NAMESPACE
