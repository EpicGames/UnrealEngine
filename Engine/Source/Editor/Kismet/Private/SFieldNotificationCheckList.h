// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "FieldNotificationId.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class ITableRow;
class STableViewBase;
class SWidget;
class UK2Node_FunctionEntry;
enum class ECheckBoxState : uint8;
template <typename ItemType> class SListView;

namespace UE::FieldNotification
{

/**
 * A widget in details panel which allows the user to choose which field notify variables/functions will be notified when the selected property changes.
 */
class SFieldNotificationCheckList : public SCompoundWidget
{
public:
	typedef SListView< TSharedPtr<FFieldNotificationId> > SComboListType;

	SLATE_BEGIN_ARGS(SFieldNotificationCheckList)
	{}
		SLATE_ARGUMENT(TWeakObjectPtr<UBlueprint>, BlueprintPtr)
		SLATE_ARGUMENT(FName, FieldName)
		SLATE_ARGUMENT(bool, IsFieldFunction)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	EVisibility GetFieldNotifyIconVisibility() const;

	TSharedRef<ITableRow> GenerateMenuItemRow(TSharedPtr<FFieldNotificationId> InItem, const TSharedRef<STableViewBase>& OwnerTable);
	void OnMenuOpenChanged(bool bOpen);

	bool IsCheckBoxEnabled(FName OtherName) const;
	ECheckBoxState OnVariableCheckboxState(FName OtherName) const;
	void OnVariableCheckBoxChanged(ECheckBoxState InNewState, FName OtherName);

	/** Iterates through all graphs in the blueprint to find function graphs that are field notify and adds their entry node to OutArray. */
	TArray<UK2Node_FunctionEntry*> GetFieldNotifyFunctionEntryNodesInBlueprint();

private:
	/** Array of all field notify variables and functions in the owning blueprint. */
	TArray<TSharedPtr<FFieldNotificationId>> FieldNotificationIdsSource;
	TWeakObjectPtr<UBlueprint> BlueprintPtr;
	TSharedPtr< SComboListType > ComboListView;
	/** The name of the selected field in the blueprint editor.*/
	FName FieldName;
};

} // namespace