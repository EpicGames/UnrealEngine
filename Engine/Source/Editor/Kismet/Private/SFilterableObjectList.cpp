// Copyright Epic Games, Inc. All Rights Reserved.


#include "SFilterableObjectList.h"

#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Misc/Attribute.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class ITableRow;
class STableViewBase;

#define LOCTEXT_NAMESPACE "SFilterableObjectList"

//////////////////////////////////////////////////////////////////////////
// SFilterableObjectList

void SFilterableObjectList::OnFilterTextChanged( const FText& InFilterText )
{
	ReapplyFilter();
}

FReply SFilterableObjectList::OnRefreshButtonClicked()
{
	RebuildObjectList();
	ReapplyFilter();
	return FReply::Handled();
}

EVisibility SFilterableObjectList::GetFilterStatusVisibility() const
{
	return IsFilterActive() ? EVisibility::Visible : EVisibility::Collapsed;
}

FText SFilterableObjectList::GetFilterStatusText() const
{
	return FText::Format(LOCTEXT("FilterStatus_ShowingXOfYFmt", "Showing {0} of {1}"), FText::AsNumber(FilteredObjectList.Num()), FText::AsNumber(LoadedObjectList.Num()));
}

FString SFilterableObjectList::GetSearchableText(UObject* Object)
{
	return Object->GetName();
}

bool SFilterableObjectList::IsFilterActive() const
{
	return FilteredObjectList.Num() != LoadedObjectList.Num();
}

void SFilterableObjectList::ReapplyFilter()
{
	RefilterObjectList();

	if (ObjectListWidget.IsValid())
	{
		ObjectListWidget->RequestListRefresh();
	}
}

void SFilterableObjectList::Construct(const FArguments& InArgs)
{
	InternalConstruct();
}

void SFilterableObjectList::InternalConstruct()
{
	RebuildObjectList();
	RefilterObjectList();

	this->ChildSlot
	[
		SNew(SVerticalBox)

		// The filter line
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)

			// Filter text box
			+SHorizontalBox::Slot()
				.FillWidth(1)
			[
				SAssignNew(FilterTextBoxWidget, SSearchBox)
					.ToolTipText( LOCTEXT("SearchBox_ToolTip", "Type words to search for") )
					.OnTextChanged( this, &SFilterableObjectList::OnFilterTextChanged )
			]

			// Refresh button (rescans for newly loaded objects; then reruns the filter on the new list)
			+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ToolTipText( LOCTEXT("Refresh_ToolTip", "Search for new entries") )
				.OnClicked( this, &SFilterableObjectList::OnRefreshButtonClicked )
				[
					SNew(SImage)
						.Image( FAppStyle::GetBrush(TEXT("AnimEditor.RefreshButton")) )
				]
			]
		]

		// The filter status line; shows how many items made it past the filter
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(STextBlock)
			.Visibility( this, &SFilterableObjectList::GetFilterStatusVisibility )
			.Text( this, &SFilterableObjectList::GetFilterStatusText )
		]

		// The (possibly filtered) list of items
		+SVerticalBox::Slot()
			.FillHeight(1)
			.Padding(2.0f)
		[
			SNew(SBorder)
			[
				SAssignNew(ObjectListWidget, SListView<UObject*>)
					.ListItemsSource(&FilteredObjectList)
					.OnGenerateRow(this, &SFilterableObjectList::OnGenerateTableRow )
			]
		]
	];
}

SFilterableObjectList::FListRow SFilterableObjectList::GenerateRowForObject(UObject* InData)
{
	return
		FListRow(
			SNew(STextBlock).Text(FText::FromString(InData->GetName())),
			FOnDragDetected()
		);
}

TSharedRef<ITableRow> SFilterableObjectList::OnGenerateTableRow(UObject* InData, const TSharedRef<STableViewBase>& OwnerTable)
{
	FListRow GenerateRow = GenerateRowForObject(InData);
	return
		SNew( STableRow< UObject* >, OwnerTable )
		. OnDragDetected( GenerateRow.OnDragDetected_Handler )
		[
			GenerateRow.Widget
		];
}

void SFilterableObjectList::RebuildObjectList()
{
	LoadedObjectList.Empty();
}

void SFilterableObjectList::RefilterObjectList()
{
	// Tokenize the search box text into a set of terms; all of them must be present to pass the filter
	TArray<FString> FilterTerms;
	if (FilterTextBoxWidget.IsValid())
	{
		FilterTextBoxWidget->GetText().ToString().ParseIntoArray(FilterTerms, TEXT(" "), true);
	}

	if (FilterTerms.Num())
	{
		FilteredObjectList.Empty();

		// Run thru each item in the list, checking it against the text filter
		for (int32 ObjectIndex = 0; ObjectIndex < LoadedObjectList.Num(); ++ObjectIndex)
		{
			UObject* TestObject = LoadedObjectList[ObjectIndex];

			FString SearchText = GetSearchableText(TestObject);

			bool bMatchesAllTerms = true;
			for (int32 FilterIndex = 0; (FilterIndex < FilterTerms.Num()) && bMatchesAllTerms; ++FilterIndex)
			{
				const bool bMatchesTerm = SearchText.Contains(FilterTerms[FilterIndex]);
				bMatchesAllTerms = bMatchesAllTerms && bMatchesTerm;
			}

			if (bMatchesAllTerms)
			{
				FilteredObjectList.Add(TestObject);
			}
		}
	}
	else
	{
		// Nothing to filter, just copy the list
		FilteredObjectList = LoadedObjectList; 
	}
}

#undef LOCTEXT_NAMESPACE
