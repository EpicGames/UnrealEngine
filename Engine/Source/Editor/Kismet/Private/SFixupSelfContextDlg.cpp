// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFixupSelfContextDlg.h"

#include "BlueprintEditor.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Blueprint.h"
#include "Framework/Views/ITypedTableView.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node_CallFunction.h"
#include "K2Node_CustomEvent.h"
#include "K2Node_Event.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/SubclassOf.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class ITableRow;
class STableViewBase;
class SWidget;
class UFunction;

#define LOCTEXT_NAMESPACE "FixupContextDialog"

TSharedRef<SWidget> FFixupSelfContextItem::CreateWidget(TArray<TSharedPtr<FString>>& InFixupOptions)
{
	return SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		[
			SNew(STextBlock)
			.Text(FText::FromName(FuncName))
		]
		+SHorizontalBox::Slot()
		[
			SAssignNew(ComboBox, STextComboBox)
			.OptionsSource(&InFixupOptions)
			.InitiallySelectedItem(InFixupOptions[0])
		];
}

void SFixupSelfContextDialog::Construct(const FArguments& InArgs, const TArray< UK2Node_CallFunction* >& InNodesToFixup, UBlueprint* InFromBP, const FBlueprintEditor* InBlueprintEditorPtr, bool bInOtherPastedNodes)
{
	NodesToFixup = InNodesToFixup;
	BlueprintEditor = InBlueprintEditorPtr;
	bOtherNodes = bInOtherPastedNodes;
	FromBP = InFromBP;

	Options.Add(MakeShared<FString>(LOCTEXT("DoNothing", "Do Nothing").ToString()));
	Options.Add(MakeShared<FString>(LOCTEXT("CreateMatchingFunction", "Create Matching Function in Blueprint").ToString()));
	Options.Add(MakeShared<FString>(LOCTEXT("RemoveNodes", "Remove Node(s)").ToString()));

	for (UK2Node_CallFunction* Node : NodesToFixup)
	{
		bool bIsNew = true;
		for (FListViewItem Func : FunctionsToFixup)
		{
			if (Func->FuncName == Node->GetFunctionName())
			{
				bIsNew = false;
				Func->Nodes.Add(Node);
			}
		}

		if (bIsNew)
		{
			FunctionsToFixup.Add(MakeShared<FFixupSelfContextItem>(Node->GetFunctionName()));
			FunctionsToFixup.Last()->Nodes.Add(Node);
		}
	}

	ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("FixupDescription", "Some function references could not be resolved in the new context. How would you like to fix them?"))
				.AutoWrapText(true)
			]

			+SVerticalBox::Slot()
			.Padding(5.0f, 5.0f)
			[
				SNew(SBox)
				.MinDesiredHeight(100.0f)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("Menu.Background"))
					[
						SNew(SListView<FListViewItem>)
						.ListItemsSource(&FunctionsToFixup)
						.SelectionMode(ESelectionMode::None)
						.OnGenerateRow(this, &SFixupSelfContextDialog::OnGenerateRow)
					]
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(5.0f, 3.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("NoNodesWarning", "WARNING: Nothing will be pasted!"))
					.ColorAndOpacity(FSlateColor(FLinearColor::Yellow))
					.Visibility(this, &SFixupSelfContextDialog::GetNoneWarningVisibility)
				]
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.HAlign(HAlign_Right)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Bottom)
					.Padding(5.0f, 3.0f)
					[
						SNew(SButton)
						.Text(LOCTEXT("Cancel", "Cancel"))
						.OnClicked(this, &SFixupSelfContextDialog::CloseWindow, false)
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Bottom)
					.Padding(5.0f, 3.0f)
					[
						SNew(SButton)
						.Text(LOCTEXT("Confirm", "Confirm"))
						.OnClicked(this, &SFixupSelfContextDialog::CloseWindow, true)
					]
				]
			]
		];
}

bool SFixupSelfContextDialog::CreateModal(const TArray<UK2Node_CallFunction*>& NodesToFixup, UBlueprint* InFromBP, const FBlueprintEditor* BlueprintEditor, bool bOtherPastedNodes)
{
	TSharedPtr<SWindow> Window;
	TSharedPtr<SFixupSelfContextDialog> Widget;

	Window = SNew(SWindow)
		.Title(LOCTEXT("FixupReferencesTitle", "Fix Self Context Function References"))
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(400.f, 300.f))
		.SupportsMaximize(true)
		.SupportsMinimize(false)
		.HasCloseButton(false)
		[
			SNew(SBorder)
			.Padding(4.f)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SAssignNew(Widget, SFixupSelfContextDialog, NodesToFixup, InFromBP, BlueprintEditor, bOtherPastedNodes)
			]
		];

	Widget->MyWindow = Window;

	GEditor->EditorAddModalWindow(Window.ToSharedRef());

	return Widget->bOutConfirmed;
}

TSharedRef<ITableRow> SFixupSelfContextDialog::OnGenerateRow(FListViewItem Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<FListViewItem>, OwnerTable)
		[
			Item->CreateWidget(Options)
		];
}

EVisibility SFixupSelfContextDialog::GetNoneWarningVisibility() const
{
	if (bOtherNodes)
	{
		return EVisibility::Hidden;
	}

	for (FListViewItem Item : FunctionsToFixup)
	{
		if (Item->ComboBox.IsValid())
		{
			int32 Strategy;
			Options.Find(Item->ComboBox->GetSelectedItem(), Strategy);

			if (FFixupSelfContextItem::EFixupStrategy(Strategy) != FFixupSelfContextItem::EFixupStrategy::RemoveNode)
			{
				return EVisibility::Hidden;
			}
		}
	}

	return EVisibility::Visible;
}

FReply SFixupSelfContextDialog::CloseWindow(bool bConfirmed)
{
	if (bConfirmed)
	{
		for (FListViewItem ItemPtr : FunctionsToFixup)
		{
			if (!ItemPtr.IsValid() || !ItemPtr->ComboBox.IsValid())
			{
				continue;
			}

			int32 Strategy;
			Options.Find(ItemPtr->ComboBox->GetSelectedItem(), Strategy);

			switch (FFixupSelfContextItem::EFixupStrategy(Strategy))
			{
			case FFixupSelfContextItem::EFixupStrategy::CreateNewFunction:
				CreateMissingFunctions(ItemPtr);
				break;
			case FFixupSelfContextItem::EFixupStrategy::RemoveNode:
				for (UK2Node_CallFunction* Node : ItemPtr->Nodes)
				{
					UEdGraph* Graph = Node ? Node->GetGraph() : nullptr;
					if (Graph)
					{
						Graph->RemoveNode(Node);
					}
				}
				break;
			case FFixupSelfContextItem::EFixupStrategy::DoNothing:
			default:
				break;
			}
		}
	}

	bOutConfirmed = bConfirmed;
	
	if (MyWindow.IsValid())
	{
		MyWindow->RequestDestroyWindow();
	}

	return FReply::Handled();
}

void SFixupSelfContextDialog::CreateMissingFunctions(FListViewItem FuncToFix)
{
	if (!BlueprintEditor || !FuncToFix.IsValid())
	{
		return;
	}

	// Search the FromBP for an Event with the name we are interested in
	UFunction* OriginalEventFunc = nullptr;
	if(FromBP && FromBP->SkeletonGeneratedClass)
	{
		TArray<UK2Node_Event*> AllEvents;
		FBlueprintEditorUtils::GetAllNodesOfClass(FromBP, AllEvents);
	
		for (UK2Node_Event* EventNode : AllEvents)
		{
			if (EventNode->GetFunctionName() == FuncToFix->FuncName)
			{
				// Attempt to find the UFunction by name on the given blueprint instead of the EventReference because
				// the EventReference will always be empty, giving a nullptr every time.				
				OriginalEventFunc = FromBP->SkeletonGeneratedClass->FindFunctionByName(FuncToFix->FuncName);				
				break;
			}
		}
	}

	if(OriginalEventFunc)
	{
		// Spawn a new event node in the BP we are copying to
		UBlueprint* BP = BlueprintEditor->GetBlueprintObj();
		UEdGraph* EventGraph = FBlueprintEditorUtils::FindEventGraph(BP);

		UK2Node_CustomEvent* NewNode = UK2Node_CustomEvent::CreateFromFunction(
			EventGraph->GetGoodPlaceForNewNode(),
			EventGraph,
			FuncToFix->FuncName.ToString(),
			OriginalEventFunc
		);

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(BP);
	}
	else
	{
		// Otherwise just spawn a regular function graph
		check(FuncToFix->Nodes.Num() > 0);
		FBlueprintEditorUtils::CreateMatchingFunction(FuncToFix->Nodes[0], BlueprintEditor->GetDefaultSchema());
	}

	// Reconstruct all the nodes so that they get the newly created function reference
	for (UK2Node_CallFunction* Node : FuncToFix->Nodes)
	{
		Node->ReconstructNode();
	}
}

#undef LOCTEXT_NAMESPACE
