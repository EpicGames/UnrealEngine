// Copyright Epic Games, Inc. All Rights Reserved.


#include "SGraphTitleBar.h"

#include "BlueprintEditor.h"
#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphSchema.h"
#include "Engine/Blueprint.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "SGraphTitleBarAddNewBookmark.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/Casts.h"
#include "Types/ISlateMetaData.h"
#include "Types/SlateEnums.h"
#include "UObject/Object.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Navigation/SBreadcrumbTrail.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SGraphTitleBar"


SGraphTitleBar::~SGraphTitleBar()
{
	// Unregister for notifications
	if (Kismet2Ptr.IsValid())
	{
		Kismet2Ptr.Pin()->OnRefresh().RemoveAll(this);
	}
}

const FSlateBrush* SGraphTitleBar::GetTypeGlyph() const
{
	check(EdGraphObj != nullptr);
	return FBlueprintEditor::GetGlyphForGraph(EdGraphObj, true);
}

FText SGraphTitleBar::GetTitleForOneCrumb(const UEdGraph* Graph)
{
	const UEdGraphSchema* Schema = Graph->GetSchema();

	FGraphDisplayInfo DisplayInfo;
	Schema->GetGraphDisplayInformation(*Graph, /*out*/ DisplayInfo);

	FFormatNamedArguments Args;
	Args.Add(TEXT("BreadcrumbDisplayName"), DisplayInfo.DisplayName);
	Args.Add(TEXT("BreadcrumbNotes"), FText::FromString(DisplayInfo.GetNotesAsString()));
	return FText::Format(LOCTEXT("BreadcrumbTitle", "{BreadcrumbDisplayName} {BreadcrumbNotes}"), Args);
}

FText SGraphTitleBar::GetTitleExtra() const
{
	check(EdGraphObj != nullptr);

	FText ExtraText = Kismet2Ptr.Pin()->GetGraphDecorationString(EdGraphObj);

	const bool bEditable = (Kismet2Ptr.Pin()->IsEditable(EdGraphObj));

	if(!bEditable)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("BaseText"), ExtraText);

		ExtraText = FText::Format(LOCTEXT("ReadOnlyWarningText", "{BaseText} (READ-ONLY)"), Args);
	}

	return ExtraText;
}

EVisibility SGraphTitleBar::IsGraphBlueprintNameVisible() const
{
	return bShowBlueprintTitle ? EVisibility::Visible : EVisibility::Collapsed;
}

void SGraphTitleBar::OnBreadcrumbClicked(UEdGraph* const& Item)
{
	OnDifferentGraphCrumbClicked.ExecuteIfBound(Item);
}

void SGraphTitleBar::Construct( const FArguments& InArgs )
{
	EdGraphObj = InArgs._EdGraphObj;
	OnDifferentGraphCrumbClicked = InArgs._OnDifferentGraphCrumbClicked;

	check(EdGraphObj);
	Kismet2Ptr = InArgs._Kismet2;
	check(Kismet2Ptr.IsValid());

	// Set-up shared breadcrumb defaults
	FMargin BreadcrumbTrailPadding = FMargin(4.f, 2.f);
	const FSlateBrush* BreadcrumbButtonImage = FAppStyle::GetBrush("BreadcrumbTrail.Delimiter");
	
	this->ChildSlot
	[
		SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush( TEXT("Graph.TitleBackground") ) )
		.HAlign(HAlign_Fill)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("EventGraphTitleBar")))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SGraphTitleBarAddNewBookmark)
				.EditorPtr(Kismet2Ptr)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SSeparator)
				.Orientation(Orient_Vertical)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				InArgs._HistoryNavigationWidget.ToSharedRef()
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SSeparator)
				.Orientation(Orient_Vertical)
			]
			// Title text/icon
			+SHorizontalBox::Slot()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding( 10.0f,5.0f )
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image( this, &SGraphTitleBar::GetTypeGlyph )
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
					+SHorizontalBox::Slot()
					.FillWidth(1.f)
					[
						SAssignNew(BreadcrumbTrailScrollBox, SScrollBox)
						.Orientation(Orient_Horizontal)
						.ScrollBarVisibility(EVisibility::Collapsed)

						+SScrollBox::Slot()
						.Padding(0.f)
						.VAlign(VAlign_Center)
						[
							SNew(SHorizontalBox)
							// show fake 'root' breadcrumb for the title
							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(BreadcrumbTrailPadding)
							[
								SNew(STextBlock)
								.Text(this, &SGraphTitleBar::GetBlueprintTitle )
								.TextStyle( FAppStyle::Get(), TEXT("GraphBreadcrumbButtonText") )
								.Visibility( this, &SGraphTitleBar::IsGraphBlueprintNameVisible )
							]
							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(SImage)
								.Image( BreadcrumbButtonImage )
								.Visibility( this, &SGraphTitleBar::IsGraphBlueprintNameVisible )
							]

							// New style breadcrumb
							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SAssignNew(BreadcrumbTrail, SBreadcrumbTrail<UEdGraph*>)
								.ButtonStyle(FAppStyle::Get(), "GraphBreadcrumbButton")
								.TextStyle(FAppStyle::Get(), "GraphBreadcrumbButtonText")
								.ButtonContentPadding( BreadcrumbTrailPadding )
								.DelimiterImage( BreadcrumbButtonImage )
								.PersistentBreadcrumbs( true )
								.OnCrumbClicked( this, &SGraphTitleBar::OnBreadcrumbClicked )
							]

							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Font( FCoreStyle::GetDefaultFontStyle("Regular", 14) )
								.ColorAndOpacity( FLinearColor(1,1,1,0.5) )
								.Text( this, &SGraphTitleBar::GetTitleExtra )
							]
						]
					]
				]
			]
		]
	];

	RebuildBreadcrumbTrail();
	BreadcrumbTrailScrollBox->ScrollToEnd();

	UBlueprint* BlueprintObj = FBlueprintEditorUtils::FindBlueprintForGraph(this->EdGraphObj);
	if (BlueprintObj)
	{
		bShowBlueprintTitle = true;
		BlueprintTitle = FText::FromString(BlueprintObj->GetFriendlyName());

		// Register for notifications to refresh UI
		if( Kismet2Ptr.IsValid() )
		{
			Kismet2Ptr.Pin()->OnRefresh().AddRaw(this, &SGraphTitleBar::Refresh);
		}
	}
}

void SGraphTitleBar::RebuildBreadcrumbTrail()
{
	// Build up a stack of graphs so we can pop them in reverse order and create breadcrumbs
	TArray<UEdGraph*> Stack;
	for (UEdGraph* OuterChain = EdGraphObj; OuterChain != nullptr; OuterChain = GetOuterGraph(OuterChain))
	{
		Stack.Push(OuterChain);
	}

	BreadcrumbTrail->ClearCrumbs(false);

	//Get the last object in the array
	UEdGraph* LastObj = nullptr;
	if( Stack.Num() > 0 )
	{
		LastObj = Stack[Stack.Num() -1];
	}

	while (Stack.Num() > 0)
	{
		UEdGraph* Graph = Stack.Pop();
		
		auto Foo = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&SGraphTitleBar::GetTitleForOneCrumb, (const UEdGraph*)Graph));
		BreadcrumbTrail->PushCrumb(Foo, Graph);
	}
}

UEdGraph* SGraphTitleBar::GetOuterGraph( UObject* Obj )
{
	if( Obj )
	{
		UObject* OuterObj = Obj->GetOuter();
		if( OuterObj )
		{
			if( OuterObj->IsA( UEdGraph::StaticClass()) )
			{
				return Cast<UEdGraph>(OuterObj);
			}
			else
			{
				return GetOuterGraph( OuterObj );
			}
		}
	}
	return nullptr;
}

FText SGraphTitleBar::GetBlueprintTitle() const
{
	return BlueprintTitle;
}

void SGraphTitleBar::Refresh()
{
	// Refresh UI on request
	if (EdGraphObj)
	{
		if (UBlueprint* BlueprintObj = FBlueprintEditorUtils::FindBlueprintForGraph(this->EdGraphObj))
		{
			BlueprintTitle = FText::FromString(BlueprintObj->GetFriendlyName());
			RebuildBreadcrumbTrail();
		}
	}
}

#undef LOCTEXT_NAMESPACE
