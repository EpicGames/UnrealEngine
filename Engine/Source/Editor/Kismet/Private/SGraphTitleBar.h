// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintUtilities.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Navigation/SBreadcrumbTrail.h"
#include "Widgets/SCompoundWidget.h"

class FBlueprintEditor;
class SFunctionEditor;
class SScrollBox;
class SWidget;
class UEdGraph;
class UObject;
struct FSlateBrush;

//////////////////////////////////////////////////////////////////////////
// SGraphTitleBar

class SGraphTitleBar : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SGraphTitleBar )
		: _EdGraphObj(nullptr)
		, _Kismet2()
		{}

		SLATE_ARGUMENT( UEdGraph*, EdGraphObj )
		SLATE_ARGUMENT( TWeakPtr<FBlueprintEditor>, Kismet2 )
		SLATE_EVENT( FEdGraphEvent, OnDifferentGraphCrumbClicked )
		SLATE_ARGUMENT( TSharedPtr<SWidget>, HistoryNavigationWidget )
	SLATE_END_ARGS()

	/** SGraphTitleBar destructor */
	~SGraphTitleBar();

	void Construct(const FArguments& InArgs);

	/** Refresh the toolbar */
	void Refresh();

protected:
	/** Owning Kismet 2 */
	TWeakPtr<FBlueprintEditor> Kismet2Ptr;

	/** Edited graph */
	UEdGraph* EdGraphObj;

	/** Pointer to the function editor widget */
	TWeakPtr<SFunctionEditor>	FuncEditorPtr;

	TSharedPtr<SScrollBox> BreadcrumbTrailScrollBox;

	/** Breadcrumb trail widget */
	TSharedPtr< SBreadcrumbTrail<UEdGraph*> > BreadcrumbTrail;

	/** Callback to call when the user wants to change the active graph via the breadcrumb trail */
	FEdGraphEvent OnDifferentGraphCrumbClicked;

	/** Should we show graph's blueprint title */
	bool bShowBlueprintTitle;

	/** Blueprint title being displayed for toolbar */
	FText BlueprintTitle;

protected:
	/** Get the icon to use */
	const FSlateBrush* GetTypeGlyph() const;

	/** Get the extra title text */
	FText GetTitleExtra() const;

	/** Helper methods */
	EVisibility IsGraphBlueprintNameVisible() const;

	void OnBreadcrumbClicked(UEdGraph* const & Item);

	void RebuildBreadcrumbTrail();

	static FText GetTitleForOneCrumb(const UEdGraph* Graph);

	/** Function to fetch outer class which is of type UEGraph. */
	UEdGraph* GetOuterGraph( UObject* Obj );

	/** Helper method used to show blueprint title in breadcrumbs */
	FText GetBlueprintTitle() const;

	/** Helper method used to create the bookmark selector widget */
	TSharedRef<SWidget> CreateBookmarkSelectionWidget();
};
