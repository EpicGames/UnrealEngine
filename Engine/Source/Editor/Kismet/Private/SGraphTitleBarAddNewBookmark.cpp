// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGraphTitleBarAddNewBookmark.h"

#include "BlueprintEditor.h"
#include "BlueprintEditorSettings.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "Engine/Blueprint.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "SPrimaryButton.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Types/SlateStructs.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SGraphTitleBarAddNewBookmark"

void SGraphTitleBarAddNewBookmark::Construct(const FArguments& InArgs)
{
	EditorContextPtr = InArgs._EditorPtr;

	SComboButton::FArguments Args;
	Args.ButtonContent()
	[
		SNew(SImage)
		.Image(this, &SGraphTitleBarAddNewBookmark::GetAddButtonIcon)
		.ColorAndOpacity(FSlateColor::UseForeground())
	]
	.MenuContent()
	[
		SNew(SBox)
		.MinDesiredWidth(300.0f)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(8)
			.FillHeight(1.f)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SNew(STextBlock)
				.Text(this, &SGraphTitleBarAddNewBookmark::GetPopupTitleText)
				.Font(FAppStyle::GetFontStyle("StandardDialog.LargeFont"))
			]
			+ SVerticalBox::Slot()
			.Padding(8, 4, 8, 8)
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(6)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("BookmarkNameFieldLabel", "Name:"))
				]
				+ SHorizontalBox::Slot()
				.Padding(1)
				.FillWidth(1.f)
				[
					SAssignNew(NameEntryWidget, SEditableTextBox)
					.SelectAllTextWhenFocused(true)
					.OnTextCommitted(this, &SGraphTitleBarAddNewBookmark::OnNameTextCommitted)
					.OnTextChanged(this, &SGraphTitleBarAddNewBookmark::OnNameTextCommitted, ETextCommit::Default)
					.Text_Lambda([this]() -> FText
					{
						return CurrentNameText;
					})
				]
			]
			+ SVerticalBox::Slot()
			.Padding(8, 4, 4, 8)
			.AutoHeight()
			.VAlign(VAlign_Bottom)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.Text(LOCTEXT("RemoveButtonLabel", "Remove"))
					.Visibility_Lambda([this]() -> EVisibility
					{
						return CurrentViewBookmarkId.IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
					})
					.OnClicked(this, &SGraphTitleBarAddNewBookmark::OnRemoveButtonClicked)
				]
				+SHorizontalBox::Slot()
				.FillWidth(1.f)
				.HAlign(HAlign_Right)
				[
					SNew(SUniformGridPanel)
					.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
					.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
					.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
					+ SUniformGridPanel::Slot(0, 0)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.Text(LOCTEXT("CancelButtonLabel", "Cancel"))
						.OnClicked_Lambda([this]() -> FReply
						{
							SetIsOpen(false);
							return FReply::Handled();
						})
					]
					+ SUniformGridPanel::Slot(1, 0)
					[
						SNew(SPrimaryButton)
						.Text(this, &SGraphTitleBarAddNewBookmark::GetAddButtonLabel)
						.OnClicked(this, &SGraphTitleBarAddNewBookmark::OnAddButtonClicked)
						.IsEnabled(this, &SGraphTitleBarAddNewBookmark::IsAddButtonEnabled)
					]
				]
			]
		]
	]
	.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
	.ToolTipText(LOCTEXT("AddBookmarkButtonToolTip", "Bookmark Current Location"))
	.OnComboBoxOpened(this, &SGraphTitleBarAddNewBookmark::OnComboBoxOpened);

	SetMenuContentWidgetToFocus(NameEntryWidget);

	SetVisibility(MakeAttributeLambda([this]()
	{
		TSharedPtr<FBlueprintEditor> EditorContext = EditorContextPtr.Pin();
		if (EditorContext.IsValid())
		{
			UEdGraph* FocusedGraph = EditorContext->GetFocusedGraph();
			if (FocusedGraph != nullptr || CurrentViewBookmarkId.IsValid())
			{
				return EVisibility::Visible;
			}
		}

		return EVisibility::Hidden;
	}));
	
	SComboButton::Construct(Args);
}

FText SGraphTitleBarAddNewBookmark::GetPopupTitleText() const
{
	if (CurrentViewBookmarkId.IsValid())
	{
		return LOCTEXT("EditBookmarkPopupTitle", "Edit Bookmark");
	}
	
	return LOCTEXT("NewBookmarkPopupTitle", "New Bookmark");
}

FText SGraphTitleBarAddNewBookmark::GetDefaultNameText() const
{
	FText ResultText = LOCTEXT("DefaultBookmarkLabel", "New Bookmark");
	TSharedPtr<FBlueprintEditor> EditorContext = EditorContextPtr.Pin();
	if (EditorContext.IsValid())
	{
		UBlueprint* Blueprint = EditorContext->GetBlueprintObj();
		check(Blueprint);

		if (CurrentViewBookmarkId.IsValid())
		{
			const FGuid& TargetNodeId = CurrentViewBookmarkId;
			auto FindCurrentBookmarkNodeLambda = [TargetNodeId](const TArray<FBPEditorBookmarkNode>& InBookmarkNodesToSearch)
			{
				return InBookmarkNodesToSearch.FindByPredicate([TargetNodeId](const FBPEditorBookmarkNode& InNode) { return InNode.NodeGuid == TargetNodeId; });
			};

			// Check for a shared bookmark
			const FBPEditorBookmarkNode* BookmarkNode = FindCurrentBookmarkNodeLambda(Blueprint->BookmarkNodes);
			if (BookmarkNode == nullptr)
			{
				// No shared bookmark found, so check the local user settings
				BookmarkNode = FindCurrentBookmarkNodeLambda(GetDefault<UBlueprintEditorSettings>()->BookmarkNodes);
			}

			if (ensure(BookmarkNode != nullptr))
			{
				ResultText = BookmarkNode->DisplayName;
			}
		}
		else
		{
			int32 i = 1;
			bool bNeedsCheck;

			do
			{
				bNeedsCheck = false;

				for (const FBPEditorBookmarkNode& BookmarkNode : Blueprint->BookmarkNodes)
				{
					if (ResultText.EqualTo(BookmarkNode.DisplayName))
					{
						const FEditedDocumentInfo* BookmarkInfo = Blueprint->Bookmarks.Find(BookmarkNode.NodeGuid);
						if (BookmarkInfo && BookmarkInfo->EditedObjectPath.ResolveObject() == EditorContext->GetFocusedGraph())
						{
							ResultText = FText::Format(LOCTEXT("DefaultBookmarkLabelWithIndex", "New Bookmark {0}"), ++i);

							bNeedsCheck = true;
							break;
						}
					}
				}
			} while (bNeedsCheck);
		}
	}

	return ResultText;
}

const FSlateBrush* SGraphTitleBarAddNewBookmark::GetAddButtonIcon() const
{
	TSharedPtr<FBlueprintEditor> EditorContext = EditorContextPtr.Pin();
	if (EditorContext.IsValid())
	{
		FGuid CurrentBookmarkId;
		EditorContext->GetViewBookmark(CurrentBookmarkId);
		if (CurrentBookmarkId.IsValid())
		{
			return FAppStyle::Get().GetBrush("GraphEditor.Bookmark"); // Filled bookmark icon
		}
	}
	return FAppStyle::Get().GetBrush("GraphEditor.Bookmark.Outline"); // Hollow bookmark icon
}

FText SGraphTitleBarAddNewBookmark::GetAddButtonLabel() const
{
	if (CurrentViewBookmarkId.IsValid())
	{
		return LOCTEXT("RenameButtonLabel", "Rename");
	}
	
	return LOCTEXT("AddButtonLabel", "Add");
}

void SGraphTitleBarAddNewBookmark::OnComboBoxOpened()
{
	CurrentViewBookmarkId.Invalidate();

	TSharedPtr<FBlueprintEditor> EditorContext = EditorContextPtr.Pin();
	if (EditorContext.IsValid())
	{
		EditorContext->GetViewBookmark(CurrentViewBookmarkId);
	}

	CurrentNameText = GetDefaultNameText();
	if (CurrentViewBookmarkId.IsValid())
	{
		OriginalNameText = CurrentNameText;
	}
	else
	{
		OriginalNameText = FText::GetEmpty();
	}

	check(NameEntryWidget.IsValid());
	NameEntryWidget->SetText(CurrentNameText);
}

FReply SGraphTitleBarAddNewBookmark::OnAddButtonClicked()
{
	TSharedPtr<FBlueprintEditor> EditorContext = EditorContextPtr.Pin();
	if (EditorContext.IsValid())
	{
		if (CurrentViewBookmarkId.IsValid())
		{
			EditorContext->RenameBookmark(CurrentViewBookmarkId, CurrentNameText);
		}
		else if (UEdGraph* FocusedGraph = EditorContext->GetFocusedGraph())
		{
			FEditedDocumentInfo NewBookmarkInfo;
			NewBookmarkInfo.EditedObjectPath = FocusedGraph;
			FVector2D Location;
			EditorContext->GetViewLocation(Location, NewBookmarkInfo.SavedZoomAmount);
			NewBookmarkInfo.SavedViewOffset = Location;
			if (FBPEditorBookmarkNode* NewNode = EditorContext->AddBookmark(CurrentNameText, NewBookmarkInfo))
			{
				EditorContext->SetViewLocation(NewBookmarkInfo.SavedViewOffset, NewBookmarkInfo.SavedZoomAmount, NewNode->NodeGuid);
			}
		}
	}

	SetIsOpen(false);

	return FReply::Handled();
}

bool SGraphTitleBarAddNewBookmark::IsAddButtonEnabled() const
{
	return !CurrentNameText.IsEmpty() && CurrentNameText.CompareTo(OriginalNameText) != 0;
}

FReply SGraphTitleBarAddNewBookmark::OnRemoveButtonClicked()
{
	TSharedPtr<FBlueprintEditor> EditorContext = EditorContextPtr.Pin();
	if (EditorContext.IsValid() && CurrentViewBookmarkId.IsValid())
	{
		EditorContext->RemoveBookmark(CurrentViewBookmarkId);
	}

	SetIsOpen(false);

	return FReply::Handled();
}

void SGraphTitleBarAddNewBookmark::OnNameTextCommitted(const FText& InText, ETextCommit::Type CommitType)
{
	CurrentNameText = InText;

	if (CommitType == ETextCommit::OnEnter)
	{
		OnAddButtonClicked();
	}
}

#undef LOCTEXT_NAMESPACE