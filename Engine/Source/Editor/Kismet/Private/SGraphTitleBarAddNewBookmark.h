// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboButton.h"

class SGraphTitleBarAddNewBookmark : public SComboButton
{
public:
	SLATE_BEGIN_ARGS(SGraphTitleBarAddNewBookmark)
		: _EditorPtr()
	{}

	SLATE_ARGUMENT(TWeakPtr<class FBlueprintEditor>, EditorPtr)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

protected:
	FText GetPopupTitleText() const;
	FText GetDefaultNameText() const;
	const FSlateBrush* GetAddButtonIcon() const;
	FText GetAddButtonLabel() const;

	void OnComboBoxOpened();
	FReply OnAddButtonClicked();
	bool IsAddButtonEnabled() const;
	FReply OnRemoveButtonClicked();
	void OnNameTextCommitted(const FText& InText, ETextCommit::Type CommitType);

private:
	TWeakPtr<class FBlueprintEditor> EditorContextPtr;
	TSharedPtr<class SEditableTextBox> NameEntryWidget;

	FText CurrentNameText;
	FText OriginalNameText;
	FGuid CurrentViewBookmarkId;
};
