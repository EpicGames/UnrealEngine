// Copyright Epic Games, Inc. All Rights Reserved.


#include "SKismetInspector.h"

#include "BitmaskLiteralDetails.h"
#include "BlueprintDetailsCustomization.h"
#include "BlueprintEditor.h"
#include "BlueprintEditorSettings.h"
#include "BlueprintMemberReferenceCustomization.h"
#include "BlueprintNamespaceUtilities.h"
#include "Components/ActorComponent.h"
#include "Components/ChildActorComponent.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "DetailsViewArgs.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNode_Documentation.h"
#include "EdGraphSchema_K2.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Blueprint.h"
#include "Engine/MemberReference.h"
#include "Engine/SCS_Node.h"
#include "FormatTextDetails.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Text/TextLayout.h"
#include "GameFramework/Actor.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IDetailCustomization.h"
#include "IDetailsView.h"
#include "IStructureDetailsView.h"
#include "Input/Events.h"
#include "Internationalization/Internationalization.h"
#include "K2Node.h"
#include "K2Node_AddComponent.h" // for GetTemplateFromNode()
#include "K2Node_BitmaskLiteral.h"
#include "K2Node_CallFunction.h"
#include "K2Node_EditablePinBase.h"
#include "K2Node_FormatText.h"
#include "K2Node_FunctionTerminator.h"
#include "K2Node_VariableGet.h"
#include "K2Node_VariableSet.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/ComponentEditorUtils.h"	// For CanEditNativeComponent()
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "Settings/EditorExperimentalSettings.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Templates/UnrealTemplate.h"
#include "Types/ISlateMetaData.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Text/STextBlock.h"

class FNotifyHook;
class FStructOnScope;
class IClassViewerFilter;
class IDetailLayoutBuilder;
class SDockTab;
class SWidget;
struct FGeometry;

#define LOCTEXT_NAMESPACE "KismetInspector"

class SKismetInspectorUneditableComponentWarning : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SKismetInspectorUneditableComponentWarning)
		: _WarningText()
		, _OnHyperlinkClicked()
	{}
		
		/** The rich text to show in the warning */
		SLATE_ATTRIBUTE(FText, WarningText)

		/** Called when the hyperlink in the rich text is clicked */
		SLATE_EVENT(FSlateHyperlinkRun::FOnClick, OnHyperlinkClicked)

	SLATE_END_ARGS()

	/** Constructs the widget */
	void Construct(const FArguments& InArgs)
	{
		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(2.0f)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Warning"))
				]
				+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(2.0f)
					[
						SNew(SRichTextBlock)
						.DecoratorStyleSet(&FAppStyle::Get())
						.Justification(ETextJustify::Left)
						.TextStyle(FAppStyle::Get(), "DetailsView.BPMessageTextStyle")
						.Text(InArgs._WarningText)
						.AutoWrapText(true)
						+ SRichTextBlock::HyperlinkDecorator(TEXT("HyperlinkDecorator"), InArgs._OnHyperlinkClicked)
					]
			]
		];
	}
};

//////////////////////////////////////////////////////////////////////////
// FKismetSelectionInfo

struct FKismetSelectionInfo
{
public:
	TArray<UActorComponent*> EditableComponentTemplates;
	TArray<UObject*> ObjectsForPropertyEditing;
};

//////////////////////////////////////////////////////////////////////////
// SKismetInspector

void SKismetInspector::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	if(bRefreshOnTick)
	{
		// if struct is valid, update struct
		if (StructToDisplay.IsValid())
		{
			UpdateFromSingleStruct(StructToDisplay);
			StructToDisplay.Reset();
		}
		else
		{
			FKismetSelectionInfo SelectionInfo;
			RefreshPropertyObjects.Remove(nullptr);
			UpdateFromObjects(RefreshPropertyObjects, SelectionInfo, RefreshOptions);
			RefreshPropertyObjects.Empty();
		}

		bRefreshOnTick = false;
	}
}

TSharedRef<SWidget> SKismetInspector::MakeContextualEditingWidget(struct FKismetSelectionInfo& SelectionInfo, const FShowDetailsOptions& Options)
{
	TSharedRef< SVerticalBox > ContextualEditingWidget = SNew( SVerticalBox );

	if(bShowTitleArea)
	{
		if (SelectedObjects.Num() == 0)
		{
			// Warning about nothing being selected
			ContextualEditingWidget->AddSlot()
			.AutoHeight()
			.HAlign( HAlign_Center )
			.Padding( 2.0f, 14.0f, 2.0f, 2.0f )
			[
				SNew( STextBlock )
				.Text( LOCTEXT("NoNodesSelected", "Select a node to edit details.") )
			];
		}
		else
		{
			// Title of things being edited
			ContextualEditingWidget->AddSlot()
			.AutoHeight()
			.Padding( 2.0f, 0.0f, 2.0f, 0.0f )
			[
				SNew(STextBlock)
				.Text(this, &SKismetInspector::GetContextualEditingWidgetTitle)
			];
		}
	}

	// Show the property editor
	PropertyView->HideFilterArea(Options.bHideFilterArea);
	PropertyView->SetObjects(SelectionInfo.ObjectsForPropertyEditing, Options.bForceRefresh);

	if (SelectionInfo.ObjectsForPropertyEditing.Num())
	{
		ContextualEditingWidget->AddSlot()
		.FillHeight( 0.9f )
		.VAlign( VAlign_Top )
		[
			SNew( SBox )
			.Visibility(this, &SKismetInspector::GetPropertyViewVisibility)
			[
				SNew( SVerticalBox )
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding( FMargin( 0,0,0,1) )
				[
					SNew(SKismetInspectorUneditableComponentWarning)
					.Visibility(this, &SKismetInspector::GetInheritedBlueprintComponentWarningVisibility)
					.WarningText(NSLOCTEXT("SKismetInspector", "BlueprintUneditableInheritedComponentWarning", "Components flagged as not editable when inherited must be edited in the <a id=\"HyperlinkDecorator\" style=\"DetailsView.BPMessageHyperlinkStyle\">Parent Blueprint</>"))
					.OnHyperlinkClicked(this, &SKismetInspector::OnInheritedBlueprintComponentWarningHyperlinkClicked)
				]
				+SVerticalBox::Slot()
				[
					PropertyView.ToSharedRef()
				]
			]
		];

		ContextualEditingWidget->AddSlot()
		.AutoHeight()
		.VAlign(VAlign_Top)
		[
			SNew(SCheckBox)
			.ToolTipText(LOCTEXT("TogglePublicView", "Toggle Public View"))
			.IsChecked(this, &SKismetInspector::GetPublicViewCheckboxState)
			.OnCheckStateChanged(this, &SKismetInspector::SetPublicViewCheckboxState)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("PublicViewCheckboxLabel", "Public View"))
			]
			.Visibility_Lambda([bShowPublicView = this->bShowPublicView]()
			{
				return bShowPublicView.Get() ? EVisibility::Visible : EVisibility::Hidden;
			})
		];
	}

	return ContextualEditingWidget;
}

void SKismetInspector::SetOwnerTab(TSharedRef<SDockTab> Tab)
{
	OwnerTab = Tab;
}

TSharedPtr<SDockTab> SKismetInspector::GetOwnerTab() const
{
	return OwnerTab.Pin();
}

bool SKismetInspector::IsSelected(UObject* Object) const
{
	for ( const TWeakObjectPtr<UObject>& SelectedObject : SelectedObjects )
	{
		if ( SelectedObject.Get() == Object )
		{
			return true;
		}
	}

	return false;
}

const TArray< TWeakObjectPtr<UObject> >& SKismetInspector::GetSelectedObjects() const
{
	return SelectedObjects;
}

void SKismetInspector::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(RefreshPropertyObjects);
}

FString SKismetInspector::GetReferencerName() const
{
	return TEXT("SKismetInspector");
}

FText SKismetInspector::GetContextualEditingWidgetTitle() const
{
	FText Title = PropertyViewTitle;
	if (Title.IsEmpty())
	{
		if (SelectedObjects.Num() == 1 && SelectedObjects[0].IsValid())
		{
			UObject* Object = SelectedObjects[0].Get();

			if (UEdGraphNode* Node = Cast<UEdGraphNode>(Object))
			{
				Title = Node->GetNodeTitle(ENodeTitleType::ListView);
			}
			else if (USCS_Node* SCSNode = Cast<USCS_Node>(Object))
			{
				if (SCSNode->ComponentTemplate != nullptr)
				{
					const FName VariableName = SCSNode->GetVariableName();
					if (VariableName != NAME_None)
					{
						Title = FText::Format(LOCTEXT("TemplateForFmt", "Template for {0}"), FText::FromName(VariableName));
					}
					else 
					{
						Title = FText::Format(LOCTEXT("Name_TemplateFmt", "{0} Template"), FText::FromString(SCSNode->ComponentTemplate->GetClass()->GetName()));
					}
				}
			}
			else if (UK2Node_AddComponent* ComponentNode = Cast<UK2Node_AddComponent>(Object))
			{
				// Edit the component template
				if (UActorComponent* Template = ComponentNode->GetTemplateFromNode())
				{
					Title = FText::Format(LOCTEXT("Name_TemplateFmt", "{0} Template"), FText::FromString(Template->GetClass()->GetName()));
				}
			}

			if (Title.IsEmpty())
			{
				Title = FText::FromString(UKismetSystemLibrary::GetDisplayName(Object));
			}
		}
		else if (SelectedObjects.Num() > 1)
		{
			UClass* BaseClass = nullptr;

			for (auto ObjectWkPtrIt = SelectedObjects.CreateConstIterator(); ObjectWkPtrIt; ++ObjectWkPtrIt)
			{
				TWeakObjectPtr<UObject> ObjectWkPtr = *ObjectWkPtrIt;
				if (ObjectWkPtr.IsValid())
				{
					UObject* Object = ObjectWkPtr.Get();
					UClass* ObjClass = Object->GetClass();

					if (UEdGraphNode* Node = Cast<UEdGraphNode>(Object))
					{
						// Hide any specifics of node types; they're all ed graph nodes
						ObjClass = UEdGraphNode::StaticClass();
					}

					// Keep track of the class of objects selected
					if (BaseClass == nullptr)
					{
						BaseClass = ObjClass;
						checkSlow(ObjClass);
					}
					while (!ObjClass->IsChildOf(BaseClass))
					{
						BaseClass = BaseClass->GetSuperClass();
					}
				}
			}

			if (BaseClass)
			{
				Title = FText::Format(LOCTEXT("MultipleObjectsSelectedFmt", "{0} {1} selected"), FText::AsNumber(SelectedObjects.Num()), FText::FromString(BaseClass->GetName() + TEXT("s")));
			}
		}
	}
	return Title;
}

void SKismetInspector::Construct(const FArguments& InArgs)
{
	bShowInspectorPropertyView = true;
	PublicViewState = ECheckBoxState::Unchecked;
	bComponenetDetailsCustomizationEnabled = false;
	bRefreshOnTick = false;

	BlueprintEditorPtr = InArgs._Kismet2;
	bShowPublicView = InArgs._ShowPublicViewControl;
	bShowTitleArea = InArgs._ShowTitleArea;
	TSharedPtr<FBlueprintEditor> Kismet2 = BlueprintEditorPtr.Pin();

	// Create a property view
	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FNotifyHook* NotifyHook = nullptr;
	if(InArgs._SetNotifyHook)
	{
		NotifyHook = Kismet2.Get();
	}

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = InArgs._HideNameArea ? FDetailsViewArgs::HideNameArea : FDetailsViewArgs::ObjectsUseNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.NotifyHook = NotifyHook;
	DetailsViewArgs.ViewIdentifier = InArgs._ViewIdentifier;
	DetailsViewArgs.ExternalScrollbar = InArgs._ExternalScrollbar;
	DetailsViewArgs.ScrollbarAlignment = InArgs._ScrollbarAlignment;
	DetailsViewArgs.bShowSectionSelector = InArgs._ShowSectionSelector;
	if (Kismet2.IsValid())
	{
		TSharedPtr<IClassViewerFilter> ImportedClassFilter = Kismet2->GetImportedClassViewerFilter();
		if (ImportedClassFilter.IsValid())
		{
			DetailsViewArgs.ClassViewerFilters.Add(ImportedClassFilter.ToSharedRef());
		}
	}

	PropertyView = EditModule.CreateDetailView( DetailsViewArgs );
		
	//@TODO: .IsEnabled( FSlateApplication::Get().GetNormalExecutionAttribute() );
	PropertyView->SetIsPropertyVisibleDelegate( FIsPropertyVisible::CreateSP(this, &SKismetInspector::IsPropertyVisible) );
	PropertyView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateSP(this, &SKismetInspector::IsPropertyEditingEnabled));

	IsPropertyEditingEnabledDelegate = InArgs._IsPropertyEditingEnabledDelegate;
	UserOnFinishedChangingProperties = InArgs._OnFinishedChangingProperties;

	TWeakPtr<SMyBlueprint> MyBlueprint = Kismet2.IsValid() ? Kismet2->GetMyBlueprintWidget() : InArgs._MyBlueprintWidget;
	
	if( MyBlueprint.IsValid() )
	{
		FOnGetDetailCustomizationInstance LayoutDelegateDetails = FOnGetDetailCustomizationInstance::CreateStatic(&FBlueprintDelegateActionDetails::MakeInstance, MyBlueprint);
		PropertyView->RegisterInstancedCustomPropertyLayout(UMulticastDelegatePropertyWrapper::StaticClass(), LayoutDelegateDetails);
		
		// Register function and variable details customization
		FOnGetDetailCustomizationInstance LayoutGraphDetails = FOnGetDetailCustomizationInstance::CreateStatic(&FBlueprintGraphActionDetails::MakeInstance, MyBlueprint, InArgs._ShowLocalVariables);
		PropertyView->RegisterInstancedCustomPropertyLayout(UEdGraph::StaticClass(), LayoutGraphDetails);
		PropertyView->RegisterInstancedCustomPropertyLayout(UK2Node_EditablePinBase::StaticClass(), LayoutGraphDetails);
		PropertyView->RegisterInstancedCustomPropertyLayout(UK2Node_CallFunction::StaticClass(), LayoutGraphDetails);

		FOnGetDetailCustomizationInstance LayoutVariableDetails = FOnGetDetailCustomizationInstance::CreateStatic(&FBlueprintVarActionDetails::MakeInstance, MyBlueprint);
		PropertyView->RegisterInstancedCustomPropertyLayout(UPropertyWrapper::StaticClass(), LayoutVariableDetails);
		PropertyView->RegisterInstancedCustomPropertyLayout(UK2Node_VariableGet::StaticClass(), LayoutVariableDetails);
		PropertyView->RegisterInstancedCustomPropertyLayout(UK2Node_VariableSet::StaticClass(), LayoutVariableDetails);

		PropertyView->RegisterInstancedCustomPropertyTypeLayout(FMemberReference::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FBlueprintMemberReferenceDetails::MakeInstance, MyBlueprint));
	}

	if (Kismet2.IsValid() && Kismet2->IsEditingSingleBlueprint())
	{
		FOnGetDetailCustomizationInstance LayoutOptionDetails = FOnGetDetailCustomizationInstance::CreateStatic(&FBlueprintGlobalOptionsDetails::MakeInstance, BlueprintEditorPtr);
		PropertyView->RegisterInstancedCustomPropertyLayout(UBlueprint::StaticClass(), LayoutOptionDetails);

		FOnGetDetailCustomizationInstance LayoutFormatTextDetails = FOnGetDetailCustomizationInstance::CreateStatic(&FFormatTextDetails::MakeInstance);
		PropertyView->RegisterInstancedCustomPropertyLayout(UK2Node_FormatText::StaticClass(), LayoutFormatTextDetails);

		FOnGetDetailCustomizationInstance LayoutBitmaskLiteralDetails = FOnGetDetailCustomizationInstance::CreateStatic(&FBitmaskLiteralDetails::MakeInstance);
		PropertyView->RegisterInstancedCustomPropertyLayout(UK2Node_BitmaskLiteral::StaticClass(), LayoutBitmaskLiteralDetails);

		FOnGetDetailCustomizationInstance LayoutDocumentationDetails = FOnGetDetailCustomizationInstance::CreateStatic(&FBlueprintDocumentationDetails::MakeInstance, BlueprintEditorPtr);
		PropertyView->RegisterInstancedCustomPropertyLayout(UEdGraphNode_Documentation::StaticClass(), LayoutDocumentationDetails);

		FOnGetDetailCustomizationInstance GraphNodeDetails = FOnGetDetailCustomizationInstance::CreateStatic(&FBlueprintGraphNodeDetails::MakeInstance, BlueprintEditorPtr);
		PropertyView->RegisterInstancedCustomPropertyLayout(UEdGraphNode::StaticClass(), GraphNodeDetails);

		PropertyView->RegisterInstancedCustomPropertyLayout(UChildActorComponent::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FChildActorComponentDetails::MakeInstance, BlueprintEditorPtr));
	}

	// Create the border that all of the content will get stuffed into
	ChildSlot
	[
		SNew(SVerticalBox)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("BlueprintInspector")))
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SAssignNew( ContextualEditingBorderWidget, SBorder )
			.Padding(0.0f)
			.BorderImage( FAppStyle::GetBrush("NoBorder") )
		]
	];

	// Update based on the current (empty) selection set
	TArray<UObject*> InitialSelectedObjects;
	FKismetSelectionInfo SelectionInfo;
	UpdateFromObjects(InitialSelectedObjects, SelectionInfo, SKismetInspector::FShowDetailsOptions(FText::GetEmpty(), true));

	// create struct to display
	FStructureDetailsViewArgs StructureViewArgs;
	StructureViewArgs.bShowObjects = true;
	StructureViewArgs.bShowAssets = true;
	StructureViewArgs.bShowClasses = true;
	StructureViewArgs.bShowInterfaces = true;

	FDetailsViewArgs ViewArgs;
	ViewArgs.bAllowSearch = false;
	ViewArgs.bHideSelectionTip = false;
	ViewArgs.bShowObjectLabel = false;
	ViewArgs.NotifyHook = NotifyHook;

	StructureDetailsView = EditModule.CreateStructureDetailView(ViewArgs, StructureViewArgs, StructToDisplay, LOCTEXT("Struct", "Struct View"));
	StructureDetailsView->GetDetailsView()->SetIsPropertyReadOnlyDelegate(FIsPropertyReadOnly::CreateSP(this, &SKismetInspector::IsStructViewPropertyReadOnly));
	StructureDetailsView->GetOnFinishedChangingPropertiesDelegate().Clear();
	StructureDetailsView->GetOnFinishedChangingPropertiesDelegate().Add(UserOnFinishedChangingProperties);
}

void SKismetInspector::EnableComponentDetailsCustomization(bool bEnable)
{
	// An "empty" instanced customization that's intended to override any registered global details customization for
	// the AActor class type. This will be applied -only- when the CDO is selected to the Details view in Components mode.
	class FActorDetailsOverrideCustomization : public IDetailCustomization
	{
	public:
		/** IDetailCustomization interface */
		virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override {}

		static TSharedRef<class IDetailCustomization> MakeInstance()
		{
			return MakeShareable(new FActorDetailsOverrideCustomization());
		}
	};

	bComponenetDetailsCustomizationEnabled = bEnable;

	if (bEnable)
	{
		FOnGetDetailCustomizationInstance ActorOverrideDetails = FOnGetDetailCustomizationInstance::CreateStatic(&FActorDetailsOverrideCustomization::MakeInstance);
		PropertyView->RegisterInstancedCustomPropertyLayout(AActor::StaticClass(), ActorOverrideDetails);

		FOnGetDetailCustomizationInstance LayoutComponentDetails = FOnGetDetailCustomizationInstance::CreateStatic(&FBlueprintComponentDetails::MakeInstance, BlueprintEditorPtr);
		PropertyView->RegisterInstancedCustomPropertyLayout(UActorComponent::StaticClass(), LayoutComponentDetails);
	}
	else
	{
		PropertyView->UnregisterInstancedCustomPropertyLayout(AActor::StaticClass());
		PropertyView->UnregisterInstancedCustomPropertyLayout(UActorComponent::StaticClass());
	}
}

/** Update the inspector window to show information on the supplied object */
void SKismetInspector::ShowDetailsForSingleObject(UObject* Object, const FShowDetailsOptions& Options)
{
	TArray<UObject*> PropertyObjects;

	if (Object != nullptr)
	{
		PropertyObjects.Add(Object);
	}

	if(PropertyView)
	{
		PropertyView->SetObjectFilter(nullptr);
	}

	ShowDetailsForObjects(PropertyObjects, Options);
}

void SKismetInspector::ShowDetailsForObjects(const TArray<UObject*>& PropertyObjects, const FShowDetailsOptions& Options)
{
	// Refresh is being deferred until the next tick, this prevents batch operations from bombarding the details view with calls to refresh
	RefreshPropertyObjects = PropertyObjects;
	RefreshOptions = Options;
	bRefreshOnTick = true;
}

/** Update the inspector window to show information on the supplied object */
void SKismetInspector::ShowSingleStruct(TSharedPtr<FStructOnScope> InStructToDisplay)
{
	static bool bIsReentrant = false;
	if (!bIsReentrant)
	{
		bIsReentrant = true;
		// When the selection is changed, we may be potentially actively editing a property,
		// if this occurs we need need to immediately clear keyboard focus
		if (FSlateApplication::Get().HasFocusedDescendants(AsShared()))
		{
			FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::Mouse);
		}
		bIsReentrant = false;
	}

	StructToDisplay = InStructToDisplay;
	// we don't defer this becasue StructDetailViews contains TSharedPtr to this sturct, 
	// not clearing until next tick causes crash
	// so will update struct view here, but updating widget will happen in the tick
	StructureDetailsView->SetStructureData(InStructToDisplay);
	bRefreshOnTick = true;
}

void SKismetInspector::AddPropertiesRecursive(FProperty* Property)
{
	if (Property != nullptr)
	{
		// Add this property
		SelectedObjectProperties.Add(Property);

		// If this is a struct or an array of structs, recursively add the child properties
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property);
		FStructProperty* StructProperty = CastField<FStructProperty>(Property);
		if(	StructProperty != nullptr && StructProperty->Struct != nullptr)
		{
			for (TFieldIterator<FProperty> StructPropIt(StructProperty->Struct); StructPropIt; ++StructPropIt)
			{
				FProperty* InsideStructProperty = *StructPropIt;
				AddPropertiesRecursive(InsideStructProperty);
			}
		}
		else if( ArrayProperty && ArrayProperty->Inner->IsA<FStructProperty>() )
		{
			AddPropertiesRecursive(ArrayProperty->Inner);
		}
	}
}

void SKismetInspector::UpdateFromSingleStruct(const TSharedPtr<FStructOnScope>& InStructToDisplay)
{
	if (StructureDetailsView.IsValid())
	{
		SelectedObjects.Empty();

		// Update our context-sensitive editing widget
		ContextualEditingBorderWidget->SetContent(StructureDetailsView->GetWidget().ToSharedRef());
	}
}

void SKismetInspector::UpdateFromObjects(const TArray<UObject*>& PropertyObjects, struct FKismetSelectionInfo& SelectionInfo, const FShowDetailsOptions& Options)
{
	// There's not an explicit point where
	// we ender a kind of component editing mode, so instead, just look at what we're selecting.
	// If we select a component, then enable the customization.
	bool bEnableComponentCustomization = false;

	TSharedPtr<FBlueprintEditor> BlueprintEditor = BlueprintEditorPtr.Pin();
	if (BlueprintEditor.IsValid())
	{
		if (BlueprintEditor->CanAccessComponentsMode())
		{
			for (UObject* PropertyObject : PropertyObjects)
			{
				if (PropertyObject && !PropertyObject->IsValidLowLevel())
				{
					ensureMsgf(false, TEXT("Object in KismetInspector is invalid, see TTP 281915"));
					continue;
				}

				if (PropertyObject->IsA<UActorComponent>())
				{
					bEnableComponentCustomization = true;
					break;
				}
			}
		}
	}

	EnableComponentDetailsCustomization(bEnableComponentCustomization);
	
	if (!Options.bForceRefresh)
	{
		// Early out if the PropertyObjects and the SelectedObjects are the same
		bool bEquivalentSets = (PropertyObjects.Num() == SelectedObjects.Num());
		if (bEquivalentSets)
		{
			// Verify the elements of the sets are equivalent
			for (int32 i = 0; i < PropertyObjects.Num(); i++)
			{
				if (PropertyObjects[i] != SelectedObjects[i].Get())
				{
					if (PropertyObjects[i] && !PropertyObjects[i]->IsValidLowLevel())
					{
						ensureMsgf(false, TEXT("Object in KismetInspector is invalid, see TTP 281915"));
						continue;
					}

					bEquivalentSets = false;
					break;
				}
			}
		}

		if (bEquivalentSets)
		{
			return;
		}
	}

	PropertyView->OnFinishedChangingProperties().Clear();
	PropertyView->OnFinishedChangingProperties().Add(UserOnFinishedChangingProperties);
	PropertyView->OnFinishedChangingProperties().AddSP(this, &SKismetInspector::OnFinishedChangingProperties);

	// Proceed to update
	SelectedObjects.Empty();

	for (auto ObjectIt = PropertyObjects.CreateConstIterator(); ObjectIt; ++ObjectIt)
	{
		if (UObject* Object = *ObjectIt)
		{
			if (!Object->IsValidLowLevel())
			{
				ensureMsgf(false, TEXT("Object in KismetInspector is invalid, see TTP 281915"));
				continue;
			}

			SelectedObjects.Add(Object);

			if (USCS_Node* SCSNode = Cast<USCS_Node>(Object))
			{
				// Edit the component template
				UActorComponent* NodeComponent = SCSNode->ComponentTemplate;
				if (NodeComponent != nullptr)
				{
					SelectionInfo.ObjectsForPropertyEditing.Add(NodeComponent);
					SelectionInfo.EditableComponentTemplates.Add(NodeComponent);
				}
			}
			else if (UK2Node* K2Node = Cast<UK2Node>(Object))
			{
				// Edit the component template if it exists
				if (UK2Node_AddComponent* ComponentNode = Cast<UK2Node_AddComponent>(K2Node))
				{
					if (UActorComponent* Template = ComponentNode->GetTemplateFromNode())
					{
						SelectionInfo.ObjectsForPropertyEditing.Add(Template);
						SelectionInfo.EditableComponentTemplates.Add(Template);
					}
				}

				// See if we should edit properties of the node
				if (K2Node->ShouldShowNodeProperties())
				{
					SelectionInfo.ObjectsForPropertyEditing.Add(Object);
				}
			}
			else if (UActorComponent* ActorComponent = Cast<UActorComponent>(Object))
			{
				AActor* Owner = ActorComponent->GetOwner();
				if(Owner != nullptr && Owner->HasAnyFlags(RF_ClassDefaultObject))
				{
					SelectionInfo.ObjectsForPropertyEditing.AddUnique(ActorComponent);
					SelectionInfo.EditableComponentTemplates.Add(ActorComponent);
				}
				else
				{
					// We're editing a component that exists outside of a CDO, so just edit the component instance directly
					SelectionInfo.ObjectsForPropertyEditing.AddUnique(ActorComponent);
				}
			}
			else
			{
				// Editing any UObject*
				SelectionInfo.ObjectsForPropertyEditing.AddUnique(Object);
			}
		}
	}

	// By default, no property filtering
	SelectedObjectProperties.Empty();

	// Add to the property filter list for any editable component templates
	if (SelectionInfo.EditableComponentTemplates.Num())
	{
		for (auto CompIt = SelectionInfo.EditableComponentTemplates.CreateIterator(); CompIt; ++CompIt)
		{
			UActorComponent* EditableComponentTemplate = *CompIt;
			check(EditableComponentTemplate != nullptr);

			// Add all properties belonging to the component template class
			for (TFieldIterator<FProperty> PropIt(EditableComponentTemplate->GetClass()); PropIt; ++PropIt)
			{
				FProperty* Property = *PropIt;
				check(Property != nullptr);

				AddPropertiesRecursive(Property);
			}

			// Attempt to locate a matching property for the current component template
			for (auto ObjIt = SelectionInfo.ObjectsForPropertyEditing.CreateIterator(); ObjIt; ++ObjIt)
			{
				UObject* Object = *ObjIt;
				check(Object != nullptr);

				if (Object != EditableComponentTemplate)
				{
					if (FObjectProperty* ObjectProperty = FindFProperty<FObjectProperty>(Object->GetClass(), EditableComponentTemplate->GetFName()))
					{
						SelectedObjectProperties.Add(ObjectProperty);
					}
					else
					{
						FProperty* ReferencingProperty = FComponentEditorUtils::GetPropertyForEditableNativeComponent(EditableComponentTemplate);
						if (ReferencingProperty == nullptr)
						{
							if (UActorComponent* Archetype = Cast<UActorComponent>(EditableComponentTemplate->GetArchetype()))
							{
								ReferencingProperty = FComponentEditorUtils::GetPropertyForEditableNativeComponent(Archetype);
							}
						}
						if (ReferencingProperty)
						{
							SelectedObjectProperties.Add(ReferencingProperty);
						}
					}
				}
			}
		}
	}

	PropertyViewTitle = Options.ForcedTitle;
	bShowComponents = Options.bShowComponents;

	// Update our context-sensitive editing widget
	ContextualEditingBorderWidget->SetContent( MakeContextualEditingWidget(SelectionInfo, Options) );
}

bool SKismetInspector::IsStructViewPropertyReadOnly(const struct FPropertyAndParent& PropertyAndParent) const
{
	const FProperty& Property = PropertyAndParent.Property;
	if (Property.HasAnyPropertyFlags(CPF_EditConst))
	{
		return true;
	}

	return false;
}

bool SKismetInspector::IsAnyParentOrContainerSelected(const FPropertyAndParent& PropertyAndParent) const
{
	for (const FProperty* CurrentProperty : PropertyAndParent.ParentProperties)
	{
		if (SelectedObjectProperties.Find(const_cast<FProperty*>(CurrentProperty)))
		{
			return true;
		}

		// the property might be the Inner property of an array (or Key/Value of a map), so check if the outer property is selected
		const FProperty* CurrentOuter = CurrentProperty->GetOwner<FProperty>();
		if (CurrentOuter != nullptr && SelectedObjectProperties.Find(const_cast<FProperty*>(CurrentOuter)))
		{
			return true;
		}
	}

	return false;
} 

bool SKismetInspector::IsPropertyVisible( const FPropertyAndParent& PropertyAndParent ) const
{
	const FProperty& Property = PropertyAndParent.Property;

	// If we are in 'instance preview' - hide anything marked 'disabled edit on instance'
	if ((ECheckBoxState::Checked == PublicViewState) && Property.HasAnyPropertyFlags(CPF_DisableEditOnInstance))
	{
		return false;
	}

	// Only hide EditInstanceOnly properties if we are editing a CDO/archetype
	bool bIsEditingTemplate = true;
	for (const TWeakObjectPtr<UObject>& SelectedObject : SelectedObjects)
	{
		UObject* Object = SelectedObject.Get();
		if (!Object || !Object->IsTemplate())
		{
			bIsEditingTemplate = false;
			break;
		}
	}

	if (bIsEditingTemplate)
	{
		// check if the property (or any of its parent properties) was added by this blueprint
		// this is necessary because of Instanced objects, which will have a different owning class yet are conceptually contained in this blueprint
		bool bVariableAddedInCurrentBlueprint = false;
		TSharedPtr<FBlueprintEditor> BlueprintEditor = BlueprintEditorPtr.Pin();
		const UBlueprint* Blueprint = BlueprintEditor.IsValid() ? BlueprintEditor->GetBlueprintObj() : nullptr;

		auto WasAddedInThisBlueprint = [Blueprint](const FProperty* Property)
		{
			if (const UClass* OwningClass = Property->GetOwnerClass())
			{
				return Blueprint && OwningClass->ClassGeneratedBy.Get() == Blueprint;
			}
			return false;
		};
		
		bVariableAddedInCurrentBlueprint |= WasAddedInThisBlueprint(&Property);

		for (const FProperty* Parent : PropertyAndParent.ParentProperties)
		{
			bVariableAddedInCurrentBlueprint |= WasAddedInThisBlueprint(Parent);
		}

		// if this property wasn't added in this blueprint, we want to filter it out if it (or any of its parents) are marked EditInstanceOnly or private
		if (!bVariableAddedInCurrentBlueprint)
		{
			if (Property.HasAnyPropertyFlags(CPF_DisableEditOnTemplate) || Property.GetBoolMetaData(FBlueprintMetadata::MD_Private))
			{
				return false;
			}

			for (const FProperty* Parent : PropertyAndParent.ParentProperties)
			{
				if (Property.HasAnyPropertyFlags(CPF_DisableEditOnTemplate) || Parent->GetBoolMetaData(FBlueprintMetadata::MD_Private))
				{
					return false;
				}
			}
		}
	}
	
	// figure out if this Blueprint variable is an Actor variable
	const FArrayProperty* ArrayProperty = CastField<const FArrayProperty>(&Property);
	const FSetProperty* SetProperty = CastField<const FSetProperty>(&Property);
	const FMapProperty* MapProperty = CastField<const FMapProperty>(&Property);

	const FProperty* TestProperty = ArrayProperty ? ArrayProperty->Inner : &Property;
	const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(TestProperty);
	bool bIsActorProperty = (ObjectProperty != nullptr && ObjectProperty->PropertyClass && ObjectProperty->PropertyClass->IsChildOf(AActor::StaticClass()));

	if (bIsEditingTemplate && Property.HasAnyPropertyFlags(CPF_DisableEditOnTemplate) && bIsActorProperty)
	{
		// Actor variables can't have default values (because Blueprint templates are library elements that can 
		// bridge multiple levels and different levels might not have the actor that the default is referencing).
		return false;
	}

	bool bIsComponent = (ObjectProperty != nullptr && ObjectProperty->PropertyClass && ObjectProperty->PropertyClass->IsChildOf(UActorComponent::StaticClass()));
	if (!bShowComponents && bIsComponent)
	{
		// Don't show sub components properties, thats what selecting components in the component tree is for.
		return false;
	}

	// Filter down to selected properties only if set.
	if (SelectedObjectProperties.Find(const_cast<FProperty*>(&Property)))
	{
		// If the current property is selected, it is visible.
		return true;
	}
	else if (PropertyAndParent.ParentProperties.Num() > 0 && SelectedObjectProperties.Num() > 0)
	{
		if (IsAnyParentOrContainerSelected(PropertyAndParent))
		{
			return true;
		}
	}
	else if (ArrayProperty || MapProperty || SetProperty)
	{
		// .Find won't work here because the items inside of the container properties are not FProperties
		for (const TWeakFieldPtr<FProperty>& CurProp : SelectedObjectProperties)
		{
			if ((ArrayProperty && (ArrayProperty->PropertyFlags & CPF_Edit) && CurProp->GetFName() == ArrayProperty->GetFName()) ||
				(MapProperty && (MapProperty->PropertyFlags & CPF_Edit) && CurProp->GetFName() == MapProperty->GetFName()) ||
				(SetProperty && (SetProperty->PropertyFlags & CPF_Edit) && CurProp->GetFName() == SetProperty->GetFName()))
			{
				return true;
			}
		}
	}

	return SelectedObjectProperties.Num() == 0;
}

void SKismetInspector::SetPropertyWindowContents(TArray<UObject*> Objects)
{
	if (FSlateApplication::IsInitialized())
	{
		check(PropertyView.IsValid());
		PropertyView->SetObjects(Objects);
	}
}

EVisibility SKismetInspector::GetPropertyViewVisibility() const
{
	return bShowInspectorPropertyView? EVisibility::Visible : EVisibility::Collapsed;
}

bool SKismetInspector::IsPropertyEditingEnabled() const
{
	bool bIsEditable = true;

	if (BlueprintEditorPtr.IsValid())
	{
		if (GetDefault<UEditorExperimentalSettings>()->bAllowPotentiallyUnsafePropertyEditing == false)
		{
			bIsEditable = BlueprintEditorPtr.Pin()->InEditingMode();
		}
		else
		{
			// This function is essentially for PIE use so if we are NOT doing PIE use the normal path
			if (GEditor->GetPIEWorldContext() == nullptr)
			{
				bIsEditable = BlueprintEditorPtr.Pin()->InEditingMode();
			}
		}
	}

	for (const TWeakObjectPtr<UObject>& SelectedObject : SelectedObjects)
	{
		if (UActorComponent* Component = Cast<UActorComponent>(SelectedObject.Get()))
		{
			if(!CastChecked<UActorComponent>(Component->GetArchetype())->IsEditableWhenInherited())
			{
				bIsEditable = false;
				break;
			}
		}
		else if(UEdGraphNode* EdGraphNode = Cast<UEdGraphNode>(SelectedObject.Get()))
		{
			if(UEdGraph* OuterGraph = EdGraphNode->GetGraph())
			{
				if(BlueprintEditorPtr.IsValid() && !BlueprintEditorPtr.Pin()->IsEditable(OuterGraph))
				{
					// Allow property editing on interface function graph terminator nodes (i.e. allow users to modify the interface function signature).
					const bool bIsInterfaceGraphTerminatorNode = FBlueprintEditorUtils::IsInterfaceGraph(OuterGraph) && EdGraphNode->IsA<UK2Node_FunctionTerminator>();
					if (!bIsInterfaceGraphTerminatorNode)
					{
						bIsEditable = false;
						break;
					}
				}
			}
		}
	}
	
	return bIsEditable && (!IsPropertyEditingEnabledDelegate.IsBound() || IsPropertyEditingEnabledDelegate.Execute());
}

EVisibility SKismetInspector::GetInheritedBlueprintComponentWarningVisibility() const
{
	bool bIsUneditableBlueprintComponent = false;

	// Check to see if any selected components are inherited from blueprint
	for (const TWeakObjectPtr<UObject>& SelectedObject : SelectedObjects)
	{
		UActorComponent* Component = Cast<UActorComponent>(SelectedObject.Get());
		bIsUneditableBlueprintComponent = Component ? !CastChecked<UActorComponent>(Component->GetArchetype())->IsEditableWhenInherited() : false;
		if (bIsUneditableBlueprintComponent)
		{
			break;
		}
	}

	return bIsUneditableBlueprintComponent ? EVisibility::Visible : EVisibility::Collapsed;
}

void SKismetInspector::OnInheritedBlueprintComponentWarningHyperlinkClicked(const FSlateHyperlinkRun::FMetadata& Metadata)
{
	if (BlueprintEditorPtr.IsValid())
	{
		UBlueprint* Blueprint = BlueprintEditorPtr.Pin()->GetBlueprintObj();
		if (Blueprint && Blueprint->ParentClass->HasAllClassFlags(CLASS_CompiledFromBlueprint))
		{
			// Open the blueprint
			GEditor->EditObject(CastChecked<UBlueprint>(Blueprint->ParentClass->ClassGeneratedBy));
		}
	}
}


ECheckBoxState SKismetInspector::GetPublicViewCheckboxState() const
{
	return PublicViewState;
}

void SKismetInspector::SetPublicViewCheckboxState( ECheckBoxState InIsChecked )
{
	PublicViewState = InIsChecked;

	//reset the details view
	TArray<UObject*> Objs;
	for(auto It(SelectedObjects.CreateIterator());It;++It)
	{
		Objs.Add(It->Get());
	}
	SelectedObjects.Empty();
	
	if(Objs.Num() > 1)
	{
		ShowDetailsForObjects(Objs);
	}
	else if(Objs.Num() == 1)
	{
		ShowDetailsForSingleObject(Objs[0], FShowDetailsOptions(PropertyViewTitle));
	}
	
	BlueprintEditorPtr.Pin()->StartEditingDefaults();
}

void SKismetInspector::OnFinishedChangingProperties(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	ImportNamespacesForPropertyValue(InPropertyChangedEvent.MemberProperty);
}

void SKismetInspector::ImportNamespacesForPropertyValue(const FProperty* InProperty) const
{
	if (!GetDefault<UBlueprintEditorSettings>()->bEnableNamespaceImportingFeatures)
	{
		return;
	}

	if (!InProperty || SelectedObjects.Num() == 0)
	{
		return;
	}

	TSharedPtr<FBlueprintEditor> BlueprintEditor = BlueprintEditorPtr.Pin();
	if (BlueprintEditor.IsValid())
	{
		// Gather all namespace identifier strings associated with the property's value for each edited object.
		TSet<FString> AssociatedNamespaces;
		for (const TWeakObjectPtr<UObject>& SelectedObjectPtr : SelectedObjects)
		{
			if (const UObject* SelectedObject = SelectedObjectPtr.Get())
			{
				const UStruct* SelectedType = SelectedObject->GetClass();

				// Ensure that the selected object type matches the property's owner.
				// For example, a details customization may select an unrelated object
				// and then customize each row with an external object reference. In
				// those cases, the customization would need to handle this explicitly.
				if (!InProperty->IsIn(SelectedType))
				{
					continue;
				}

				FBlueprintNamespaceUtilities::GetPropertyValueNamespaces(InProperty, SelectedObject, AssociatedNamespaces);
			}
		}

		// Auto-import any namespace(s) associated with the property's value into the current editor context.
		if (AssociatedNamespaces.Num() > 0)
		{
			FBlueprintEditor::FImportNamespaceExParameters Params;
			Params.NamespacesToImport = MoveTemp(AssociatedNamespaces);
			BlueprintEditor->ImportNamespaceEx(Params);
		}
	}
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
