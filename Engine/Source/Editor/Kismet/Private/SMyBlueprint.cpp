// Copyright Epic Games, Inc. All Rights Reserved.


#include "SMyBlueprint.h"

#include "Animation/AnimClassInterface.h"
#include "AnimationGraph.h"
#include "AnimationStateMachineGraph.h"
#include "AnimationStateMachineSchema.h"
#include "AnimGraphNode_StateMachineBase.h"
#include "BPDelegateDragDropAction.h"
#include "BPFunctionDragDropAction.h"
#include "BPGraphClipboardData.h"
#include "BPVariableDragDropAction.h"
#include "BlueprintEditor.h"
#include "BlueprintEditorCommands.h"
#include "BlueprintEditorModule.h"
#include "BlueprintEditorSettings.h"
#include "Components/ActorComponent.h"
#include "Components/TimelineComponent.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailLayoutBuilder.h"
#include "Dialogs/Dialogs.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EditorCategoryUtils.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/MemberReference.h"
#include "Engine/TimelineTemplate.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GenericPlatform/GenericApplication.h"
#include "GraphActionNode.h"
#include "GraphEditorActions.h"
#include "GraphEditorDragDropAction.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformCrt.h"
#include "Input/DragAndDrop.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "K2Node.h"
#include "K2Node_AddComponent.h"
#include "K2Node_CallFunction.h"
#include "K2Node_Composite.h"
#include "K2Node_CreateDelegate.h"
#include "K2Node_EditablePinBase.h"
#include "K2Node_Event.h"
#include "K2Node_EventNodeInterface.h"
#include "K2Node_ExternalGraphInterface.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_Tunnel.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/CString.h"
#include "Misc/Guid.h"
#include "Misc/StringOutputDevice.h"
#include "ObjectEditorUtils.h"
#include "ObjectTools.h"
#include "SBlueprintPalette.h"
#include "SGraphActionMenu.h"
#include "SKismetInspector.h"
#include "SPositiveActionButton.h"
#include "SReplaceNodeReferences.h"
#include "SSubobjectBlueprintEditor.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "SourceCodeNavigation.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/Casts.h"
#include "Templates/Less.h"
#include "Templates/SubclassOf.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"
#include "Textures/SlateIcon.h"
#include "Types/ISlateMetaData.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/Script.h"
#include "UObject/UObjectBaseUtility.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"

struct FGeometry;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "MyBlueprint"

//////////////////////////////////////////////////////////////////////////

// Magic values to differentiate Variables and Graphs on the clipboard
static const TCHAR* VAR_PREFIX = TEXT("BPVar");
static const TCHAR* GRAPH_PREFIX = TEXT("BPGraph");

//////////////////////////////////////////////////////////////////////////

void FMyBlueprintCommands::RegisterCommands() 
{
	UI_COMMAND( OpenGraph, "Open Graph", "Opens up this function, macro, or event graph's graph panel up.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( OpenGraphInNewTab, "Open in New Tab", "Opens up this function, macro, or event graph's graph panel up in a new tab. Hold down Ctrl and double click for shortcut.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( OpenExternalGraph, "Open External Graph", "Opens up this external graph's graph panel in its own asset editor", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( FocusNode, "Focus", "Focuses on the associated node", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( FocusNodeInNewTab, "Focus in New Tab", "Focuses on the associated node in a new tab", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ImplementFunction, "Implement event", "Implements this overridable function as a new event.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( DeleteEntry, "Delete", "Deletes this function or variable from this blueprint.", EUserInterfaceActionType::Button, FInputChord(EKeys::Delete), FInputChord(EKeys::BackSpace));
	UI_COMMAND( PasteVariable, "Paste Variable", "Pastes the variable to this blueprint.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND( PasteLocalVariable, "Paste Local Variable", "Pastes the variable to this scope.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND( PasteFunction, "Paste Function", "Pastes the function to this blueprint.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND( PasteMacro, "Paste Macro", "Pastes the macro to this blueprint.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND( GotoNativeVarDefinition, "Goto Code Definition", "Goto the native code definition of this variable", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( MoveVariableToParent, "Move to Parent Class", "Moves the variable to its parent class", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( MoveFunctionToParent, "Move to Parent Class", "Moves the function to its parent class", EUserInterfaceActionType::Button, FInputChord() );
}

//////////////////////////////////////////////////////////////////////////

class FMyBlueprintCategoryDragDropAction : public FGraphEditorDragDropAction
{
public:
	DRAG_DROP_OPERATOR_TYPE(FMyBlueprintCategoryDragDropAction, FGraphEditorDragDropAction)

	virtual void HoverTargetChanged() override
	{
		const FSlateBrush* StatusSymbol = FAppStyle::GetBrush(TEXT("NoBrush")); 
		FText Message = DraggedCategory;

		FFormatNamedArguments Args;
		Args.Add(TEXT("DraggedCategory"), DraggedCategory);

		if (!HoveredCategoryName.IsEmpty())
		{
			if(HoveredCategoryName.EqualTo(DraggedCategory))
			{
				StatusSymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));

				
				Message = FText::Format( LOCTEXT("MoveCatOverSelf", "Cannot insert category '{DraggedCategory}' before itself."), Args );
			}
			else
			{
				StatusSymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
				Args.Add(TEXT("HoveredCategory"), HoveredCategoryName);
				Message = FText::Format( LOCTEXT("MoveCatOK", "Move category '{DraggedCategory}' before '{HoveredCategory}'"), Args );
			}
		}
		else if (HoveredAction.IsValid())
		{
			StatusSymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
			Message = LOCTEXT("MoveCatOverAction", "Can only insert before another category.");
		}
		else
		{
			StatusSymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
			Message = FText::Format(LOCTEXT("MoveCatAction", "Moving category '{DraggedCategory}'"), Args);
		}

		SetSimpleFeedbackMessage(StatusSymbol, FLinearColor::White, Message);
	}
	
	virtual FReply DroppedOnCategory(FText OnCategory) override
	{
		// Get MyBlueprint via MyBlueprintPtr
		TSharedPtr<SMyBlueprint> MyBlueprint = MyBlueprintPtr.Pin();
		if(MyBlueprint.IsValid())
		{
			// Move the category in the blueprint category sort list
			MyBlueprint->MoveCategoryBeforeCategory( DraggedCategory, OnCategory );
		}

		return FReply::Handled();
	}

	static TSharedRef<FMyBlueprintCategoryDragDropAction> New(const FText& InCategory, TSharedPtr<SMyBlueprint> InMyBlueprint)
	{
		TSharedRef<FMyBlueprintCategoryDragDropAction> Operation = MakeShareable(new FMyBlueprintCategoryDragDropAction);
		Operation->DraggedCategory = InCategory;
		Operation->MyBlueprintPtr = InMyBlueprint;
		Operation->Construct();
		return Operation;
	}

	/** Category we were dragging */
	FText DraggedCategory;
	/** MyBlueprint widget we dragged from */
	TWeakPtr<SMyBlueprint>	MyBlueprintPtr;
};

//////////////////////////////////////////////////////////////////////////
// FGraphActionSort

// Helper structure to aid category sorting
struct FGraphActionSort
{
public:
	FGraphActionSort(TArray<FName>& BlueprintCategorySorting)
		: bCategoriesModified(false)
		, CategorySortIndices(BlueprintCategorySorting)
	{
		CategoryUsage.Init(0, CategorySortIndices.Num());
	}

	void AddAction(const FString& Category, TSharedPtr<FEdGraphSchemaAction> Action)
	{
		// Find root category
		int32 RootCategoryDelim = Category.Find(TEXT("|"));
		FName RootCategory = RootCategoryDelim == INDEX_NONE ? *Category : *Category.Left(RootCategoryDelim);
		// Get root sort index
		const int32 SortIndex = GetSortIndex(RootCategory) + Action->GetSectionID();

		SortedActions.Add(SortIndex, Action);
		SortedActionsByCategoryRoot.Add(RootCategory, Action);
	}

	void AddAction(TSharedPtr<FEdGraphSchemaAction> Action)
	{
		const FString UserCategoryName = FEditorCategoryUtils::GetCategoryDisplayString(Action->GetCategory().ToString());
		AddAction(UserCategoryName, Action);
	}

	void GetAllActions(FGraphActionListBuilderBase& OutActions)
	{
		SortedActions.KeySort(TLess<int32>());

		for (const TPair<int32, TSharedPtr<FEdGraphSchemaAction>>& Iter : SortedActions)
		{
			OutActions.AddAction(Iter.Value);
		}
	}

	void CleanupCategories()
	{
		// Scrub unused categories from the blueprint
		if (bCategoriesModified)
		{
			for (int32 CategoryIdx = CategoryUsage.Num() - 1; CategoryIdx >= 0; CategoryIdx--)
			{
				if (CategoryUsage[CategoryIdx] == 0)
				{
					CategorySortIndices.RemoveAt(CategoryIdx);
				}
			}
			bCategoriesModified = false;
		}
	}

	bool HasCategory(const FString& Category, const NodeSectionID::Type SectionID) const
	{
		const int32 RootCategoryDelim = Category.Find(TEXT("|"));
		const FName RootCategory = RootCategoryDelim == INDEX_NONE ? *Category : *Category.Left(RootCategoryDelim);
		for (TMultiMap<FName, TSharedPtr<FEdGraphSchemaAction>>::TConstKeyIterator It = SortedActionsByCategoryRoot.CreateConstKeyIterator(RootCategory); It; ++It)
		{
			if (It.Value()->GetSectionID() == SectionID)
			{
				//It contains the root but might not contain the full path.
				if (RootCategoryDelim == INDEX_NONE || It.Value()->GetCategory().ToString().StartsWith(Category))
				{
					return true;
				}
			}
		}
		return false;
	}

private:
	const int32 GetSortIndex(FName Category)
	{
		int32 SortIndex = CategorySortIndices.Find(Category);

		if (SortIndex == INDEX_NONE)
		{
			bCategoriesModified = true;
			SortIndex = CategorySortIndices.Add(Category);
			CategoryUsage.Add(0);
		}
		CategoryUsage[SortIndex]++;
		// Spread the sort values so we can fine tune sorting
		SortIndex *= 1000;

		return SortIndex + SortedActions.Num();
	}

private:
	/** Signals if the blueprint categories have been modified and require cleanup */
	bool bCategoriesModified;
	/** Tracks category usage to aid removal of unused categories */
	TArray<int32> CategoryUsage;
	/** Reference to the category sorting in the blueprint */
	TArray<FName>& CategorySortIndices;
	/** Map used to sort Graph actions */
	TMultiMap<int32, TSharedPtr<FEdGraphSchemaAction>> SortedActions;
	TMultiMap<FName, TSharedPtr<FEdGraphSchemaAction>> SortedActionsByCategoryRoot;
};

//////////////////////////////////////////////////////////////////////////

void SMyBlueprint::Construct(const FArguments& InArgs, TWeakPtr<FBlueprintEditor> InBlueprintEditor, const UBlueprint* InBlueprint )
{
	bNeedsRefresh = false;
	bShowReplicatedVariablesOnly = false;

	BlueprintEditorPtr = InBlueprintEditor;
	EdGraph = nullptr;
	
	TSharedPtr<SWidget> ToolbarBuilderWidget = TSharedPtr<SWidget>();

	if( InBlueprintEditor.IsValid() )
	{
		Blueprint = BlueprintEditorPtr.Pin()->GetBlueprintObj();

		CommandList = MakeShareable(new FUICommandList);
		
		CommandList->Append(InBlueprintEditor.Pin()->GetToolkitCommands());

		CommandList->MapAction( FMyBlueprintCommands::Get().OpenGraph,
			FExecuteAction::CreateSP(this, &SMyBlueprint::OnOpenGraph),
			FCanExecuteAction(), FIsActionChecked(),
			FIsActionButtonVisible::CreateSP(this, &SMyBlueprint::CanOpenGraph) );
	
		CommandList->MapAction( FMyBlueprintCommands::Get().OpenGraphInNewTab,
			FExecuteAction::CreateSP(this, &SMyBlueprint::OnOpenGraphInNewTab),
			FCanExecuteAction(), FIsActionChecked(),
			FIsActionButtonVisible::CreateSP(this, &SMyBlueprint::CanOpenGraph) );

		CommandList->MapAction( FMyBlueprintCommands::Get().OpenExternalGraph,
			FExecuteAction::CreateSP(this, &SMyBlueprint::OnOpenExternalGraph),
			FCanExecuteAction(), FIsActionChecked(),
			FIsActionButtonVisible::CreateSP(this, &SMyBlueprint::CanOpenExternalGraph) );
		
		CommandList->MapAction( FMyBlueprintCommands::Get().FocusNode,
			FExecuteAction::CreateSP(this, &SMyBlueprint::OnFocusNode),
			FCanExecuteAction(), FIsActionChecked(),
			FIsActionButtonVisible::CreateSP(this, &SMyBlueprint::CanFocusOnNode) );

		CommandList->MapAction( FMyBlueprintCommands::Get().FocusNodeInNewTab,
			FExecuteAction::CreateSP(this, &SMyBlueprint::OnFocusNodeInNewTab),
			FCanExecuteAction(), FIsActionChecked(),
			FIsActionButtonVisible::CreateSP(this, &SMyBlueprint::CanFocusOnNode) );

		CommandList->MapAction( FMyBlueprintCommands::Get().ImplementFunction,
			FExecuteAction::CreateSP(this, &SMyBlueprint::OnImplementFunction),
			FCanExecuteAction(), FIsActionChecked(),
			FIsActionButtonVisible::CreateSP(this, &SMyBlueprint::CanImplementFunction) );
		
		CommandList->MapAction( FGraphEditorCommands::Get().FindReferences,
			FExecuteAction::CreateSP(this, &SMyBlueprint::OnFindReference, /*bSearchAllBlueprints=*/false, EGetFindReferenceSearchStringFlags::Legacy),
			FCanExecuteAction(),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateSP(this, &SMyBlueprint::CanFindReference) );
		
		CommandList->MapAction( FGraphEditorCommands::Get().FindReferencesByNameLocal,
			FExecuteAction::CreateSP(this, &SMyBlueprint::OnFindReference, /*bSearchAllBlueprints=*/false, EGetFindReferenceSearchStringFlags::None),
			FCanExecuteAction(),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateSP(this, &SMyBlueprint::CanFindReference) );

		
		CommandList->MapAction( FGraphEditorCommands::Get().FindReferencesByNameGlobal,
			FExecuteAction::CreateSP(this, &SMyBlueprint::OnFindReference, /*bSearchAllBlueprints=*/true, EGetFindReferenceSearchStringFlags::None),
			FCanExecuteAction(),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateSP(this, &SMyBlueprint::CanFindReference) );
		
		CommandList->MapAction( FGraphEditorCommands::Get().FindReferencesByClassMemberLocal,
			FExecuteAction::CreateSP(this, &SMyBlueprint::OnFindReference, /*bSearchAllBlueprints=*/false, EGetFindReferenceSearchStringFlags::UseSearchSyntax),
			FCanExecuteAction(),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateSP(this, &SMyBlueprint::CanFindReference) );
		
		CommandList->MapAction( FGraphEditorCommands::Get().FindReferencesByClassMemberGlobal,
			FExecuteAction::CreateSP(this, &SMyBlueprint::OnFindReference, /*bSearchAllBlueprints=*/true, EGetFindReferenceSearchStringFlags::UseSearchSyntax),
			FCanExecuteAction(),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateSP(this, &SMyBlueprint::CanFindReference) );

		CommandList->MapAction( FGraphEditorCommands::Get().FindAndReplaceReferences,
			FExecuteAction::CreateSP(this, &SMyBlueprint::OnFindAndReplaceReference),
			FCanExecuteAction(),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateSP(this, &SMyBlueprint::CanFindAndReplaceReference) );
	
		CommandList->MapAction( FMyBlueprintCommands::Get().DeleteEntry,
			FExecuteAction::CreateSP(this, &SMyBlueprint::OnDeleteEntry),
			FCanExecuteAction::CreateSP(this, &SMyBlueprint::CanDeleteEntry) );

		CommandList->MapAction( FGenericCommands::Get().Duplicate,
			FExecuteAction::CreateSP(this, &SMyBlueprint::OnDuplicateAction),
			FCanExecuteAction::CreateSP(this, &SMyBlueprint::CanDuplicateAction),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateSP(this, &SMyBlueprint::IsDuplicateActionVisible) );

		CommandList->MapAction( FMyBlueprintCommands::Get().MoveVariableToParent,
			FExecuteAction::CreateSP(this, &SMyBlueprint::OnMoveToParent),
			FCanExecuteAction(),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateSP(this, &SMyBlueprint::CanMoveVariableToParent) );

		CommandList->MapAction( FMyBlueprintCommands::Get().MoveFunctionToParent,
			 FExecuteAction::CreateSP(this, &SMyBlueprint::OnMoveToParent),
			 FCanExecuteAction(),
			 FIsActionChecked(),
			 FIsActionButtonVisible::CreateSP(this, &SMyBlueprint::CanMoveFunctionToParent) );

		CommandList->MapAction( FMyBlueprintCommands::Get().GotoNativeVarDefinition,
			FExecuteAction::CreateSP(this, &SMyBlueprint::GotoNativeCodeVarDefinition),
			FCanExecuteAction(),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateSP(this, &SMyBlueprint::IsNativeVariable) );
		ToolbarBuilderWidget = SNullWidget::NullWidget;
	
		CommandList->MapAction(FGenericCommands::Get().Rename,
			FExecuteAction::CreateSP(this, &SMyBlueprint::OnRequestRenameOnActionNode),
			FCanExecuteAction::CreateSP(this, &SMyBlueprint::CanRequestRenameOnActionNode));

		CommandList->MapAction(FGenericCommands::Get().Copy,
			FExecuteAction::CreateSP(this, &SMyBlueprint::OnCopy),
			FCanExecuteAction::CreateSP(this, &SMyBlueprint::CanCopy));
		
		CommandList->MapAction(FGenericCommands::Get().Cut,
			FExecuteAction::CreateSP(this, &SMyBlueprint::OnCut),
			FCanExecuteAction::CreateSP(this, &SMyBlueprint::CanCut));

		CommandList->MapAction(FGenericCommands::Get().Paste,
			FExecuteAction::CreateSP(this, &SMyBlueprint::OnPasteGeneric),
			FCanExecuteAction(), FIsActionChecked(),
			FIsActionButtonVisible::CreateSP(this, &SMyBlueprint::CanPasteGeneric));

		CommandList->MapAction(FMyBlueprintCommands::Get().PasteVariable,
			FExecuteAction::CreateSP(this, &SMyBlueprint::OnPasteVariable),
			FCanExecuteAction(), FIsActionChecked(),
			FIsActionButtonVisible::CreateSP(this, &SMyBlueprint::CanPasteVariable));

		CommandList->MapAction(FMyBlueprintCommands::Get().PasteLocalVariable,
			FExecuteAction::CreateSP(this, &SMyBlueprint::OnPasteLocalVariable),
			FCanExecuteAction(), FIsActionChecked(),
			FIsActionButtonVisible::CreateSP(this, &SMyBlueprint::CanPasteLocalVariable));

		CommandList->MapAction(FMyBlueprintCommands::Get().PasteFunction,
			FExecuteAction::CreateSP(this, &SMyBlueprint::OnPasteFunction),
			FCanExecuteAction(), FIsActionChecked(),
			FIsActionButtonVisible::CreateSP(this, &SMyBlueprint::CanPasteFunction));

		CommandList->MapAction(FMyBlueprintCommands::Get().PasteMacro,
			FExecuteAction::CreateSP(this, &SMyBlueprint::OnPasteMacro),
			FCanExecuteAction(), FIsActionChecked(),
			FIsActionButtonVisible::CreateSP(this, &SMyBlueprint::CanPasteMacro));
	}
	else
	{
		// we're in read only mode when there's no blueprint editor:
		Blueprint = const_cast<UBlueprint*>(InBlueprint);
		check(Blueprint);
		ToolbarBuilderWidget = SNew(SBox);
	}

	TSharedPtr<SWidget> AddNewMenu = SNew(SPositiveActionButton)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("MyBlueprintAddNewCombo")))
		.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
		.Text(LOCTEXT("AddNewLabel", "Add"))
		.ToolTipText(LOCTEXT("AddNewToolTip", "Add a new Variable, Graph, Function, Macro, or Event Dispatcher."))
		.IsEnabled(this, &SMyBlueprint::IsEditingMode)
		.OnGetMenuContent(this, &SMyBlueprint::CreateAddNewMenuWidget);

	FMenuBuilder ViewOptions(true, nullptr);

	ViewOptions.AddMenuEntry(
		LOCTEXT("ShowInheritedVariables", "Show Inherited Variables"),
		LOCTEXT("ShowInheritedVariablesTooltip", "Should inherited variables from parent classes and blueprints be shown in the tree?"),
		FSlateIcon(),
		FUIAction( 
			FExecuteAction::CreateSP( this, &SMyBlueprint::OnToggleShowInheritedVariables ),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP( this, &SMyBlueprint::IsShowingInheritedVariables )
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton,
		TEXT("MyBlueprint_ShowInheritedVariables")
	);

	ViewOptions.AddMenuEntry(
		LOCTEXT("ShowEmptySections", "Show Empty Sections"),
		LOCTEXT("ShowEmptySectionsTooltip", "Should we show empty sections? eg. Graphs, Functions...etc."),
		FSlateIcon(),
		FUIAction( 
			FExecuteAction::CreateSP( this, &SMyBlueprint::OnToggleShowEmptySections ),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SMyBlueprint::IsShowingEmptySections)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton,
		TEXT("MyBlueprint_ShowEmptySections")
	);

	ViewOptions.AddMenuEntry(
		LOCTEXT("ShowReplicatedVariablesOnly", "Show Replicated Variables Only"),
		LOCTEXT("ShowReplicatedVariablesOnlyTooltip", "Should we only show variables that are replicated?"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SMyBlueprint::OnToggleShowReplicatedVariablesOnly),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SMyBlueprint::IsShowingReplicatedVariablesOnly)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton,
		TEXT("MyBlueprint_ShowReplicatedVariablesOnly")
	);

	ViewOptions.AddMenuEntry(
		LOCTEXT("AlwaysShowInterfacesInOverrides", "Show interfaces in the function override menu"),
		LOCTEXT("AlwaysShowInterfacesInOverridesTooltip", "Should we always display interface functions/events in the override menu?"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SMyBlueprint::OnToggleAlwaysShowInterfacesInOverrides),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SMyBlueprint::GetAlwaysShowInterfacesInOverrides)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton,
		TEXT("MyBlueprint_AlwaysShowInterfacesInOverrides")
	);

	ViewOptions.AddMenuEntry(
		LOCTEXT("AlwaysShowAccessSpecifier", "Show access specifier in the My Blueprint View"),
		LOCTEXT("AlwaysShowAccessSpecifierTooltip", "Should we always display the access specifier of functions in the function menu?"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SMyBlueprint::OnToggleShowAccessSpecifier),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SMyBlueprint::GetShowAccessSpecifier)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton,
		TEXT("MyBlueprint_AlwaysShowAccessSpecifier")
	);

	SAssignNew(FilterBox, SSearchBox)
		.OnTextChanged( this, &SMyBlueprint::OnFilterTextChanged );

	// create the main action list piece of this widget
	SAssignNew(GraphActionMenu, SGraphActionMenu, false)
		.OnGetFilterText(this, &SMyBlueprint::GetFilterText)
		.OnCreateWidgetForAction(this, &SMyBlueprint::OnCreateWidgetForAction)
		.OnCollectAllActions(this, &SMyBlueprint::CollectAllActions)
		.OnCollectStaticSections(this, &SMyBlueprint::CollectStaticSections)
		.OnActionDragged(this, &SMyBlueprint::OnActionDragged)
		.OnCategoryDragged(this, &SMyBlueprint::OnCategoryDragged)
		.OnActionSelected(this, &SMyBlueprint::OnGlobalActionSelected)
		.OnActionDoubleClicked(this, &SMyBlueprint::OnActionDoubleClicked)
		.OnContextMenuOpening(this, &SMyBlueprint::OnContextMenuOpening)
		.OnCategoryTextCommitted(this, &SMyBlueprint::OnCategoryNameCommitted)
		.OnCanRenameSelectedAction(this, &SMyBlueprint::CanRequestRenameOnActionNode)
		.OnGetSectionTitle(this, &SMyBlueprint::OnGetSectionTitle)
		.OnGetSectionWidget(this, &SMyBlueprint::OnGetSectionWidget)
		.OnActionMatchesName(this, &SMyBlueprint::HandleActionMatchesName)
		.DefaultRowExpanderBaseIndentLevel(1)
		.AlphaSortItems(false)
		.UseSectionStyling(true);


	// now piece together all the content for this widget
	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.Padding(4.0f)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("MyBlueprintPanel")))
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					ToolbarBuilderWidget.ToSharedRef()
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(0, 0, 4, 0)
					[
						AddNewMenu.ToSharedRef()
					]

					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.VAlign(VAlign_Center)
					[
						FilterBox.ToSharedRef()
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2, 0, 0, 0)
					[
						SNew(SComboButton)
						.ContentPadding(0.0f)
						.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
						.HasDownArrow(false)
						.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ViewOptions")))
						.ButtonContent()
						[
							SNew(SImage)
							.ColorAndOpacity(FSlateColor::UseForeground())
							.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
						]
						.MenuContent()
						[
							ViewOptions.MakeWidget()
						]
					]
				]
			]
		]
		
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			GraphActionMenu.ToSharedRef()
		]
	];
	
	ResetLastPinType();

	if( !BlueprintEditorPtr.IsValid() )
	{
		Refresh();
	}

	TMap<int32, bool> ExpandedSections;
	ExpandedSections.Add(NodeSectionID::VARIABLE, true);
	ExpandedSections.Add(NodeSectionID::FUNCTION, true);
	ExpandedSections.Add(NodeSectionID::MACRO, true);
	ExpandedSections.Add(NodeSectionID::DELEGATE, true);
	ExpandedSections.Add(NodeSectionID::GRAPH, true);
	ExpandedSections.Add(NodeSectionID::ANIMGRAPH, true);
	ExpandedSections.Add(NodeSectionID::ANIMLAYER, true);
	ExpandedSections.Add(NodeSectionID::LOCAL_VARIABLE, true);

	GraphActionMenu->SetSectionExpansion(ExpandedSections);

	FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &SMyBlueprint::OnObjectPropertyChanged);
}

SMyBlueprint::~SMyBlueprint()
{
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
}

void SMyBlueprint::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if(bNeedsRefresh)
	{
		Refresh();
	}
}

FReply SMyBlueprint::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList.IsValid() && CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SMyBlueprint::OnCategoryNameCommitted(const FText& InNewText, ETextCommit::Type InTextCommit, TWeakPtr< FGraphActionNode > InAction )
{
	// Remove excess whitespace and prevent categories with just spaces
	FText CategoryName = FText::TrimPrecedingAndTrailing(InNewText);

	TArray<TSharedPtr<FEdGraphSchemaAction>> Actions;
	GraphActionMenu->GetCategorySubActions(InAction, Actions);

	if (Actions.Num())
	{
		const FScopedTransaction Transaction( LOCTEXT( "RenameCategory", "Rename Category" ) );

		GetBlueprintObj()->Modify();

		for (int32 i = 0; i < Actions.Num(); ++i)
		{
			if (Actions[i]->GetTypeId() == FEdGraphSchemaAction_K2Var::StaticGetTypeId())
			{
				FEdGraphSchemaAction_K2Var* VarAction = (FEdGraphSchemaAction_K2Var*)Actions[i].Get();

				if(FProperty* TargetProperty = VarAction->GetProperty())
				{
					UClass* OuterClass = VarAction->GetProperty()->GetOwnerChecked<UClass>();
					const bool bIsNativeVar = (OuterClass->ClassGeneratedBy == NULL);

					// If the variable is not native and it's outer is the skeleton generated class, we can rename the category
					if(!bIsNativeVar && OuterClass == GetBlueprintObj()->SkeletonGeneratedClass)
					{
						FBlueprintEditorUtils::SetBlueprintVariableCategory(GetBlueprintObj(), VarAction->GetVariableName(), NULL, CategoryName, true);
					}
				}
			}
			else if (Actions[i]->GetTypeId() == FEdGraphSchemaAction_K2LocalVar::StaticGetTypeId())
			{
				FEdGraphSchemaAction_K2LocalVar* LocalVarAction = (FEdGraphSchemaAction_K2LocalVar*)Actions[i].Get();

				FBlueprintEditorUtils::SetBlueprintVariableCategory(GetBlueprintObj(), LocalVarAction->GetVariableName(), CastChecked<UStruct>(LocalVarAction->GetVariableScope()), CategoryName, true);
			}
			else if (Actions[i]->GetTypeId() == FEdGraphSchemaAction_K2Delegate::StaticGetTypeId())
			{
				FEdGraphSchemaAction_K2Delegate* DelegateAction = (FEdGraphSchemaAction_K2Delegate*)Actions[i].Get();
				FBlueprintEditorUtils::SetBlueprintVariableCategory(GetBlueprintObj(), DelegateAction->GetDelegateProperty()->GetFName(), NULL, CategoryName, true);
			}
			else if (Actions[i]->GetTypeId() == FEdGraphSchemaAction_K2Graph::StaticGetTypeId())
			{
				// Do not allow renaming of any graph actions outside of the following
				if(Actions[i]->GetSectionID() == NodeSectionID::FUNCTION || Actions[i]->GetSectionID() == NodeSectionID::MACRO || Actions[i]->GetSectionID() == NodeSectionID::ANIMLAYER)
				{
					FEdGraphSchemaAction_K2Graph* GraphAction = (FEdGraphSchemaAction_K2Graph*)Actions[i].Get();

					// Don't allow changing the category of a graph who's parent is not the current Blueprint
					if(GraphAction && !FBlueprintEditorUtils::IsPaletteActionReadOnly(Actions[i], BlueprintEditorPtr.Pin()) && FBlueprintEditorUtils::FindBlueprintForGraph(GraphAction->EdGraph) == GetBlueprintObj())
					{
						FReply ReplyBySchema = FReply::Unhandled();
						if (GraphAction->EdGraph)
						{
							if (UEdGraphSchema* Schema = (UEdGraphSchema*)GraphAction->EdGraph->GetSchema())
							{
								TArray<FString> CategoryParts;
								TWeakPtr< FGraphActionNode > CurrentActionNode = InAction;
								while (CurrentActionNode.IsValid())
								{
									FString CurrentDisplayName = CurrentActionNode.Pin()->GetDisplayName().ToString();
									if (!CurrentDisplayName.IsEmpty())
									{
										CategoryParts.Insert(CurrentDisplayName, 0);
									}
									CurrentActionNode = CurrentActionNode.Pin()->GetParentNode();
								}

								FString OldCategoryPath = FString::Join(CategoryParts, TEXT("|"));
								CategoryParts.Last() = CategoryName.ToString();
								FString NewCategoryPath = FString::Join(CategoryParts, TEXT("|"));

								FString CurrentCategoryPath = GraphAction->GetCategory().ToString();
								if (CurrentCategoryPath == OldCategoryPath || CurrentCategoryPath.StartsWith(OldCategoryPath + TEXT("|"), ESearchCase::CaseSensitive))
								{
									NewCategoryPath = NewCategoryPath + CurrentCategoryPath.RightChop(OldCategoryPath.Len());
								}

								ReplyBySchema = Schema->TrySetGraphCategory(GraphAction->EdGraph, FText::FromString(NewCategoryPath));
							}
						}

						if (!ReplyBySchema.IsEventHandled())
						{
							GraphAction->MovePersistentItemToCategory(CategoryName);
						}
					}
				}
			}
		}
		Refresh();
		FBlueprintEditorUtils::MarkBlueprintAsModified(GetBlueprintObj());
		SelectItemByName(FName(*CategoryName.ToString()), ESelectInfo::OnMouseClick, InAction.Pin()->SectionID, true);
	}
}

FText SMyBlueprint::OnGetSectionTitle( int32 InSectionID )
{
	FText SeparatorTitle;
	/* Setup an appropriate name for the section for this node */
	switch( InSectionID )
	{
	case NodeSectionID::VARIABLE:
		SeparatorTitle = NSLOCTEXT("GraphActionNode", "Variables", "Variables");
		break;
	case NodeSectionID::COMPONENT:
		SeparatorTitle = NSLOCTEXT("GraphActionNode", "Components", "Components");
		break;
	case NodeSectionID::FUNCTION:
		if ( OverridableFunctionActions.Num() > 0 )
		{
			SeparatorTitle = FText::Format(NSLOCTEXT("GraphActionNode", "FunctionsOverridableFormat", "Functions <TinyText.Subdued>({0} Overridable)</>"), FText::AsNumber(OverridableFunctionActions.Num()));
		}
		else
		{
			SeparatorTitle = NSLOCTEXT("GraphActionNode", "Functions", "Functions");
		}

		break;
	case NodeSectionID::FUNCTION_OVERRIDABLE:
		SeparatorTitle = NSLOCTEXT("GraphActionNode", "OverridableFunctions", "Overridable Functions");
		break;
	case NodeSectionID::MACRO:
		SeparatorTitle = NSLOCTEXT("GraphActionNode", "Macros", "Macros");
		break;
	case NodeSectionID::INTERFACE:
		SeparatorTitle = NSLOCTEXT("GraphActionNode", "Interfaces", "Interfaces");
		break;
	case NodeSectionID::DELEGATE:
		SeparatorTitle = NSLOCTEXT("GraphActionNode", "EventDispatchers", "Event Dispatchers");
		break;	
	case NodeSectionID::GRAPH:
		SeparatorTitle = NSLOCTEXT("GraphActionNode", "Graphs", "Graphs");
		break;
	case NodeSectionID::ANIMGRAPH:
		SeparatorTitle = NSLOCTEXT("GraphActionNode", "AnimationGraphs", "Animation Graphs");
		break;
	case NodeSectionID::ANIMLAYER:
		SeparatorTitle = NSLOCTEXT("GraphActionNode", "AnimationLayers", "Animation Layers");
		break;
	case NodeSectionID::USER_ENUM:
		SeparatorTitle = NSLOCTEXT("GraphActionNode", "Userenums", "User Enums");
		break;	
	case NodeSectionID::LOCAL_VARIABLE:
		if ( GetFocusedGraph() )
		{
			SeparatorTitle = FText::Format(NSLOCTEXT("GraphActionNode", "LocalVariables_Focused", "Local Variables <TinyText.Subdued>({0})</>"), FText::FromName(GetFocusedGraph()->GetFName()));
		}
		else
		{
			SeparatorTitle = NSLOCTEXT("GraphActionNode", "LocalVariables", "Local Variables");
		}
		break;
	case NodeSectionID::USER_STRUCT:
		SeparatorTitle = NSLOCTEXT("GraphActionNode", "Userstructs", "User Structs");
		break;	
	default:
	case NodeSectionID::NONE:
		SeparatorTitle = FText::GetEmpty();
		break;
	}
	return SeparatorTitle;
}

TSharedRef<SWidget> SMyBlueprint::OnGetSectionWidget(TSharedRef<SWidget> RowWidget, int32 InSectionID)
{
	TWeakPtr<SWidget> WeakRowWidget = RowWidget;

	FText AddNewText;
	FName MetaDataTag;

	switch ( InSectionID )
	{
	case NodeSectionID::VARIABLE:
		AddNewText = LOCTEXT("AddNewVariable", "Variable");
		MetaDataTag = TEXT("AddNewVariable");
		break;
	case NodeSectionID::FUNCTION:
		AddNewText = LOCTEXT("AddNewFunction", "Function");
		MetaDataTag = TEXT("AddNewFunction");

		if ( OverridableFunctionActions.Num() > 0 )
		{
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SAssignNew(FunctionSectionButton, SComboButton)
					.IsEnabled(this, &SMyBlueprint::IsEditingMode)
					.Visibility(EVisibility::SelfHitTestInvisible)
					.ForegroundColor(FAppStyle::GetSlateColor("DefaultForeground"))
					.OnGetMenuContent(this, &SMyBlueprint::OnGetFunctionListMenu)
					.ContentPadding(0.0f)
					.HasDownArrow(true)
					.ButtonContent()
					[
						SNew(STextBlock)
						.Font(IDetailLayoutBuilder::GetDetailFontBold())
						.Text(LOCTEXT("Override", "Override"))
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(2,0,0,0)
				[
					CreateAddToSectionButton(InSectionID, WeakRowWidget, AddNewText, MetaDataTag)
				];
		}

		break;
	case NodeSectionID::MACRO:
		AddNewText = LOCTEXT("AddNewMacro", "Macro");
		MetaDataTag = TEXT("AddNewMacro");
		break;
	case NodeSectionID::DELEGATE:
		AddNewText = LOCTEXT("AddNewDelegate", "Event Dispatcher");
		MetaDataTag = TEXT("AddNewDelegate");
		break;
	case NodeSectionID::GRAPH:
		AddNewText = LOCTEXT("AddNewGraph", "New Graph");
		MetaDataTag = TEXT("AddNewGraph");
		break;
	case NodeSectionID::ANIMLAYER:
		AddNewText = LOCTEXT("AddNewAnimLayer", "New Animation Layer");
		MetaDataTag = TEXT("AddNewAnimLayer");
		break;
	case NodeSectionID::LOCAL_VARIABLE:
		AddNewText = LOCTEXT("AddNewLocalVariable", "Local Variable");
		MetaDataTag = TEXT("AddNewLocalVariable");
		break;
	default:
		return SNullWidget::NullWidget;
	}

	return CreateAddToSectionButton(InSectionID, WeakRowWidget, AddNewText, MetaDataTag);
}

TSharedRef<SWidget> SMyBlueprint::CreateAddToSectionButton(int32 InSectionID, TWeakPtr<SWidget> WeakRowWidget, FText AddNewText, FName MetaDataTag)
{
	return 
		SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.OnClicked(this, &SMyBlueprint::OnAddButtonClickedOnSection, InSectionID)
		.IsEnabled(this, &SMyBlueprint::CanAddNewElementToSection, InSectionID)
		.ContentPadding(FMargin(1, 0))
		.AddMetaData<FTagMetaData>(FTagMetaData(MetaDataTag))
		.ToolTipText(AddNewText)
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		];
}

FReply SMyBlueprint::OnAddButtonClickedOnSection(int32 InSectionID)
{
	TSharedPtr<FBlueprintEditor> BlueprintEditor = BlueprintEditorPtr.Pin();

	switch ( InSectionID )
	{
	case NodeSectionID::VARIABLE:
		CommandList->ExecuteAction(FBlueprintEditorCommands::Get().AddNewVariable.ToSharedRef());
		break;
	case NodeSectionID::FUNCTION:
		CommandList->ExecuteAction(FBlueprintEditorCommands::Get().AddNewFunction.ToSharedRef());
		break;
	case NodeSectionID::MACRO:
		CommandList->ExecuteAction(FBlueprintEditorCommands::Get().AddNewMacroDeclaration.ToSharedRef());
		break;
	case NodeSectionID::DELEGATE:
		CommandList->ExecuteAction(FBlueprintEditorCommands::Get().AddNewDelegate.ToSharedRef());
		break;
	case NodeSectionID::GRAPH:
		CommandList->ExecuteAction(FBlueprintEditorCommands::Get().AddNewEventGraph.ToSharedRef());
		break;
	case NodeSectionID::ANIMLAYER:
		CommandList->ExecuteAction(FBlueprintEditorCommands::Get().AddNewAnimationLayer.ToSharedRef());
		break;
	case NodeSectionID::LOCAL_VARIABLE:
		OnAddNewLocalVariable();
		break;
	}

	return FReply::Handled();
}

bool SMyBlueprint::CanAddNewElementToSection(int32 InSectionID) const
{
	if (!IsEditingMode())
	{
		return false;
	}

	if (UBlueprint* CurrentBlueprint = GetBlueprintObj())
	{
		switch (InSectionID)
		{
		case NodeSectionID::VARIABLE:
			return CurrentBlueprint->SupportsGlobalVariables();
		case NodeSectionID::FUNCTION:
			return CurrentBlueprint->SupportsFunctions();
		case NodeSectionID::MACRO:
			return CurrentBlueprint->SupportsMacros();
		case NodeSectionID::DELEGATE:
			return CurrentBlueprint->SupportsDelegates();
		case NodeSectionID::GRAPH:
			return CurrentBlueprint->SupportsEventGraphs();
		case NodeSectionID::ANIMLAYER:
			return CurrentBlueprint->SupportsAnimLayers();
		case NodeSectionID::LOCAL_VARIABLE:
			return CurrentBlueprint->SupportsLocalVariables();
		default:
			break;
		}
	}

	return false;
}

bool SMyBlueprint::HandleActionMatchesName(FEdGraphSchemaAction* InAction, const FName& InName) const
{
	if (BlueprintEditorPtr.IsValid())
	{
		return BlueprintEditorPtr.Pin()->OnActionMatchesName(InAction, InName);
	}
	return false;
}

TSharedRef<SWidget> SMyBlueprint::OnGetFunctionListMenu()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

	BuildOverridableFunctionsMenu(MenuBuilder);

	TSharedRef<SWidget> MenuWidget = MenuBuilder.MakeWidget();
	
	// force user focus onto the menu widget:
	if(FunctionSectionButton.IsValid())
	{
		FunctionSectionButton->SetMenuContentWidgetToFocus(MenuWidget);
	}

	return MenuWidget;
}

void SMyBlueprint::BuildOverridableFunctionsMenu(FMenuBuilder& MenuBuilder)
{
	// Sort by function name so that it's easier for users to find the function they're looking for:
    OverridableFunctionActions.Sort([](const TSharedPtr<FEdGraphSchemaAction_K2Graph> &LHS, const TSharedPtr<FEdGraphSchemaAction_K2Graph> &RHS)
    {
        return LHS->GetMenuDescription().CompareToCaseIgnored(RHS->GetMenuDescription()) < 0;
    });
	
	MenuBuilder.BeginSection("OverrideFunction", LOCTEXT("OverrideFunction", "Override Function"));
	{
		for (TSharedPtr<FEdGraphSchemaAction_K2Graph>& OverrideAction : OverridableFunctionActions)
		{
			// Check if function data is valid and skip this entry if it's a private function
			if (OverrideAction->FuncName != NAME_None)
			{
				if (const UFunction* OverrideActionFunction = FindUField<UFunction>(Blueprint->SkeletonGeneratedClass, OverrideAction->FuncName))
				{
					if (OverrideActionFunction->HasAnyFunctionFlags(FUNC_Private))
					{
						continue;
					}
				}
			}
			
			UClass* const OverrideFuncClass = FBlueprintEditorUtils::GetOverrideFunctionClass(GetBlueprintObj(), OverrideAction->FuncName);
			
			// Add the function name and tooltip 
			TSharedRef<SHorizontalBox> FunctionBox = SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.Padding(FMargin(2.0f, 0.0f, 20.0f, 0.0f))
				[
					SNew(STextBlock)
					.Text(OverrideAction->GetMenuDescription())
					.ToolTipText(OverrideAction->GetTooltipDescription())					
				]
				// Where the function came from function came from
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.Padding(FMargin(1.0f, 0.0f, 0.0f, 0.0f))
				[
					SNew(STextBlock)
					.Text(OverrideFuncClass ? OverrideFuncClass->GetDisplayNameText() : FText::GetEmpty())
					.ToolTipText(OverrideAction->GetTooltipDescription())
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				];

			MenuBuilder.AddMenuEntry(
				FUIAction(
					FExecuteAction::CreateSP(this, &SMyBlueprint::ImplementFunction, OverrideAction),
					FCanExecuteAction::CreateSP(this, &SMyBlueprint::IsEditingMode)),
				FunctionBox,
				NAME_None,
				OverrideAction->GetTooltipDescription(),
				EUserInterfaceActionType::Button
				);
		}
	}
	MenuBuilder.EndSection();
}

bool SMyBlueprint::CanRequestRenameOnActionNode(TWeakPtr<FGraphActionNode> InSelectedNode) const
{
	bool bIsReadOnly = true;

	// If checking if renaming is available on a category node, the category must have a non-native entry
	if (InSelectedNode.Pin()->IsCategoryNode())
	{
		TArray<TSharedPtr<FEdGraphSchemaAction>> Actions;
		GraphActionMenu->GetCategorySubActions(InSelectedNode, Actions);

		for (TSharedPtr<FEdGraphSchemaAction> Action : Actions)
		{
			if (Action->GetPersistentItemDefiningObject().IsPotentiallyEditable())
			{
				bIsReadOnly = false;
				break;
			}
		}
	}
	else if (InSelectedNode.Pin()->IsActionNode())
	{
		bIsReadOnly = FBlueprintEditorUtils::IsPaletteActionReadOnly(InSelectedNode.Pin()->Action, BlueprintEditorPtr.Pin());
		if(!bIsReadOnly)
		{
			bIsReadOnly = !InSelectedNode.Pin()->Action->CanBeRenamed();
		}
	}

	return IsEditingMode() && !bIsReadOnly;
}

void SMyBlueprint::Refresh()
{
	bNeedsRefresh = false;

	// Conform to our interfaces here to ensure we catch any newly added functions
	FBlueprintEditorUtils::ConformImplementedInterfaces(GetBlueprintObj());

	GraphActionMenu->RefreshAllActions(/*bPreserveExpansion=*/ true);
}

TSharedRef<SWidget> SMyBlueprint::OnCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData)
{
	return BlueprintEditorPtr.IsValid() ? SNew(SBlueprintPaletteItem, InCreateData, BlueprintEditorPtr.Pin()) : SNew(SBlueprintPaletteItem, InCreateData, GetBlueprintObj());
}

void SMyBlueprint::GetChildGraphs(UEdGraph* InEdGraph, int32 const SectionId, FGraphActionSort& SortList, const FText& ParentCategory) const
{
	check(InEdGraph);

	// Grab display info
	FGraphDisplayInfo EdGraphDisplayInfo;
	if (const UEdGraphSchema* Schema = InEdGraph->GetSchema())
	{
		Schema->GetGraphDisplayInformation(*InEdGraph, EdGraphDisplayInfo);
	}
	const FText EdGraphDisplayName = EdGraphDisplayInfo.DisplayName;

	// Grab children graphs
	for (UEdGraph* Graph : InEdGraph->SubGraphs)
	{
		if (Graph == nullptr)
		{
			ensureMsgf(Graph != nullptr, TEXT("A subgraph of %s was null"), *GetPathNameSafe(InEdGraph));
			continue;
		}

		FGraphDisplayInfo ChildGraphDisplayInfo;
		if (const UEdGraphSchema* ChildSchema = Graph->GetSchema())
		{
			ChildSchema->GetGraphDisplayInformation(*Graph, ChildGraphDisplayInfo);
		}

		FText DisplayText = ChildGraphDisplayInfo.DisplayName;

		FText Category;
		if (!ParentCategory.IsEmpty())
		{
			Category = FText::Format(FText::FromString(TEXT("{0}|{1}")), ParentCategory, EdGraphDisplayName);
		}
		else
		{
			Category = EdGraphDisplayName;
		}

		const FName DisplayName = FName(*DisplayText.ToString());
		FText ChildTooltip = DisplayText;
		FText ChildDesc = MoveTemp(DisplayText);

		TSharedPtr<FEdGraphSchemaAction_K2Graph> NewChildAction = MakeShareable(new FEdGraphSchemaAction_K2Graph(EEdGraphSchemaAction_K2Graph::Subgraph, Category, MoveTemp(ChildDesc), MoveTemp(ChildTooltip), 1, SectionId));
		NewChildAction->FuncName = DisplayName;
		NewChildAction->EdGraph = Graph;
		SortList.AddAction(NewChildAction);
		
		GetChildGraphs(Graph, SectionId, SortList, Category);
		GetChildEvents(Graph, SectionId, SortList, Category);
	}
}

void SMyBlueprint::GetChildEvents(UEdGraph const* InEdGraph, int32 const SectionId, FGraphActionSort& SortList, const FText& ParentCategory, bool bInAddChildGraphs) const
{
	if (!ensure(InEdGraph != NULL))
	{
		return;
	}

	// ask the schema first to provide the child events
	if (UEdGraphSchema const* Schema = InEdGraph->GetSchema())
	{
		TArray<TSharedPtr<FEdGraphSchemaAction>> ActionsFromSchema;
		if(Schema->TryToGetChildEvents(InEdGraph, SectionId, ActionsFromSchema, ParentCategory))
		{
			for(TSharedPtr<FEdGraphSchemaAction> ActionFromSchema : ActionsFromSchema)
			{
				SortList.AddAction(ActionFromSchema);
			}
			return;
		}
	}

	// grab the parent graph's name
	FGraphDisplayInfo EdGraphDisplayInfo;
	if (UEdGraphSchema const* Schema = InEdGraph->GetSchema())
	{
		Schema->GetGraphDisplayInformation(*InEdGraph, EdGraphDisplayInfo);
	}
	FText EdGraphDisplayName = EdGraphDisplayInfo.DisplayName;
	FText ActionCategory;
	if (!ParentCategory.IsEmpty())
	{
		ActionCategory = FText::Format(FText::FromString(TEXT("{0}|{1}")), ParentCategory, EdGraphDisplayName);
	}
	else
	{
		ActionCategory = MoveTemp(EdGraphDisplayName);
	}

	for (UEdGraphNode* GraphNode : InEdGraph->Nodes)
	{
		if (GraphNode)
		{
			TSharedPtr<FEdGraphSchemaAction> EventNodeAction;
			if(GraphNode->GetClass()->ImplementsInterface(UK2Node_EventNodeInterface::StaticClass()))
			{
				EventNodeAction = CastChecked<IK2Node_EventNodeInterface>(GraphNode)->GetEventNodeAction(ActionCategory);
				EventNodeAction->SectionID = SectionId;
				SortList.AddAction(EventNodeAction);
			}

			if(bInAddChildGraphs && GraphNode->GetClass()->ImplementsInterface(UK2Node_ExternalGraphInterface::StaticClass()))
			{
				TArray<UEdGraph*> ExternalGraphs = CastChecked<IK2Node_ExternalGraphInterface>(GraphNode)->GetExternalGraphs();
				for(UEdGraph* ExternalGraph : ExternalGraphs)
				{
					FText ExternalGraphCategory;
					if(EventNodeAction.IsValid())
					{
						ExternalGraphCategory = FText::Format(FText::FromString(TEXT("{0}|{1}")), ActionCategory, EventNodeAction->GetMenuDescription());
					}
					else
					{
						ExternalGraphCategory = ActionCategory;
					}
					
					// Dont add child graphs, to avoid circular references creating infinite recursion
					const bool bAddChildGraphs = false;
					AddEventForFunctionGraph(ExternalGraph, SectionId, SortList, ExternalGraphCategory, bAddChildGraphs);
				}
			}
		}
	}
}

void SMyBlueprint::GetLocalVariables(FGraphActionSort& SortList) const
{
	// We want to pull local variables from the top level function graphs
	UEdGraph* TopLevelGraph = FBlueprintEditorUtils::GetTopLevelGraph(GetFocusedGraph());
	if( TopLevelGraph )
	{
		TArray<FBPVariableDescription> LocalVariables;
		bool bSchemaImplementsGetLocalVariables = false;
	
		// grab the parent graph's name
		FGraphDisplayInfo EdGraphDisplayInfo;
		if (UEdGraphSchema const* Schema = TopLevelGraph->GetSchema())
		{
			Schema->GetGraphDisplayInformation(*TopLevelGraph, EdGraphDisplayInfo);

			// Try to get the local variables from the schema
			bSchemaImplementsGetLocalVariables = Schema->GetLocalVariables(GetFocusedGraph(), LocalVariables);
			for (const FBPVariableDescription& Variable : LocalVariables)
			{
				FText Category = Variable.Category;
				if (Variable.Category.EqualTo(UEdGraphSchema_K2::VR_DefaultCategory))
				{
					Category = FText::GetEmpty();
				}

				TSharedPtr<FEdGraphSchemaAction> Action =  Schema->MakeActionFromVariableDescription(GetFocusedGraph(), Variable);
				if (Action.IsValid())
				{
					SortList.AddAction(Action);
				}						
			}
		}

		// If the schema did not return any local variables, try to get them from the function entry
		if (!bSchemaImplementsGetLocalVariables)
		{
			TArray<UK2Node_FunctionEntry*> FunctionEntryNodes;
			TopLevelGraph->GetNodesOfClass<UK2Node_FunctionEntry>(FunctionEntryNodes);

			// Search in all FunctionEntry nodes for their local variables
			FText ActionCategory;
			for (UK2Node_FunctionEntry* const FunctionEntry : FunctionEntryNodes)
			{
				for (const FBPVariableDescription& Variable : FunctionEntry->LocalVariables)
				{
					FText Category = Variable.Category;
					if (Variable.Category.EqualTo(UEdGraphSchema_K2::VR_DefaultCategory))
					{
						Category = FText::GetEmpty();
					}

					UFunction* Func = FindUField<UFunction>(GetBlueprintObj()->SkeletonGeneratedClass, TopLevelGraph->GetFName());
					if (Func)
					{
						TSharedPtr<FEdGraphSchemaAction_K2LocalVar> NewVarAction = MakeShareable(new FEdGraphSchemaAction_K2LocalVar(Category, FText::FromName(Variable.VarName), FText::GetEmpty(), 0, NodeSectionID::LOCAL_VARIABLE));
						NewVarAction->SetVariableInfo(Variable.VarName, Func, Variable.VarType.PinCategory == UEdGraphSchema_K2::PC_Boolean);
						SortList.AddAction(NewVarAction);
					}
				}
			}
		}
	}
}

EVisibility SMyBlueprint::GetLocalActionsListVisibility() const
{
	if( !BlueprintEditorPtr.IsValid())
	{
		return EVisibility::Visible;
	}

	if( BlueprintEditorPtr.IsValid() && BlueprintEditorPtr.Pin()->NewDocument_IsVisibleForType(FBlueprintEditor::CGT_NewLocalVariable))
	{
		return EVisibility::Visible;
	}
	return EVisibility::Collapsed;
}

void SMyBlueprint::AddEventForFunctionGraph(UEdGraph* InEdGraph, int32 const SectionId, FGraphActionSort& SortList, const FText& ParentCategory, bool bAddChildGraphs) const
{
	UBlueprint* BlueprintObj = GetBlueprintObj();
	check(BlueprintObj);
	
	FGraphDisplayInfo DisplayInfo;
	InEdGraph->GetSchema()->GetGraphDisplayInformation(*InEdGraph, DisplayInfo);

	FText FunctionCategory = InEdGraph->GetSchema()->GetGraphCategory(InEdGraph);
	if (FunctionCategory.IsEmpty() && BlueprintObj->SkeletonGeneratedClass != nullptr)
	{
		UFunction* Function = BlueprintObj->SkeletonGeneratedClass->FindFunctionByName(InEdGraph->GetFName());
		if (Function != nullptr)
		{
			FunctionCategory = FObjectEditorUtils::GetCategoryText(Function);
		}
	}

	// Default, so place in 'non' category
	if (FunctionCategory.EqualTo(FText::FromString(BlueprintObj->GetName())) || FunctionCategory.EqualTo(UEdGraphSchema_K2::VR_DefaultCategory))
	{
		FunctionCategory = FText::GetEmpty();
	}

	FText ActionCategory;
	if (!ParentCategory.IsEmpty())
	{
		ActionCategory = FText::Format(FText::FromString(TEXT("{0}|{1}")), ParentCategory, FunctionCategory);
	}
	else
	{
		ActionCategory = MoveTemp(FunctionCategory);
	}
	
	//@TODO: Should be a bit more generic (or the AnimGraph shouldn't be stored as a FunctionGraph...)
	const bool bIsConstructionScript = InEdGraph->GetFName() == UEdGraphSchema_K2::FN_UserConstructionScript;

	TSharedPtr<FEdGraphSchemaAction_K2Graph> NewFuncAction = MakeShareable(new FEdGraphSchemaAction_K2Graph(EEdGraphSchemaAction_K2Graph::Function, ActionCategory, DisplayInfo.DisplayName, DisplayInfo.Tooltip, bIsConstructionScript ? 2 : 1, SectionId));
	NewFuncAction->FuncName = InEdGraph->GetFName();
	NewFuncAction->EdGraph = InEdGraph;

	const FString UserCategoryName = FEditorCategoryUtils::GetCategoryDisplayString(ActionCategory.ToString());
	SortList.AddAction(UserCategoryName, NewFuncAction);

	if(bAddChildGraphs)
	{
		GetChildGraphs(InEdGraph, NewFuncAction->GetSectionID(), SortList, ActionCategory);
	}
	GetChildEvents(InEdGraph, NewFuncAction->GetSectionID(), SortList, ActionCategory, bAddChildGraphs);
}

void SMyBlueprint::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	UBlueprint* BlueprintObj = GetBlueprintObj();
	check(BlueprintObj);

	TSharedPtr<FBlueprintEditor> BlueprintEditor = BlueprintEditorPtr.Pin();

	EFieldIteratorFlags::SuperClassFlags FieldIteratorSuperFlag = EFieldIteratorFlags::IncludeSuper;
	if ( ShowUserVarsOnly() )
	{
		FieldIteratorSuperFlag = EFieldIteratorFlags::ExcludeSuper;
	}

	bool bShowReplicatedOnly = IsShowingReplicatedVariablesOnly();

	// Initialise action sorting instance
	FGraphActionSort SortList( BlueprintObj->CategorySorting );
	// List of names of functions we implement
	ImplementedFunctionCache.Empty();

	// Fill with functions names we've already collected for rename, to ensure we do not add the same function multiple times.
	TArray<FName> OverridableFunctionNames;

	// Grab Variables
	for (TFieldIterator<FProperty> PropertyIt(BlueprintObj->SkeletonGeneratedClass, FieldIteratorSuperFlag); PropertyIt; ++PropertyIt)
	{
		FProperty* Property = *PropertyIt;
		FName PropName = Property->GetFName();

		// If we're showing only replicated, ignore the rest
		if (bShowReplicatedOnly && (!Property->HasAnyPropertyFlags(CPF_Net | CPF_RepNotify) || Property->HasAnyPropertyFlags(CPF_RepSkip)))
		{
			continue;
		}
		
		// Don't show delegate properties, there is special handling for these
		const bool bMulticastDelegateProp = Property->IsA(FMulticastDelegateProperty::StaticClass());
		const bool bDelegateProp = (Property->IsA(FDelegateProperty::StaticClass()) || bMulticastDelegateProp);
		const bool bShouldShowAsVar = (!Property->HasAnyPropertyFlags(CPF_Parm) && Property->HasAllPropertyFlags(CPF_BlueprintVisible)) && !bDelegateProp;
		const bool bShouldShowAsDelegate = !Property->HasAnyPropertyFlags(CPF_Parm) && bMulticastDelegateProp 
			&& Property->HasAnyPropertyFlags(CPF_BlueprintAssignable | CPF_BlueprintCallable);
		FObjectPropertyBase* Obj = CastField<FObjectPropertyBase>(Property);
		if(!bShouldShowAsVar && !bShouldShowAsDelegate)
		{
			continue;
		}

		const FText PropertyTooltip = Property->GetToolTipText();
		const FName PropertyName = Property->GetFName();
		const FText PropertyDesc = Property->IsNative() ? Property->GetDisplayNameText() : FText::FromName(PropertyName);

		FText CategoryName = FObjectEditorUtils::GetCategoryText(Property);
		FText PropertyCategory = FObjectEditorUtils::GetCategoryText(Property);
		const FString UserCategoryName = FEditorCategoryUtils::GetCategoryDisplayString( PropertyCategory.ToString() );

		if (CategoryName.EqualTo(FText::FromString(BlueprintObj->GetName())) || CategoryName.EqualTo(UEdGraphSchema_K2::VR_DefaultCategory))
		{
			CategoryName = FText::GetEmpty();		// default, so place in 'non' category
			PropertyCategory = FText::GetEmpty();
		}

		if (bShouldShowAsVar)
		{
			const bool bComponentProperty = Obj && Obj->PropertyClass ? Obj->PropertyClass->IsChildOf<UActorComponent>() : false;

			// By default components go into the variable section under the component category unless a custom category is specified.
			if ( bComponentProperty && CategoryName.IsEmpty() )
			{
				PropertyCategory = LOCTEXT("Components", "Components");
			}

			TSharedPtr<FEdGraphSchemaAction_K2Var> NewVarAction = MakeShareable(new FEdGraphSchemaAction_K2Var(PropertyCategory, PropertyDesc, PropertyTooltip, 0, NodeSectionID::VARIABLE));
			const FArrayProperty* ArrayProperty = CastField<const FArrayProperty>(Property);
			const FProperty* TestProperty = ArrayProperty ? ArrayProperty->Inner : Property;
			NewVarAction->SetVariableInfo(PropertyName, BlueprintObj->SkeletonGeneratedClass, CastField<FBoolProperty>(TestProperty) != nullptr);
			SortList.AddAction( UserCategoryName, NewVarAction );
		}
		else if (bShouldShowAsDelegate && BlueprintEditor.IsValid() && BlueprintEditor->AreDelegatesAllowed())
		{
			TSharedPtr<FEdGraphSchemaAction_K2Delegate> NewDelegateAction;
			// Delegate is visible in MyBlueprint when not-native or its category name is not empty.
			if (Property->HasAllPropertyFlags(CPF_Edit) || !PropertyCategory.IsEmpty())
			{
				NewDelegateAction = MakeShareable(new FEdGraphSchemaAction_K2Delegate(PropertyCategory, PropertyDesc, PropertyTooltip, 0, NodeSectionID::DELEGATE));
				NewDelegateAction->SetVariableInfo(PropertyName, BlueprintObj->SkeletonGeneratedClass, false);
				SortList.AddAction( UserCategoryName, NewDelegateAction );
			}

			UClass* OwnerClass = Property->GetOwnerChecked<UClass>();
			UEdGraph* Graph = FBlueprintEditorUtils::GetDelegateSignatureGraphByName(BlueprintObj, PropertyName);
			if (Graph && OwnerClass && (BlueprintObj == OwnerClass->ClassGeneratedBy))
			{
				if (NewDelegateAction.IsValid())
				{
					NewDelegateAction->EdGraph = Graph;
				}
				ImplementedFunctionCache.Add(PropertyName);
			}
		}
	}

	// Grab what events are implemented in the event graphs so they don't show up in the menu if they are already implemented
	for (UEdGraph* const Graph : BlueprintObj->EventGraphs)
	{
		if (Graph && !Graph->IsUnreachable())
		{
			FName GraphName = Graph->GetFName();
			ImplementedFunctionCache.Add(GraphName);
			OverridableFunctionNames.Add(GraphName);
		}
	}

	// Grab functions implemented by the blueprint
	for (UEdGraph* Graph : BlueprintObj->FunctionGraphs)
	{
		check(Graph);

		int32 SectionID = NodeSectionID::FUNCTION;

		if(Graph->IsA<UAnimationGraph>())
		{
			const bool bIsDefaultAnimGraph = Graph->GetFName() == UEdGraphSchema_K2::GN_AnimGraph;
			if(bIsDefaultAnimGraph)
			{
				SectionID = NodeSectionID::ANIMGRAPH;
			}
			else
			{
				SectionID = NodeSectionID::ANIMLAYER;
			}
		}

		AddEventForFunctionGraph(Graph, SectionID, SortList, FText::GetEmpty(), true);

		ImplementedFunctionCache.Add(Graph->GetFName());
	}

	if(BlueprintEditor.IsValid() && BlueprintEditor->AreMacrosAllowed())
	{
		// Grab macros implemented by the blueprint
		for (int32 i = 0; i < BlueprintObj->MacroGraphs.Num(); i++)
		{
			UEdGraph* Graph = BlueprintObj->MacroGraphs[i];
			check(Graph);
		
			const FName MacroName = Graph->GetFName();

			FGraphDisplayInfo DisplayInfo;
			Graph->GetSchema()->GetGraphDisplayInformation(*Graph, DisplayInfo);

			FText MacroCategory = GetGraphCategory(Graph);

			TSharedPtr<FEdGraphSchemaAction_K2Graph> NewMacroAction = MakeShareable(new FEdGraphSchemaAction_K2Graph(EEdGraphSchemaAction_K2Graph::Macro, MacroCategory, DisplayInfo.PlainName, DisplayInfo.Tooltip, 1, NodeSectionID::MACRO));
			NewMacroAction->FuncName = MacroName;
			NewMacroAction->EdGraph = Graph;

			const FString UserCategoryName = FEditorCategoryUtils::GetCategoryDisplayString(MacroCategory.ToString());
			SortList.AddAction(UserCategoryName, NewMacroAction);

			GetChildGraphs(Graph, NewMacroAction->GetSectionID(), SortList, MacroCategory);
			GetChildEvents(Graph, NewMacroAction->GetSectionID(), SortList, MacroCategory);

			ImplementedFunctionCache.Add(MacroName);
		}
	}

	OverridableFunctionActions.Reset();

	// Cache potentially overridable functions
	UClass* ParentClass = BlueprintObj->SkeletonGeneratedClass ? BlueprintObj->SkeletonGeneratedClass->GetSuperClass() : *BlueprintObj->ParentClass;
	for ( TFieldIterator<UFunction> FunctionIt(ParentClass, EFieldIteratorFlags::IncludeSuper); FunctionIt; ++FunctionIt )
	{
		const UFunction* Function = *FunctionIt;
		const FName FunctionName = Function->GetFName();

		UClass *OuterClass = CastChecked<UClass>(Function->GetOuter());
		// ignore skeleton classes and convert them into their "authoritative" types so they
		// can be found in the graph
		if(UBlueprintGeneratedClass *GeneratedOuterClass = Cast<UBlueprintGeneratedClass>(OuterClass))
		{
			OuterClass = GeneratedOuterClass->GetAuthoritativeClass();
		}

		if (    UEdGraphSchema_K2::CanKismetOverrideFunction(Function) 
			 && !OverridableFunctionNames.Contains(FunctionName) 
			 && !ImplementedFunctionCache.Contains(FunctionName) 
			 && !FObjectEditorUtils::IsFunctionHiddenFromClass(Function, ParentClass)
			 && !FBlueprintEditorUtils::FindOverrideForFunction(BlueprintObj, OuterClass, Function->GetFName())
			 && Blueprint->AllowFunctionOverride(Function)
		   )
		{
			FText FunctionTooltip = FText::FromString(ObjectTools::GetDefaultTooltipForFunction(Function));
			FText FunctionDesc = K2Schema->GetFriendlySignatureName(Function);
			if ( FunctionDesc.IsEmpty() )
			{
				FunctionDesc = FText::FromString(Function->GetName());
			}

			if (Function->HasMetaData(FBlueprintMetadata::MD_DeprecatedFunction))
			{
				FunctionDesc = FBlueprintEditorUtils::GetDeprecatedMemberMenuItemName(FunctionDesc);
			}

			FText FunctionCategory = FObjectEditorUtils::GetCategoryText(Function);

			TSharedPtr<FEdGraphSchemaAction_K2Graph> NewFuncAction = MakeShareable(new FEdGraphSchemaAction_K2Graph(EEdGraphSchemaAction_K2Graph::Function, FunctionCategory, FunctionDesc, FunctionTooltip, 1, NodeSectionID::FUNCTION_OVERRIDABLE));
			NewFuncAction->FuncName = FunctionName;

			OverridableFunctionActions.Add(NewFuncAction);
			OverridableFunctionNames.Add(FunctionName);
		}
	}

	auto IsInAnimBPLambda = [&BlueprintObj](const FName FunctionName, FText& FunctionCategory) -> bool
	{
		if (BlueprintObj->SkeletonGeneratedClass != nullptr)
		{
			if (UFunction * Function = BlueprintObj->SkeletonGeneratedClass->FindFunctionByName(FunctionName))
			{
				FunctionCategory = FObjectEditorUtils::GetCategoryText(Function);

				if (IAnimClassInterface * AnimClassInterface = IAnimClassInterface::GetFromClass(BlueprintObj->SkeletonGeneratedClass))
				{
					if (IAnimClassInterface::IsAnimBlueprintFunction(AnimClassInterface, Function))
					{
						return true;
					}
				}
			}
		}
		return false;
	};

	// Also functions implemented from interfaces
	for (int32 i=0; i < BlueprintObj->ImplementedInterfaces.Num(); i++)
	{
		FBPInterfaceDescription& InterfaceDesc = BlueprintObj->ImplementedInterfaces[i];
		UClass* InterfaceClass = InterfaceDesc.Interface.Get();
		if (InterfaceClass)
		{
			for (TFieldIterator<UFunction> FunctionIt(InterfaceClass, EFieldIteratorFlags::IncludeSuper); FunctionIt; ++FunctionIt)
			{
				const UFunction* Function = *FunctionIt;
				const FName FunctionName = Function->GetFName();

				if (FunctionName != UEdGraphSchema_K2::FN_ExecuteUbergraphBase)
				{
					FText FunctionTooltip = Function->GetToolTipText();
					FText FunctionDesc = K2Schema->GetFriendlySignatureName(Function);

					FText FunctionCategory;
					bool bIsAnimFunction = IsInAnimBPLambda(FunctionName, FunctionCategory);

					TSharedPtr<FEdGraphSchemaAction_K2Graph> NewFuncAction = MakeShareable(new FEdGraphSchemaAction_K2Graph(EEdGraphSchemaAction_K2Graph::Interface, FunctionCategory, FunctionDesc, FunctionTooltip, 1, bIsAnimFunction ? NodeSectionID::ANIMLAYER : NodeSectionID::INTERFACE));

					NewFuncAction->FuncName = FunctionName;
					OutAllActions.AddAction(NewFuncAction);

					// Find the graph that this function is on so the user can double click and open it from the interfaces menu
					for (UEdGraph* const Graph : InterfaceDesc.Graphs)
					{
						if (Graph && Graph->GetFName() == FunctionName)
						{
							NewFuncAction->EdGraph = Graph;
							break;
						}
					}

					// if this function is not in the interfaces menu, then allow it to be put in the override function menu
					if (GetAlwaysShowInterfacesInOverrides())
					{
						OverridableFunctionActions.Add(NewFuncAction);
						OverridableFunctionNames.Add(FunctionName);
					}

					if(bIsAnimFunction && NewFuncAction->EdGraph)
					{
						GetChildGraphs(NewFuncAction->EdGraph, NewFuncAction->GetSectionID(), SortList, FunctionCategory);
						GetChildEvents(NewFuncAction->EdGraph, NewFuncAction->GetSectionID(), SortList, FunctionCategory);
					}
				}
			}
		}
	}

	// also walk up the class chain to look for overridable functions in natively implemented interfaces
	for (UClass* TempClass = BlueprintObj->ParentClass; TempClass; TempClass = TempClass->GetSuperClass())
	{
		for (int32 Idx = 0; Idx < TempClass->Interfaces.Num(); ++Idx)
		{
			FImplementedInterface const& I = TempClass->Interfaces[Idx];
			
			// same as above, make a function?
			for (TFieldIterator<UFunction> FunctionIt(I.Class, EFieldIteratorFlags::IncludeSuper); FunctionIt; ++FunctionIt)
			{
				const UFunction* Function = *FunctionIt;
				const FName FunctionName = Function->GetFName();

				if (UEdGraphSchema_K2::CanKismetOverrideFunction(Function) && !ImplementedFunctionCache.Contains(FunctionName))
				{
					FText FunctionTooltip = Function->GetToolTipText();
					FText FunctionDesc = K2Schema->GetFriendlySignatureName(Function);

					FText FunctionCategory = FObjectEditorUtils::GetCategoryText(Function);
					bool bIsAnimFunction = IsInAnimBPLambda(FunctionName, FunctionCategory);

					TSharedPtr<FEdGraphSchemaAction_K2Graph> NewFuncAction = MakeShareable(new FEdGraphSchemaAction_K2Graph(EEdGraphSchemaAction_K2Graph::Interface, FunctionCategory, FunctionDesc, FunctionTooltip, 1, bIsAnimFunction ? NodeSectionID::ANIMLAYER : NodeSectionID::INTERFACE));
					NewFuncAction->FuncName = FunctionName;
					
					if (!OverridableFunctionNames.Contains(FunctionName))
					{
						OverridableFunctionActions.Add(NewFuncAction);
						OverridableFunctionNames.Add(FunctionName);
					}

					OutAllActions.AddAction(NewFuncAction);
				}
			}
		}
	}

	
	if(BlueprintEditor.IsValid() && BlueprintEditor->AreEventGraphsAllowed())
	{
		// Grab ubergraph pages
		for (int32 i = 0; i < BlueprintObj->UbergraphPages.Num(); i++)
		{
			UEdGraph* Graph = BlueprintObj->UbergraphPages[i];
			check(Graph);
		
			FGraphDisplayInfo DisplayInfo;
			Graph->GetSchema()->GetGraphDisplayInformation(*Graph, DisplayInfo);

			TSharedPtr<FEdGraphSchemaAction_K2Graph> NeUbergraphAction = MakeShareable(new FEdGraphSchemaAction_K2Graph(EEdGraphSchemaAction_K2Graph::Graph, FText::GetEmpty(), DisplayInfo.PlainName, DisplayInfo.Tooltip, 2, NodeSectionID::GRAPH));
			NeUbergraphAction->FuncName = Graph->GetFName();
			NeUbergraphAction->EdGraph = Graph;
			OutAllActions.AddAction(NeUbergraphAction);

			GetChildGraphs(Graph, NeUbergraphAction->GetSectionID(), SortList);
			GetChildEvents(Graph, NeUbergraphAction->GetSectionID(), SortList);
		}
	}

	// Grab intermediate pages
	for (int32 i = 0; i < BlueprintObj->IntermediateGeneratedGraphs.Num(); i++)
	{
		UEdGraph* Graph = BlueprintObj->IntermediateGeneratedGraphs[i];
		check(Graph);
		
		const FName IntermediateName(*(FString(TEXT("$INTERMEDIATE$_")) + Graph->GetName()));
		FString IntermediateTooltip = IntermediateName.ToString();
		FString IntermediateDesc = IntermediateName.ToString();
		TSharedPtr<FEdGraphSchemaAction_K2Graph> NewIntermediateAction = MakeShareable(new FEdGraphSchemaAction_K2Graph(EEdGraphSchemaAction_K2Graph::Graph, FText::GetEmpty(), FText::FromString(IntermediateDesc), FText::FromString(IntermediateTooltip), 1));
		NewIntermediateAction->FuncName = IntermediateName;
		NewIntermediateAction->EdGraph = Graph;
		OutAllActions.AddAction(NewIntermediateAction);

		GetChildGraphs(Graph, NewIntermediateAction->GetSectionID(), SortList);
		GetChildEvents(Graph, NewIntermediateAction->GetSectionID(), SortList);
	}

	if (GetLocalActionsListVisibility().IsVisible())
	{
		GetLocalVariables(SortList);
	}

	// Inject a placeholder for each pinned category that is missing.
	FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>("Kismet");
	for (const UE::BlueprintEditor::FAdditionalCategory& Category : BlueprintEditorModule.GetAdditionalBlueprintCategories(BlueprintObj))
	{
		if (Category.Placeholders.IsEmpty())
		{
			continue;
		}

		for (const UE::BlueprintEditor::FAdditionalCategory::FPlaceholder& Placeholder : Category.Placeholders)
		{
			const FString CategoryDisplayName = FEditorCategoryUtils::GetCategoryDisplayString(Category.Category.ToString());
			if (!SortList.HasCategory(CategoryDisplayName, Placeholder.SectionID))
			{
				TSharedPtr<FEdGraphSchemaAction_Dummy> Action = MakeShareable(new FEdGraphSchemaAction_Dummy(
					Category.Category,
					Placeholder.MenuDescription,
					Placeholder.Tooltip,
					0,
					FText::GetEmpty(),
					static_cast<int32>(Placeholder.SectionID)));
				SortList.AddAction(CategoryDisplayName, Action);
			}
		}
	}

	// Add all the sorted variables, components, functions, etc...
	SortList.CleanupCategories();
	SortList.GetAllActions(OutAllActions);
}

void SMyBlueprint::CollectStaticSections(TArray<int32>& StaticSectionIDs)
{
	TSharedPtr<FBlueprintEditor> BlueprintEditor = BlueprintEditorPtr.Pin();
	const bool bIsEditor = BlueprintEditor.IsValid();

	if ( IsShowingEmptySections() )
	{
		if (!bIsEditor || (BlueprintEditor->NewDocument_IsVisibleForType(FBlueprintEditor::CGT_NewEventGraph) && BlueprintEditor->IsSectionVisible(NodeSectionID::GRAPH)))
		{
			StaticSectionIDs.Add(NodeSectionID::GRAPH);
		}
		if (!bIsEditor || (BlueprintEditor->NewDocument_IsVisibleForType(FBlueprintEditor::CGT_NewAnimationLayer) && BlueprintEditor->IsSectionVisible(NodeSectionID::ANIMLAYER)))
		{
			StaticSectionIDs.Add(NodeSectionID::ANIMLAYER);
		}
		if (!bIsEditor || (BlueprintEditor->NewDocument_IsVisibleForType(FBlueprintEditor::CGT_NewMacroGraph) && BlueprintEditor->IsSectionVisible(NodeSectionID::MACRO)))
		{
			StaticSectionIDs.Add(NodeSectionID::MACRO);
		}
		if (!bIsEditor || (BlueprintEditor->NewDocument_IsVisibleForType(FBlueprintEditor::CGT_NewFunctionGraph) && BlueprintEditor->IsSectionVisible(NodeSectionID::FUNCTION)))
		{
			StaticSectionIDs.Add(NodeSectionID::FUNCTION);
		}
		if (!bIsEditor || (BlueprintEditor->NewDocument_IsVisibleForType(FBlueprintEditor::CGT_NewVariable) && BlueprintEditor->IsSectionVisible(NodeSectionID::VARIABLE)))
		{
			StaticSectionIDs.Add(NodeSectionID::VARIABLE);
		}
		if (!bIsEditor || (BlueprintEditor->FBlueprintEditor::AddNewDelegateIsVisible() && BlueprintEditor->IsSectionVisible(NodeSectionID::DELEGATE)))
		{
			StaticSectionIDs.Add(NodeSectionID::DELEGATE);
		}
	}

	if ( GetLocalActionsListVisibility().IsVisible() && (!bIsEditor || BlueprintEditor->IsSectionVisible(NodeSectionID::LOCAL_VARIABLE)))
	{
		StaticSectionIDs.Add(NodeSectionID::LOCAL_VARIABLE);
	}
}

bool SMyBlueprint::IsShowingInheritedVariables() const
{
	return GetMutableDefault<UBlueprintEditorSettings>()->bShowInheritedVariables;
}

void SMyBlueprint::OnToggleShowInheritedVariables()
{
	UBlueprintEditorSettings* Settings = GetMutableDefault<UBlueprintEditorSettings>();
	Settings->bShowInheritedVariables = !Settings->bShowInheritedVariables;
	Settings->PostEditChange();
	Settings->SaveConfig();

	Refresh();
}

void SMyBlueprint::OnToggleShowEmptySections()
{
	UBlueprintEditorSettings* Settings = GetMutableDefault<UBlueprintEditorSettings>();
	Settings->bShowEmptySections = !Settings->bShowEmptySections;
	Settings->PostEditChange();
	Settings->SaveConfig();

	Refresh();
}

bool SMyBlueprint::IsShowingEmptySections() const
{
	return GetMutableDefault<UBlueprintEditorSettings>()->bShowEmptySections;
}

void SMyBlueprint::OnToggleShowReplicatedVariablesOnly()
{
	bShowReplicatedVariablesOnly = !bShowReplicatedVariablesOnly;
	Refresh();
}

void SMyBlueprint::OnToggleAlwaysShowInterfacesInOverrides()
{
	UBlueprintEditorSettings* Settings = GetMutableDefault<UBlueprintEditorSettings>();
	Settings->bAlwaysShowInterfacesInOverrides = !Settings->bAlwaysShowInterfacesInOverrides;
	Settings->PostEditChange();
	Settings->SaveConfig();
	Refresh();
}

bool SMyBlueprint::GetAlwaysShowInterfacesInOverrides() const
{
	return GetMutableDefault<UBlueprintEditorSettings>()->bAlwaysShowInterfacesInOverrides;
}

void SMyBlueprint::OnToggleShowParentClassInOverrides()
{
	UBlueprintEditorSettings* Settings = GetMutableDefault<UBlueprintEditorSettings>();
	Settings->bShowParentClassInOverrides = !Settings->bShowParentClassInOverrides;
	Settings->PostEditChange();
	Settings->SaveConfig();
	Refresh();
}

bool SMyBlueprint::GetShowParentClassInOverrides() const
{
	return GetMutableDefault<UBlueprintEditorSettings>()->bShowParentClassInOverrides;
}

void SMyBlueprint::OnToggleShowAccessSpecifier()
{
	UBlueprintEditorSettings* Settings = GetMutableDefault<UBlueprintEditorSettings>();
	Settings->bShowAccessSpecifier = !Settings->bShowAccessSpecifier;
	Settings->PostEditChange();
	Settings->SaveConfig();
	Refresh();
}

bool SMyBlueprint::GetShowAccessSpecifier() const 
{
	return GetMutableDefault<UBlueprintEditorSettings>()->bShowAccessSpecifier;
}

bool SMyBlueprint::IsShowingReplicatedVariablesOnly() const
{
	return bShowReplicatedVariablesOnly;
}

FReply SMyBlueprint::OnActionDragged( const TArray< TSharedPtr<FEdGraphSchemaAction> >& InActions, const FPointerEvent& MouseEvent )
{
	if (!BlueprintEditorPtr.IsValid())
	{
		return FReply::Unhandled();
	}

	TSharedPtr<FEdGraphSchemaAction> InAction( InActions.Num() > 0 ? InActions[0] : nullptr );
	if(InAction.IsValid())
	{
		auto AnalyticsDelegate = FNodeCreationAnalytic::CreateSP( this, &SMyBlueprint::UpdateNodeCreation );

		if(InAction->GetTypeId() == FEdGraphSchemaAction_K2Graph::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2Graph* FuncAction = (FEdGraphSchemaAction_K2Graph*)InAction.Get();

			if (FuncAction->EdGraph)
			{
				if (FuncAction->EdGraph->GetSchema()->CanGraphBeDropped(InAction))
				{
					return FuncAction->EdGraph->GetSchema()->BeginGraphDragAction(InAction, FPointerEvent());
				}
			}

			if (FuncAction->GraphType == EEdGraphSchemaAction_K2Graph::Function ||FuncAction->GraphType == EEdGraphSchemaAction_K2Graph::Interface)
			{
				// Callback function to report that the user cannot drop this function in the graph
				auto CanDragDropAction = [](TSharedPtr<FEdGraphSchemaAction> /*DropAction*/, UEdGraph* HoveredGraphIn, FText& ImpededReasonOut, int32 FunctionFlags)->bool //bool bIsBlueprintCallableFunction, bool bIsPureFunction)->bool
				{
					if ((FunctionFlags & (FUNC_BlueprintCallable | FUNC_BlueprintPure)) == 0)
					{
						ImpededReasonOut = LOCTEXT("NonBlueprintCallable", "This function was not marked as Blueprint Callable and cannot be placed in a graph!");
						return false;
					}
					
					if (const UEdGraphSchema_K2* K2Schema = Cast<UEdGraphSchema_K2>(HoveredGraphIn->GetSchema()))
					{
						if ((FunctionFlags & (FUNC_BlueprintPure)) == 0)
						{
							if (K2Schema->DoesGraphSupportImpureFunctions(HoveredGraphIn) == false)
							{
								ImpededReasonOut = LOCTEXT("GraphNoSupportImpureF", "The target graph does not support Impure Functions.");
								return false;
							}
						}
					}
					else
					{
						// TODO : Check if this else has to block everything or just explicit schemas
						if (const UAnimationStateMachineSchema* AnimationStateMachineSchema = Cast<UAnimationStateMachineSchema>(HoveredGraphIn->GetSchema()))
						{
							ImpededReasonOut = LOCTEXT("GraphNoSupportFunctions", "The target graph does not support Functions.");
							return false;
						}
					}

					return true;
				};

				int32 FunctionFlags = 0;

				if (FuncAction->EdGraph)
				{
					for (UEdGraphNode* GraphNode : FuncAction->EdGraph->Nodes)
					{
						if (UK2Node_FunctionEntry* Node = Cast<UK2Node_FunctionEntry>(GraphNode))
						{
							FunctionFlags |= Node->GetFunctionFlags();
						}
					}
				}

				return FReply::Handled().BeginDragDrop(FKismetFunctionDragDropAction::New(
					InAction, 
					FuncAction->FuncName, 
					GetBlueprintObj()->SkeletonGeneratedClass, 
					FMemberReference(), 
					AnalyticsDelegate, 
					FKismetDragDropAction::FCanBeDroppedDelegate::CreateLambda(CanDragDropAction, FunctionFlags)));
			}
			else if (FuncAction->GraphType == EEdGraphSchemaAction_K2Graph::Macro)
			{
				if ((FuncAction->EdGraph != NULL) && GetBlueprintObj()->BlueprintType != BPTYPE_MacroLibrary)
				{
					return FReply::Handled().BeginDragDrop(FKismetMacroDragDropAction::New(InAction, FuncAction->FuncName, GetBlueprintObj(), FuncAction->EdGraph, AnalyticsDelegate));
				}
			}
		}
		else if(InAction->GetTypeId() == FEdGraphSchemaAction_K2Delegate::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2Delegate* DelegateAction = (FEdGraphSchemaAction_K2Delegate*)InAction.Get();
			check(DelegateAction->GetDelegateName() != NAME_None);
			if (UClass* VarClass = DelegateAction->GetDelegateClass())
			{
				const bool bIsAltDown = MouseEvent.IsAltDown();
				const bool bIsCtrlDown = MouseEvent.IsLeftControlDown() || MouseEvent.IsRightControlDown();
				
				TSharedRef<FKismetVariableDragDropAction> DragOperation = FKismetDelegateDragDropAction::New(InAction, DelegateAction->GetDelegateName(), VarClass, AnalyticsDelegate);
				DragOperation->SetAltDrag(bIsAltDown);
				DragOperation->SetCtrlDrag(bIsCtrlDown);
				return FReply::Handled().BeginDragDrop(DragOperation);
			}
		}
		else if( InAction->GetTypeId() == FEdGraphSchemaAction_K2LocalVar::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2LocalVar* VarAction = (FEdGraphSchemaAction_K2LocalVar*)InAction.Get();
			if (UStruct* VariableScope = Cast<UStruct>(VarAction->GetVariableScope()))
			{
				TSharedRef<FKismetVariableDragDropAction> DragOperation = FKismetVariableDragDropAction::New(InAction, VarAction->GetVariableName(), VariableScope, AnalyticsDelegate);
				DragOperation->SetAltDrag(MouseEvent.IsAltDown());
				DragOperation->SetCtrlDrag(MouseEvent.IsLeftControlDown() || MouseEvent.IsRightControlDown());
				return FReply::Handled().BeginDragDrop(DragOperation);
			}
		}
		else if(InAction->GetTypeId() == FEdGraphSchemaAction_K2Var::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2Var* VarAction = (FEdGraphSchemaAction_K2Var*)InAction.Get();
			if (UClass* VarClass = VarAction->GetVariableClass())
			{
				TSharedRef<FKismetVariableDragDropAction> DragOperation = FKismetVariableDragDropAction::New(InAction, VarAction->GetVariableName(), VarClass, AnalyticsDelegate);
				DragOperation->SetAltDrag(MouseEvent.IsAltDown());
				DragOperation->SetCtrlDrag(MouseEvent.IsLeftControlDown() || MouseEvent.IsRightControlDown());
				return FReply::Handled().BeginDragDrop(DragOperation);
			}
		}
		else if( InAction->IsA(FEdGraphSchemaAction_BlueprintVariableBase::StaticGetTypeId()))
		{
			FEdGraphSchemaAction_BlueprintVariableBase* VarAction = (FEdGraphSchemaAction_BlueprintVariableBase*)InAction.Get();

			if (GetFocusedGraph()->GetSchema()->CanGraphBeDropped(InAction))
			{
				return GetFocusedGraph()->GetSchema()->BeginGraphDragAction(InAction, MouseEvent);
			}		
		}
		else if (InAction->GetTypeId() == FEdGraphSchemaAction_K2Event::StaticGetTypeId())
		{	
			// Check if it's a custom event, it is preferable to drop a call function for custom events than to focus on the node
			FEdGraphSchemaAction_K2Event* FuncAction = (FEdGraphSchemaAction_K2Event*)InAction.Get();
			if (UK2Node_Event* Event = Cast<UK2Node_Event>(FuncAction->NodeTemplate))
			{
				UFunction* const Function = FFunctionFromNodeHelper::FunctionFromNode(Event);

				// Callback function to report that the user cannot drop this function in the graph
				auto CanDragDropAction = [](TSharedPtr<FEdGraphSchemaAction> /*DropAction*/, UEdGraph* /*HoveredGraphIn*/, FText& ImpededReasonOut, UFunction* Func)->bool
				{
					// If this function is not BP callable then don't let it be dropped
					if (Func && !(Func->FunctionFlags & (FUNC_BlueprintCallable | FUNC_BlueprintPure)))
					{
						ImpededReasonOut = LOCTEXT("NonBlueprintCallableEvent", "This event was not marked as Blueprint Callable and cannot be placed in a graph!");
						return false;
					}

					return true;
				};

				TSharedRef< FKismetFunctionDragDropAction> DragOperation =
					FKismetFunctionDragDropAction::New(
						InAction, (Function ? Function->GetFName() : Event->GetFName()),
						GetBlueprintObj()->SkeletonGeneratedClass,
						FMemberReference(),
						AnalyticsDelegate,
						FKismetDragDropAction::FCanBeDroppedDelegate::CreateLambda(CanDragDropAction, Function)
					);
				return FReply::Handled().BeginDragDrop(DragOperation);	
			}
		}
	}

	return FReply::Unhandled();
}

FReply SMyBlueprint::OnCategoryDragged(const FText& InCategory, const FPointerEvent& MouseEvent)
{
	TSharedRef<FMyBlueprintCategoryDragDropAction> DragOperation = FMyBlueprintCategoryDragDropAction::New(InCategory, SharedThis(this));
	return FReply::Handled().BeginDragDrop(DragOperation);
}

void SMyBlueprint::OnGlobalActionSelected(const TArray< TSharedPtr<FEdGraphSchemaAction> >& InActions, ESelectInfo::Type InSelectionType)
{
	if (InSelectionType == ESelectInfo::OnMouseClick  || InSelectionType == ESelectInfo::OnKeyPress || InSelectionType == ESelectInfo::OnNavigation || InActions.Num() == 0)
	{
		OnActionSelected(InActions);
	}
}

void SMyBlueprint::OnActionSelected( const TArray< TSharedPtr<FEdGraphSchemaAction> >& InActions )
{
	TSharedPtr<FEdGraphSchemaAction> InAction(InActions.Num() > 0 ? InActions[0] : NULL);
	UBlueprint* CurrentBlueprint = Blueprint;
	TSharedPtr<SKismetInspector> CurrentInspector = Inspector.Pin();

	TSharedPtr<FBlueprintEditor> BlueprintEditor = BlueprintEditorPtr.Pin();

	if (BlueprintEditor.IsValid())
	{
		BlueprintEditor->SetUISelectionState(FBlueprintEditor::SelectionState_MyBlueprint);

		CurrentBlueprint = BlueprintEditor->GetBlueprintObj();
		CurrentInspector = BlueprintEditor->GetInspector();
	}
	OnActionSelectedHelper(InAction, BlueprintEditorPtr, Blueprint, CurrentInspector.ToSharedRef());
}

void SMyBlueprint::OnActionSelectedHelper(TSharedPtr<FEdGraphSchemaAction> InAction, TWeakPtr< FBlueprintEditor > InBlueprintEditor, UBlueprint* CurrentBlueprint, TSharedRef<SKismetInspector> CurrentInspector)
{
	if (InAction.IsValid())
	{
		if (InAction->GetTypeId() == FEdGraphSchemaAction_K2Graph::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2Graph* GraphAction = (FEdGraphSchemaAction_K2Graph*)InAction.Get();

			if (GraphAction->EdGraph)
			{
				FGraphDisplayInfo DisplayInfo;
				const UEdGraphSchema* Schema = GraphAction->EdGraph->GetSchema();
				check(Schema != nullptr);
				Schema->GetGraphDisplayInformation(*GraphAction->EdGraph, DisplayInfo);
				CurrentInspector->ShowDetailsForSingleObject(GraphAction->EdGraph, SKismetInspector::FShowDetailsOptions(DisplayInfo.PlainName));
			}
		}
		else if (InAction->GetTypeId() == FEdGraphSchemaAction_K2Delegate::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2Delegate* DelegateAction = (FEdGraphSchemaAction_K2Delegate*)InAction.Get();
			if (FMulticastDelegateProperty* Property = DelegateAction->GetDelegateProperty())
			{
				CurrentInspector->ShowDetailsForSingleObject(Property->GetUPropertyWrapper(), SKismetInspector::FShowDetailsOptions(FText::FromString(Property->GetName())));
			}
		}
		else if (InAction->GetTypeId() == FEdGraphSchemaAction_K2Var::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2Var* VarAction = (FEdGraphSchemaAction_K2Var*)InAction.Get();

			SKismetInspector::FShowDetailsOptions Options(FText::FromName(VarAction->GetVariableName()));
			Options.bForceRefresh = true;

			FProperty* Prop = VarAction->GetProperty();
			UPropertyWrapper* PropWrap = (Prop ? Prop->GetUPropertyWrapper() : nullptr);
			CurrentInspector->ShowDetailsForSingleObject(PropWrap, Options);
		}
		else if (InAction->GetTypeId() == FEdGraphSchemaAction_K2LocalVar::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2LocalVar* VarAction = (FEdGraphSchemaAction_K2LocalVar*)InAction.Get();

			SKismetInspector::FShowDetailsOptions Options(FText::FromName(VarAction->GetVariableName()));

			FProperty* Prop = VarAction->GetProperty();
			UPropertyWrapper* PropWrap = (Prop ? Prop->GetUPropertyWrapper() : nullptr);
			CurrentInspector->ShowDetailsForSingleObject(PropWrap, Options);
		}
		else if (InAction->IsAVariable())
		{
			FEdGraphSchemaAction_BlueprintVariableBase* VarAction = (FEdGraphSchemaAction_BlueprintVariableBase*)InAction.Get();
			if (UEdGraph* Graph = GetFocusedGraph())
			{
				if (BlueprintEditorPtr.IsValid())
				{
					BlueprintEditorPtr.Pin()->SelectLocalVariable(Graph, VarAction->GetVariableName());	
				}
			}
		}
		else if (InAction->GetTypeId() == FEdGraphSchemaAction_K2Enum::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2Enum* EnumAction = (FEdGraphSchemaAction_K2Enum*)InAction.Get();

			SKismetInspector::FShowDetailsOptions Options(FText::FromName(EnumAction->GetPathName()));
			Options.bForceRefresh = true;

			CurrentInspector->ShowDetailsForSingleObject(EnumAction->Enum, Options);
		}
		else if (InAction->GetTypeId() == FEdGraphSchemaAction_K2Struct::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2Struct* StructAction = (FEdGraphSchemaAction_K2Struct*)InAction.Get();

			SKismetInspector::FShowDetailsOptions Options(FText::FromName(StructAction->GetPathName()));
			Options.bForceRefresh = true;

			CurrentInspector->ShowDetailsForSingleObject(StructAction->Struct, Options);
		}
		else if (InAction->GetTypeId() == FEdGraphSchemaAction_K2TargetNode::StaticGetTypeId() ||
			InAction->GetTypeId() == FEdGraphSchemaAction_K2Event::StaticGetTypeId() ||
			InAction->GetTypeId() == FEdGraphSchemaAction_K2InputAction::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2TargetNode* TargetNodeAction = (FEdGraphSchemaAction_K2TargetNode*)InAction.Get();
			
			UK2Node* NodeTemplate = TargetNodeAction->NodeTemplate.Get();
			check(NodeTemplate != nullptr)

			SKismetInspector::FShowDetailsOptions Options(NodeTemplate->GetNodeTitle(ENodeTitleType::EditableTitle));
			CurrentInspector->ShowDetailsForSingleObject(NodeTemplate, Options);
		}
		else
		{
			CurrentInspector->ShowDetailsForObjects(TArray<UObject*>());
		}
	}
	else
	{
		CurrentInspector->ShowDetailsForObjects(TArray<UObject*>());
	}
}

void SMyBlueprint::OnActionDoubleClicked(const TArray< TSharedPtr<FEdGraphSchemaAction> >& InActions)
{
	if ( !BlueprintEditorPtr.IsValid() )
	{
		return;
	}

	TSharedPtr<FEdGraphSchemaAction> InAction(InActions.Num() > 0 ? InActions[0] : NULL);
	ExecuteAction(InAction);
}

void SMyBlueprint::ExecuteAction(TSharedPtr<FEdGraphSchemaAction> InAction)
{
	// Force it to open in a new document if shift is pressed
	const bool bIsShiftPressed = FSlateApplication::Get().GetModifierKeys().IsShiftDown();
	FDocumentTracker::EOpenDocumentCause OpenMode = bIsShiftPressed ? FDocumentTracker::ForceOpenNewDocument : FDocumentTracker::OpenNewDocument;

	UBlueprint* BlueprintObj = BlueprintEditorPtr.Pin()->GetBlueprintObj();
	if(InAction.IsValid())
	{
		if(InAction->GetTypeId() == FEdGraphSchemaAction_K2Graph::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2Graph* GraphAction = (FEdGraphSchemaAction_K2Graph*)InAction.Get();

			if (GraphAction->EdGraph)
			{
				BlueprintEditorPtr.Pin()->JumpToHyperlink(GraphAction->EdGraph);
			}
			else if(IsAnInterfaceEvent(GraphAction))
			{
				// Focus it's node in the event graph
				UFunction* OverrideFunc = nullptr;
				UClass* const OverrideFuncClass = FBlueprintEditorUtils::GetOverrideFunctionClass(BlueprintObj, GraphAction->FuncName, &OverrideFunc);
				if (OverrideFunc)
				{
					FName EventName = OverrideFunc->GetFName();
					// check if event has been implemented
					if (UK2Node_Event* ExistingNode = FBlueprintEditorUtils::FindOverrideForFunction(BlueprintObj, OverrideFuncClass, EventName))
					{
						FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(ExistingNode);
					}
					else
					{
						// if there isn't an associated node, make one and focus it
						ImplementFunction(GraphAction);
					}
				}
			}
		}
		if (InAction->GetTypeId() == FEdGraphSchemaAction_K2Delegate::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2Delegate* DelegateAction = (FEdGraphSchemaAction_K2Delegate*)InAction.Get();

			if (DelegateAction->EdGraph)
			{
				BlueprintEditorPtr.Pin()->JumpToHyperlink(DelegateAction->EdGraph);
			}
		}
		else if(InAction->GetTypeId() == FEdGraphSchemaAction_K2Var::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2Var* VarAction = (FEdGraphSchemaAction_K2Var*)InAction.Get();
			
			// timeline variables
			const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(VarAction->GetProperty());
			if (ObjectProperty &&
				ObjectProperty->PropertyClass &&
				ObjectProperty->PropertyClass->IsChildOf(UTimelineComponent::StaticClass()))
			{
				for (int32 i=0; i<BlueprintObj->Timelines.Num(); i++)
				{
					// Convert the Timeline's name to a variable name before comparing it to the variable
					if (BlueprintObj->Timelines[i]->GetVariableName() == VarAction->GetVariableName())
					{
						BlueprintEditorPtr.Pin()->JumpToHyperlink(BlueprintObj->Timelines[i]);
					}
				}
			}
		}
		else if (InAction->GetTypeId() == FEdGraphSchemaAction_K2Event::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2Event* EventNodeAction = (FEdGraphSchemaAction_K2Event*)InAction.Get();
			FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(EventNodeAction->NodeTemplate);
		}
		else if (InAction->GetTypeId() == FEdGraphSchemaAction_K2TargetNode::StaticGetTypeId() || 
			InAction->GetTypeId() == FEdGraphSchemaAction_K2InputAction::StaticGetTypeId())
		{
			FEdGraphSchemaAction_K2TargetNode* TargetNodeAction = (FEdGraphSchemaAction_K2TargetNode*)InAction.Get();
			FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(TargetNodeAction->NodeTemplate);
		}
		else
		{
			InAction->OnDoubleClick(Blueprint);
		}
	}
}

template<class SchemaActionType> SchemaActionType* SelectionAsType( const TSharedPtr< SGraphActionMenu >& GraphActionMenu )
{
	TArray<TSharedPtr<FEdGraphSchemaAction> > SelectedActions;
	GraphActionMenu->GetSelectedActions(SelectedActions);

	SchemaActionType* Selection = NULL;

	TSharedPtr<FEdGraphSchemaAction> SelectedAction( SelectedActions.Num() > 0 ? SelectedActions[0] : NULL );
	if ( SelectedAction.IsValid() &&
		 SelectedAction->GetTypeId() == SchemaActionType::StaticGetTypeId() )
	{
		// TODO Why not? StaticCastSharedPtr<>()

		Selection = (SchemaActionType*)SelectedActions[0].Get();
	}

	return Selection;
}

FEdGraphSchemaAction_K2Enum* SMyBlueprint::SelectionAsEnum() const
{
	return SelectionAsType<FEdGraphSchemaAction_K2Enum>( GraphActionMenu );
}


FEdGraphSchemaAction_K2Struct* SMyBlueprint::SelectionAsStruct() const
{
	return SelectionAsType<FEdGraphSchemaAction_K2Struct>( GraphActionMenu );
}

FEdGraphSchemaAction_K2Graph* SMyBlueprint::SelectionAsGraph() const
{
	return SelectionAsType<FEdGraphSchemaAction_K2Graph>( GraphActionMenu );
}

FEdGraphSchemaAction_K2Var* SMyBlueprint::SelectionAsVar() const
{
	return SelectionAsType<FEdGraphSchemaAction_K2Var>( GraphActionMenu );
}

FEdGraphSchemaAction_K2LocalVar* SMyBlueprint::SelectionAsLocalVar() const
{
	return SelectionAsType<FEdGraphSchemaAction_K2LocalVar>(GraphActionMenu);
}

FEdGraphSchemaAction_BlueprintVariableBase* SMyBlueprint::SelectionAsBlueprintVariable() const
{
	TArray<TSharedPtr<FEdGraphSchemaAction> > SelectedActions;
	GraphActionMenu->GetSelectedActions(SelectedActions);

	FEdGraphSchemaAction_BlueprintVariableBase* Selection = nullptr;

	TSharedPtr<FEdGraphSchemaAction> SelectedAction( SelectedActions.Num() > 0 ? SelectedActions[0] : nullptr );
	if ( SelectedAction.IsValid() && SelectedAction->IsA(FEdGraphSchemaAction_BlueprintVariableBase::StaticGetTypeId()))
	{
		Selection = (FEdGraphSchemaAction_BlueprintVariableBase*)SelectedAction.Get();
	}

	return Selection;
}

FEdGraphSchemaAction_K2Delegate* SMyBlueprint::SelectionAsDelegate() const
{
	return SelectionAsType<FEdGraphSchemaAction_K2Delegate>( GraphActionMenu );
}

FEdGraphSchemaAction_K2Event* SMyBlueprint::SelectionAsEvent() const
{
	return SelectionAsType<FEdGraphSchemaAction_K2Event>( GraphActionMenu );
}

FEdGraphSchemaAction_K2InputAction* SMyBlueprint::SelectionAsInputAction() const
{
	return SelectionAsType<FEdGraphSchemaAction_K2InputAction>(GraphActionMenu);
}

bool SMyBlueprint::SelectionIsCategory() const
{
	return !SelectionHasContextMenu();
}

bool SMyBlueprint::SelectionHasContextMenu() const
{
	TArray<TSharedPtr<FEdGraphSchemaAction> > SelectedActions;
	GraphActionMenu->GetSelectedActions(SelectedActions);
	return SelectedActions.Num() > 0;
}

FText SMyBlueprint::GetGraphCategory(UEdGraph* InGraph) const
{
	FText ReturnCategory;

	// Pull the category from the required metadata based on the types of nodes we can discover in the graph
	UK2Node_EditablePinBase* EntryNode = FBlueprintEditorUtils::GetEntryNode(InGraph);
	if (UK2Node_FunctionEntry* FunctionEntryNode = Cast<UK2Node_FunctionEntry>(EntryNode))
	{
		ReturnCategory = FunctionEntryNode->MetaData.Category;
	}
	else if (UK2Node_Tunnel* TypedEntryNode = ExactCast<UK2Node_Tunnel>(EntryNode))
	{
		ReturnCategory = TypedEntryNode->MetaData.Category;
	}

	// Empty the category if it's default, we don't want to display the "default" category and items will just appear without a category
	if(ReturnCategory.EqualTo(UEdGraphSchema_K2::VR_DefaultCategory))
	{
		ReturnCategory = FText::GetEmpty();
	}

	return ReturnCategory;
}

void SMyBlueprint::GetSelectedItemsForContextMenu(TArray<FComponentEventConstructionData>& OutSelectedItems) const
{
	FEdGraphSchemaAction_K2Var* Var = SelectionAsVar();
	if ( Var != NULL )
	{
		FObjectProperty* ComponentProperty = CastField<FObjectProperty>(Var->GetProperty());

		if ( ComponentProperty != NULL &&
			 ComponentProperty->PropertyClass != NULL &&
			 ComponentProperty->PropertyClass->IsChildOf( UActorComponent::StaticClass() ) )
		{
			FComponentEventConstructionData NewItem;
			NewItem.VariableName = Var->GetVariableName();
			NewItem.Component = Cast<UActorComponent>(ComponentProperty->PropertyClass->GetDefaultObject());

			OutSelectedItems.Add( NewItem );
		}
	}
}

TSharedPtr<SWidget> SMyBlueprint::OnContextMenuOpening()
{
	if( !BlueprintEditorPtr.IsValid() )
	{
		return TSharedPtr<SWidget>();
	}

	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder( bShouldCloseWindowAfterMenuSelection, CommandList);
	
	// Check if the selected action is valid for a context menu
	if (SelectionHasContextMenu())
	{
		FEdGraphSchemaAction_K2Var* Var = SelectionAsVar();
		FEdGraphSchemaAction_K2Graph* Graph = SelectionAsGraph();
		FEdGraphSchemaAction_K2Event* Event = SelectionAsEvent();
		const bool bExpandFindReferences = Graph || Event || Var;
		
		MenuBuilder.BeginSection("BasicOperations");
		{
			MenuBuilder.AddMenuEntry(FMyBlueprintCommands::Get().OpenGraph);
			MenuBuilder.AddMenuEntry(FMyBlueprintCommands::Get().OpenGraphInNewTab);
			MenuBuilder.AddMenuEntry(FMyBlueprintCommands::Get().OpenExternalGraph);
			MenuBuilder.AddMenuEntry(FMyBlueprintCommands::Get().FocusNode);
			MenuBuilder.AddMenuEntry(FMyBlueprintCommands::Get().FocusNodeInNewTab);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename, NAME_None, LOCTEXT("Rename", "Rename"), LOCTEXT("Rename_Tooltip", "Renames this function or variable from blueprint.") );
			MenuBuilder.AddMenuEntry(FMyBlueprintCommands::Get().ImplementFunction);

			// Depending on context, FindReferences can be a button or an expandable menu. For example, the context menu
			// for functions now lets you choose whether to do search by-name (fast) or by-function (smart).
			if (!bExpandFindReferences)
			{
				// No expandable menu: display the simple 'Find References' action
				MenuBuilder.AddMenuEntry(FGraphEditorCommands::Get().FindReferences);
			}
			else
			{
				// Insert "Find References" sub-menu here
				MenuBuilder.AddSubMenu(
					LOCTEXT("FindReferences_Label", "Find References"),
					LOCTEXT("FindReferences_Tooltip", "Options for finding references to class members"),
					FNewMenuDelegate::CreateStatic(&FGraphEditorCommands::BuildFindReferencesMenu),
					false,
					FSlateIcon()
				);
			}
			
			MenuBuilder.AddMenuEntry(FGraphEditorCommands::Get().FindAndReplaceReferences);
			MenuBuilder.AddMenuEntry(FMyBlueprintCommands::Get().GotoNativeVarDefinition);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Duplicate);
			MenuBuilder.AddMenuEntry(FMyBlueprintCommands::Get().MoveVariableToParent);
			MenuBuilder.AddMenuEntry(FMyBlueprintCommands::Get().MoveFunctionToParent);
			MenuBuilder.AddMenuEntry(FMyBlueprintCommands::Get().DeleteEntry);
		}
		MenuBuilder.EndSection();

		if ( Var && BlueprintEditorPtr.IsValid() && FBlueprintEditorUtils::DoesSupportEventGraphs(GetBlueprintObj()) )
		{
			FObjectProperty* ComponentProperty = CastField<FObjectProperty>(Var->GetProperty());

			if ( ComponentProperty && ComponentProperty->PropertyClass &&
				 ComponentProperty->PropertyClass->IsChildOf( UActorComponent::StaticClass() ) )
			{
				if( FBlueprintEditorUtils::CanClassGenerateEvents( ComponentProperty->PropertyClass ))
				{
					TSharedPtr<FBlueprintEditor> BlueprintEditor(BlueprintEditorPtr.Pin());

					// If the selected item is valid, and is a component of some sort, build a context menu
					// of events appropriate to the component.
					MenuBuilder.AddSubMenu(	LOCTEXT("AddEventSubMenu", "Add Event"), 
											LOCTEXT("AddEventSubMenu_ToolTip", "Add Event"), 
											FNewMenuDelegate::CreateStatic(	&SSubobjectBlueprintEditor::BuildMenuEventsSection,
																											BlueprintEditor->GetBlueprintObj(), ComponentProperty->PropertyClass.Get(), 
												FCanExecuteAction::CreateRaw(this, &SMyBlueprint::IsEditingMode),
												FGetSelectedObjectsDelegate::CreateSP(this, &SMyBlueprint::GetSelectedItemsForContextMenu)));
				}
			}
		}
		// If this is a function graph than we should add the option to convert it to an event if possible
		else if( Graph && Graph->EdGraph )
		{
			// The first function entry node will have all the information that the conversion needs
			// (the interface method entry in the tree might not have a real graph though, if it comes from a parent unchanged or is an event that hasn't been implemented yet)
			UK2Node_FunctionEntry* EntryNode = nullptr;
			if (Graph->EdGraph != nullptr)
			{
				for( UEdGraphNode* Node : Graph->EdGraph->Nodes)
				{
					if (UK2Node_FunctionEntry* TypedNode = Cast<UK2Node_FunctionEntry>(Node))
					{
						EntryNode = TypedNode;
						break;
					}
				}
			}

			TSharedPtr<FBlueprintEditor> BlueprintEditor(BlueprintEditorPtr.Pin());
			if( EntryNode && BlueprintEditor.IsValid() &&
				FBlueprintEditorUtils::IsFunctionConvertableToEvent(BlueprintEditor->GetBlueprintObj(), EntryNode->FindSignatureFunction()) )
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("MyBlueprint_Conversion_Func", "Convert function to event"), FText(), FSlateIcon(),
					FExecuteAction::CreateLambda([BlueprintEditor, EntryNode]()
					{
						// ConvertFunctionIfValid handles any bad state, so no need for additional messaging
						BlueprintEditor->ConvertFunctionIfValid(EntryNode);
					})
				);
			}

			if (const UEdGraphSchema* Schema = Graph->EdGraph->GetSchema())
			{
				if (Schema->AllowsFunctionVariants())
				{
					MenuBuilder.AddMenuEntry(
						LOCTEXT("MyBlueprint_Add_Func_Variant", "Add Variant"), FText(), FSlateIcon(),
						FExecuteAction::CreateLambda([BlueprintEditor, Graph]()
						{
							BlueprintEditor->AddNewFunctionVariant(Graph->EdGraph);
						})
					);
				}
			}
		}
		// If this is an event, allow us to convert it to a function graph if possible
		else if( Event )
		{
			TSharedPtr<FBlueprintEditor> BlueprintEditor(BlueprintEditorPtr.Pin());			
			UK2Node_Event* EventNode = Cast<UK2Node_Event>(Event->NodeTemplate);
			
			if( BlueprintEditor.IsValid() && EventNode )
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("MyBlueprint_Conversion_Event", "Convert event to function"), FText(), FSlateIcon(),
					FExecuteAction::CreateLambda([BlueprintEditor, EventNode]()
					{
						// The ConvertEventIfValid function handles all bad states, so there's no need for further validation
						BlueprintEditor->ConvertEventIfValid(EventNode);
					})
				);
			}			
		}
	}
	else
	{
		BuildAddNewMenu(MenuBuilder);
	}

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SMyBlueprint::CreateAddNewMenuWidget()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

	BuildAddNewMenu(MenuBuilder);

	return MenuBuilder.MakeWidget();
}

void SMyBlueprint::BuildAddNewMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("AddNewItem", LOCTEXT("AddOperations", "Add New"));

	if(UBlueprint* CurrentBlueprint = GetBlueprintObj())
	{
		if (CurrentBlueprint->SupportsGlobalVariables())
		{
			MenuBuilder.AddMenuEntry(FBlueprintEditorCommands::Get().AddNewVariable);
			MenuBuilder.AddMenuEntry(FMyBlueprintCommands::Get().PasteVariable);
		}
		if (CurrentBlueprint->SupportsLocalVariables())
		{
			MenuBuilder.AddMenuEntry(FBlueprintEditorCommands::Get().AddNewLocalVariable);
			MenuBuilder.AddMenuEntry(FMyBlueprintCommands::Get().PasteLocalVariable);
		}
		if (CurrentBlueprint->SupportsFunctions())
		{
			MenuBuilder.AddMenuEntry(FBlueprintEditorCommands::Get().AddNewFunction);
			MenuBuilder.AddMenuEntry(FMyBlueprintCommands::Get().PasteFunction);

			// If we cannot handle Function Graphs, we cannot handle function overrides
			if (OverridableFunctionActions.Num() > 0 && BlueprintEditorPtr.Pin()->NewDocument_IsVisibleForType(FBlueprintEditor::CGT_NewFunctionGraph))
			{
				MenuBuilder.AddSubMenu(
					LOCTEXT("OverrideFunction", "Override Function"),
					FText::GetEmpty(),
					FNewMenuDelegate::CreateSP(this, &SMyBlueprint::BuildOverridableFunctionsMenu),
					false,
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintEditor.AddNewFunction.Small"));
			}
		}

		if (CurrentBlueprint->SupportsMacros())
		{
			MenuBuilder.AddMenuEntry(FBlueprintEditorCommands::Get().AddNewMacroDeclaration);
			MenuBuilder.AddMenuEntry(FMyBlueprintCommands::Get().PasteMacro);
		}
		if (CurrentBlueprint->SupportsEventGraphs())
		{
			MenuBuilder.AddMenuEntry(FBlueprintEditorCommands::Get().AddNewEventGraph);
		}
		if (CurrentBlueprint->SupportsDelegates())
		{
			MenuBuilder.AddMenuEntry(FBlueprintEditorCommands::Get().AddNewDelegate);
		}
		if (CurrentBlueprint->SupportsAnimLayers())
		{
			MenuBuilder.AddMenuEntry(FBlueprintEditorCommands::Get().AddNewAnimationLayer);
		}
	}
	MenuBuilder.EndSection();
}

bool SMyBlueprint::CanOpenGraph() const
{
	const FEdGraphSchemaAction_K2Graph* GraphAction = SelectionAsGraph();
	const bool bGraph = GraphAction && GraphAction->EdGraph;
	const FEdGraphSchemaAction_K2Delegate* DelegateAction = SelectionAsDelegate();
	const bool bDelegate = DelegateAction && DelegateAction->EdGraph;
	return (bGraph || bDelegate) && BlueprintEditorPtr.IsValid();
}

bool SMyBlueprint::CanOpenExternalGraph() const 
{
	const FEdGraphSchemaAction_K2Graph* GraphAction = SelectionAsGraph();
	const bool bGraph = GraphAction && GraphAction->EdGraph;
	return CanOpenGraph() && bGraph && !BlueprintEditorPtr.Pin()->IsGraphInCurrentBlueprint(GraphAction->EdGraph);
}

void SMyBlueprint::OpenGraph(FDocumentTracker::EOpenDocumentCause InCause, bool bOpenExternalGraphInNewEditor)
{
	UEdGraph* GraphToOpen = nullptr;

	if (FEdGraphSchemaAction_K2Graph* GraphAction = SelectionAsGraph())
	{
		GraphToOpen = GraphAction->EdGraph;
		// If we have no graph then this is an interface event, so focus on the event graph
		if (!GraphToOpen)
		{
			GraphToOpen = FBlueprintEditorUtils::FindEventGraph(GetBlueprintObj());
		}
	}
	else if (FEdGraphSchemaAction_K2Delegate* DelegateAction = SelectionAsDelegate())
	{
		GraphToOpen = DelegateAction->EdGraph;
	}
	else if (FEdGraphSchemaAction_K2Event* EventAction = SelectionAsEvent())
	{
		GraphToOpen = EventAction->NodeTemplate->GetGraph();
	}
	else if (FEdGraphSchemaAction_K2InputAction* InputAction = SelectionAsInputAction())
	{
		GraphToOpen = InputAction->NodeTemplate->GetGraph();
	}
	
	if (GraphToOpen)
	{
		if(bOpenExternalGraphInNewEditor && !BlueprintEditorPtr.Pin()->IsGraphInCurrentBlueprint(GraphToOpen))
		{
			if(UBlueprint* OtherBlueprint = FBlueprintEditorUtils::FindBlueprintForGraph(GraphToOpen))
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(OtherBlueprint);
				if(IBlueprintEditor* OtherBlueprintEditor = static_cast<IBlueprintEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(OtherBlueprint, true)))
				{
					OtherBlueprintEditor->JumpToHyperlink(GraphToOpen, false);
				}
			}
		}
		else
		{
			BlueprintEditorPtr.Pin()->OpenDocument(GraphToOpen, InCause);
		}
	}
}


void SMyBlueprint::OnOpenGraph()
{
	OpenGraph(FDocumentTracker::OpenNewDocument);
}

void SMyBlueprint::OnOpenGraphInNewTab()
{
	OpenGraph(FDocumentTracker::ForceOpenNewDocument);	
}

void SMyBlueprint::OnOpenExternalGraph()
{
	OpenGraph(FDocumentTracker::OpenNewDocument, true);
}

bool SMyBlueprint::CanFocusOnNode() const
{
	FEdGraphSchemaAction_K2Event const* const EventAction = SelectionAsEvent();
	FEdGraphSchemaAction_K2InputAction const* const InputAction = SelectionAsInputAction();
	UK2Node_Event* ExistingNode = nullptr;

	if (FEdGraphSchemaAction_K2Graph* GraphAction = SelectionAsGraph())
	{
		// Is this an event implemented from an interface?
		UBlueprint* BlueprintObj = GetBlueprintObj();		
		UFunction* OverrideFunc = nullptr;
		UClass* const OverrideFuncClass = FBlueprintEditorUtils::GetOverrideFunctionClass(BlueprintObj, GraphAction->FuncName, &OverrideFunc);

		if (OverrideFunc)
		{
			// Add to event graph
			FName EventName = OverrideFunc->GetFName();
			ExistingNode = FBlueprintEditorUtils::FindOverrideForFunction(BlueprintObj, OverrideFuncClass, EventName);
		}
	}

	return (EventAction && EventAction->NodeTemplate) || (InputAction && InputAction->NodeTemplate) || ExistingNode;
}

void SMyBlueprint::OnFocusNode()
{
	FEdGraphSchemaAction_K2Event* EventAction = SelectionAsEvent();
	FEdGraphSchemaAction_K2InputAction* InputAction = SelectionAsInputAction();
	if (EventAction || InputAction)
	{
		UK2Node* Node = EventAction ? EventAction->NodeTemplate : InputAction->NodeTemplate;
		FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(Node);
	}
	else if (FEdGraphSchemaAction_K2Graph* GraphAction = SelectionAsGraph())
	{
		// Is this an event implemented from an interface?
		UBlueprint* BlueprintObj = GetBlueprintObj();
		UFunction* OverrideFunc = nullptr;
		UClass* const OverrideFuncClass = FBlueprintEditorUtils::GetOverrideFunctionClass(BlueprintObj, GraphAction->FuncName, &OverrideFunc);

		if (OverrideFunc)
		{
			// Add to event graph
			FName EventName = OverrideFunc->GetFName();
			if (UK2Node_Event* ExistingNode = FBlueprintEditorUtils::FindOverrideForFunction(BlueprintObj, OverrideFuncClass, EventName))
			{
				FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(ExistingNode);
			}
		}
	}
}

void SMyBlueprint::OnFocusNodeInNewTab()
{
	OpenGraph(FDocumentTracker::ForceOpenNewDocument);
	OnFocusNode();
}

bool SMyBlueprint::CanImplementFunction() const
{
	FEdGraphSchemaAction_K2Graph* GraphAction = SelectionAsGraph();
	return GraphAction && GraphAction->EdGraph == nullptr && !CanFocusOnNode();
}

void SMyBlueprint::OnImplementFunction()
{
	if ( FEdGraphSchemaAction_K2Graph* GraphAction = SelectionAsGraph() )
	{
		ImplementFunction(GraphAction);
	}
}

void SMyBlueprint::ImplementFunction(TSharedPtr<FEdGraphSchemaAction_K2Graph> GraphAction)
{
	ImplementFunction(GraphAction.Get());
}

void SMyBlueprint::ImplementFunction(FEdGraphSchemaAction_K2Graph* GraphAction)
{
	UBlueprint* BlueprintObj = GetBlueprintObj();
	check(BlueprintObj && BlueprintObj->SkeletonGeneratedClass);
	if(BlueprintObj->BlueprintType == BPTYPE_MacroLibrary)
	{
		// macro libraries cannot have functions, attempting to create a function in 
		// one would trigger an assertion failure
		FNotificationInfo Info( LOCTEXT("NoFunctionsInMacro", 
				"Blueprint Macro Libraries cannot implement functions"));
		Info.bUseLargeFont = false;
		Info.ExpireDuration = 5.0f;
		Info.bFireAndForget = true;
		FSlateNotificationManager::Get().AddNotification(Info);
		return;
	}

	// Ensure that we are conforming to all current interfaces so that if there has been an additional
	// interface function added we just focus to it instead of creating a new one
	FBlueprintEditorUtils::ConformImplementedInterfaces(BlueprintObj);

	UFunction* OverrideFunc = nullptr;
	UClass* const OverrideFuncClass = FBlueprintEditorUtils::GetOverrideFunctionClass(BlueprintObj, GraphAction->FuncName, &OverrideFunc);
	check(OverrideFunc);
	// Some types of blueprints don't have an event graph (IE gameplay ability blueprints), in that case just make a new graph, even
	// for events:
	UEdGraph* EventGraph = FBlueprintEditorUtils::FindEventGraph(BlueprintObj);
	if (UEdGraphSchema_K2::FunctionCanBePlacedAsEvent(OverrideFunc) && !IsImplementationDesiredAsFunction(OverrideFunc) && EventGraph)
	{
		// Add to event graph
		FName EventName = OverrideFunc->GetFName();
		UK2Node_Event* ExistingNode = FBlueprintEditorUtils::FindOverrideForFunction(BlueprintObj, OverrideFuncClass, EventName);

		if (ExistingNode)
		{
			FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(ExistingNode);
		}
		else
		{
			UK2Node_Event* NewEventNode = FEdGraphSchemaAction_K2NewNode::SpawnNode<UK2Node_Event>(
				EventGraph,
				EventGraph->GetGoodPlaceForNewNode(),
				EK2NewNodeFlags::SelectNewNode,
				[EventName, OverrideFuncClass](UK2Node_Event* NewInstance)
				{
					NewInstance->EventReference.SetExternalMember(EventName, OverrideFuncClass);
					NewInstance->bOverrideFunction = true;
				}
			);
			if (NewEventNode)
			{
				FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(NewEventNode);
			}
		}
	}
	else
	{
		// If there is an already existing graph of this function than just open that
		// Needed for implementing interface functions on the base class through the override menu
		UEdGraph* const ExistingGraph = FindObject<UEdGraph>(BlueprintObj, *GraphAction->FuncName.ToString());
		if (ExistingGraph)
		{
			FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(ExistingGraph);
		}
		else
		{
			const FScopedTransaction Transaction(LOCTEXT("CreateOverrideFunctionGraph", "Create Override Function Graph"));
			BlueprintObj->Modify();
			// Implement the function graph
			UEdGraph* const NewGraph = FBlueprintEditorUtils::CreateNewGraph(BlueprintObj, GraphAction->FuncName, UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());
			FBlueprintEditorUtils::AddFunctionGraph(BlueprintObj, NewGraph, /*bIsUserCreated=*/ false, OverrideFuncClass);
			NewGraph->Modify();
			BlueprintEditorPtr.Pin()->OpenDocument(NewGraph, FDocumentTracker::OpenNewDocument);
		}
	}
}

bool SMyBlueprint::IsImplementationDesiredAsFunction(const UFunction* OverrideFunc) const
{	
	// If the original function was created in a parent blueprint, then prefer a BP function
	if (OverrideFunc)
	{
		FName OverrideName = *OverrideFunc->GetName();
		TSet<FName> GraphNames;
		FBlueprintEditorUtils::GetAllGraphNames(GetBlueprintObj(), GraphNames);
		for (const FName & Name : GraphNames)
		{
			if (Name == OverrideName)
			{
				return true;
			}
		}
	}
	
	// Otherwise, we would prefer an event
	return false;
}

void SMyBlueprint::OnFindReference(bool bSearchAllBlueprints, const EGetFindReferenceSearchStringFlags Flags)
{
	FString SearchTerm;
	for(const UEdGraph* UberGraph : GetBlueprintObj()->UbergraphPages)
	{
		if(const UEdGraphSchema* Schema = UberGraph->GetSchema())
		{
			TArray<TSharedPtr<FEdGraphSchemaAction> > SelectedActions;
			GraphActionMenu->GetSelectedActions(SelectedActions);
			if(SelectedActions.Num() == 1)
			{
				if (const FEdGraphSchemaAction* GraphAction = SelectedActions[0].Get())
				{
					SearchTerm = Schema->GetFindReferenceSearchTerm(GraphAction);
					if(!SearchTerm.IsEmpty())
					{
						BlueprintEditorPtr.Pin()->SummonSearchUI(true, SearchTerm);
						return;
					}
				}
			}
		}
	}

	bool bUseQuotes = true;

	if (FEdGraphSchemaAction_K2Graph* GraphAction = SelectionAsGraph())
	{
		bool bSearchTermGenerated = false;
		if (EnumHasAnyFlags(Flags, EGetFindReferenceSearchStringFlags::UseSearchSyntax))
		{
			// Attempt to resolve function, then attempt to generate search term from it
			if (const UFunction* Function = GraphAction->GetFunction())
			{
				if (FindInBlueprintsHelpers::ConstructSearchTermFromFunction(Function, SearchTerm))
				{
					bSearchTermGenerated = true;
					bUseQuotes = false;	
				}
			}
		}

		// If a search term was not generated yet, just search by name. Either the graph doesn't represent a
		// function or we failed to resolve a dependency for generating a query.
		if (!bSearchTermGenerated)
		{
			SearchTerm = GraphAction->FuncName.ToString();
		}
	}
	else if (FEdGraphSchemaAction_K2Var* VarAction = SelectionAsVar())
	{
		bool bSearchTermGenerated = false;
		if (EnumHasAnyFlags(Flags, EGetFindReferenceSearchStringFlags::UseSearchSyntax))
		{
			// Attempt to resolve property, then generate search term from it
			if (const FProperty* Property = VarAction->GetProperty())
			{
				FMemberReference MemberReference;
				MemberReference.SetFromField<FProperty>(Property, true, Property->GetOwnerClass());
				SearchTerm = MemberReference.GetReferenceSearchString(Property->GetOwnerClass());
				bSearchTermGenerated = true;
				bUseQuotes = false;
			}
		}

		// If a search term was not generated yet, just search by variable name
		if (!bSearchTermGenerated)
		{
			SearchTerm = VarAction->GetVariableName().ToString();
		}
	}
	else if (FEdGraphSchemaAction_K2LocalVar* LocalVarAction = SelectionAsLocalVar())
	{
		SearchTerm = FString::Printf(TEXT("Nodes(VariableReference(MemberName=+\"%s\" && MemberScope=+\"%s\"))"), *LocalVarAction->GetVariableName().ToString(), *LocalVarAction->GetVariableScope()->GetName());
		bUseQuotes = false;
	}
	else if (FEdGraphSchemaAction_K2Delegate* DelegateAction = SelectionAsDelegate())
	{
		SearchTerm = DelegateAction->GetDelegateName().ToString();
	}
	else if (FEdGraphSchemaAction_K2Enum* EnumAction = SelectionAsEnum())
	{
		SearchTerm = EnumAction->Enum->GetName();
	}
	else if (FEdGraphSchemaAction_K2Struct* StructAction = SelectionAsStruct())
	{
		SearchTerm = StructAction->Struct->GetName();
	}
	else if (FEdGraphSchemaAction_K2Event* EventAction = SelectionAsEvent())
	{
		bool bSearchTermGenerated = false;
		if (EnumHasAnyFlags(Flags, EGetFindReferenceSearchStringFlags::UseSearchSyntax))
		{
			if (const UK2Node_Event* EventNode = Cast<UK2Node_Event>(EventAction->NodeTemplate))
			{
				SearchTerm = EventNode->GetFindReferenceSearchString(Flags);
				bSearchTermGenerated = true;
				bUseQuotes = false;
			}
		}

		if (!bSearchTermGenerated)
		{
			SearchTerm = EventAction->NodeTemplate->GetFindReferenceSearchString(EGetFindReferenceSearchStringFlags::None);
			bUseQuotes = false;
		}
	}
	else if (FEdGraphSchemaAction_K2InputAction* InputAction = SelectionAsInputAction())
	{
		SearchTerm = InputAction->NodeTemplate ? 
			InputAction->NodeTemplate->GetNodeTitle(ENodeTitleType::FullTitle).ToString() :
			InputAction->GetMenuDescription().ToString();
	}

	if (!SearchTerm.IsEmpty())
	{
		if (bUseQuotes)
		{
			SearchTerm = FString::Printf(TEXT("\"%s\""), *SearchTerm);
		}
		
		const bool bSetFindWithinBlueprint = !bSearchAllBlueprints;
		BlueprintEditorPtr.Pin()->SummonSearchUI(bSetFindWithinBlueprint, SearchTerm);
	}
}

bool SMyBlueprint::CanFindReference() const
{
	// Nothing relevant to the category will ever be found, unless the name of the category overlaps with another item
	if (SelectionIsCategory())
	{
		return false;
	}

	return true;
}

void SMyBlueprint::OnFindAndReplaceReference()
{
	TSharedPtr<FBlueprintEditor> PinnedEditor = BlueprintEditorPtr.Pin();
	if (PinnedEditor.IsValid())
	{
		if (FEdGraphSchemaAction_K2Var* VarAction = SelectionAsVar())
		{
			PinnedEditor->SummonFindAndReplaceUI();
			if (PinnedEditor->GetReplaceReferencesWidget().IsValid())
			{
				PinnedEditor->GetReplaceReferencesWidget()->SetSourceVariable(VarAction->GetProperty());
			}
		}
	}
}

bool SMyBlueprint::CanFindAndReplaceReference() const
{
	return SelectionAsVar() != nullptr;
}

void SMyBlueprint::OnDeleteGraph(UEdGraph* InGraph, EEdGraphSchemaAction_K2Graph::Type InGraphType)
{
	if (InGraph && InGraph->bAllowDeletion)
	{
		if (const UEdGraphSchema* Schema = InGraph->GetSchema())
		{
			if (Schema->TryDeleteGraph(InGraph))
			{
				return;
			}
		}

		const FScopedTransaction Transaction( LOCTEXT("RemoveGraph", "Remove Graph") );
		GetBlueprintObj()->Modify();

		InGraph->Modify();

		if (InGraphType == EEdGraphSchemaAction_K2Graph::Subgraph)
		{
			// Remove any composite nodes bound to this graph
			TArray<UK2Node_Composite*> AllCompositeNodes;
			FBlueprintEditorUtils::GetAllNodesOfClass<UK2Node_Composite>(GetBlueprintObj(), AllCompositeNodes);

			const bool bDontRecompile = true;
			for (UK2Node_Composite* CompNode : AllCompositeNodes)
			{
				if (CompNode->BoundGraph == InGraph)
				{
					FBlueprintEditorUtils::RemoveNode(GetBlueprintObj(), CompNode, bDontRecompile);
				}
			}
		}

		FBlueprintEditorUtils::RemoveGraph(GetBlueprintObj(), InGraph, EGraphRemoveFlags::Recompile);
		BlueprintEditorPtr.Pin()->CloseDocumentTab(InGraph);

		for (TObjectIterator<UK2Node_CreateDelegate> It(RF_ClassDefaultObject, /** bIncludeDerivedClasses */ true, /** InternalExcludeFlags */ EInternalObjectFlags::Garbage); It; ++It)
		{
			if (It->GetGraph() != InGraph)
			{
				if (IsValid(*It) && IsValid(It->GetGraph()))
				{
					It->HandleAnyChange();
				}
			}
		}

		InGraph = NULL;
	}
}

UEdGraph* SMyBlueprint::GetFocusedGraph() const
{
	TSharedPtr<FBlueprintEditor> BlueprintEditorPtrPinned = BlueprintEditorPtr.Pin();
	if( BlueprintEditorPtrPinned.IsValid() )
	{
		return BlueprintEditorPtrPinned->GetFocusedGraph();
	}

	return EdGraph;
}

void SMyBlueprint::OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (InObject == Blueprint && (InPropertyChangedEvent.ChangeType != EPropertyChangeType::ValueSet && InPropertyChangedEvent.ChangeType != EPropertyChangeType::ArrayClear))
	{
		bNeedsRefresh = true;
	}
}

bool SMyBlueprint::IsEditingMode() const
{
	TSharedPtr<FBlueprintEditor> BlueprintEditorSPtr = BlueprintEditorPtr.Pin();
	return BlueprintEditorSPtr.IsValid() && BlueprintEditorSPtr->InEditingMode();
}

bool SMyBlueprint::IsAnInterfaceEvent(FEdGraphSchemaAction_K2Graph* InAction)
{
	return InAction->GraphType == EEdGraphSchemaAction_K2Graph::Interface && !InAction->EdGraph;
}

void SMyBlueprint::OnDeleteDelegate(FEdGraphSchemaAction_K2Delegate* InDelegateAction)
{
	UEdGraph* GraphToActOn = InDelegateAction->EdGraph;
	UBlueprint* BlueprintObj = GetBlueprintObj();
	if (GraphToActOn && BlueprintObj)
	{
		const FScopedTransaction Transaction( LOCTEXT("RemoveDelegate", "Remove Event Dispatcher") );
		BlueprintObj->Modify();

		BlueprintEditorPtr.Pin()->CloseDocumentTab(GraphToActOn);
		GraphToActOn->Modify();

		FBlueprintEditorUtils::RemoveMemberVariable(BlueprintObj, GraphToActOn->GetFName());
		FBlueprintEditorUtils::RemoveGraph(BlueprintObj, GraphToActOn, EGraphRemoveFlags::Recompile);

		for (TObjectIterator<UK2Node_CreateDelegate> It(RF_ClassDefaultObject, /** bIncludeDerivedClasses */ true, /** InternalExcludeFlags */ EInternalObjectFlags::Garbage); It; ++It)
		{
			if (IsValid(*It) && IsValid(It->GetGraph()))
			{
				It->HandleAnyChange();
			}
		}
	}
}

namespace UE::Blueprint::Private
{
	// Given a type and value name, display a deletion confirmation warning.
	// Returns true if the user 'cancelled' the action, interpreted as an early exit prior to deletion.
	static bool DisplayInUseWarningAndEarlyExit(const FName& DisplayTypeName, const FName& DisplayValueName)
	{
		const FText DeleteConfirmationPrompt = FText::Format(LOCTEXT("DeleteConfirmationPrompt", "{0} {1} is in use! Do you really want to delete it?")
			, { FText::FromName(DisplayTypeName), FText::FromName(DisplayValueName) }
		);
		const FText DeleteConfirmationTitle = FText::Format(LOCTEXT("DeleteConfirmationTitle", "Delete {0}")
			, { FText::FromName(DisplayTypeName) }
		);
		const FString DeleteConfirmationIniSetting = FString::Format(TEXT("DeleteConfirmation{0}_Warning"), { DisplayTypeName.ToString() });

		// Warn the user that this may result in data loss
		FSuppressableWarningDialog::FSetupInfo Info(DeleteConfirmationPrompt, DeleteConfirmationTitle, DeleteConfirmationIniSetting);
		Info.ConfirmText = LOCTEXT("DeleteConfirmation_Yes", "Yes");
		Info.CancelText = LOCTEXT("DeleteConfirmation_No", "No");

		FSuppressableWarningDialog DeleteFunctionInUse(Info);
		return DeleteFunctionInUse.ShowModal() == FSuppressableWarningDialog::Cancel;
	}
}

void SMyBlueprint::OnDeleteEntry()
{
	if (FEdGraphSchemaAction_K2Graph* GraphAction = SelectionAsGraph())
	{
		// Currently only function graphs are supported for in-use detection and deletion warnings
		if (GraphAction->GraphType == EEdGraphSchemaAction_K2Graph::Function)
		{
			if (FBlueprintEditorUtils::IsFunctionUsed(GetBlueprintObj(), GraphAction->FuncName))
			{
				if (UE::Blueprint::Private::DisplayInUseWarningAndEarlyExit("Function", GraphAction->FuncName))
				{
					return;
				}
			}
		}

		OnDeleteGraph(GraphAction->EdGraph, GraphAction->GraphType);
	}
	else if (FEdGraphSchemaAction_K2Delegate* DelegateAction = SelectionAsDelegate())
	{
		OnDeleteDelegate(DelegateAction);
	}
	else if (FEdGraphSchemaAction_K2Var* VarAction = SelectionAsVar())
	{
		if(FBlueprintEditorUtils::IsVariableUsed(GetBlueprintObj(), VarAction->GetVariableName()))
		{
			if (UE::Blueprint::Private::DisplayInUseWarningAndEarlyExit("Variable", VarAction->GetVariableName()))
			{
				return;
			}
		}

		const FScopedTransaction Transaction( LOCTEXT( "RemoveVariable", "Remove Variable" ) );

		GetBlueprintObj()->Modify();
		FBlueprintEditorUtils::RemoveMemberVariable(GetBlueprintObj(), VarAction->GetVariableName());
	}
	else if (FEdGraphSchemaAction_K2LocalVar* LocalVarAction = SelectionAsLocalVar())
	{
		if (FBlueprintEditorUtils::IsVariableUsed(GetBlueprintObj(), LocalVarAction->GetVariableName(), FBlueprintEditorUtils::FindScopeGraph(GetBlueprintObj(), CastChecked<UStruct>(LocalVarAction->GetVariableScope()))))
		{
			if (UE::Blueprint::Private::DisplayInUseWarningAndEarlyExit("Local Variable", LocalVarAction->GetVariableName()))
			{
				return;
			}
		}

		const FScopedTransaction Transaction( LOCTEXT( "RemoveLocalVariable", "Remove Local Variable" ) );

		GetBlueprintObj()->Modify();

		UEdGraph* FunctionGraph = FBlueprintEditorUtils::GetTopLevelGraph(GetFocusedGraph());
		TArray<UK2Node_FunctionEntry*> FunctionEntryNodes;
		FunctionGraph->GetNodesOfClass<UK2Node_FunctionEntry>(FunctionEntryNodes);
		check(FunctionEntryNodes.Num() == 1);
		FunctionEntryNodes[0]->Modify();

		FBlueprintEditorUtils::RemoveLocalVariable(GetBlueprintObj(), CastChecked<UStruct>(LocalVarAction->GetVariableScope()), LocalVarAction->GetVariableName());
	}
	else if (FEdGraphSchemaAction_BlueprintVariableBase* BPVarAction = SelectionAsBlueprintVariable())
	{
		if (BPVarAction->IsVariableUsed())
		{
			if (UE::Blueprint::Private::DisplayInUseWarningAndEarlyExit("Variable", BPVarAction->GetVariableName()))
			{
				return;
			}
		}

		const FScopedTransaction Transaction( LOCTEXT( "RemoveLocalVariable", "Remove Local Variable" ) );

		GetBlueprintObj()->Modify();
		BPVarAction->DeleteVariable();		
	}
	else if (FEdGraphSchemaAction_K2Event* EventAction = SelectionAsEvent())
	{
		const FScopedTransaction Transaction(LOCTEXT( "RemoveEventNode", "Remove EventNode"));

		GetBlueprintObj()->Modify();
		FBlueprintEditorUtils::RemoveNode(GetBlueprintObj(), EventAction->NodeTemplate);
	}
	else if (SelectionIsCategory())
	{
		TArray<TSharedPtr<FEdGraphSchemaAction>> Actions;
		GraphActionMenu->GetSelectedCategorySubActions(Actions);
		if (Actions.Num())
		{
			FText TransactionTitle;

			switch((NodeSectionID::Type)Actions[0]->GetSectionID())
			{
			case NodeSectionID::VARIABLE:
			case NodeSectionID::LOCAL_VARIABLE:
				{
					TransactionTitle = LOCTEXT( "BulkRemoveVariables", "Bulk Remove Variables" );
					break;
				}
			case NodeSectionID::DELEGATE:
				{
					TransactionTitle = LOCTEXT( "BulkRemoveDelegates", "Bulk Remove Delegates" );
					break;
				}
			case NodeSectionID::FUNCTION:
				{
					TransactionTitle = LOCTEXT( "BulkRemoveFunctions", "Bulk Remove Functions" );
					break;
				}
			case NodeSectionID::MACRO:
				{
					TransactionTitle = LOCTEXT( "BulkRemoveMacros", "Bulk Remove Macros" );
					break;
				}
			default:
				{
					TransactionTitle = LOCTEXT( "BulkRemove", "Bulk Remove Items" );
				}
			}

			FScopedTransaction Transaction( TransactionTitle);

			bool bModified = false;

			GetBlueprintObj()->Modify();
			for (int32 i = 0; i < Actions.Num(); ++i)
			{
				if (Actions[i]->GetTypeId() == FEdGraphSchemaAction_K2Var::StaticGetTypeId())
				{
					FEdGraphSchemaAction_K2Var* Var = (FEdGraphSchemaAction_K2Var*)Actions[i].Get();
					
					FBlueprintEditorUtils::RemoveMemberVariable(GetBlueprintObj(), Var->GetVariableName());
					bModified = true;
				}
				else if (Actions[i]->GetTypeId() == FEdGraphSchemaAction_K2LocalVar::StaticGetTypeId())
				{
					FEdGraphSchemaAction_K2LocalVar* K2LocalVarAction = (FEdGraphSchemaAction_K2LocalVar*)Actions[i].Get();

					FBlueprintEditorUtils::RemoveLocalVariable(GetBlueprintObj(), CastChecked<UStruct>(K2LocalVarAction->GetVariableScope()), K2LocalVarAction->GetVariableName());
					bModified = true;
				}
				else if (Actions[i]->GetTypeId() == FEdGraphSchemaAction_K2Graph::StaticGetTypeId())
				{
					FEdGraphSchemaAction_K2Graph* K2GraphAction = (FEdGraphSchemaAction_K2Graph*)Actions[i].Get();
					if(K2GraphAction->EdGraph->bAllowDeletion)
					{
						OnDeleteGraph(K2GraphAction->EdGraph, K2GraphAction->GraphType);
						bModified = true;
					}
				}
				else if (Actions[i]->GetTypeId() == FEdGraphSchemaAction_K2Delegate::StaticGetTypeId())
				{
					OnDeleteDelegate((FEdGraphSchemaAction_K2Delegate*)Actions[i].Get());
					bModified = true;
				}
			}

			if(!bModified)
			{
				Transaction.Cancel();
			}
		}
	}

	Refresh();
	BlueprintEditorPtr.Pin()->GetInspector()->ShowDetailsForObjects(TArray<UObject*>());
}

struct FDeleteEntryHelper
{
	static bool CanDeleteVariable(const UBlueprint* Blueprint, FName VarName)
	{
		check(NULL != Blueprint);

		const FProperty* VariableProperty = FindFProperty<FProperty>(Blueprint->SkeletonGeneratedClass, VarName);
		const UClass* VarSourceClass = VariableProperty->GetOwnerChecked<const UClass>();
		const bool bIsBlueprintVariable = (VarSourceClass == Blueprint->SkeletonGeneratedClass);
		const int32 VarInfoIndex = FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, VariableProperty->GetFName());
		const bool bHasVarInfo = (VarInfoIndex != INDEX_NONE);

		return bIsBlueprintVariable && bHasVarInfo;
	}
};

bool SMyBlueprint::CanDeleteEntry() const
{
	// Cannot delete entries while not in editing mode
	if(!IsEditingMode())
	{
		return false;
	}

	if (FEdGraphSchemaAction_K2Graph* GraphAction = SelectionAsGraph())
	{
		return (GraphAction->EdGraph ? GraphAction->EdGraph->bAllowDeletion : false);
	}
	else if (FEdGraphSchemaAction_K2Delegate* DelegateAction = SelectionAsDelegate())
	{
		return (DelegateAction->EdGraph != nullptr) && (DelegateAction->EdGraph->bAllowDeletion) &&
			FDeleteEntryHelper::CanDeleteVariable(GetBlueprintObj(), DelegateAction->GetDelegateName());
	}
	else if (FEdGraphSchemaAction_K2Var* VarAction = SelectionAsVar())
	{
		return FDeleteEntryHelper::CanDeleteVariable(GetBlueprintObj(), VarAction->GetVariableName());
	}
	else if (FEdGraphSchemaAction_K2Event* EventAction = SelectionAsEvent())
	{
		return EventAction->NodeTemplate != nullptr;
	}
	else if (FEdGraphSchemaAction_K2LocalVar* LocalVariable = SelectionAsLocalVar())
	{
		return true;
	}
	else if (FEdGraphSchemaAction_BlueprintVariableBase* BPVariable = SelectionAsBlueprintVariable())
	{
		return true;
	}
	else if (SelectionIsCategory())
	{
		// Can't delete categories if they can't be renamed, that means they are native
		if(GraphActionMenu->CanRequestRenameOnActionNode())
		{
			return true;
		}
	}
	else if (FEdGraphSchemaAction* Action = SelectionAsType<FEdGraphSchemaAction>(GraphActionMenu))
	{
		return Action->CanBeDeleted();
	}

	return false;
}

bool SMyBlueprint::IsDuplicateActionVisible() const
{
	if (FEdGraphSchemaAction_K2Graph* GraphAction = SelectionAsGraph())
	{
		// Functions in interface Blueprints cannot be duplicated
		if(GetBlueprintObj()->BlueprintType != BPTYPE_Interface)
		{
			// Only display it for valid function graphs
			return GraphAction->EdGraph && GraphAction->EdGraph->GetSchema()->CanDuplicateGraph(GraphAction->EdGraph);
		}
	}
	else if (SelectionAsVar() || SelectionAsLocalVar())
	{
		return true;
	}
	return false;
}

bool SMyBlueprint::CanDuplicateAction() const
{
	// Cannot delete entries while not in editing mode
	if (!IsEditingMode())
	{
		return false;
	}

	if (FEdGraphSchemaAction_K2Graph* GraphAction = SelectionAsGraph())
	{
		// Only support function graph duplication
		if(GraphAction->EdGraph)
		{
			return GraphAction->EdGraph->GetSchema()->CanDuplicateGraph(GraphAction->EdGraph);
		}
	}
	else if(FEdGraphSchemaAction_K2Var* VarAction = SelectionAsVar())
	{
		// if the property is not an allowable Blueprint variable type, do not allow the variable to be duplicated.
		// Some actions (timelines) exist as variables but cannot be used in a user-defined variable.
		const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(VarAction->GetProperty());
		if (ObjectProperty &&
			ObjectProperty->PropertyClass &&
			!UEdGraphSchema_K2::IsAllowableBlueprintVariableType(ObjectProperty->PropertyClass))
		{
			return false;
		}
		return true;
	}
	else if(SelectionAsBlueprintVariable())
	{
		return true;
	}
	return false;
}

void SMyBlueprint::OnDuplicateAction()
{
	FName DuplicateActionName = NAME_None;

	if (FEdGraphSchemaAction_K2Graph* GraphAction = SelectionAsGraph())
	{
		// Only StateMachine, function, anim graph and macro duplication is supported
		EGraphType GraphType = GraphAction->EdGraph->GetSchema()->GetGraphType(GraphAction->EdGraph);
		check(GraphType == GT_StateMachine || GraphType == GT_Function || GraphType == GT_Macro || GraphType == GT_Animation);

		if (GraphType == GT_StateMachine)
		{
			// StateMachine is handled using the BlueprintEditor copy / paste functionality
			if (const UAnimationStateMachineGraph* AnimationStateMachineGraph = Cast<UAnimationStateMachineGraph>(GraphAction->EdGraph))
			{
				BlueprintEditorPtr.Pin()->SelectAndDuplicateNode(AnimationStateMachineGraph->OwnerAnimGraphNode.Get());
			}
		}
		else
		{
			const FScopedTransaction Transaction(LOCTEXT("DuplicateGraph", "Duplicate Graph"));
			UBlueprint* BlueprintObj = GetBlueprintObj();
			UEdGraph* GraphToDuplicate = GraphAction->EdGraph;

			BlueprintObj->Modify();

			UEdGraph* DuplicatedGraph = GraphToDuplicate->GetSchema()->DuplicateGraph(GraphToDuplicate);
			check(DuplicatedGraph);

			DuplicatedGraph->Modify();

			// Generate new Guids and component templates for all relevant nodes in the graph
			// *NOTE* this cannot occur during PostDuplicate, node Guids and component templates need to remain static during duplication for Blueprint compilation
			for (UEdGraphNode* EdGraphNode : DuplicatedGraph->Nodes)
			{
				if (EdGraphNode)
				{
					EdGraphNode->CreateNewGuid();

					if (UK2Node_AddComponent* AddComponentNode = Cast<UK2Node_AddComponent>(EdGraphNode))
					{
						AddComponentNode->MakeNewComponentTemplate();
					}
				}
			}

			if (GraphType == GT_Function || GraphType == GT_Animation)
			{
				BlueprintObj->FunctionGraphs.Add(DuplicatedGraph);
			}
			else if (GraphType == GT_Macro)
			{
				BlueprintObj->MacroGraphs.Add(DuplicatedGraph);
			}

			// Now that we've taken ownership of the graph (assigned it into macrographs
			// or functiongraphs, as appropriate) it will be properly categorized
			// and we can run post rename logic reliably:
			FName NewGraphName = FBlueprintEditorUtils::FindUniqueKismetName(BlueprintObj, GraphToDuplicate->GetFName().GetPlainNameString());
			FBlueprintEditorUtils::RenameGraph(DuplicatedGraph, NewGraphName.ToString());

			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(BlueprintObj);

			BlueprintEditorPtr.Pin()->OpenDocument(DuplicatedGraph, FDocumentTracker::ForceOpenNewDocument);
			DuplicateActionName = DuplicatedGraph->GetFName();
		}
	}
	else if (FEdGraphSchemaAction_K2Var* VarAction = SelectionAsVar())
	{
		const FScopedTransaction Transaction( LOCTEXT( "DuplicateVariable", "Duplicate Variable" ) );
		GetBlueprintObj()->Modify();

		DuplicateActionName = FBlueprintEditorUtils::DuplicateVariable(GetBlueprintObj(), nullptr, VarAction->GetVariableName());
		if(DuplicateActionName == NAME_None)
		{
			// the variable was probably inherited from a C++ class

			FEdGraphPinType VarPinType;
			GetDefault<UEdGraphSchema_K2>()->ConvertPropertyToPinType(VarAction->GetProperty(), VarPinType);
			FBlueprintEditorUtils::AddMemberVariable(GetBlueprintObj(), FBlueprintEditorUtils::FindUniqueKismetName(Blueprint, VarAction->GetVariableName().ToString()), VarPinType);
		}
	}
	else if (FEdGraphSchemaAction_K2LocalVar* LocalVarAction = SelectionAsLocalVar())
	{
		const FScopedTransaction Transaction( LOCTEXT( "Duplicate Local Variable", "Duplicate Local Variable" ) );
		GetBlueprintObj()->Modify();

		DuplicateActionName = FBlueprintEditorUtils::DuplicateVariable(GetBlueprintObj(), Cast<UStruct>(LocalVarAction->GetVariableScope()), LocalVarAction->GetVariableName());
	}

	// Select and rename the duplicated action
	if(DuplicateActionName != NAME_None)
	{
		SelectItemByName(DuplicateActionName);
		Refresh();
		OnRequestRenameOnActionNode();
	}
}

void SMyBlueprint::GotoNativeCodeVarDefinition()
{
	if( FEdGraphSchemaAction_K2Var* VarAction = SelectionAsVar() )
	{
		if( FProperty* VarProperty = VarAction->GetProperty() )
		{
			FSourceCodeNavigation::NavigateToProperty( VarProperty );
		}
	}
}

bool SMyBlueprint::IsNativeVariable() const
{
	if( FEdGraphSchemaAction_K2Var* VarAction = SelectionAsVar() )
	{
		FProperty* VarProperty = VarAction->GetProperty();

		if( VarProperty && VarProperty->IsNative())
		{
			return true;
		}
	}
	return false;
}

void SMyBlueprint::OnMoveToParent()
{
	if (UBlueprint* ParentBlueprint = UBlueprint::GetBlueprintFromClass(Blueprint->ParentClass))
	{
		if (FEdGraphSchemaAction_K2Var* VarAction = SelectionAsVar())
		{
			int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, VarAction->GetVariableName());
			if (VarIndex != INDEX_NONE)
			{
				FScopedTransaction Transaction(LOCTEXT("MoveVarToParent", "Move Variable To Parent"));
				Blueprint->Modify();
				ParentBlueprint->Modify();

				FBPVariableDescription VarDesc = Blueprint->NewVariables[VarIndex];
				Blueprint->NewVariables.RemoveAt(VarIndex);

				// We need to manually pull the DefaultValue from the FProperty to set it on the new parent class variable
				{
					// Grab property off blueprint's current CDO
					UClass* GeneratedClass = Blueprint->GeneratedClass;
					UObject* GeneratedCDO = GeneratedClass->GetDefaultObject();

					if (FProperty* TargetProperty = FindFProperty<FProperty>(GeneratedClass, VarDesc.VarName))
					{
						if (void* OldPropertyAddr = TargetProperty->ContainerPtrToValuePtr<void>(GeneratedCDO))
						{
							// If there is a property for variable, it means the original default value was already copied, so it can be safely overridden
							VarDesc.DefaultValue.Empty();
							TargetProperty->ExportTextItem_Direct(VarDesc.DefaultValue, OldPropertyAddr, OldPropertyAddr, nullptr, PPF_SerializedAsImportText);
						}
					}
				}

				ParentBlueprint->NewVariables.Add(VarDesc);

				FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
				FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(ParentBlueprint);
				FBlueprintEditorUtils::ValidateBlueprintChildVariables(ParentBlueprint, VarDesc.VarName);
			}
		}
		else if (FEdGraphSchemaAction_K2Graph* GraphAction = SelectionAsGraph())
		{
			TSharedPtr<FBlueprintEditor> PinnedEditor = BlueprintEditorPtr.Pin();
			if (PinnedEditor.IsValid())
			{
				FScopedTransaction Transaction(LOCTEXT("MoveFuncToParent", "Move Function To Parent"));
				Blueprint->Modify();
				ParentBlueprint->Modify();

				FBPGraphClipboardData CopiedGraph(GraphAction->EdGraph);
				if (CopiedGraph.CreateAndPopulateGraph(ParentBlueprint, CopiedGraph.GetOriginalBlueprint(), PinnedEditor.Get(), GraphAction->GetCategory()))
				{
					PinnedEditor->CloseDocumentTab(GraphAction->EdGraph);
					OnDeleteEntry();
				}
			}
		}

		// open editor for parent blueprint
		if (UObject* ParentClass = Blueprint->ParentClass)
		{
			UBlueprintGeneratedClass* ParentBlueprintGeneratedClass = Cast<UBlueprintGeneratedClass>(ParentClass);
			if (ParentBlueprintGeneratedClass != NULL)
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ParentBlueprintGeneratedClass->ClassGeneratedBy);
			}
		}
	}
}

bool SMyBlueprint::CanMoveVariableToParent() const
{
	bool bCanMove = false;

	TSharedPtr<FBlueprintEditor> PinnedEditor = BlueprintEditorPtr.Pin();
	if (PinnedEditor.IsValid() && PinnedEditor->IsParentClassABlueprint())
	{
		if (FEdGraphSchemaAction_K2Var* VarAction = SelectionAsVar())
		{
			// If this variable is new to this class
			UBlueprint* SourceBlueprint;
			int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndexAndBlueprint(Blueprint, VarAction->GetVariableName(), SourceBlueprint);
			bCanMove = (VarIndex != INDEX_NONE) && (SourceBlueprint == Blueprint);
		}
	}

	return bCanMove;
}

bool SMyBlueprint::CanMoveFunctionToParent() const
{
	bool bCanMove = false;

	TSharedPtr<FBlueprintEditor> PinnedEditor = BlueprintEditorPtr.Pin();
	if (PinnedEditor.IsValid() && PinnedEditor->IsParentClassABlueprint())
	{
		if (FEdGraphSchemaAction_K2Graph* GraphAction = SelectionAsGraph())
		{
			if (CanDeleteEntry())
			{
				return PinnedEditor->IsGraphInCurrentBlueprint(GraphAction->EdGraph) && GraphAction->GraphType == EEdGraphSchemaAction_K2Graph::Function;
			}
		}
	}

	return bCanMove;
}

void SMyBlueprint::OnCopy()
{
	FString OutputString;

	if (FEdGraphSchemaAction_K2Var* VarAction = SelectionAsVar())
	{
		UBlueprint* SourceBlueprint;
		int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndexAndBlueprint(Blueprint, VarAction->GetVariableName(), SourceBlueprint);
		if (VarIndex != INDEX_NONE)
		{
			// make a copy of the Variable description so we can set the default value
			FBPVariableDescription Description = SourceBlueprint->NewVariables[VarIndex];

			//Grab property of blueprint's current CDO
			UClass* GeneratedClass = SourceBlueprint->GeneratedClass;
			UObject* GeneratedCDO = GeneratedClass->GetDefaultObject();
			FProperty* TargetProperty = FindFProperty<FProperty>(GeneratedClass, Description.VarName);

			if (TargetProperty)
			{
				// Grab the address of where the property is actually stored (UObject* base, plus the offset defined in the property)
				void* OldPropertyAddr = TargetProperty->ContainerPtrToValuePtr<void>(GeneratedCDO);
				if (OldPropertyAddr)
				{
					TargetProperty->ExportTextItem_Direct(Description.DefaultValue, OldPropertyAddr, OldPropertyAddr, nullptr, PPF_SerializedAsImportText);
				}
			}

			FBPVariableDescription::StaticStruct()->ExportText(OutputString, &Description, &Description, nullptr, 0, nullptr, false);
			OutputString = VAR_PREFIX + OutputString;
		}
	}
	else if (FEdGraphSchemaAction_K2LocalVar* LocalVarAction = SelectionAsLocalVar())
	{
		FBPVariableDescription* Description = FBlueprintEditorUtils::FindLocalVariable(Blueprint, CastChecked<UStruct>(LocalVarAction->GetVariableScope()), LocalVarAction->GetVariableName());

		if (Description)
		{
			FBPVariableDescription::StaticStruct()->ExportText(OutputString, Description, Description, nullptr, 0, nullptr, false);
			OutputString = VAR_PREFIX + OutputString;
		}
	}
	else if (FEdGraphSchemaAction_BlueprintVariableBase* BPVariable = SelectionAsBlueprintVariable())
	{
		if (const UEdGraph* FocusedGraph = Cast<UEdGraph>(BPVariable->GetVariableScope()))
		{
			if (const UEdGraphSchema* Schema = FocusedGraph->GetSchema())
			{
				TArray<FBPVariableDescription> LocalVariables;
				Schema->GetLocalVariables(FocusedGraph, LocalVariables);
				for (const FBPVariableDescription& VariableDescription : LocalVariables)
				{
					if (VariableDescription.VarName == BPVariable->GetVariableName())
					{
						FBPVariableDescription::StaticStruct()->ExportText(OutputString, &VariableDescription, &VariableDescription, nullptr, 0, nullptr, false);
						OutputString = VAR_PREFIX + OutputString;
						break;
					}
				}
			}
		}
	}
	else if (FEdGraphSchemaAction_K2Graph* GraphAction = SelectionAsGraph())
	{
		if (!Blueprint->ExportGraphToText(GraphAction->EdGraph, OutputString))
		{
			FBPGraphClipboardData FuncData(GraphAction->EdGraph);
			FBPGraphClipboardData::StaticStruct()->ExportText(OutputString, &FuncData, &FuncData, nullptr, 0, nullptr, false);
			OutputString = GRAPH_PREFIX + OutputString;
		}
	}

	if (!OutputString.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(OutputString.GetCharArray().GetData());
	}
}

bool SMyBlueprint::CanCopy() const
{
	if (FEdGraphSchemaAction_K2Var* VarAction = SelectionAsVar())
	{
		return FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, VarAction->GetVariableName()) != INDEX_NONE;
	}
	else if (FEdGraphSchemaAction_K2LocalVar* LocalVarAction = SelectionAsLocalVar())
	{
		return FBlueprintEditorUtils::FindLocalVariable(Blueprint, Cast<UStruct>(LocalVarAction->GetVariableScope()), LocalVarAction->GetVariableName()) != nullptr;
	}
	else if (FEdGraphSchemaAction_BlueprintVariableBase* BPVariable = SelectionAsBlueprintVariable())
	{
		if (const UEdGraph* FocusedGraph = Cast<UEdGraph>(BPVariable->GetVariableScope()))
		{
			if (const UEdGraphSchema* Schema = FocusedGraph->GetSchema())
			{
				TArray<FBPVariableDescription> LocalVariables;
				Schema->GetLocalVariables(FocusedGraph, LocalVariables);

				for (const FBPVariableDescription& VariableDescription : LocalVariables)
				{
					if (VariableDescription.VarName == BPVariable->GetVariableName())
					{
						return true;
					}
				}
			}
		}
		return false;		
	}
	else if (FEdGraphSchemaAction_K2Graph* GraphAction = SelectionAsGraph())
	{
		if (GraphAction->GraphType == EEdGraphSchemaAction_K2Graph::Function ||
			GraphAction->GraphType == EEdGraphSchemaAction_K2Graph::Macro)
		{
			return true;
		}
	}

	return false;
}

void SMyBlueprint::OnCut()
{
	OnCopy();
	OnDeleteEntry();
}

bool SMyBlueprint::CanCut() const
{
	return CanCopy() && CanDeleteEntry();
}

void SMyBlueprint::OnPasteGeneric()
{
	// prioritize pasting as a member variable if possible
	if (CanPasteVariable())
	{
		OnPasteVariable();
	}
	else if (CanPasteLocalVariable())
	{
		OnPasteLocalVariable();
	}
	else if (CanPasteFunction())
	{
		OnPasteFunction();
	}
	else if (CanPasteMacro())
	{
		OnPasteMacro();
	}
}

bool SMyBlueprint::CanPasteGeneric()
{
	return CanPasteVariable() || CanPasteLocalVariable() || CanPasteFunction() || CanPasteMacro();
}

void SMyBlueprint::OnPasteVariable()
{
	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);
	if (!ensure(ClipboardText.StartsWith(VAR_PREFIX, ESearchCase::CaseSensitive)))
	{
		return;
	}

	FBPVariableDescription Description;
	FStringOutputDevice Errors;
	const TCHAR* Import = ClipboardText.GetCharArray().GetData() + FCString::Strlen(VAR_PREFIX);
	FBPVariableDescription::StaticStruct()->ImportText(Import, &Description, nullptr, PPF_None, &Errors, FBPVariableDescription::StaticStruct()->GetName());
	if (Errors.IsEmpty())
	{
		FBPVariableDescription NewVar = FBlueprintEditorUtils::DuplicateVariableDescription(Blueprint, Description);
		if (NewVar.VarGuid.IsValid())
		{
			FScopedTransaction Transaction(FText::Format(LOCTEXT("PasteVariable", "Paste Variable: {0}"), FText::FromName(NewVar.VarName)));
			Blueprint->Modify();

			NewVar.Category = GetPasteCategory();

			Blueprint->NewVariables.Add(NewVar);

			// Potentially adjust variable names for any child blueprints
			FBlueprintEditorUtils::ValidateBlueprintChildVariables(Blueprint, NewVar.VarName);
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);

			SelectItemByName(NewVar.VarName);
		}
	}
}

void SMyBlueprint::OnPasteLocalVariable()
{
	TSharedPtr<FBlueprintEditor> PinnedEditor = BlueprintEditorPtr.Pin();
	if (PinnedEditor.IsValid())
	{
		UEdGraph* FocusedGraph = PinnedEditor->GetFocusedGraph();
		if (FocusedGraph)
		{
			TArray<UK2Node_FunctionEntry*> FunctionEntry;
			FocusedGraph->GetNodesOfClass<UK2Node_FunctionEntry>(FunctionEntry);

			FString ClipboardText;
			FPlatformApplicationMisc::ClipboardPaste(ClipboardText);
			if (!ensure(ClipboardText.StartsWith(VAR_PREFIX, ESearchCase::CaseSensitive)))
			{
				return;
			}

			FBPVariableDescription Description;
			FStringOutputDevice Errors;
			const TCHAR* Import = ClipboardText.GetCharArray().GetData() + FCString::Strlen(VAR_PREFIX);
			FBPVariableDescription::StaticStruct()->ImportText(Import, &Description, nullptr, 0, &Errors, FBPVariableDescription::StaticStruct()->GetName());
			if (Errors.IsEmpty())
			{
				FBPVariableDescription NewVar = FBlueprintEditorUtils::DuplicateVariableDescription(Blueprint, Description);
				if (NewVar.VarGuid.IsValid())
				{
					FScopedTransaction Transaction(FText::Format(LOCTEXT("PasteLocalVariable", "Paste Local Variable: {0}"), FText::FromName(NewVar.VarName)));

					NewVar.Category = GetPasteCategory();
					NewVar.RemoveMetaData(FBlueprintMetadata::MD_Private);

					if (FunctionEntry.Num() == 1)
					{
						FunctionEntry[0]->Modify();
						FunctionEntry[0]->LocalVariables.Add(NewVar);
					}
					else
					{
						BlueprintEditorPtr.Pin()->OnPasteNewLocalVariable(NewVar);
					}

					// Potentially adjust variable names for any child blueprints
					FBlueprintEditorUtils::ValidateBlueprintChildVariables(Blueprint, NewVar.VarName);
					FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);

					SelectItemByName(NewVar.VarName);
				}
			}
		}
	}
}

bool SMyBlueprint::CanPasteVariable() const
{
	TSharedPtr<FBlueprintEditor> PinnedEditor = BlueprintEditorPtr.Pin();
	if (PinnedEditor.IsValid() && !PinnedEditor->NewDocument_IsVisibleForType(FBlueprintEditor::CGT_NewVariable))
	{
		return false;
	}

	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);
	if (ClipboardText.StartsWith(VAR_PREFIX, ESearchCase::CaseSensitive))
	{
		FBPVariableDescription Description;
		FStringOutputDevice Errors;
		const TCHAR* Import = ClipboardText.GetCharArray().GetData() + FCString::Strlen(VAR_PREFIX);
		FBPVariableDescription::StaticStruct()->ImportText(Import, &Description, nullptr, 0, &Errors, FBPVariableDescription::StaticStruct()->GetName());

		return Errors.IsEmpty();
	}

	return false;
}

bool SMyBlueprint::CanPasteLocalVariable() const
{
	TSharedPtr<FBlueprintEditor> PinnedEditor = BlueprintEditorPtr.Pin();
	if (PinnedEditor.IsValid() && !PinnedEditor->NewDocument_IsVisibleForType(FBlueprintEditor::CGT_NewLocalVariable))
	{
		return false;
	}

	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);
	if (ClipboardText.StartsWith(VAR_PREFIX, ESearchCase::CaseSensitive))
	{
		FBPVariableDescription Description;
		FStringOutputDevice Errors;
		const TCHAR* Import = ClipboardText.GetCharArray().GetData() + FCString::Strlen(VAR_PREFIX);
		FBPVariableDescription::StaticStruct()->ImportText(Import, &Description, nullptr, 0, &Errors, FBPVariableDescription::StaticStruct()->GetName());

		return Errors.IsEmpty();
	}
	
	return false;
}

void SMyBlueprint::OnPasteFunction()
{
	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);

	if (Blueprint->TryImportGraphFromText(ClipboardText))
	{
		return;
	}

	if (!ensure(ClipboardText.StartsWith(GRAPH_PREFIX, ESearchCase::CaseSensitive)))
	{
		return;
	}

	FBPGraphClipboardData FuncData;
	FStringOutputDevice Errors;
	const TCHAR* Import = ClipboardText.GetCharArray().GetData() + FCString::Strlen(GRAPH_PREFIX);
	FBPGraphClipboardData::StaticStruct()->ImportText(Import, &FuncData, nullptr, 0, &Errors, FBPGraphClipboardData::StaticStruct()->GetName());
	if (Errors.IsEmpty() && FuncData.IsValid())
	{
		FScopedTransaction Transaction(LOCTEXT("PasteFunction", "Paste Function"));

		TSharedPtr<FBlueprintEditor> PinnedEditor = BlueprintEditorPtr.Pin();
		if (PinnedEditor.IsValid())
		{
			Blueprint->Modify();
			UEdGraph* Graph = FuncData.CreateAndPopulateGraph(Blueprint, FuncData.GetOriginalBlueprint(), PinnedEditor.Get(), GetPasteCategory());

			if (Graph)
			{
				PinnedEditor->OpenDocument(Graph, FDocumentTracker::OpenNewDocument);
				SelectItemByName(Graph->GetFName());
				Refresh();
				OnRequestRenameOnActionNode();
			}
			else
			{
				Transaction.Cancel();
			}
		}
	}
}

bool SMyBlueprint::CanPasteFunction() const
{
	TSharedPtr<FBlueprintEditor> PinnedEditor = BlueprintEditorPtr.Pin();
	if (PinnedEditor.IsValid() && !PinnedEditor->NewDocument_IsVisibleForType(FBlueprintEditor::CGT_NewFunctionGraph))
	{
		return false;
	}

	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);

	if (Blueprint->CanImportGraphFromText(ClipboardText))
	{
		return true;
	}

	if (ClipboardText.StartsWith(GRAPH_PREFIX, ESearchCase::CaseSensitive))
	{
		FBPGraphClipboardData FuncData;
		FStringOutputDevice Errors;
		const TCHAR* Import = ClipboardText.GetCharArray().GetData() + FCString::Strlen(GRAPH_PREFIX);
		FBPGraphClipboardData::StaticStruct()->ImportText(Import, &FuncData, nullptr, 0, &Errors, FBPGraphClipboardData::StaticStruct()->GetName());

		return Errors.IsEmpty() && FuncData.IsFunction();
	}

	return false;
}

void SMyBlueprint::OnPasteMacro()
{
	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);
	if (!ensure(ClipboardText.StartsWith(GRAPH_PREFIX, ESearchCase::CaseSensitive)))
	{
		return;
	}

	FBPGraphClipboardData FuncData;
	FStringOutputDevice Errors;
	const TCHAR* Import = ClipboardText.GetCharArray().GetData() + FCString::Strlen(GRAPH_PREFIX);
	FBPGraphClipboardData::StaticStruct()->ImportText(Import, &FuncData, nullptr, 0, &Errors, FBPGraphClipboardData::StaticStruct()->GetName());
	if (Errors.IsEmpty() && FuncData.IsValid())
	{
		FScopedTransaction Transaction(LOCTEXT("PasteMacro", "Paste Macro"));

		TSharedPtr<FBlueprintEditor> PinnedEditor = BlueprintEditorPtr.Pin();
		if (PinnedEditor.IsValid())
		{
			Blueprint->Modify();
			UEdGraph* Graph = FuncData.CreateAndPopulateGraph(Blueprint, FuncData.GetOriginalBlueprint(), PinnedEditor.Get(), GetPasteCategory());

			if (Graph)
			{
				PinnedEditor->OpenDocument(Graph, FDocumentTracker::OpenNewDocument);
				SelectItemByName(Graph->GetFName());
				OnRequestRenameOnActionNode();
			}
			else
			{
				Transaction.Cancel();
			}
		}
	}
}

bool SMyBlueprint::CanPasteMacro() const
{
	TSharedPtr<FBlueprintEditor> PinnedEditor = BlueprintEditorPtr.Pin();
	if (PinnedEditor.IsValid() && !PinnedEditor->NewDocument_IsVisibleForType(FBlueprintEditor::CGT_NewMacroGraph))
	{
		return false;
	}

	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);
	if (ClipboardText.StartsWith(GRAPH_PREFIX, ESearchCase::CaseSensitive))
	{
		FBPGraphClipboardData MacroData;
		FStringOutputDevice Errors;
		const TCHAR* Import = ClipboardText.GetCharArray().GetData() + FCString::Strlen(GRAPH_PREFIX);
		FBPGraphClipboardData::StaticStruct()->ImportText(Import, &MacroData, nullptr, 0, &Errors, FBPGraphClipboardData::StaticStruct()->GetName());

		return Errors.IsEmpty() && MacroData.IsMacro();
	}

	return false;
}

FText SMyBlueprint::GetPasteCategory() const
{
	if (SelectionIsCategory() && GraphActionMenu.IsValid())
	{
		FString CategoryName = GraphActionMenu->GetSelectedCategoryName();
		if (!CategoryName.IsEmpty())
		{
			return FText::FromString(GraphActionMenu->GetSelectedCategoryName());
		}
	}
	
	return UEdGraphSchema_K2::VR_DefaultCategory;
}

void SMyBlueprint::OnResetItemFilter()
{
	FilterBox->SetText(FText::GetEmpty());
}

void SMyBlueprint::EnsureLastPinTypeValid()
{
	LastPinType.bIsWeakPointer = false;
	LastFunctionPinType.bIsWeakPointer = false;

	const bool bLastPinTypeValid = (UEdGraphSchema_K2::PC_Struct != LastPinType.PinCategory) || LastPinType.PinSubCategoryObject.IsValid();
	const bool bLastFunctionPinTypeValid = (UEdGraphSchema_K2::PC_Struct != LastFunctionPinType.PinCategory) || LastFunctionPinType.PinSubCategoryObject.IsValid();
	const bool bConstType = LastPinType.bIsConst || LastFunctionPinType.bIsConst;
	if (!bLastPinTypeValid || !bLastFunctionPinTypeValid || bConstType)
	{
		ResetLastPinType();
	}
}

void SMyBlueprint::ResetLastPinType()
{
	LastPinType.ResetToDefaults();
	LastPinType.PinCategory = UEdGraphSchema_K2::PC_Boolean;
	LastFunctionPinType = LastPinType;
}

void SMyBlueprint::UpdateNodeCreation()
{
	if( BlueprintEditorPtr.IsValid() )
	{
		BlueprintEditorPtr.Pin()->UpdateNodeCreationStats( ENodeCreateAction::MyBlueprintDragPlacement );
	}
}

FReply SMyBlueprint::OnAddNewLocalVariable()
{
	if( BlueprintEditorPtr.IsValid() )
	{
		BlueprintEditorPtr.Pin()->OnAddNewLocalVariable();
	}

	return FReply::Handled();
}

void SMyBlueprint::OnFilterTextChanged( const FText& InFilterText )
{
	GraphActionMenu->GenerateFilteredItems(false);
}

FText SMyBlueprint::GetFilterText() const
{
	return FilterBox->GetText();
}

void SMyBlueprint::OnRequestRenameOnActionNode()
{
	// Attempt to rename in both menus, only one of them will have anything selected
	GraphActionMenu->OnRequestRenameOnActionNode();
}

bool SMyBlueprint::CanRequestRenameOnActionNode() const
{
	TArray<TSharedPtr<FEdGraphSchemaAction> > SelectedActions;
	GraphActionMenu->GetSelectedActions(SelectedActions);

	// If there is anything selected in the GraphActionMenu, check the item for if it can be renamed.
	if (SelectedActions.Num() || SelectionIsCategory())
	{
		return GraphActionMenu->CanRequestRenameOnActionNode();
	}
	return false;
}

void SMyBlueprint::SelectItemByName(const FName& ItemName, ESelectInfo::Type SelectInfo, int32 SectionId/* = INDEX_NONE*/, bool bIsCategory/* = false*/)
{
	// Check if the graph action menu is being told to clear
	if(ItemName == NAME_None)
	{
		ClearGraphActionMenuSelection();
	}
	else
	{
		// Attempt to select the item in the main graph action menu
		const bool bSucceededAtSelecting = GraphActionMenu->SelectItemByName(ItemName, SelectInfo, SectionId, bIsCategory);
		if (!bSucceededAtSelecting)
		{
			// We failed to select the item, maybe because it was filtered out?
			// Reset the item filter and try again (we don't do this first because someone went to the effort of typing
			// a filter and probably wants to keep it unless it is getting in the way, as it just has)
			OnResetItemFilter();
			GraphActionMenu->SelectItemByName(ItemName, SelectInfo, SectionId, bIsCategory);
		}
	}
}

void SMyBlueprint::ClearGraphActionMenuSelection()
{
	GraphActionMenu->SelectItemByName(NAME_None);
}

void SMyBlueprint::ExpandCategory(const FText& CategoryName)
{
	GraphActionMenu->ExpandCategory(CategoryName);
}

bool SMyBlueprint::MoveCategoryBeforeCategory(const FText& InCategoryToMove, const FText& InTargetCategory)
{
	bool bResult = false;

	FString CategoryToMoveString = InCategoryToMove.ToString();
	FString TargetCategoryString = InTargetCategory.ToString();
	if (UBlueprint* BlueprintObj = BlueprintEditorPtr.Pin()->GetBlueprintObj())
	{
		FScopedTransaction Transaction(LOCTEXT("ReorderCategories", "Reorder Categories"));
		BlueprintObj->Modify();

		// Find root categories
		int32 RootCategoryDelim = CategoryToMoveString.Find(TEXT("|"), ESearchCase::CaseSensitive);
		FName CategoryToMove = RootCategoryDelim == INDEX_NONE ? *CategoryToMoveString : *CategoryToMoveString.Left(RootCategoryDelim);
		RootCategoryDelim = TargetCategoryString.Find(TEXT("|"), ESearchCase::CaseSensitive);
		FName TargetCategory = RootCategoryDelim == INDEX_NONE ? *TargetCategoryString : *TargetCategoryString.Left(RootCategoryDelim);

		TArray<FName>& CategorySort = BlueprintObj->CategorySorting;

		// Remove existing sort index
		const int32 RemovalIndex = CategorySort.Find(CategoryToMove);
		if (RemovalIndex != INDEX_NONE)
		{
			CategorySort.RemoveAt(RemovalIndex);
		}

		// Update the Category sort order and refresh ( if the target category has an entry )
		const int32 InsertIndex = CategorySort.Find(TargetCategory);
		if (InsertIndex != INDEX_NONE)
		{
			CategorySort.Insert(CategoryToMove, InsertIndex);
			Refresh();
			bResult = true;
		}
	}

	return bResult;
}

#undef LOCTEXT_NAMESPACE
