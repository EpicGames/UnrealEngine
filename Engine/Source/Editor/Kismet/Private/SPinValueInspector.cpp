// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPinValueInspector.h"

#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "EdGraph/EdGraphNode.h"
#include "Engine/Blueprint.h"
#include "Framework/Application/SlateApplication.h"
#include "GenericPlatform/GenericWindowDefinition.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetDebugUtilities.h"
#include "Layout/Children.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "SlotBase.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "SPinValueInspector"

///////////////////////////////////////////////////////////////////////////////

TSharedPtr<SWindow> FPinValueInspectorTooltip::TooltipWindow = nullptr;
TSharedPtr<SToolTip> FPinValueInspectorTooltip::TooltipWidget = nullptr;
TSharedPtr<SPinValueInspector> FPinValueInspectorTooltip::ValueInspectorWidget = nullptr;
TSharedPtr<FPinValueInspectorTooltip> FPinValueInspectorTooltip::Instance = nullptr;

///////////////////////////////////////////////////////////////////////////////

class SPinValueInspector_ConstrainedBox : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPinValueInspector_ConstrainedBox)
		: _MinWidth()
		, _MaxWidth()
		, _MinHeight()
		, _MaxHeight()
	{}
		SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_ATTRIBUTE(TOptional<float>, MinWidth)
		SLATE_ATTRIBUTE(TOptional<float>, MaxWidth)
		SLATE_ATTRIBUTE(TOptional<float>, MinHeight)
		SLATE_ATTRIBUTE(TOptional<float>, MaxHeight)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		MinWidth = InArgs._MinWidth;
		MaxWidth = InArgs._MaxWidth;
		MinHeight = InArgs._MinHeight;
		MaxHeight = InArgs._MaxHeight;

		ChildSlot
		[
			InArgs._Content.Widget
		];
	}
	
	virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override
	{
		const float MinWidthVal = MinWidth.Get().Get(0.0f);
		const float MaxWidthVal = MaxWidth.Get().Get(0.0f);
		const float MinHeightVal = MinHeight.Get().Get(0.0f);
		const float MaxHeightVal = MaxHeight.Get().Get(0.0f);

		if (MinWidthVal == 0.0f && MaxWidthVal == 0.0f && MinHeightVal == 0.0f && MaxHeightVal == 0.0f)
		{
			return SCompoundWidget::ComputeDesiredSize(LayoutScaleMultiplier);
		}

		FVector2D ChildSize = ChildSlot.GetWidget()->GetDesiredSize();

		if (MinWidthVal != 0.0f || MaxWidthVal != 0.0f)
		{
			ChildSize.X = FMath::Max(MinWidthVal, ChildSize.X);
			if (MaxWidthVal > MinWidthVal)
			{
				ChildSize.X = FMath::Min(MaxWidthVal, ChildSize.X);
			}
		}
		if (CachedXSize.IsSet() && ChildSize.X < *CachedXSize)
		{
			ChildSize.X = *CachedXSize;
		}

		if (MinHeightVal != 0.0f || MaxHeightVal != 0.0f)
		{
			ChildSize.Y = FMath::Max(MinHeightVal, ChildSize.Y);
			if (MaxHeightVal > MinHeightVal)
			{
				ChildSize.Y = FMath::Min(MaxHeightVal, ChildSize.Y);
			}
		}

		if ((!CachedXSize.IsSet() && ChildSize.X != 0.0f) ||
			(CachedXSize.IsSet() && ChildSize.X > *CachedXSize))
		{
			CachedXSize.Reset();
			CachedXSize.Emplace(ChildSize.X);
		}

		return ChildSize;
	}

	void RequestResize()
	{
		CachedXSize.Reset();
	}

private:
	TAttribute<TOptional<float>> MinWidth;
	TAttribute<TOptional<float>> MaxWidth;
	TAttribute<TOptional<float>> MinHeight;
	TAttribute<TOptional<float>> MaxHeight;

	// must be mutable to be changed in ComputeDesiredSize
	mutable TOptional<double> CachedXSize;
};

void SPinValueInspector::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SSearchBox)
				.Visibility(this, &SPinValueInspector::GetSearchFilterVisibility)
				.OnTextChanged(this, &SPinValueInspector::OnSearchTextChanged)
		]
		+SVerticalBox::Slot()
		[
			SAssignNew(ConstrainedBox, SPinValueInspector_ConstrainedBox)
				.MinWidth(300.0f)
				.MaxWidth(800.0f)
				.MaxHeight(300.0f)
				[
					SAssignNew(TreeViewWidget, SKismetDebugTreeView)
						//.OnContextMenuOpening
						.OnExpansionChanged(this, &SPinValueInspector::OnExpansionChanged)
						.HeaderRow
						(
							SNew(SHeaderRow)
								.Visibility(EVisibility::Collapsed)
							+ SHeaderRow::Column(SKismetDebugTreeView::ColumnId_Name)
							+ SHeaderRow::Column(SKismetDebugTreeView::ColumnId_Value)
						)
				]
		]
	];
}

bool SPinValueInspector::ShouldShowSearchFilter() const
{
	// Only expose the search filter if child nodes are present (e.g. struct/container types).
	for (const FDebugTreeItemPtr& RootNode : TreeViewWidget->GetRootTreeItems())
	{
		if (RootNode->HasChildren())
		{
			return true;
		}
	}

	return false;
}

EVisibility SPinValueInspector::GetSearchFilterVisibility() const
{
	return ShouldShowSearchFilter() ? EVisibility::Visible : EVisibility::Collapsed;
}

void SPinValueInspector::OnSearchTextChanged(const FText& InSearchText)
{
	TreeViewWidget->ClearExpandedItems();
	TreeViewWidget->SetSearchText(InSearchText);
}

void SPinValueInspector::OnExpansionChanged(FDebugTreeItemPtr InItem, bool bItemIsExpanded)
{
	ConstrainedBox->RequestResize();
}

void SPinValueInspector::AddTreeItemUnique(const FDebugTreeItemPtr& Item)
{
	TreeViewWidget->AddTreeItemUnique(Item);
}

void SPinValueInspector::PopulateTreeView()
{
	// Locate the class property associated with the source pin and set it to the root node.
	const UEdGraphPin* GraphPin = PinRef.Get();
	if (!GraphPin)
	{
		return;
	}

	const UEdGraphNode* GraphNode = GraphPin->GetOwningNodeUnchecked();
	if (!GraphNode)
	{
		TreeViewWidget->AddTreeItemUnique(SKismetDebugTreeView::MakeMessageItem(LOCTEXT("InvalidNode", "Owning Node Not Found").ToString()));
		return;
	}

	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(GraphNode);
	if (!Blueprint)
	{
		TreeViewWidget->AddTreeItemUnique(SKismetDebugTreeView::MakeMessageItem(LOCTEXT("InvalidBlueprint", "Owning Blueprint Not Found").ToString()));
		return;
	}
	
	if (GraphPin->Direction == EGPD_Input)
	{
		for (const UEdGraphPin* Pin : GraphPin->LinkedTo)
		{
			AddPinToTreeView(Pin, Blueprint);
		}
	}
	else
	{
		AddPinToTreeView(GraphPin, Blueprint);
	}
}

void SPinValueInspector::AddPinToTreeView(const UEdGraphPin* Pin, UBlueprint* Blueprint)
{
	TSharedPtr<FPropertyInstanceInfo> DebugInfo;
	const FKismetDebugUtilities::EWatchTextResult WatchStatus = FKismetDebugUtilities::GetDebugInfo(DebugInfo, Blueprint, Blueprint->GetObjectBeingDebugged(), Pin);
	if (WatchStatus != FKismetDebugUtilities::EWTR_Valid)
	{
		switch (WatchStatus)
		{
		case FKismetDebugUtilities::EWTR_NotInScope:
			TreeViewWidget->AddTreeItemUnique(SKismetDebugTreeView::MakeMessageItem(LOCTEXT("NotInScope", "Not in scope").ToString()));
			break;

		case FKismetDebugUtilities::EWTR_NoProperty:
			TreeViewWidget->AddTreeItemUnique(SKismetDebugTreeView::MakeMessageItem(LOCTEXT("UnknownProperty", "No debug data").ToString()));
			break;

		default:
		case FKismetDebugUtilities::EWTR_NoDebugObject:
			TreeViewWidget->AddTreeItemUnique(SKismetDebugTreeView::MakeMessageItem(LOCTEXT("NoDebugObject", "No debug object").ToString()));
			break;
		}

		return;
	}

	if (ensureMsgf(DebugInfo.IsValid(), TEXT("GetDebugInfo returned EWTR_Valid, but DebugInfo wasn't valid")))
	{
		TreeViewWidget->AddTreeItemUnique(SKismetDebugTreeView::MakeWatchLineItem(Pin, Blueprint->GetObjectBeingDebugged()));
	}
}

void SPinValueInspector::RequestUpdateFilteredItems()
{
	TreeViewWidget->RequestUpdateFilteredItems();
}

const FEdGraphPinReference& SPinValueInspector::GetPinRef() const
{
	return PinRef;
}

void SPinValueInspector::SetPinRef(const FEdGraphPinReference& InPinRef)
{
	TreeViewWidget->ClearTreeItems();
	PinRef = InPinRef;
	PopulateTreeView();
	ConstrainedBox->RequestResize();
}

TWeakPtr<FPinValueInspectorTooltip> FPinValueInspectorTooltip::SummonTooltip(FEdGraphPinReference InPinRef, TSharedPtr<SPinValueInspector> InContentWidgetOverride)
{
	if (!FSlateApplication::Get().GetVisibleMenuWindow().Get())
	{
		if (ensureMsgf(InPinRef.Get(), TEXT("SPinValueInspector::SummonTooltip was called with an invalid Pin")))
		{
			if (!TooltipWindow.IsValid())
			{
				CreatePinValueTooltipWindow();
			}
			else if (Instance.IsValid())
			{
				Instance->DismissTooltip();
			}

			Instance = MakeShared<FPinValueInspectorTooltip>();
			if (TooltipWidget.IsValid() && InContentWidgetOverride)
			{
				TooltipWidget->SetContentWidget(InContentWidgetOverride->AsShared());
				InContentWidgetOverride->SetPinRef(InPinRef);
				ValueInspectorWidget->SetPinRef({ });
			}
			else
			{
				ValueInspectorWidget->SetPinRef(InPinRef);
			}

			TooltipWindow->ShowWindow();
			return Instance;
		}
	}

	return nullptr;
}

void FPinValueInspectorTooltip::ShutdownTooltip()
{
	TooltipWindow.Reset();
	TooltipWidget.Reset();
	ValueInspectorWidget.Reset();
}

void FPinValueInspectorTooltip::MoveTooltip(const UE::Slate::FDeprecateVector2DParameter& InNewLocation)
{
	if (ensureMsgf(TooltipWindow.IsValid(), TEXT("SPinValueInspector::MoveTooltip was called before any tooltip was created")))
	{
		TooltipWindow->MoveWindowTo(InNewLocation);
	}
}

void FPinValueInspectorTooltip::TryDismissTooltip(bool bForceDismiss)
{
	if (Instance.IsValid() && ValueInspectorWidget.IsValid())
	{
		// if we can't inspect the pin, force it to close
		if (!FKismetDebugUtilities::CanInspectPinValue(ValueInspectorWidget->PinRef.Get()) || TooltipCanClose() || bForceDismiss)
		{
			DismissTooltip();
		}
	}
}

void FPinValueInspectorTooltip::DismissTooltip()
{
	if (TooltipHostsMenu())
	{
		FSlateApplication::Get().DismissAllMenus();
	}

	ValueInspectorWidget->SetPinRef(FEdGraphPinReference());

	TooltipWindow->HideWindow();
	if (TooltipWidget->GetContentWidget() != ValueInspectorWidget)
	{
		TooltipWidget->SetContentWidget(ValueInspectorWidget->AsShared());
	}

	Instance.Reset();
}

void FPinValueInspectorTooltip::CreatePinValueTooltipWindow()
{
	TooltipWindow = SNew(SWindow)
		.Type(EWindowType::Notification) // EWindowType::Tooltip disables interactivity with child widgets
		.CreateTitleBar(false)
		.IsTopmostWindow(true)
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		.IsPopupWindow(true)
		.SizingRule(ESizingRule::Autosized)
		.ActivationPolicy(EWindowActivationPolicy::Never);

	FSlateApplication::Get().AddWindow(TooltipWindow.ToSharedRef());
	TooltipWindow->SetContent(SAssignNew(TooltipWidget, SToolTip));
	TooltipWidget->SetContentWidget(SAssignNew(ValueInspectorWidget, SPinValueInspector));
}

bool FPinValueInspectorTooltip::TooltipCanClose() const
{
	return !TooltipWindow->IsHovered() && !TooltipHostsMenu();
}

bool FPinValueInspectorTooltip::TooltipHostsMenu() const
{
	TSharedPtr<SWindow> MenuHostWindow = FSlateApplication::Get().GetVisibleMenuWindow();
	return MenuHostWindow.IsValid() && (MenuHostWindow == TooltipWindow);
}

#undef LOCTEXT_NAMESPACE
