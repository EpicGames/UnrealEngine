// Copyright Epic Games, Inc. All Rights Reserved.

#include "SReplaceNodeReferences.h"

#include "Algo/RemoveIf.h"
#include "BlueprintEditor.h"
#include "Components/ActorComponent.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "EdGraphSchema_K2.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EditorCategoryUtils.h"
#include "Engine/Blueprint.h"
#include "Engine/MemberReference.h"
#include "FindInBlueprints.h"
#include "Framework/Views/ITypedTableView.h"
#include "HAL/Platform.h"
#include "ImaginaryBlueprintData.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "ObjectEditorUtils.h"
#include "ReplaceNodeReferencesHelper.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class ITableRow;
class SWidget;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "SNodeVariableReferences"

class FTargetCategoryReplaceReferences : public FTargetReplaceReferences
{
public:
	FTargetCategoryReplaceReferences(FText InCategoryTitle)
		: CategoryTitle(InCategoryTitle)
	{}

	// FTargetReplaceReferences interface
	virtual TSharedRef<SWidget> CreateWidget() const override
	{
		return 
			SNew(STextBlock)
			.Text(CategoryTitle);
	}

	virtual bool GetMemberReference(FMemberReference& OutVariableReference) const override
	{
		return false;
	}

	virtual FText GetDisplayTitle() const override
	{
		return CategoryTitle;
	}

	virtual bool IsCategory() const override { return true; }
	// End of FTargetReplaceReferences interface

public:
	/** Category title to display for this item */
	FText CategoryTitle;
};

class FTargetVariableReplaceReferences : public FTargetReplaceReferences
{
public:
	// FTargetReplaceReferences interface
	virtual TSharedRef<SWidget> CreateWidget() const override
	{
		return 
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 0.0f)
			[
				SNew(SLayeredImage, GetSecondaryIcon(), GetSecondaryIconColor())
				.Image(GetIcon())
				.ColorAndOpacity(GetIconColor())
			]

			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(GetDisplayTitle())
				.ToolTipText(TooltipWarning)
			]
			// type of the variable
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.Padding(FMargin(1.0f, 0.0f, 0.0f, 0.0f))
			[
				SNew(STextBlock)
				.Text(GetPinClass() ? GetPinClass()->GetDisplayNameText() : FText::GetEmpty())
				.ToolTipText(TooltipWarning)
				.ColorAndOpacity(GetVariableTypeColor())
			];
	}

	virtual bool GetMemberReference(FMemberReference& OutVariableReference) const override
	{
		OutVariableReference = VariableReference;
		return true;
	}

	virtual FText GetDisplayTitle() const override
	{
		return FText::FromName(VariableReference.GetMemberName());
	}

	virtual const struct FSlateBrush* GetIcon() const override
	{
		return FBlueprintEditorUtils::GetIconFromPin(PinType);
	}

	virtual FSlateColor GetIconColor() const override
	{
		UEdGraphSchema_K2 const* K2Schema = GetDefault<UEdGraphSchema_K2>();
		return K2Schema->GetPinTypeColor(PinType);
	}

	virtual const struct FSlateBrush* GetSecondaryIcon() const override
	{
		return FBlueprintEditorUtils::GetSecondaryIconFromPin(PinType);
	}

	virtual FSlateColor GetSecondaryIconColor() const override
	{
		UEdGraphSchema_K2 const* K2Schema = GetDefault<UEdGraphSchema_K2>();
		return K2Schema->GetSecondaryPinTypeColor(PinType);
	}
	// End of FTargetReplaceReferences interface

	/** gets the UClass of the object being referenced by PinType */
	UClass *GetPinClass() const
	{
		if (PinType.PinCategory == UEdGraphSchema_K2::PC_Object && PinType.PinSubCategoryObject != nullptr)
		{
			return Cast<UClass>(PinType.PinSubCategoryObject);
		}
		return nullptr;
	}

	/** gets the color that the pin class will be displayed as in the menu */
	FSlateColor GetVariableTypeColor() const
	{
		const FSlateColor TypeColor = TooltipWarning.IsEmpty() ? FSlateColor(EStyleColor::AccentGreen) : FSlateColor(EStyleColor::AccentYellow);
		const FLinearColor LinearColor = TypeColor.GetSpecifiedColor();
		return FSlateColor(LinearColor.Desaturate(.2f));
	}

public:
	/** Variable reference for this item */
	FMemberReference VariableReference;

	/** Pin type representing the FProperty of this item */
	FEdGraphPinType PinType;

	/** Used to warn if the user is going to commit a potentially destructive action */
	FText TooltipWarning = FText::GetEmpty();
};

void SReplaceNodeReferences::Construct(const FArguments& InArgs, TSharedPtr<class FBlueprintEditor> InBlueprintEditor)
{
	BlueprintEditor = InBlueprintEditor;
	Refresh();
	bFindWithinBlueprint = false;
	bShowReplacementsWhenFinished = true;

	ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(3.0f, 5.0f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("FindWhat", "Find what:"))
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				[
					SNew(SBox)
					.MinDesiredWidth(150.0f)
					[
						SAssignNew( SourceReferencesComboBox, SComboButton )
						.OnGetMenuContent(this, &SReplaceNodeReferences::GetSourceMenuContent)
						.ContentPadding(0.0f)
						.ToolTipText(this, &SReplaceNodeReferences::GetSourceDisplayText)
						.HasDownArrow(true)
						.ButtonContent()
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(2.0f, 0.0f)
							[
								SNew(
									SLayeredImage,
									TAttribute<const FSlateBrush*>(this, &SReplaceNodeReferences::GetSecondarySourceReferenceIcon),
									TAttribute<FSlateColor>(this, &SReplaceNodeReferences::GetSecondarySourceReferenceIconColor)
								)
								.Image(this, &SReplaceNodeReferences::GetSourceReferenceIcon)
								.ColorAndOpacity(this, &SReplaceNodeReferences::GetSourceReferenceIconColor)
							]

							+SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Text(this, &SReplaceNodeReferences::GetSourceDisplayText)
							]
						]
					]
				]
			]

			+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(3.0f, 5.0f)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ReplaceWith", "Replace with:"))
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Left)
					[
						SNew(SBox)
						.MinDesiredWidth(150.0f)
						[
							SAssignNew( TargetReferencesComboBox, SComboButton )
							.OnGetMenuContent(this, &SReplaceNodeReferences::GetTargetMenuContent)
							.ContentPadding(0.0f)
							.ToolTipText(this, &SReplaceNodeReferences::GetTargetDisplayText)
							.HasDownArrow(true)
							.ButtonContent()
							[
								SNew(SHorizontalBox)
								+SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								.Padding(2.0f, 0.0f)
								[
									SNew(
										SLayeredImage,
										TAttribute<const FSlateBrush*>(this, &SReplaceNodeReferences::GetSecondaryTargetIcon),
										TAttribute<FSlateColor>(this, &SReplaceNodeReferences::GetSecondaryTargetIconColor)
									)
									.Image(this, &SReplaceNodeReferences::GetTargetIcon)
									.ColorAndOpacity(this, &SReplaceNodeReferences::GetTargetIconColor)
								]

								+SHorizontalBox::Slot()
								.VAlign(VAlign_Center)
								[
									SNew(STextBlock)
									.Text(this, &SReplaceNodeReferences::GetTargetDisplayText)
								]
							]
						]
					]
				]

			+SVerticalBox::Slot()
				[
					SNew(SBox)
						.MinDesiredHeight(150.0f)
						[
							SAssignNew(FindInBlueprints, SFindInBlueprints, InBlueprintEditor)
								.bIsSearchWindow(false)
								.bHideSearchBar(true)
								.bHideFindGlobalButton(true)
						]

				]

			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(3.0f, 5.0f)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SCheckBox)
						.IsChecked(this, &SReplaceNodeReferences::GetLocalCheckBoxState)
						.OnCheckStateChanged(this, &SReplaceNodeReferences::OnLocalCheckBoxChanged)
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(this, &SReplaceNodeReferences::GetLocalCheckBoxLabelText)
					]
				]

			+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.Padding(2.0f)
					.HAlign(HAlign_Left)
					.AutoWidth()
					[
						SNew(SButton)
						.Text(this, &SReplaceNodeReferences::GetFindAllButtonText)
						.ToolTipText(this, &SReplaceNodeReferences::GetFindAndReplaceToolTipText, false)
						.OnClicked(this, &SReplaceNodeReferences::OnFindAll)
						.IsEnabled(this, &SReplaceNodeReferences::CanBeginSearch, false)
					]

					+SHorizontalBox::Slot()
					.Padding(2.0f)
					.HAlign(HAlign_Left)
					.AutoWidth()
					[
						SNew(SButton)
						.Text(this, &SReplaceNodeReferences::GetFindAndReplaceAllButtonText)
						.ToolTipText(this, &SReplaceNodeReferences::GetFindAndReplaceToolTipText, true)
						.OnClicked(this, &SReplaceNodeReferences::OnFindAndReplaceAll)
						.IsEnabled(this, &SReplaceNodeReferences::CanBeginSearch, true)
					]
					
					+SHorizontalBox::Slot()
					.Padding(2.0f)
					.HAlign(HAlign_Left)
					.AutoWidth()
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SCheckBox)
							.IsChecked(this, &SReplaceNodeReferences::GetShowReplacementsCheckBoxState)
							.OnCheckStateChanged(this, &SReplaceNodeReferences::OnShowReplacementsCheckBoxChanged)
						]

						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						[
							SNew(STextBlock)
							.Text(this, &SReplaceNodeReferences::GetShowReplacementsCheckBoxLabelText)
						]
					]

					+SHorizontalBox::Slot()
					.Padding(2.0f)
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Bottom)
					.FillWidth(1.0f)
					[
						SNew(STextBlock)
						.Text(this, &SReplaceNodeReferences::GetStatusText)
					]
				]
		];
}

void SReplaceNodeReferences::Refresh()
{
	SetSourceVariable(nullptr);
	PossibleTargetVariableList.Empty();
	PossibleSourceVariableList.Empty();
	TargetClass = BlueprintEditor.Pin()->GetBlueprintObj()->SkeletonGeneratedClass;
	GatherAllAvailableBlueprintVariables(TargetClass, true);
	GatherAllAvailableBlueprintVariables(TargetClass, false);
}

void SReplaceNodeReferences::SetSourceVariable(FProperty* InProperty)
{
	if (InProperty)
	{
		FEdGraphPinType OldSourcePinType = SourcePinType;
		UEdGraphSchema_K2 const* K2Schema = GetDefault<UEdGraphSchema_K2>();
		K2Schema->ConvertPropertyToPinType(InProperty, SourcePinType);

		SourceProperty = InProperty;

		// If the type has changed, reset the target
		if (SourcePinType != OldSourcePinType)
		{
			SelectedTargetReferenceItem.Reset();
		}

		PossibleTargetVariableList.Empty();
		GatherAllAvailableBlueprintVariables(TargetClass, true);

		if (AvailableTargetReferencesTreeView.IsValid())
		{
			AvailableTargetReferencesTreeView->RequestTreeRefresh();
		}
	}
	else
	{
		SourceProperty = nullptr;
		SelectedTargetReferenceItem.Reset();
	}

	// Reset the FindInBlueprints results
	if (FindInBlueprints.IsValid())
	{
		FindInBlueprints->ClearResults();
	}
}

SReplaceNodeReferences::~SReplaceNodeReferences()
{

}

TSharedRef<SWidget>	SReplaceNodeReferences::GetTargetMenuContent()
{
	return SAssignNew(AvailableTargetReferencesTreeView, SReplaceReferencesTreeViewType)
		.TreeItemsSource( &PossibleTargetVariableList )
		.OnSelectionChanged(this, &SReplaceNodeReferences::OnTargetSelectionChanged)
		.OnGenerateRow( this, &SReplaceNodeReferences::OnGenerateRow )
		.OnGetChildren( this, &SReplaceNodeReferences::OnGetChildren );
}

TSharedRef<SWidget> SReplaceNodeReferences::GetSourceMenuContent()
{
	return SAssignNew(AvailableSourceReferencesTreeView, SReplaceReferencesTreeViewType)
		.TreeItemsSource(&PossibleSourceVariableList)
		.OnSelectionChanged(this, &SReplaceNodeReferences::OnSourceSelectionChanged)
		.OnGenerateRow(this, &SReplaceNodeReferences::OnGenerateRow)
		.OnGetChildren(this, &SReplaceNodeReferences::OnGetChildren);
}

void SReplaceNodeReferences::GatherAllAvailableBlueprintVariables(UClass* InTargetClass, bool bForTarget)
{
	if (InTargetClass == nullptr)
	{
		return;
	}

	if (bForTarget)
	{
		GatherAllAvailableBlueprintVariables(InTargetClass->GetSuperClass(), bForTarget);
	}

	TMap<FString, TSharedPtr< FTargetCategoryReplaceReferences > > CategoryMap;

	UObject* PathObject = InTargetClass->ClassGeneratedBy ? InTargetClass->ClassGeneratedBy.Get() : InTargetClass;
	TSharedPtr< FTargetCategoryReplaceReferences > BlueprintCategory = MakeShareable(new FTargetCategoryReplaceReferences(FText::FromString(PathObject->GetPathName())));
	for (TFieldIterator<FProperty> PropertyIt(InTargetClass, EFieldIteratorFlags::ExcludeSuper); PropertyIt; ++PropertyIt)
	{
		FProperty* Property = *PropertyIt;

		if (Property == SourceProperty)
		{
			continue;
		}
		FName PropName = Property->GetFName();

		// Don't show delegate properties, there is special handling for these
		const bool bMulticastDelegateProp = Property->IsA(FMulticastDelegateProperty::StaticClass());
		const bool bDelegateProp = (Property->IsA(FDelegateProperty::StaticClass()) || bMulticastDelegateProp);
		const bool bShouldShowAsVar = (!Property->HasAnyPropertyFlags(CPF_Parm) && Property->HasAllPropertyFlags(CPF_BlueprintVisible)) && !bDelegateProp;
		const bool bShouldShowAsDelegate = !Property->HasAnyPropertyFlags(CPF_Parm) && bMulticastDelegateProp 
			&& Property->HasAnyPropertyFlags(CPF_BlueprintAssignable | CPF_BlueprintCallable);
		FObjectPropertyBase* Obj = CastField<FObjectPropertyBase>(Property);
		if(!bShouldShowAsVar && !bShouldShowAsDelegate)
		{
			continue;
		}

		const FText PropertyTooltip = Property->GetToolTipText();
		const FName PropertyName = Property->GetFName();
		const FText PropertyDesc = FText::FromName(PropertyName);

		FText CategoryName = FObjectEditorUtils::GetCategoryText(Property);
		FText PropertyCategory = FObjectEditorUtils::GetCategoryText(Property);
		const FString UserCategoryName = FEditorCategoryUtils::GetCategoryDisplayString( PropertyCategory.ToString() );

		UEdGraphSchema_K2 const* K2Schema = GetDefault<UEdGraphSchema_K2>();
		if (CategoryName.EqualTo(FText::FromString(PathObject->GetName())) || CategoryName.EqualTo(UEdGraphSchema_K2::VR_DefaultCategory))
		{
			CategoryName = FText::GetEmpty();		// default, so place in 'non' category
			PropertyCategory = FText::GetEmpty();
		}

		if (bShouldShowAsVar)
		{
			const bool bComponentProperty = Obj && Obj->PropertyClass ? Obj->PropertyClass->IsChildOf<UActorComponent>() : false;

			// By default components go into the variable section under the component category unless a custom category is specified.
			if ( bComponentProperty && CategoryName.IsEmpty() )
			{
				PropertyCategory = LOCTEXT("Components", "Components");
			}

			TSharedPtr< FTargetCategoryReplaceReferences > CategoryReference = BlueprintCategory;
			if (!CategoryName.IsEmpty())
			{
				TSharedPtr< FTargetCategoryReplaceReferences >* CategoryReferencePtr = CategoryMap.Find(PropertyCategory.ToString());
				if (CategoryReferencePtr)
				{
					CategoryReference = *CategoryReferencePtr;
				}
				else
				{
					CategoryReference = MakeShareable(new FTargetCategoryReplaceReferences(PropertyCategory));
					CategoryMap.Add(PropertyCategory.ToString(), CategoryReference);
				}
			}

			TSharedPtr< FTargetVariableReplaceReferences > VariableItem = MakeShareable(new FTargetVariableReplaceReferences);
			VariableItem->VariableReference.SetFromField<FProperty>(Property, true, InTargetClass);

			FEdGraphPinType Type;
			K2Schema->ConvertPropertyToPinType(Property, VariableItem->PinType);

			// check whether this is a valid replacement using inheritance
			bool bValidInheritance = false;
			if (SourcePinType.PinCategory == UEdGraphSchema_K2::PC_Object &&
				SourcePinType.PinSubCategoryObject != nullptr)
			{
				const UClass* VariableItemClass = VariableItem->GetPinClass();
				const UClass* SourceClass = Cast<UClass>(SourcePinType.PinSubCategoryObject);

				// ensure the casting succeeded
				if (VariableItemClass && SourceClass)
				{
					if (VariableItemClass->IsChildOf(SourceClass) || SourceClass->IsChildOf(VariableItemClass))
					{
						bValidInheritance = true;
						if (VariableItemClass != SourceClass)
						{
							VariableItem->TooltipWarning = FText::Format(
								LOCTEXT("TooltipWarning", "Warning: Replacing a reference of type '{0}' with a reference of type '{1}' may break connections"),
								SourceClass->GetDisplayNameText(),
								VariableItemClass->GetDisplayNameText()
							);
						}
					}
				}
			}

			if (!bForTarget || bValidInheritance || VariableItem->PinType == SourcePinType)
			{
				CategoryReference->Children.Add(VariableItem);

				// If this is the first Child, add the category to the main Blueprint category
				if (CategoryReference != BlueprintCategory && CategoryReference->Children.Num() == 1)
				{
					BlueprintCategory->Children.Add(CategoryReference);
				}
			}
		}
	}

	TArray<FTreeViewItem>& CurrentList = bForTarget ? PossibleTargetVariableList : PossibleSourceVariableList;

	if (BlueprintCategory->Children.Num())
	{
		CurrentList.Add(BlueprintCategory);
		// Sort markers
		struct FCompareCategoryTitles
		{
			FORCEINLINE bool operator()(const TSharedPtr<FTargetReplaceReferences> A, const TSharedPtr<FTargetReplaceReferences> B) const
			{
				if (A->Children.Num() > 0 && B->Children.Num() == 0)
				{
					return true;
				}
				else if (A->Children.Num() == 0 && B->Children.Num() > 0)
				{
					return false;
				}
				return A->GetDisplayTitle().CompareTo(B->GetDisplayTitle()) < 0;
			}
		};
		BlueprintCategory->Children.Sort(FCompareCategoryTitles());
	}
	
	// Conditionally add "No variables found"
	if (TargetClass == InTargetClass && CurrentList.Num() == 0)
	{
		if (bForTarget)
		{
			TSharedPtr< FTargetCategoryReplaceReferences > NoneFound = MakeShared<FTargetCategoryReplaceReferences>(LOCTEXT("NoReplacements", "No viable replacements found!"));
			CurrentList.Add(NoneFound);
		}
		else
		{
			TSharedPtr< FTargetCategoryReplaceReferences > NoneFound = MakeShared<FTargetCategoryReplaceReferences>(LOCTEXT("NoSources", "No replaceable variables found!"));
			CurrentList.Add(NoneFound);
		}
	}
}

TSharedRef<ITableRow> SReplaceNodeReferences::OnGenerateRow(FTreeViewItem InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew( STableRow< TSharedPtr<FFindInBlueprintsResult> >, OwnerTable )
		[
			InItem->CreateWidget()
		];
}

void SReplaceNodeReferences::OnGetChildren( FTreeViewItem InItem, TArray< FTreeViewItem >& OutChildren )
{
	OutChildren += InItem->Children;
}

FReply SReplaceNodeReferences::OnFindAll()
{
	if (bFindWithinBlueprint)
	{
		OnSubmitSearchQuery(false);
	}
	else
	{
		FFindInBlueprintCachingOptions CachingOptions;
		CachingOptions.MinimiumVersionRequirement = EFiBVersion::FIB_VER_VARIABLE_REFERENCE;
		CachingOptions.OnFinished = FSimpleDelegate::CreateSP(this, &SReplaceNodeReferences::OnSubmitSearchQuery, false);
		FindInBlueprints->CacheAllBlueprints(CachingOptions);
	}
	return FReply::Handled();
}

FReply SReplaceNodeReferences::OnFindAndReplaceAll()
{
	if (SelectedTargetReferenceItem.IsValid())
	{
		if (bFindWithinBlueprint)
		{
			OnSubmitSearchQuery(true);
		}
		else
		{
			FFindInBlueprintCachingOptions CachingOptions;
			CachingOptions.MinimiumVersionRequirement = EFiBVersion::FIB_VER_VARIABLE_REFERENCE;
			CachingOptions.OnFinished = FSimpleDelegate::CreateSP(this, &SReplaceNodeReferences::OnSubmitSearchQuery, true);
			FindInBlueprints->CacheAllBlueprints(CachingOptions);
		}
	}
	return FReply::Handled();
}

void SReplaceNodeReferences::OnSubmitSearchQuery(bool bFindAndReplace)
{
	if (SourceProperty != nullptr)
	{
		FString SearchTerm;

		FMemberReference SourceVariableReference;
		SourceVariableReference.SetFromField<FProperty>(SourceProperty, true, SourceProperty->GetOwnerClass());
		SearchTerm = SourceVariableReference.GetReferenceSearchString(SourceProperty->GetOwnerClass());

		FOnSearchComplete OnSearchComplete;
		if (bFindAndReplace)
		{
			OnSearchComplete = FOnSearchComplete::CreateSP(this, &SReplaceNodeReferences::FindAllReplacementsComplete);
		}

		FStreamSearchOptions SearchOptions;
		SearchOptions.ImaginaryDataFilter = ESearchQueryFilter::NodesFilter;
		SearchOptions.MinimiumVersionRequirement = EFiBVersion::FIB_VER_VARIABLE_REFERENCE;
		FindInBlueprints->MakeSearchQuery(SearchTerm, bFindWithinBlueprint, SearchOptions, OnSearchComplete);
	}
}

void SReplaceNodeReferences::FindAllReplacementsComplete(TArray<FImaginaryFiBDataSharedPtr>& InRawDataList)
{
	if (InRawDataList.Num() == 0)
	{
		return;
	}

	if (!bFindWithinBlueprint)
	{
		SReplaceReferencesConfirmation::EDialogResponse Response = SReplaceReferencesConfirmation::CreateModal(&InRawDataList);

		if (Response == SReplaceReferencesConfirmation::EDialogResponse::Cancel)
		{
			return;
		}
	}

	if (SourceProperty != nullptr && SelectedTargetReferenceItem.IsValid())
	{
		FMemberReference SourceVariableReference;
		UClass* SourcePropertyScope = SourceProperty->GetOwnerClass();
		SourceVariableReference.SetFromField<FProperty>(SourceProperty, SourcePropertyScope);
		FMemberReference TargetVariableReference;
		if (SelectedTargetReferenceItem->GetMemberReference(TargetVariableReference) && SourceVariableReference.ResolveMember<FProperty>(SourcePropertyScope))
		{
			TSharedPtr<FBlueprintEditor> PinnedEditor = BlueprintEditor.Pin();
			if (PinnedEditor.IsValid())
			{
				UBlueprint* Blueprint = PinnedEditor->GetBlueprintObj();
				if (Blueprint)
				{
					const FScopedTransaction Transaction(GetTransactionTitle(TargetVariableReference));
					Blueprint->Modify();

					// Note: SourceProperty will be reset after this step! This occurs via a Refresh() that's triggered by an OnBlueprintChanged event.
					FReplaceNodeReferencesHelper::ReplaceReferences(SourceVariableReference, TargetVariableReference, Blueprint, InRawDataList);

					if (bShowReplacementsWhenFinished)
					{
						// @todo - Possibly move this into a local SFindInBlueprints context that's separate from the replacement context.
						// That way we could limit the results to just the local Blueprint if 'bFindWithinBlueprint' is toggled on. For now
						// we're just utilizing one of the "floating" global FiB nomad tabs to display the results, which will not have an
						// associated Blueprint editor context, so we cannot perform a local Blueprint search there.
						TSharedPtr<SFindInBlueprints> GlobalResults = FFindInBlueprintSearchManager::Get().GetGlobalFindResults();
						if (GlobalResults)
						{
							FStreamSearchOptions SearchOptions;
							SearchOptions.ImaginaryDataFilter = ESearchQueryFilter::NodesFilter;
							SearchOptions.MinimiumVersionRequirement = EFiBVersion::FIB_VER_VARIABLE_REFERENCE;
							GlobalResults->MakeSearchQuery(TargetVariableReference.GetReferenceSearchString(SourcePropertyScope), /*bInIsFindWithinBlueprint =*/ false, SearchOptions);
						}
					}
				}
			}
		}
	}
}

void SReplaceNodeReferences::OnTargetSelectionChanged(FTreeViewItem Selection, ESelectInfo::Type SelectInfo)
{
	// When the user is navigating, do not act upon the selection change
	if(SelectInfo == ESelectInfo::OnNavigation || (Selection.IsValid() && Selection->IsCategory()))
	{
		return;
	}
	
	SelectedTargetReferenceItem = Selection;
	TargetReferencesComboBox->SetIsOpen(false);
}

void SReplaceNodeReferences::OnSourceSelectionChanged(FTreeViewItem Selection, ESelectInfo::Type SelectInfo)
{
	// When the user is navigating, do not act upon the selection change
	if (SelectInfo == ESelectInfo::OnNavigation || (Selection.IsValid() && Selection->IsCategory()))
	{
		return;
	}

	FMemberReference NewSource;
	if (Selection->GetMemberReference(NewSource))
	{
		SetSourceVariable(NewSource.ResolveMember<FProperty>(TargetClass));
	}

	SourceReferencesComboBox->SetIsOpen(false);
}

FText SReplaceNodeReferences::GetSourceDisplayText() const
{
	if (SourceProperty == nullptr)
	{
		return LOCTEXT("UnselectedSourceReference", "Please select a source reference!");
	}
	return FText::FromString(SourceProperty->GetName());
}

const FSlateBrush* SReplaceNodeReferences::GetSourceReferenceIcon() const
{
	if (SourceProperty != nullptr)
	{
		return FBlueprintEditorUtils::GetIconFromPin(SourcePinType);
	}

	return nullptr;
}

FSlateColor SReplaceNodeReferences::GetSourceReferenceIconColor() const
{
	UEdGraphSchema_K2 const* K2Schema = GetDefault<UEdGraphSchema_K2>();
	return K2Schema->GetPinTypeColor(SourcePinType);
}

const FSlateBrush* SReplaceNodeReferences::GetSecondarySourceReferenceIcon() const
{
	if (SourceProperty != nullptr)
	{
		return FBlueprintEditorUtils::GetSecondaryIconFromPin(SourcePinType);
	}

	return nullptr;
}

FSlateColor SReplaceNodeReferences::GetSecondarySourceReferenceIconColor() const
{
	UEdGraphSchema_K2 const* K2Schema = GetDefault<UEdGraphSchema_K2>();
	return K2Schema->GetSecondaryPinTypeColor(SourcePinType);
}

FText SReplaceNodeReferences::GetFindAllButtonText() const
{
	if (bFindWithinBlueprint)
	{
		const UBlueprint* BlueprintObj = BlueprintEditor.Pin()->GetBlueprintObj();
		FFormatNamedArguments Args;
		Args.Add(TEXT("BP"), BlueprintObj ? FText::FromString(BlueprintObj->GetName()) : FText::FromString(TEXT("<UNKNOWN>")));
		return FText::Format(LOCTEXT("FindLocal", "Find References in {BP}"), Args);
	}
	else
	{
		return LOCTEXT("FindAll", "Find All References");
	}
}

FText SReplaceNodeReferences::GetFindAndReplaceAllButtonText() const
{
	if (bFindWithinBlueprint)
	{
		const UBlueprint* BlueprintObj = BlueprintEditor.Pin()->GetBlueprintObj();
		FFormatNamedArguments Args;
		Args.Add(TEXT("BP"), BlueprintObj ? FText::FromString(BlueprintObj->GetName()) : FText::FromString(TEXT("<UNKNOWN>")));
		return FText::Format(LOCTEXT("ReplaceLocal", "Find and Replace References in {BP}"), Args);
	}
	else
	{
		return LOCTEXT("ReplaceAll", "Find and Replace All References");
	}
}

FText SReplaceNodeReferences::GetFindAndReplaceToolTipText(bool bFindAndReplace) const
{
	if (CanBeginSearch(bFindAndReplace))
	{
		return FText::GetEmpty();
	}
	else
	{
		if (SourceProperty == nullptr)
		{
			return LOCTEXT("PickSourceVariable", "Pick a source variable from the menu!");
		}
		else if (bFindAndReplace && !SelectedTargetReferenceItem.IsValid())
		{
			return LOCTEXT("PickTarget", "Pick a target variable to replace with from the menu!");
		}
		else
		{
			return LOCTEXT("SearchInProgress", "A search is already in progress!");
		}
	}
}

bool SReplaceNodeReferences::CanBeginSearch(bool bFindAndReplace) const
{
	bool bCanSearch = SourceProperty != nullptr && !IsSearchInProgress();

	if (bFindAndReplace)
	{
		bCanSearch = bCanSearch && SelectedTargetReferenceItem.IsValid();
	}

	return bCanSearch;
}

void SReplaceNodeReferences::OnLocalCheckBoxChanged(ECheckBoxState Checked)
{
	bFindWithinBlueprint = (Checked == ECheckBoxState::Checked);
}

ECheckBoxState SReplaceNodeReferences::GetLocalCheckBoxState() const
{
	if (FindInBlueprints.IsValid())
	{
		return bFindWithinBlueprint ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	return ECheckBoxState::Unchecked;
}

FText SReplaceNodeReferences::GetShowReplacementsCheckBoxLabelText() const
{
	return LOCTEXT("ShowReplacements", "Show Replacements when complete?");
}

void SReplaceNodeReferences::OnShowReplacementsCheckBoxChanged(ECheckBoxState Checked)
{
	bShowReplacementsWhenFinished = (Checked == ECheckBoxState::Checked);
}

ECheckBoxState SReplaceNodeReferences::GetShowReplacementsCheckBoxState() const
{
	return bShowReplacementsWhenFinished ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

FText SReplaceNodeReferences::GetLocalCheckBoxLabelText() const
{
	if (BlueprintEditor.IsValid())
	{
		const UBlueprint* BlueprintObj = BlueprintEditor.Pin()->GetBlueprintObj();

		FFormatNamedArguments Args;
		Args.Add(TEXT("BlueprintClass"), BlueprintObj ? FText::FromString(BlueprintObj->GetName()) : FText::GetEmpty());
		return FText::Format(LOCTEXT("OnlyLocal", "Only show and replace results from {BlueprintClass} class?"), Args);
	}

	return FText::GetEmpty();
}

FText SReplaceNodeReferences::GetStatusText() const
{
	if (IsSearchInProgress())
	{
		if (FFindInBlueprintSearchManager::Get().IsCacheInProgress())
		{
			return LOCTEXT("Caching", "Caching...");
		}
		else
		{
			return LOCTEXT("Searching", "Searching...");
		}
	}

	return FText::GetEmpty();
}

bool SReplaceNodeReferences::IsSearchInProgress() const
{
	return FindInBlueprints.IsValid() && 
		   (FindInBlueprints->IsSearchInProgress() || FFindInBlueprintSearchManager::Get().IsCacheInProgress());
}

FText SReplaceNodeReferences::GetTransactionTitle(const FMemberReference& TargetReference) const
{
	FText BlueprintName;

	if (bFindWithinBlueprint && BlueprintEditor.IsValid())
	{
		const UBlueprint* BlueprintObj = BlueprintEditor.Pin()->GetBlueprintObj();

		BlueprintName = BlueprintObj ? FText::FromString(BlueprintObj->GetName()) : FText::GetEmpty();
	}

	FFormatNamedArguments Args;
	Args.Add(TEXT("Source"), FText::FromString(SourceProperty->GetName()));
	Args.Add(TEXT("Target"), FText::FromName(TargetReference.GetMemberName()));
	Args.Add(TEXT("Scope"), bFindWithinBlueprint ? BlueprintName : LOCTEXT("AllBlueprints", "All Blueprints"));
	return FText::Format(LOCTEXT("FindReplaceAllTransaction", "{Source} replaced with {Target} in {Scope}"), Args);
}

FText SReplaceNodeReferences::GetTargetDisplayText() const
{
	FText ReturnText = LOCTEXT("UnselectedTargetReference", "Please select a target reference!");

	if (SelectedTargetReferenceItem.IsValid())
	{
		ReturnText = SelectedTargetReferenceItem->GetDisplayTitle();
	}
	return ReturnText;
}

const FSlateBrush* SReplaceNodeReferences::GetTargetIcon() const
{
	const FSlateBrush* ReturnBrush = nullptr;

	if (SelectedTargetReferenceItem.IsValid())
	{
		ReturnBrush = SelectedTargetReferenceItem->GetIcon();
	}
	return ReturnBrush;
}

FSlateColor SReplaceNodeReferences::GetTargetIconColor() const
{
	FSlateColor ReturnColor = FLinearColor::White;

	if (SelectedTargetReferenceItem.IsValid())
	{
		ReturnColor = SelectedTargetReferenceItem->GetIconColor();
	}
	return ReturnColor;
}

const FSlateBrush* SReplaceNodeReferences::GetSecondaryTargetIcon() const
{
	const FSlateBrush* ReturnBrush = nullptr;

	if (SelectedTargetReferenceItem.IsValid())
	{
		ReturnBrush = SelectedTargetReferenceItem->GetSecondaryIcon();
	}
	return ReturnBrush;
}

FSlateColor SReplaceNodeReferences::GetSecondaryTargetIconColor() const
{
	FSlateColor ReturnColor = FLinearColor::White;

	if (SelectedTargetReferenceItem.IsValid())
	{
		ReturnColor = SelectedTargetReferenceItem->GetSecondaryIconColor();
	}
	return ReturnColor;
}

////////////////////////////////////////////////
// Replace References Confirmation

TSharedRef<SWidget> FReplaceConfirmationListItem::CreateWidget()
{
	return SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SCheckBox)
			.IsChecked_Raw(this, &FReplaceConfirmationListItem::IsChecked)
			.OnCheckStateChanged_Raw(this, &FReplaceConfirmationListItem::OnCheckStateChanged)
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(5.0f, 3.0f)
		[
			SNew(STextBlock)
			.Text(Blueprint ? FText::FromString(Blueprint->GetPathName()) : FText::FromString(TEXT("<UNKNOWN>")))
		];
}

void FReplaceConfirmationListItem::OnCheckStateChanged(ECheckBoxState State)
{
	bReplace = State == ECheckBoxState::Checked;
}

ECheckBoxState FReplaceConfirmationListItem::IsChecked() const
{
	return bReplace ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SReplaceReferencesConfirmation::Construct(const FArguments& InArgs)
{
	RawFindData = InArgs._FindResults;
	Response = EDialogResponse::Cancel;

	if (RawFindData)
	{
		for (FImaginaryFiBDataSharedPtr Data : *RawFindData)
		{
			const UBlueprint* DataBlueprint = Data->GetBlueprint();

			const FListViewItem* FoundItem = AffectedBlueprints.FindByPredicate([DataBlueprint](FListViewItem Item)
				{
					return Item->GetBlueprint() == DataBlueprint;
				});

			if (!FoundItem)
			{
				AffectedBlueprints.Add(MakeShared<FReplaceConfirmationListItem>(Data->GetBlueprint()));
			}
		}
	}

	ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ReplaceIn", "Replace references in the following Blueprints:"))
			]

			+SVerticalBox::Slot()
			.Padding(5.0f, 5.0f)
			[
				SNew(SBox)
				.MinDesiredHeight(100.0f)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("Menu.Background"))
					[
						SNew(SListView<FListViewItem>)
						.ListItemsSource(&AffectedBlueprints)
						.SelectionMode(ESelectionMode::None)
						.OnGenerateRow(this, &SReplaceReferencesConfirmation::OnGenerateRow)
					]
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.HAlign(HAlign_Right)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Bottom)
					.Padding(5.0f, 3.0f)
					[
						SNew(SButton)
						.Text(LOCTEXT("Confirm", "Confirm"))
						.OnClicked(this, &SReplaceReferencesConfirmation::CloseWindow, EDialogResponse::Confirm)
					]
				
					+SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Bottom)
					.Padding(5.0f, 3.0f)
					[
						SNew(SButton)
						.Text(LOCTEXT("Cancel", "Cancel"))
						.OnClicked(this, &SReplaceReferencesConfirmation::CloseWindow, EDialogResponse::Cancel)
					]
				]
			]
		];
}

SReplaceReferencesConfirmation::EDialogResponse SReplaceReferencesConfirmation::CreateModal(TArray<FImaginaryFiBDataSharedPtr>* InFindResults)
{
	TSharedPtr<SWindow> Window;
	TSharedPtr<SReplaceReferencesConfirmation> Widget;

	Window = SNew(SWindow)
		.Title(LOCTEXT("ConfirmReplace", "Confirm Replacements"))
		.SizingRule(ESizingRule::UserSized)
		.MinWidth(400.f)
		.MinHeight(300.f)
		.SupportsMaximize(true)
		.SupportsMinimize(false)
		[
			SNew(SBorder)
			.Padding(4.f)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SAssignNew(Widget, SReplaceReferencesConfirmation)
				.FindResults(InFindResults)
			]
		];

	Widget->MyWindow = Window;

	GEditor->EditorAddModalWindow(Window.ToSharedRef());

	return Widget->Response;
}

TSharedRef<ITableRow> SReplaceReferencesConfirmation::OnGenerateRow(FListViewItem Item, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return SNew(STableRow<FListViewItem>, OwnerTable)
		[
			Item->CreateWidget()
		];
}

FReply SReplaceReferencesConfirmation::CloseWindow(EDialogResponse InResponse)
{
	if (InResponse == EDialogResponse::Confirm && RawFindData)
	{
		// Filter the Results if necessary
		for (FListViewItem Item : AffectedBlueprints)
		{
			if (!Item->ShouldReplace())
			{
				const UBlueprint* BP = Item->GetBlueprint();
				RawFindData->SetNum(Algo::RemoveIf(*RawFindData, [BP](FImaginaryFiBDataSharedPtr Data) { return Data->GetBlueprint() == BP; }));
			}
		}
	}

	Response = InResponse;
	MyWindow->RequestDestroyWindow();
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
