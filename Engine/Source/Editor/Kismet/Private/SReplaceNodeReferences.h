// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphPin.h"
#include "FindInBlueprintManager.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Misc/Optional.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

class FBlueprintEditor;
class FProperty;
class ITableRow;
class SComboButton;
class SFindInBlueprints;
class SWidget;
class SWindow;
class UBlueprint;
class UClass;
struct FMemberReference;
struct FSlateBrush;

class FTargetReplaceReferences
{
public:
	virtual ~FTargetReplaceReferences() = default;

	/** Returns a generated widget to represent this target item reference */
	virtual TSharedRef<SWidget> CreateWidget() const = 0;

	/**
	 * Retrieves the MemberReference represented by this item, if any
	 *
	 * @param OutVariableReference		The output MemberReference represented by this item
	 * @return							TRUE if successful at having a MemberReference
	 */
	virtual bool GetMemberReference(FMemberReference& OutVariableReference) const = 0;

	/** Returns the display title for this item */
	virtual FText GetDisplayTitle() const = 0;

	/** TRUE if this item is a category and nothing else */
	virtual bool IsCategory() const { return false; }

	/** Returns the Icon representing this reference */
	virtual const struct FSlateBrush* GetIcon() const { return nullptr; }

	/** Returns the Icon Color of this reference */
	virtual FSlateColor GetIconColor() const { return FLinearColor::White; }

	/** Returns the Secondary Icon representing this reference */
	virtual const struct FSlateBrush* GetSecondaryIcon() const { return nullptr; }

	/** Returns the Secondary Icon Color of this reference */
	virtual FSlateColor GetSecondaryIconColor() const { return FLinearColor::White; }

public:
	/** Children members to sub-list in the tree */
	TArray< TSharedPtr<FTargetReplaceReferences> > Children;
};

class SReplaceNodeReferences : public SCompoundWidget
{
protected:
	typedef TSharedPtr< FTargetReplaceReferences > FTreeViewItem;
	typedef STreeView<FTreeViewItem>  SReplaceReferencesTreeViewType;
public:
	SLATE_BEGIN_ARGS( SReplaceNodeReferences ) {}
	SLATE_END_ARGS()

		SReplaceNodeReferences()
		: SourceProperty(nullptr)
	{

	}

	void Construct(const FArguments& InArgs, TSharedPtr<class FBlueprintEditor> InBlueprintEditor);
	~SReplaceNodeReferences();

	/** Forces a refresh on this widget when things in the Blueprint Editor have changed */
	void Refresh();

	/** Sets a source variable reference to replace */
	void SetSourceVariable(FProperty* InProperty);

protected:

	/* Called when a new row is being generated */
	TSharedRef<ITableRow> OnGenerateRow(FTreeViewItem InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/* Get the children of a row */
	void OnGetChildren( FTreeViewItem InItem, TArray< FTreeViewItem >& OutChildren );

	/* Returns the menu content for the Target Reference drop down section of the combo button */
	TSharedRef<SWidget>	GetTargetMenuContent();
	
	/* Returns the menu content for the Source Reference drop down section of the combo button */
	TSharedRef<SWidget>	GetSourceMenuContent();

	/** Callback when selection in the target combo button has changed */
	void OnTargetSelectionChanged(FTreeViewItem Selection, ESelectInfo::Type SelectInfo);
	
	/** Callback when selection in the target combo button has changed */
	void OnSourceSelectionChanged(FTreeViewItem Selection, ESelectInfo::Type SelectInfo);

	/** 
	 * Submits a search query and potentially does a mass replace on results
	 *
	 * @param bInFindAndReplace		TRUE if it's a Find-and-Replace action, FALSE if it should do a search of items that will be replaced
	 */
	void OnSubmitSearchQuery(bool bFindAndReplace);

	/** Callback for "Find All" button */
	FReply OnFindAll();

	/** Callback for "Find and Replace All" button */
	FReply OnFindAndReplaceAll();

	/** Callback when the search for "Find and Replace All" is complete so that the replacements can begin */
	void FindAllReplacementsComplete(TArray<FImaginaryFiBDataSharedPtr>& InRawDataList);

	/** 
	 * Gathers all Blueprint Variable references from the Target Class
	 * 
	 * @param InTargetClass Class to gather variables from
	 * @param bForTarget    TRUE if we need to check that the type is the same as the source and recurse parent classes, FALSE if picking a new source
	 */
	void GatherAllAvailableBlueprintVariables(UClass* InTargetClass, bool bForTarget);

	/** Returns the display text for the target reference */
	FText GetTargetDisplayText() const;

	/** Returns the icon for the target reference */
	const struct FSlateBrush* GetTargetIcon() const;

	/** Returns the icon color for the target reference */
	FSlateColor GetTargetIconColor() const;
	
	/** Returns the icon for the target reference */
	const struct FSlateBrush* GetSecondaryTargetIcon() const;

	/** Returns the icon color for the target reference */
	FSlateColor GetSecondaryTargetIconColor() const;

	/** Returns the display text for the source reference */
	FText GetSourceDisplayText() const;

	/** Returns the icon for the source reference */
	const FSlateBrush* GetSourceReferenceIcon() const;

	/** Returns the icon color for the source reference */
	FSlateColor GetSourceReferenceIconColor() const;
	
	/** Returns the secondary icon for the source reference */
	const FSlateBrush* GetSecondarySourceReferenceIcon() const;

	/** Returns the secondary icon color for the source reference */
	FSlateColor GetSecondarySourceReferenceIconColor() const;

	/** Returns the text for the Find All button */
	FText GetFindAllButtonText() const;

	/** Returns the text for the Find and Replace All button */
	FText GetFindAndReplaceAllButtonText() const;

	/** Returns tool tip text for the "Find All" and "Find And Replace All" buttons */
	FText GetFindAndReplaceToolTipText(bool bFindAndReplace) const;

	/** Returns whether or not a search can be initiated right now */
	bool CanBeginSearch(bool bFindAndReplace) const;

	/** Callback for when the "Only Local Results" CheckBox is changed */
	void OnLocalCheckBoxChanged(ECheckBoxState Checked);

	/** Returns the current state of the "Only Local Results" CheckBox */
	ECheckBoxState GetLocalCheckBoxState() const;

	/** Returns the label text for the "OnlyLocalResults" CheckBox */
	FText GetLocalCheckBoxLabelText() const;

	/** Callback for when the "Show When Finished" CheckBox is changed */
	void OnShowReplacementsCheckBoxChanged(ECheckBoxState Checked);

	/** Returns the current state of the "Show When Finished" CheckBox */
	ECheckBoxState GetShowReplacementsCheckBoxState() const;

	/** Returns the label text for the "Show When Finished" CheckBox */
	FText GetShowReplacementsCheckBoxLabelText() const;

	/** Returns the text to display in the bottom right corner of the window */
	FText GetStatusText() const;

	/** Determines whether a search is actively in progress */
	bool IsSearchInProgress() const;

	/**
	 * Builds a title for the transaction of Replacing references 
	 * 
	 * @param TargetReference  The Target variable that will be replacing the source
	 */
	FText GetTransactionTitle(const FMemberReference& TargetReference) const;

protected:
	/** Combo box for selecting the target reference */
	TSharedPtr< SComboButton > TargetReferencesComboBox;

	/** Tree view for displaying available target references */
	TSharedPtr< SReplaceReferencesTreeViewType > AvailableTargetReferencesTreeView;

	/** List of items used for the root of the target picker tree */
	TArray< FTreeViewItem > PossibleTargetVariableList;

	/** Combo box for selecting the source reference */
	TSharedPtr< SComboButton > SourceReferencesComboBox;

	/** Tree view for displaying available source references */
	TSharedPtr< SReplaceReferencesTreeViewType > AvailableSourceReferencesTreeView;

	/** List of items used for the root of the source picker tree */
	TArray< FTreeViewItem > PossibleSourceVariableList;

	/** Target SKEL_ class that is being referenced by this window */
	UClass* TargetClass;

	/** Blueprint editor that owns this window */
	TWeakPtr< FBlueprintEditor > BlueprintEditor;

	/** Cached SourcePinType for the property the user wants to replace */
	FEdGraphPinType SourcePinType;

	/** Cached SourceProperty that the user wants to replace */
	FProperty* SourceProperty;

	/** Find-in-Blueprints window used for making search queries and presenting results to the user */
	TSharedPtr< SFindInBlueprints > FindInBlueprints;

	/** Currently selected target reference */
	FTreeViewItem SelectedTargetReferenceItem;

	/** Whether to search only within the currently open blueprint */
	bool bFindWithinBlueprint;

	/** Whether to show a FindResults tab for the replacements after completing action */
	bool bShowReplacementsWhenFinished;
};

/** List item for the Confirmation Dialog */
class FReplaceConfirmationListItem
{
public:
	FReplaceConfirmationListItem(const UBlueprint* InBlueprint) : Blueprint(InBlueprint), bReplace(true) {}

	/** Gets the blueprint this Item refers to */
	const UBlueprint* GetBlueprint() const { return Blueprint; }

	/** Gets whether the references in this blueprint will be replace when the modal is closed */
	bool ShouldReplace() const { return bReplace; }

	/** Sets whether the references in this blueprint will be replace when the modal is closed */
	void SetShouldReplace(bool bInReplace) { bReplace = bInReplace; }

	/** Sets whether the references in this blueprint will be replace when the modal is closed */
	TSharedRef<SWidget> CreateWidget();
private:
	/** Callback for Checkbox status changed */
	void OnCheckStateChanged(ECheckBoxState State);

	/** Callback for getting Checkbox State */
	ECheckBoxState IsChecked() const;

private:
	/** The Blueprint this item represents*/
	const UBlueprint* Blueprint;

	/** Whether or not to replace references in this blueprint */
	bool bReplace;
};

/** Widget for the ReplaceNodeReferences Confirmation Dialog */
class SReplaceReferencesConfirmation : public SCompoundWidget
{
private:
	typedef TSharedPtr<FReplaceConfirmationListItem> FListViewItem;
public:
	enum class EDialogResponse { Confirm, Cancel };

	SLATE_BEGIN_ARGS(SReplaceReferencesConfirmation)
		: _FindResults(nullptr)
		{}
		
		SLATE_ARGUMENT(TArray< FImaginaryFiBDataSharedPtr >*, FindResults)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** 
	 * Creates a Confirmation Modal, this function will not return until the Dialog is closed 
	 *
	 * @param InFindResults The Results to filter with this dialog
	 * @return A constructed Confirmation Widget
	 */
	static EDialogResponse CreateModal(TArray< FImaginaryFiBDataSharedPtr >* InFindResults);

private:
	/** Generates a row for a List Item */
	TSharedRef<ITableRow> OnGenerateRow(FListViewItem Item, const TSharedRef<STableViewBase>& OwnerTable) const;

	/** Closes the window with the given response */
	FReply CloseWindow(EDialogResponse InResponse);
private:
	/** The list of unique blueprints that are affected */
	TArray< FListViewItem > AffectedBlueprints;

	/** The find results to modify */
	TArray< FImaginaryFiBDataSharedPtr >* RawFindData;

	/** The User's response */
	EDialogResponse Response;

	/** Window to close when dialog completed */
	TSharedPtr<SWindow> MyWindow;
};
