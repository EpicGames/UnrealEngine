// Copyright Epic Games, Inc. All Rights Reserved.


#include "SSCSEditor.h"

#include "ActorEditorUtils.h"
#include "AddToProjectConfig.h"
#include "Algo/Find.h"
#include "AssetRegistry/AssetData.h"
#include "AssetSelection.h"
#include "BPVariableDragDropAction.h"
#include "BlueprintEditor.h"
#include "BlueprintEditorModule.h"
#include "BlueprintEditorSettings.h"
#include "ClassIconFinder.h"
#include "ClassViewerFilter.h"
#include "ComponentAssetBroker.h"
#include "ComponentInstanceDataCache.h"
#include "ComponentVisualizerManager.h"
#include "Components/ChildActorComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Containers/EnumAsByte.h"
#include "CoreGlobals.h"
#include "CreateBlueprintFromActorDialog.h"
#include "Dialogs/Dialogs.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "EdGraph/EdGraph.h"
#include "EdGraphSchema_K2.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "Engine/InheritableComponentHandler.h"
#include "Engine/MemberReference.h"
#include "Engine/SCS_Node.h"
#include "Engine/SimpleConstructionScript.h"
#include "Engine/World.h"
#include "FeaturedClasses.inl"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GameProjectGenerationModule.h"
#include "GraphEditorActions.h"
#include "IDocumentation.h"
#include "ISCSEditorUICustomization.h"
#include "Input/DragAndDrop.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_ComponentBoundEvent.h"
#include "K2Node_Variable.h"
#include "K2Node_VariableGet.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/ChildActorComponentEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Layout/Children.h"
#include "Layout/Geometry.h"
#include "Layout/Margin.h"
#include "Layout/WidgetPath.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Logging/MessageLog.h"
#include "Math/Color.h"
#include "Math/NumericLimits.h"
#include "Math/Rotator.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Misc/CString.h"
#include "Misc/FeedbackContext.h"
#include "Misc/Guid.h"
#include "ObjectTools.h"
#include "PropertyPath.h"
#include "SPositiveActionButton.h"
#include "ScopedTransaction.h"
#include "Selection.h"
#include "Settings/EditorStyleSettings.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateIconFinder.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Subsystems/PanelExtensionSubsystem.h"
#include "Templates/Function.h"
#include "Textures/SlateIcon.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "ToolMenu.h"
#include "ToolMenuContext.h"
#include "ToolMenuDelegates.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "Toolkits/ToolkitManager.h"
#include "Trace/Detail/Channel.h"
#include "TutorialMetaData.h"
#include "Types/ISlateMetaData.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectHash.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakFieldPtr.h"
#include "UnrealEdGlobals.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Views/SHeaderRow.h"

class FClassViewerInitializationOptions;
class FExtender;
class ITableRow;
class IToolkit;
class SWidget;
class SWindow;
class UEdGraphNode;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "SSCSEditor"

//////////////////////////////////////////////////////////////////////////
// FSCSEditorTreeNode

FSCSEditorTreeNode::FSCSEditorTreeNode(FSCSEditorTreeNode::ENodeType InNodeType)
	: NodeType(InNodeType)
	, FilterFlags((uint8)EFilteredState::Unknown)
{
}

FName FSCSEditorTreeNode::GetNodeID() const
{
	FName ItemName = GetVariableName();
	if (ItemName == NAME_None)
	{
		UActorComponent* ComponentTemplateOrInstance = GetComponentTemplate();
		if (ComponentTemplateOrInstance != nullptr)
		{
			ItemName = ComponentTemplateOrInstance->GetFName();
		}
	}
	return ItemName;
}

FName FSCSEditorTreeNode::GetVariableName() const
{
	return NAME_None;
}

FString FSCSEditorTreeNode::GetDisplayString() const
{
	return TEXT("GetDisplayString not overridden");
}

FText FSCSEditorTreeNode::GetDisplayName() const
{
	return LOCTEXT("GetDisplayNameNotOverridden", "GetDisplayName not overridden");
}

class USCS_Node* FSCSEditorTreeNode::GetSCSNode() const
{
	return nullptr;
}

FSCSEditorTreeNode::ENodeType FSCSEditorTreeNode::GetNodeType() const
{
	return NodeType;
}

bool FSCSEditorTreeNode::IsAttachedTo(FSCSEditorTreeNodePtrType InNodePtr) const
{ 
	FSCSEditorTreeNodePtrType TestParentPtr = ParentNodePtr;
	while(TestParentPtr.IsValid())
	{
		if(TestParentPtr == InNodePtr)
		{
			return true;
		}

		TestParentPtr = TestParentPtr->ParentNodePtr;
	}

	return false; 
}

bool FSCSEditorTreeNode::MatchesFilterType(const UClass* InFilterType) const
{
	// All nodes will pass the type filter by default.
	return true;
}

bool FSCSEditorTreeNode::RefreshFilteredState(const UClass* InFilterType, const TArray<FString>& InFilterTerms, bool bRecursive)
{
	bool bHasAnyVisibleChildren = false;
	if (bRecursive)
	{
		for (FSCSEditorTreeNodePtrType Child : GetChildren())
		{
			bHasAnyVisibleChildren |= Child->RefreshFilteredState(InFilterType, InFilterTerms, bRecursive);
		}
	}

	// Don't check a root actor node - it doesn't have a valid variable name. Let it recache based on children and hide itself based on their filter states.
	if (GetNodeType() == FSCSEditorTreeNode::RootActorNode)
	{
		SetCachedFilterState(bHasAnyVisibleChildren, /*bUpdateParent =*/!bRecursive);
		return bHasAnyVisibleChildren;
	}

	bool bIsFilteredOut = InFilterType && !MatchesFilterType(InFilterType);
	if (!bIsFilteredOut && 
		GetNodeType() != FSCSEditorTreeNode::SeparatorNode)
	{
		FString DisplayStr = GetDisplayString();
		for (const FString& FilterTerm : InFilterTerms)
		{
			if (!DisplayStr.Contains(FilterTerm))
			{
				bIsFilteredOut = true;
			}
		}
	}

	// if we're not recursing, then assume this is for a new node and we need to update the parent
	// otherwise, assume the parent was hit as part of the recursion
	const bool bUpdateParent = !bRecursive;
	SetCachedFilterState(!bIsFilteredOut, bUpdateParent);
	return !bIsFilteredOut;
}

void FSCSEditorTreeNode::SetCachedFilterState(bool bMatchesFilter, bool bUpdateParent)
{
	bool bFlagsChanged = false;
	if ((FilterFlags & EFilteredState::Unknown) == EFilteredState::Unknown)
	{
		FilterFlags   = 0x00;
		bFlagsChanged = true;
	}

	if (bMatchesFilter)
	{
		bFlagsChanged |= (FilterFlags & EFilteredState::MatchesFilter) == 0;
		FilterFlags |= EFilteredState::MatchesFilter;
	}
	else
	{
		bFlagsChanged |= (FilterFlags & EFilteredState::MatchesFilter) != 0;
		FilterFlags &= ~EFilteredState::MatchesFilter;
	}

	const bool bHadChildMatch = (FilterFlags & EFilteredState::ChildMatches) != 0;
	// refresh the cached child state (don't update the parent, we'll do that below if it's needed)
	RefreshCachedChildFilterState(/*bUpdateParent =*/false);

	bFlagsChanged |= bHadChildMatch != ((FilterFlags & EFilteredState::ChildMatches) != 0);
	if (bUpdateParent && bFlagsChanged)
	{
		ApplyFilteredStateToParent();
	}
}

void FSCSEditorTreeNode::RefreshCachedChildFilterState(bool bUpdateParent)
{
	const bool bContainedMatch = !IsFlaggedForFiltration();

	FilterFlags &= ~EFilteredState::ChildMatches;
	for (FSCSEditorTreeNodePtrType Child : Children)
	{
		// Separator nodes should not contribute to child matches for the parent nodes
		if (Child->GetNodeType() == FSCSEditorTreeNode::SeparatorNode)
		{
			continue;
		}
		
		if (!Child->IsFlaggedForFiltration())
		{
			FilterFlags |= EFilteredState::ChildMatches;
			break;
		}
	}
	const bool bContainsMatch = !IsFlaggedForFiltration();

	const bool bStateChange = bContainedMatch != bContainsMatch;
	if (bUpdateParent && bStateChange)
	{
		ApplyFilteredStateToParent();
	}
}

void FSCSEditorTreeNode::ApplyFilteredStateToParent()
{
	FSCSEditorTreeNode* Child = this;
	while (Child->ParentNodePtr.IsValid())
	{
		FSCSEditorTreeNode* Parent = Child->ParentNodePtr.Get();

		if ( !IsFlaggedForFiltration() )
		{
			if ((Parent->FilterFlags & EFilteredState::ChildMatches) == 0)
			{
				Parent->FilterFlags |= EFilteredState::ChildMatches;
			}
			else
			{
				// all parents from here on up should have the flag
				break;
			}
		}
		// have to see if this was the only child contributing to this flag
		else if (Parent->FilterFlags & EFilteredState::ChildMatches)
		{
			Parent->FilterFlags &= ~EFilteredState::ChildMatches;
			for (const FSCSEditorTreeNodePtrType& Sibling : Parent->Children)
			{
				if (Sibling.Get() == Child)
				{
					continue;
				}

				if (Sibling->FilterFlags & EFilteredState::FilteredInMask)
				{
					Parent->FilterFlags |= EFilteredState::ChildMatches;
					break;
				}
			}

			if (Parent->FilterFlags & EFilteredState::ChildMatches)
			{
				// another child added the flag back
				break;
			}
		}
		Child = Parent;
	}
}

FSCSEditorTreeNodePtrType FSCSEditorTreeNode::FindClosestParent(TArray<FSCSEditorTreeNodePtrType> InNodes)
{
	uint32 MinDepth = MAX_uint32;
	FSCSEditorTreeNodePtrType ClosestParentNodePtr;

	for(int32 i = 0; i < InNodes.Num() && MinDepth > 1; ++i)
	{
		if(InNodes[i].IsValid())
		{
			uint32 CurDepth = 0;
			if(InNodes[i]->FindChild(GetComponentTemplate(), true, &CurDepth).IsValid())
			{
				if(CurDepth < MinDepth)
				{
					MinDepth = CurDepth;
					ClosestParentNodePtr = InNodes[i];
				}
			}
		}
	}

	return ClosestParentNodePtr;
}

void FSCSEditorTreeNode::SetActorRootNode(FSCSEditorActorNodePtrType InActorNodePtr)
{
	ActorRootNodePtr = InActorNodePtr;

	for (FSCSEditorTreeNodePtrType& ChildPtr : Children)
	{
		if (ChildPtr.IsValid())
		{
			ChildPtr->SetActorRootNode(InActorNodePtr);
		}
	}
}

void FSCSEditorTreeNode::AddChild(FSCSEditorTreeNodePtrType InChildNodePtr)
{
	// Ensure the node is not already parented elsewhere
	if(InChildNodePtr->GetParent().IsValid())
	{
		InChildNodePtr->GetParent()->RemoveChild(InChildNodePtr);
	}

	// If this is an actor node, start a new subtree
	if (IsActorNode())
	{
		ActorRootNodePtr = StaticCastSharedRef<FSCSEditorTreeNodeActorBase>(AsShared());
	}

	// Link the child node to the actor root node for this subtree
	InChildNodePtr->SetActorRootNode(ActorRootNodePtr);

	// Add the given node as a child and link its parent
	Children.AddUnique(InChildNodePtr);
	InChildNodePtr->ParentNodePtr = AsShared();

	if (InChildNodePtr->FilterFlags != EFilteredState::Unknown && !InChildNodePtr->IsFlaggedForFiltration())
	{
		FSCSEditorTreeNodePtrType AncestorPtr = InChildNodePtr->ParentNodePtr;
		while (AncestorPtr.IsValid() && (AncestorPtr->FilterFlags & EFilteredState::ChildMatches) == 0)
		{
			AncestorPtr->FilterFlags |= EFilteredState::ChildMatches;
			AncestorPtr = AncestorPtr->GetParent();
		}
	}

	if (InChildNodePtr->IsComponentNode())
	{
		USCS_Node* SCS_Node = GetSCSNode();
		const UActorComponent* ComponentTemplate = GetObject<UActorComponent>();

		// Add a child node to the SCS tree node if not already present
		USCS_Node* SCS_ChildNode = InChildNodePtr->GetSCSNode();
		if (SCS_ChildNode != NULL)
		{
			// Get the SCS instance that owns the child node
			USimpleConstructionScript* SCS = SCS_ChildNode->GetSCS();
			if (SCS != NULL)
			{
				// If the parent is also a valid SCS node
				if (SCS_Node != NULL)
				{
					// If the parent and child are both owned by the same SCS instance
					if (SCS_Node->GetSCS() == SCS)
					{
						// Add the child into the parent's list of children
						if (!SCS_Node->GetChildNodes().Contains(SCS_ChildNode))
						{
							SCS_Node->AddChildNode(SCS_ChildNode);
						}
					}
					else
					{
						// Adds the child to the SCS root set if not already present
						SCS->AddNode(SCS_ChildNode);

						// Set parameters to parent this node to the "inherited" SCS node
						SCS_ChildNode->SetParent(SCS_Node);
					}
				}
				else if (ComponentTemplate != NULL)
				{
					// Adds the child to the SCS root set if not already present
					SCS->AddNode(SCS_ChildNode);

					// Set parameters to parent this node to the native component template
					SCS_ChildNode->SetParent(Cast<const USceneComponent>(ComponentTemplate));
				}
				else
				{
					// Adds the child to the SCS root set if not already present
					SCS->AddNode(SCS_ChildNode);
				}
			}
		}
		else if (IsInstancedComponent())
		{
			USceneComponent* ChildInstance = Cast<USceneComponent>(InChildNodePtr->GetComponentTemplate());
			if (ensure(ChildInstance != nullptr))
			{
				USceneComponent* ParentInstance = Cast<USceneComponent>(GetComponentTemplate());
				if (ensure(ParentInstance != nullptr))
				{
					// Handle attachment at the instance level
					if (ChildInstance->GetAttachParent() != ParentInstance)
					{
						AActor* Owner = ParentInstance->GetOwner();
						if (Owner->GetRootComponent() == ChildInstance)
						{
							Owner->SetRootComponent(ParentInstance);
						}
						ChildInstance->AttachToComponent(ParentInstance, FAttachmentTransformRules::KeepWorldTransform);
					}
				}
			}
		}
	}
}

FSCSEditorTreeNodePtrType FSCSEditorTreeNode::AddChild(USCS_Node* InSCSNode, bool bInIsInherited)
{
	// Ensure that the given SCS node is valid
	check(InSCSNode != NULL);

	// If it doesn't already exist as a child node
	FSCSEditorTreeNodePtrType ChildNodePtr = FindChild(InSCSNode);
	if(!ChildNodePtr.IsValid())
	{
		// Add a child node to the SCS editor tree
		ChildNodePtr = MakeShareable(new FSCSEditorTreeNodeComponent(InSCSNode, bInIsInherited));
		AddChild(ChildNodePtr);
	}

	return ChildNodePtr;
}

FSCSEditorTreeNodePtrType FSCSEditorTreeNode::AddChildFromComponent(UActorComponent* InComponentTemplate)
{
	// Ensure that the given component template is valid
	check(InComponentTemplate != NULL);

	// If it doesn't already exist in the SCS editor tree
	FSCSEditorTreeNodePtrType ChildNodePtr = FindChild(InComponentTemplate);
	if(!ChildNodePtr.IsValid())
	{
		// Add a child node to the SCS editor tree
		ChildNodePtr = FactoryNodeFromComponent(InComponentTemplate);
		AddChild(ChildNodePtr);
	}

	return ChildNodePtr;
}

// Tries to find a SCS node that was likely responsible for creating the specified instance component.  Note: This is not always possible to do!
USCS_Node* FSCSEditorTreeNode::FindSCSNodeForInstance(const UActorComponent* InstanceComponent, UClass* ClassToSearch)
{ 
	if ((ClassToSearch != nullptr) && InstanceComponent->IsCreatedByConstructionScript())
	{
		for (UClass* TestClass = ClassToSearch; TestClass->ClassGeneratedBy != nullptr; TestClass = TestClass->GetSuperClass())
		{
			if (UBlueprint* TestBP = Cast<UBlueprint>(TestClass->ClassGeneratedBy))
			{
				if (TestBP->SimpleConstructionScript != nullptr)
				{
					if (USCS_Node* Result = TestBP->SimpleConstructionScript->FindSCSNode(InstanceComponent->GetFName()))
					{
						return Result;
					}
				}
			}
		}
	}

	return nullptr;
}

FSCSEditorTreeNodePtrType FSCSEditorTreeNode::FactoryNodeFromComponent(UActorComponent* InComponent)
{
	check(InComponent);

	bool bComponentIsInAnInstance = false;

	AActor* Owner = InComponent->GetOwner();
	if ((Owner != nullptr) && !Owner->HasAnyFlags(RF_ClassDefaultObject|RF_ArchetypeObject))
	{
		bComponentIsInAnInstance = true;
	}

	if (bComponentIsInAnInstance)
	{
		if (InComponent->CreationMethod == EComponentCreationMethod::Instance)
		{
			return MakeShareable(new FSCSEditorTreeNodeInstanceAddedComponent(Owner, InComponent));
		}
		else
		{
			return MakeShareable(new FSCSEditorTreeNodeInstancedInheritedComponent(Owner, InComponent));
		}
	}

	// Not an instanced component, either an SCS node or a native component in BP edit mode
	return MakeShareable(new FSCSEditorTreeNodeComponent(InComponent));
}

FSlateColor FSCSEditorTreeNode::GetColorTintForIcon(FSCSEditorTreeNodePtrType InNode)
{
	const FLinearColor InheritedBlueprintComponentColor(0.08f, 0.35f, 0.6f);
	const FLinearColor InstancedInheritedBlueprintComponentColor(0.08f, 0.35f, 0.6f);
	const FLinearColor InheritedNativeComponentColor(0.7f, 0.9f, 0.7f);
	const FLinearColor IntroducedHereColor(FLinearColor::White);

	if (InNode->IsInheritedComponent())
	{
		if (InNode->IsNativeComponent())
		{
			return InheritedNativeComponentColor;
		}
		else if (InNode->IsInstancedComponent())
		{
			return InstancedInheritedBlueprintComponentColor;
		}
		else
		{
			return InheritedBlueprintComponentColor;
		}
	}
	else
	{
		return IntroducedHereColor;
	}
}

void FSCSEditorTreeNode::CloseOngoingCreateTransaction()
{
	OngoingCreateTransaction.Reset();
}

FSCSEditorTreeNodePtrType FSCSEditorTreeNode::FindChild(const USCS_Node* InSCSNode, bool bRecursiveSearch, uint32* OutDepth) const
{
	FSCSEditorTreeNodePtrType Result;

	// Ensure that the given SCS node is valid
	if(InSCSNode != NULL)
	{
		// Look for a match in our set of child nodes
		for(int32 ChildIndex = 0; ChildIndex < Children.Num() && !Result.IsValid(); ++ChildIndex)
		{
			if(InSCSNode == Children[ChildIndex]->GetSCSNode())
			{
				Result = Children[ChildIndex];
			}
			else if(bRecursiveSearch)
			{
				Result = Children[ChildIndex]->FindChild(InSCSNode, true, OutDepth);
			}
		}
	}

	if(OutDepth && Result.IsValid())
	{
		*OutDepth += 1;
	}

	return Result;
}

FSCSEditorTreeNodePtrType FSCSEditorTreeNode::FindChild(const UActorComponent* InComponentTemplate, bool bRecursiveSearch, uint32* OutDepth) const
{
	FSCSEditorTreeNodePtrType Result;

	// Ensure that the given component template is valid
	if(InComponentTemplate != NULL)
	{
		// Look for a match in our set of child nodes
		for(int32 ChildIndex = 0; ChildIndex < Children.Num() && !Result.IsValid(); ++ChildIndex)
		{
			if(InComponentTemplate == Children[ChildIndex]->GetComponentTemplate())
			{
				Result = Children[ChildIndex];
			}
			else if(bRecursiveSearch)
			{
				Result = Children[ChildIndex]->FindChild(InComponentTemplate, true, OutDepth);
			}
		}
	}

	if(OutDepth && Result.IsValid())
	{
		*OutDepth += 1;
	}

	return Result;
}

FSCSEditorTreeNodePtrType FSCSEditorTreeNode::FindChild(const FName& InVariableOrInstanceName, bool bRecursiveSearch, uint32* OutDepth) const
{
	FSCSEditorTreeNodePtrType Result;

	// Ensure that the given name is valid
	if(InVariableOrInstanceName != NAME_None)
	{
		// Look for a match in our set of child nodes
		for(int32 ChildIndex = 0; ChildIndex < Children.Num() && !Result.IsValid(); ++ChildIndex)
		{
			FName ItemName = Children[ChildIndex]->GetVariableName();
			if(ItemName == NAME_None && Children[ChildIndex]->GetNodeType() == ComponentNode)
			{
				UActorComponent* ComponentTemplateOrInstance = Children[ChildIndex]->GetComponentTemplate();
				check(ComponentTemplateOrInstance != nullptr);
				ItemName = ComponentTemplateOrInstance->GetFName();
			}

			if(InVariableOrInstanceName == ItemName)
			{
				Result = Children[ChildIndex];
			}
			else if(bRecursiveSearch)
			{
				Result = Children[ChildIndex]->FindChild(InVariableOrInstanceName, true, OutDepth);
			}
		}
	}

	if(OutDepth && Result.IsValid())
	{
		*OutDepth += 1;
	}

	return Result;
}

void FSCSEditorTreeNode::RemoveChild(FSCSEditorTreeNodePtrType InChildNodePtr)
{
	// Remove the given node as a child and reset its parent link
	Children.Remove(InChildNodePtr);
	InChildNodePtr->ParentNodePtr.Reset();
	InChildNodePtr->RemoveMeAsChild();

	// Reset its actor root link
	InChildNodePtr->ActorRootNodePtr.Reset();

	if (InChildNodePtr->IsFlaggedForFiltration())
	{
		RefreshCachedChildFilterState(/*bUpdateParent =*/true);
	}
}

void FSCSEditorTreeNode::OnRequestRename(TUniquePtr<FScopedTransaction> InOngoingCreateTransaction)
{
	OngoingCreateTransaction = MoveTemp(InOngoingCreateTransaction); // Take responsibility to end the 'create + give initial name' transaction.
	RenameRequestedDelegate.ExecuteIfBound();
}

void FSCSEditorTreeNode::OnCompleteRename(const FText& InNewName)
{
	// If a 'create + give initial name' transaction exists, end it, the object is expected to have its initial name.
	CloseOngoingCreateTransaction();
}

UObject* FSCSEditorTreeNode::GetOrCreateEditableObjectForBlueprint(UBlueprint* InBlueprint) const
{
	if (CanEdit())
	{
		return WeakObjectPtr.Get();
	}

	return nullptr;
}

//////////////////////////////////////////////////////////////////////////
// FSCSEditorTreeNodeComponentBase

FName FSCSEditorTreeNodeComponentBase::GetVariableName() const
{
	FName VariableName = NAME_None;

	USCS_Node* SCS_Node = GetSCSNode();
	UActorComponent* ComponentTemplate = GetComponentTemplate();

	if (IsInstancedComponent() && (SCS_Node == nullptr) && (ComponentTemplate != nullptr))
	{
		if (ComponentTemplate->GetOwner())
		{
			SCS_Node = FindSCSNodeForInstance(ComponentTemplate, ComponentTemplate->GetOwner()->GetClass());
		}
	}

	if (SCS_Node != NULL)
	{
		// Use the same variable name as is obtained by the compiler
		VariableName = SCS_Node->GetVariableName();
	}
	else if (ComponentTemplate != NULL)
	{
		// Try to find the component anchor variable name (first looks for an exact match then scans for any matching variable that points to the archetype in the CDO)
		VariableName = FComponentEditorUtils::FindVariableNameGivenComponentInstance(ComponentTemplate);
	}

	return VariableName;
}

FString FSCSEditorTreeNodeComponentBase::GetDisplayString() const
{
	FName VariableName = GetVariableName();
	UActorComponent* ComponentTemplate = GetComponentTemplate();

	UBlueprint* Blueprint = GetBlueprint();
	UClass* VariableOwner = (Blueprint != nullptr) ? *Blueprint->SkeletonGeneratedClass : nullptr;
	FProperty* VariableProperty = FindFProperty<FProperty>(VariableOwner, VariableName);

	bool const bHasValidVarName = (VariableName != NAME_None);
	bool const bIsArrayVariable = bHasValidVarName && (VariableOwner != nullptr) && 
		VariableProperty && VariableProperty->IsA<FArrayProperty>();

	// Only display SCS node variable names in the tree if they have not been autogenerated
	if (bHasValidVarName && !bIsArrayVariable)
	{
		if (IsNativeComponent() && GetDefault<UEditorStyleSettings>()->bShowNativeComponentNames)
		{
			FStringFormatNamedArguments Args;
			Args.Add(TEXT("VarName"), VariableProperty && VariableProperty->IsNative() ? VariableProperty->GetDisplayNameText().ToString() : VariableName.ToString());
			Args.Add(TEXT("CompName"), ComponentTemplate->GetName());
			return FString::Format(TEXT("{VarName} ({CompName})"), Args);
		}
		else
		{
			return VariableName.ToString();
		}
	}
	else if ( ComponentTemplate != nullptr )
	{
		return ComponentTemplate->GetFName().ToString();
	}
	else
	{
		FString UnnamedString = LOCTEXT("UnnamedToolTip", "Unnamed").ToString();
		FString NativeString = IsNativeComponent() ? LOCTEXT("NativeToolTip", "Native ").ToString() : TEXT("");

		if (ComponentTemplate != NULL)
		{
			return FString::Printf(TEXT("[%s %s%s]"), *UnnamedString, *NativeString, *ComponentTemplate->GetClass()->GetName());
		}
		else
		{
			return FString::Printf(TEXT("[%s %s]"), *UnnamedString, *NativeString);
		}
	}
}

bool FSCSEditorTreeNodeComponentBase::CanReparent() const
{
	if (FChildActorComponentEditorUtils::IsChildActorSubtreeNode(AsShared()))
	{
		// Cannot reparent nodes within a child actor node subtree.
		return false;
	}

	return !IsInheritedComponent() && !IsDefaultSceneRoot() && IsSceneComponent();
}

UBlueprint* FSCSEditorTreeNodeComponentBase::GetBlueprint() const
{
	if (const USCS_Node* SCS_Node = GetSCSNode())
	{
		if (const USimpleConstructionScript* SCS = SCS_Node->GetSCS())
		{
			return SCS->GetBlueprint();
		}
	}
	else if (const UActorComponent* ActorComponent = GetObject<UActorComponent>())
	{
		if (const AActor* Actor = ActorComponent->GetOwner())
		{
			return UBlueprint::GetBlueprintFromClass(Actor->GetClass());
		}
	}
	
	return nullptr;
}

FSCSEditorChildActorNodePtrType FSCSEditorTreeNodeComponentBase::GetChildActorNode()
{
	if (!ChildActorNodePtr.IsValid() || ChildActorNodePtr->GetChildActorComponent() != GetObject<UActorComponent>())
	{
		if (const UChildActorComponent* ChildActorComponent = GetObject<UChildActorComponent>())
		{
			AActor* ChildActor = IsInstanced() ? ChildActorComponent->GetChildActor() : ChildActorComponent->GetChildActorTemplate();
			ChildActorNodePtr = MakeShareable(new FSCSEditorTreeNodeChildActor(ChildActor));

			check(ChildActorNodePtr.IsValid());
			ChildActorNodePtr->SetOwnerNode(this->AsShared());
		}
	}

	return ChildActorNodePtr;
}

bool FSCSEditorTreeNodeComponentBase::MatchesFilterType(const UClass* InFilterType) const
{
	check(InFilterType);

	if (const UActorComponent* ComponentObject = GetObject<UActorComponent>())
	{
		const UClass* ComponentClass = ComponentObject->GetClass();
		check(ComponentClass);

		if (ComponentClass->IsChildOf(InFilterType))
		{
			return true;
		}
	}

	return false;
}

//////////////////////////////////////////////////////////////////////////
// FSCSEditorTreeNodeInstancedInheritedComponent

FSCSEditorTreeNodeInstancedInheritedComponent::FSCSEditorTreeNodeInstancedInheritedComponent(AActor* Owner, UActorComponent* ComponentInstance)
{
	check(ComponentInstance != nullptr);

	InstancedComponentOwnerPtr = Owner;

	SetObject(ComponentInstance);
}

bool FSCSEditorTreeNodeInstancedInheritedComponent::IsNativeComponent() const
{
	if (UActorComponent* Template = GetComponentTemplate())
	{
		return Template->CreationMethod == EComponentCreationMethod::Native;
	}
	else
	{
		return false;
	}
}

bool FSCSEditorTreeNodeInstancedInheritedComponent::IsRootComponent() const
{
	UActorComponent* Template = GetComponentTemplate();

	if (AActor* OwnerActor = InstancedComponentOwnerPtr.Get())
	{
		if (OwnerActor->GetRootComponent() == Template)
		{
			return true;
		}
	}

	return false;
}

bool FSCSEditorTreeNodeInstancedInheritedComponent::IsInheritedSCSNode() const
{
	return false;
}

bool FSCSEditorTreeNodeInstancedInheritedComponent::IsDefaultSceneRoot() const
{
	return false;
}

bool FSCSEditorTreeNodeInstancedInheritedComponent::CanEdit() const
{
	UActorComponent* Template = GetComponentTemplate();
	return (Template ? Template->IsEditableWhenInherited() : false);
}

FText FSCSEditorTreeNodeInstancedInheritedComponent::GetDisplayName() const
{
	FName VariableName = GetVariableName();
	if (VariableName != NAME_None)
	{
		return FText::FromName(VariableName);
	}

	return FText::GetEmpty();
}

//////////////////////////////////////////////////////////////////////////
// FSCSEditorTreeNodeInstanceAddedComponent

FSCSEditorTreeNodeInstanceAddedComponent::FSCSEditorTreeNodeInstanceAddedComponent(AActor* Owner, UActorComponent* InComponentTemplate)
{
	check(InComponentTemplate);
	InstancedComponentName = InComponentTemplate->GetFName();
	InstancedComponentOwnerPtr = Owner;
	SetObject(InComponentTemplate);
}

bool FSCSEditorTreeNodeInstanceAddedComponent::IsRootComponent() const
{
	bool bIsRoot = true;
	UActorComponent* Template = GetComponentTemplate();

	if (Template != NULL)
	{
		AActor* CDO = Template->GetOwner();
		if (CDO != NULL)
		{
			// Evaluate to TRUE if we have a valid component reference that matches the native root component
			bIsRoot = (Template == CDO->GetRootComponent());
		}
	}

	return bIsRoot;
}

bool FSCSEditorTreeNodeInstanceAddedComponent::IsDefaultSceneRoot() const
{
	if (USceneComponent* SceneComponent = Cast<USceneComponent>(GetComponentTemplate()))
	{
		return SceneComponent->GetFName() == USceneComponent::GetDefaultSceneRootVariableName();
	}

	return false;
}

FString FSCSEditorTreeNodeInstanceAddedComponent::GetDisplayString() const
{
	return InstancedComponentName.ToString();
}

FText FSCSEditorTreeNodeInstanceAddedComponent::GetDisplayName() const
{
	return FText::FromName(InstancedComponentName);
}

void FSCSEditorTreeNodeInstanceAddedComponent::RemoveMeAsChild()
{
	USceneComponent* ChildInstance = Cast<USceneComponent>(GetComponentTemplate());
	if (ensure(ChildInstance))
	{
		// Handle detachment at the instance level
		ChildInstance->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);
	}
}

void FSCSEditorTreeNodeInstanceAddedComponent::OnCompleteRename(const FText& InNewName)
{
	// If the 'rename' was part of an ongoing component creation, ensure the transaction is ended when the local object goes out of scope. (Must complete after the rename transaction below)
	TUniquePtr<FScopedTransaction> ScopedCreateTransaction(MoveTemp(OngoingCreateTransaction));

	// If a 'create' transaction is opened, the rename will be folded into it and will be invisible to the 'undo' as create + give a name is really just one operation from the user point of view.
	FScopedTransaction TransactionContext(LOCTEXT("RenameComponentVariable", "Rename Component Variable"));

	UActorComponent* ComponentInstance = GetComponentTemplate();
	if(ComponentInstance == nullptr)
	{
		return;
	}

	ERenameFlags RenameFlags = REN_DontCreateRedirectors;
	
	// name collision could occur due to e.g. our archetype being updated and causing a conflict with our ComponentInstance:
	FString NewNameAsString = InNewName.ToString();
	if(StaticFindObject(UObject::StaticClass(), ComponentInstance->GetOuter(), *NewNameAsString) == nullptr)
	{
		ComponentInstance->Rename(*NewNameAsString, nullptr, RenameFlags);
		InstancedComponentName = *NewNameAsString;
	}
	else
	{
		UObject* Collision = StaticFindObject(UObject::StaticClass(), ComponentInstance->GetOuter(), *NewNameAsString);
		if(Collision != ComponentInstance)
		{
			// use whatever name the ComponentInstance currently has:
			InstancedComponentName = ComponentInstance->GetFName();
		}
	}
}


//////////////////////////////////////////////////////////////////////////
// FSCSEditorTreeNodeComponent

FSCSEditorTreeNodeComponent::FSCSEditorTreeNodeComponent(USCS_Node* InSCSNode, bool bInIsInheritedSCS)
	: bIsInheritedSCS(bInIsInheritedSCS)
	, SCSNodePtr(InSCSNode)
{
	SetObject(( InSCSNode != nullptr ) ? InSCSNode->ComponentTemplate : nullptr);
}

FSCSEditorTreeNodeComponent::FSCSEditorTreeNodeComponent(UActorComponent* InComponentTemplate)
	: bIsInheritedSCS(false)
	, SCSNodePtr(nullptr)
{
	check(InComponentTemplate != nullptr);

	SetObject(InComponentTemplate);
	AActor* Owner = InComponentTemplate->GetOwner();
	if (Owner != nullptr)
	{
		ensureMsgf(Owner->HasAnyFlags(RF_ClassDefaultObject|RF_ArchetypeObject), TEXT("Use a different node class for instanced components"));
	}
}

bool FSCSEditorTreeNodeComponent::IsNativeComponent() const
{
	return GetSCSNode() == NULL && GetComponentTemplate() != NULL;
}

bool FSCSEditorTreeNodeComponent::IsRootComponent() const
{
	bool bIsRoot = true;
	USCS_Node* SCS_Node = GetSCSNode();
	UActorComponent* ComponentTemplate = GetComponentTemplate();

	if (SCS_Node != NULL)
	{
		USimpleConstructionScript* SCS = SCS_Node->GetSCS();
		if (SCS != NULL)
		{
			// Evaluate to TRUE if we have an SCS node reference, it is contained in the SCS root set and does not have an external parent
			bIsRoot = SCS->GetRootNodes().Contains(SCS_Node) && SCS_Node->ParentComponentOrVariableName == NAME_None;
		}
	}
	else if (ComponentTemplate != NULL)
	{
		AActor* CDO = ComponentTemplate->GetOwner();
		if (CDO != NULL)
		{
			// Evaluate to TRUE if we have a valid component reference that matches the native root component
			bIsRoot = (ComponentTemplate == CDO->GetRootComponent());
		}
	}

	return bIsRoot;
}

bool FSCSEditorTreeNodeComponent::IsInheritedSCSNode() const
{
	return bIsInheritedSCS;
}

bool FSCSEditorTreeNodeComponent::IsDefaultSceneRoot() const
{
	if (USCS_Node* SCS_Node = GetSCSNode())
	{
		USimpleConstructionScript* SCS = SCS_Node->GetSCS();
		if (SCS != nullptr)
		{
			return SCS_Node == SCS->GetDefaultSceneRootNode();
		}
	}

	return false;
}

bool FSCSEditorTreeNodeComponent::CanEdit() const
{
	bool bCanEdit = false;

	if (!IsNativeComponent())
	{
		USCS_Node* SCS_Node = GetSCSNode();
		bCanEdit = (SCS_Node != nullptr);
	}
	else if (UActorComponent* ComponentTemplate = GetComponentTemplate())
	{
		bCanEdit = (FComponentEditorUtils::GetPropertyForEditableNativeComponent(ComponentTemplate) != nullptr);
	}

	return bCanEdit;
}

FText FSCSEditorTreeNodeComponent::GetDisplayName() const
{
	FName VariableName = GetVariableName();
	if (VariableName != NAME_None)
	{
		return FText::FromName(VariableName);
	}
	return FText::GetEmpty();
}

class USCS_Node* FSCSEditorTreeNodeComponent::GetSCSNode() const
{
	return SCSNodePtr.Get();
}

UObject* FSCSEditorTreeNodeComponent::GetOrCreateEditableObjectForBlueprint(UBlueprint* InBlueprint) const
{
	if (CanEdit() && !IsNativeComponent() && IsInheritedSCSNode())
	{
		return INTERNAL_GetOverridenComponentTemplate(InBlueprint);
	}

	return FSCSEditorTreeNode::GetOrCreateEditableObjectForBlueprint(InBlueprint);
}

UActorComponent* FSCSEditorTreeNode::FindComponentInstanceInActor(const AActor* InActor) const
{
	USCS_Node* SCS_Node = GetSCSNode();
	UActorComponent* ComponentTemplate = GetComponentTemplate();

	UActorComponent* ComponentInstance = NULL;
	if (InActor != NULL)
	{
		if (SCS_Node != NULL)
		{
			FName VariableName = SCS_Node->GetVariableName();
			if (VariableName != NAME_None)
			{
				UWorld* World = InActor->GetWorld();
				FObjectPropertyBase* Property = FindFProperty<FObjectPropertyBase>(InActor->GetClass(), VariableName);
				if (Property != NULL)
				{
					// Return the component instance that's stored in the property with the given variable name
					ComponentInstance = Cast<UActorComponent>(Property->GetObjectPropertyValue_InContainer(InActor));
				}
				else if (World != nullptr && World->WorldType == EWorldType::EditorPreview)
				{
					// If this is the preview actor, return the cached component instance that's being used for the preview actor prior to recompiling the Blueprint
					ComponentInstance = SCS_Node->EditorComponentInstance.Get();
				}
			}
		}
		else if (ComponentTemplate != NULL)
		{
			TInlineComponentArray<UActorComponent*> Components;
			InActor->GetComponents(Components);
			ComponentInstance = FComponentEditorUtils::FindMatchingComponent(ComponentTemplate, Components);
		}
	}

	return ComponentInstance;
}

void FSCSEditorTreeNodeComponent::OnCompleteRename(const FText& InNewName)
{
	// If the 'rename' was part of the creation process, we need to complete the creation transaction as the component has a user confirmed name. (Must complete after rename transaction below)
	TUniquePtr<FScopedTransaction> ScopedOngoingCreateTransaction(MoveTemp(OngoingCreateTransaction));

	// If a 'create' transaction is opened, the rename will be folded into it and will be invisible to the 'undo' as 'create + give initial name' means creating an object from the user point of view.
	FScopedTransaction RenameTransaction(LOCTEXT("RenameComponentVariable", "Rename Component Variable"));

	FBlueprintEditorUtils::RenameComponentMemberVariable(GetBlueprint(), GetSCSNode(), FName(*InNewName.ToString()));
}

void FSCSEditorTreeNodeComponent::RemoveMeAsChild()
{
	// Bypass removal logic if we're part of a child actor subtree
	if (FChildActorComponentEditorUtils::IsChildActorSubtreeNode(AsShared()))
	{
		return;
	}

	// Remove the SCS node from the SCS tree, if present
	if (USCS_Node* SCS_ChildNode = GetSCSNode())
	{
		USimpleConstructionScript* SCS = SCS_ChildNode->GetSCS();
		if (SCS != NULL)
		{
			SCS->RemoveNode(SCS_ChildNode);
		}
	}
}

UActorComponent* FSCSEditorTreeNodeComponent::INTERNAL_GetOverridenComponentTemplate(UBlueprint* Blueprint) const
{
	UActorComponent* OverriddenComponent = nullptr;

	FComponentKey Key(GetSCSNode());

	const bool bBlueprintCanOverrideComponentFromKey = Key.IsValid()
		&& Blueprint
		&& Blueprint->ParentClass
		&& Blueprint->ParentClass->IsChildOf(Key.GetComponentOwner());

	if (bBlueprintCanOverrideComponentFromKey)
	{
		const bool bCreateIfNecessary = true;
		UInheritableComponentHandler* InheritableComponentHandler = Blueprint->GetInheritableComponentHandler(bCreateIfNecessary);
		if (InheritableComponentHandler)
		{
			OverriddenComponent = InheritableComponentHandler->GetOverridenComponentTemplate(Key);
			if (!OverriddenComponent && bCreateIfNecessary)
			{
				OverriddenComponent = InheritableComponentHandler->CreateOverridenComponentTemplate(Key);
			}
		}
	}
	return OverriddenComponent;
}

//////////////////////////////////////////////////////////////////////////
// FSCSEditorTreeNodeActorBase

FSCSEditorTreeNodePtrType FSCSEditorTreeNodeActorBase::GetOwnerNode() const
{
	return OwnerNodePtr;
}

void FSCSEditorTreeNodeActorBase::SetOwnerNode(FSCSEditorTreeNodePtrType NewOwnerNode)
{
	OwnerNodePtr = NewOwnerNode;
}

FSCSEditorTreeNodePtrType FSCSEditorTreeNodeActorBase::GetSceneRootNode() const
{
	return SceneRootNodePtr;
}

void FSCSEditorTreeNodeActorBase::SetSceneRootNode(FSCSEditorTreeNodePtrType NewSceneRootNode)
{
	if (SceneRootNodePtr.IsValid())
	{
		ComponentNodes.Remove(SceneRootNodePtr);
	}

	SceneRootNodePtr = NewSceneRootNode;

	if (!ComponentNodes.Contains(SceneRootNodePtr))
	{
		ComponentNodes.Add(SceneRootNodePtr);
	}
}

const TArray<FSCSEditorTreeNodePtrType>& FSCSEditorTreeNodeActorBase::GetComponentNodes() const
{
	return ComponentNodes;
}

FName FSCSEditorTreeNodeActorBase::GetNodeID() const
{
	if (const AActor* Actor = GetObject<AActor>())
	{
		return Actor->GetFName();
	}
	return NAME_None;
}

bool FSCSEditorTreeNodeActorBase::IsInstanced() const
{
	if (const AActor* Actor = GetObject<AActor>())
	{
		return !Actor->IsTemplate();
	}

	return false;
}

void FSCSEditorTreeNodeActorBase::AddChild(FSCSEditorTreeNodePtrType InChildNodePtr)
{
	if (InChildNodePtr.IsValid() && InChildNodePtr->IsComponentNode())
	{
		ComponentNodes.Add(InChildNodePtr);
		if (!SceneRootNodePtr.IsValid() && InChildNodePtr->IsSceneComponent())
		{
			SetSceneRootNode(InChildNodePtr);
		}
	}

	Super::AddChild(InChildNodePtr);
}

void FSCSEditorTreeNodeActorBase::RemoveChild(FSCSEditorTreeNodePtrType InChildNodePtr)
{
	if (InChildNodePtr.IsValid() && InChildNodePtr->IsComponentNode())
	{
		ComponentNodes.Remove(InChildNodePtr);
		if (InChildNodePtr == SceneRootNodePtr)
		{
			SceneRootNodePtr.Reset();
		}
	}

	Super::RemoveChild(InChildNodePtr);
}

UBlueprint* FSCSEditorTreeNodeActorBase::GetBlueprint() const
{
	if (const AActor* Actor = GetObject<AActor>())
	{
		return UBlueprint::GetBlueprintFromClass(Actor->GetClass());
	}

	return nullptr;
}

//////////////////////////////////////////////////////////////////////////
// FSCSEditorTreeNodeRootActor

void FSCSEditorTreeNodeRootActor::OnCompleteRename(const FText& InNewName)
{
	if (AActor* Actor = GetMutableObject<AActor>())
	{
		if (Actor->IsActorLabelEditable() && !InNewName.ToString().Equals(Actor->GetActorLabel(), ESearchCase::CaseSensitive))
		{
			const FScopedTransaction Transaction(LOCTEXT("SCSEditorRenameActorTransaction", "Rename Actor"));
			FActorLabelUtilities::RenameExistingActor(Actor, InNewName.ToString());
		}
	}

	// Not expected to reach here with an ongoing create transaction, but if it does, end it.
	CloseOngoingCreateTransaction();
}

void FSCSEditorTreeNodeRootActor::AddChild(FSCSEditorTreeNodePtrType InChildNodePtr)
{
	if (InChildNodePtr.IsValid() && InChildNodePtr->IsComponentNode())
	{
		TSharedPtr<FSCSEditorTreeNodeSeparator> NewSeparatorNodePtr;
		const bool bIsSceneComponentNode = InChildNodePtr->IsSceneComponent();

		// Make sure separators are shown
		if (bIsSceneComponentNode && !SceneComponentSeparatorNodePtr.IsValid())
		{
			NewSeparatorNodePtr = MakeShareable(new FSCSEditorTreeNodeSeparator());

			SceneComponentSeparatorNodePtr = NewSeparatorNodePtr;
		}
		else if (!bIsSceneComponentNode && !NonSceneComponentSeparatorNodePtr.IsValid())
		{
			NewSeparatorNodePtr = MakeShareable(new FSCSEditorTreeNodeSeparator());
			NewSeparatorNodePtr->AddFilteredComponentType(USceneComponent::StaticClass());

			NonSceneComponentSeparatorNodePtr = NewSeparatorNodePtr;
		}

		if (NewSeparatorNodePtr.IsValid())
		{
			FSCSEditorTreeNodeActorBase::AddChild(NewSeparatorNodePtr);
			NewSeparatorNodePtr->RefreshFilteredState(CachedFilterType, CachedFilterTerms, false);
		}
	}

	FSCSEditorTreeNodeActorBase::AddChild(InChildNodePtr);
}

void FSCSEditorTreeNodeRootActor::RemoveChild(FSCSEditorTreeNodePtrType InChildNodePtr)
{
	const TArray<FSCSEditorTreeNodePtrType>& ComponentNodePtrs = GetComponentNodes();
	const int32 IndexOfFirstSceneComponent = ComponentNodePtrs.IndexOfByPredicate([](const FSCSEditorTreeNodePtrType& NodePtr)
	{
		return NodePtr->IsSceneComponent();
	});

	{
		TSharedPtr<FSCSEditorTreeNodeSeparator> SceneComponentSeparatorNodeSharedPtr = SceneComponentSeparatorNodePtr.Pin();
		if (IndexOfFirstSceneComponent == -1 && SceneComponentSeparatorNodeSharedPtr.IsValid())
		{
			FSCSEditorTreeNodeActorBase::RemoveChild(SceneComponentSeparatorNodeSharedPtr);
		}
	}

	const int32 IndexOfFirstNonSceneComponent = ComponentNodePtrs.IndexOfByPredicate([](const FSCSEditorTreeNodePtrType& NodePtr)
	{
		return !NodePtr->IsSceneComponent();
	});

	{
		TSharedPtr<FSCSEditorTreeNodeSeparator> NonSceneComponentSeparatorNodeSharedPtr = NonSceneComponentSeparatorNodePtr.Pin();

		if (IndexOfFirstNonSceneComponent == -1 && NonSceneComponentSeparatorNodeSharedPtr.IsValid())
		{
			FSCSEditorTreeNodeActorBase::RemoveChild(NonSceneComponentSeparatorNodeSharedPtr);
		}
	}

	FSCSEditorTreeNodeActorBase::RemoveChild(InChildNodePtr);
}

bool FSCSEditorTreeNodeRootActor::RefreshFilteredState(const UClass* InFilterType, const TArray<FString>& InFilterTerms, bool bRecursive)
{
	CachedFilterType = InFilterType;
	CachedFilterTerms = InFilterTerms;

	return FSCSEditorTreeNodeActorBase::RefreshFilteredState(InFilterType, InFilterTerms, bRecursive);
}

//////////////////////////////////////////////////////////////////////////
// FSCSEditorTreeNodeChildActor

bool FSCSEditorTreeNodeChildActor::IsFlaggedForFiltration() const
{
	// Not filtered out if any children are flagged as such
	for (const FSCSEditorTreeNodePtrType& Child : GetChildren())
	{
		if (!Child->IsFlaggedForFiltration())
		{
			return false;
		}
	}

	FSCSEditorTreeNodePtrType OwnerNode = GetOwnerNode();
	if (OwnerNode.IsValid())
	{
		// Mirror the owning node's filtered state
		return OwnerNode->IsFlaggedForFiltration();
	}

	return false;
}

UChildActorComponent* FSCSEditorTreeNodeChildActor::GetChildActorComponent() const
{
	FSCSEditorTreeNodePtrType OwnerNode = GetOwnerNode();
	if (OwnerNode.IsValid())
	{
		return Cast<UChildActorComponent>(OwnerNode->GetComponentTemplate());
	}

	return nullptr;
}

//////////////////////////////////////////////////////////////////////////
// FSCSEditorTreeNodeSeparator

bool FSCSEditorTreeNodeSeparator::MatchesFilterType(const UClass* InFilterType) const
{
	check(InFilterType);

	for (const UClass* FilteredType : FilteredTypes)
	{
		// If types match here, it means we should filter out the separator, so return false.
		if (InFilterType->IsChildOf(FilteredType))
		{
			return false;
		}
	}

	return true;
}

void FSCSEditorTreeNodeSeparator::AddFilteredComponentType(const TSubclassOf<UActorComponent>& InFilterType)
{
	if (const UClass* FilterType = InFilterType.Get())
	{
		FilteredTypes.Add(FilterType);
	}
}


#undef LOCTEXT_NAMESPACE