// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSCSEditorViewport.h"

#include "BlueprintEditor.h"
#include "BlueprintEditorCommands.h"
#include "BlueprintEditorSettings.h"
#include "BlueprintEditorTabs.h"
#include "Containers/EnumAsByte.h"
#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "Editor/EditorEngine.h"
#include "EditorViewportClient.h"
#include "EditorViewportCommands.h"
#include "Engine/Engine.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/World.h"
#include "SceneInterface.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GenericPlatform/ICursor.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "PreviewScene.h"
#include "RHIDefinitions.h"
#include "SCSEditorViewportClient.h"
#include "SEditorViewportToolBarMenu.h"
#include "SSubobjectEditor.h"
#include "STransformViewportToolbar.h"
#include "SViewportToolBar.h"
#include "Slate/SceneViewport.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "ToolMenus.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SOverlay.h"

class FDragDropEvent;
class SDockTab;
class SWidget;
struct FGeometry;

#define LOCTEXT_NAMESPACE "SSCSEditorViewportToolBar"

namespace UE::SCSEditor::Private
{

bool IsViewModeSupported(EViewModeIndex InViewModeIndex)
{
	switch (InViewModeIndex)
	{
	case VMI_Unlit:
	case VMI_Lit:
	case VMI_BrushWireframe:
	case VMI_ReflectionOverride:
	case VMI_CollisionPawn:
	case VMI_CollisionVisibility:
	case VMI_PathTracing:
	case VMI_Clay:
	case VMI_Zebra:
	case VMI_FrontBackFace:
	case VMI_RandomColor:
		return true;
	default:
		return false;
	}
}

bool DoesViewModeMenuShowSection(UE::UnrealEd::EHidableViewModeMenuSections)
{
	return false;
}

} // namespace UE::SCSEditor::Private


/*-----------------------------------------------------------------------------
   SSCSEditorViewport
-----------------------------------------------------------------------------*/
void SSCSEditorViewport::Construct(const FArguments& InArgs)
{
	bIsActiveTimerRegistered = false;

	// Save off the Blueprint editor reference, we'll need this later
	BlueprintEditorPtr = InArgs._BlueprintEditor;

	// Store the pre-configured mode tools (if provided)
	ModeTools = InArgs._ModeTools;

	SEditorViewport::Construct( SEditorViewport::FArguments() );

	// Restore last used feature level
	if (ViewportClient.IsValid())
	{
		UWorld* World = ViewportClient->GetPreviewScene()->GetWorld();
		if (World != nullptr && GWorld != nullptr && GWorld->Scene != nullptr)
		{
			World->ShaderPlatformChanged(GWorld->Scene->GetShaderPlatform());
		}
	}

	// Use a delegate to inform the attached world of preview platform changes.
	UEditorEngine* Editor = (UEditorEngine*)GEngine;
	PreviewFeatureLevelChangedHandle = Editor->OnPreviewShaderPlatformChanged().AddLambda([this](EShaderPlatform NewShaderPlatform)
		{
			if(ViewportClient.IsValid())
			{
				UWorld* World = ViewportClient->GetPreviewScene()->GetWorld();
				if (World != nullptr)
				{
					World->ShaderPlatformChanged(NewShaderPlatform);

					// Refresh the preview scene. Don't change the camera.
					RequestRefresh(false);
				}
			}
		});

	// Refresh the preview scene
	RequestRefresh(true);
}

SSCSEditorViewport::~SSCSEditorViewport()
{
	UEditorEngine* Editor = (UEditorEngine*)GEngine;
	Editor->OnPreviewShaderPlatformChanged().Remove(PreviewFeatureLevelChangedHandle);
}

bool SSCSEditorViewport::IsVisible() const
{
	// We consider the viewport to be visible if the reference is valid
	return ViewportWidget.IsValid() && SEditorViewport::IsVisible();
}

TSharedRef<FEditorViewportClient> SSCSEditorViewport::MakeEditorViewportClient()
{
	FPreviewScene* PreviewScene = BlueprintEditorPtr.Pin()->GetPreviewScene();

	// Construct a new viewport client instance.
	ViewportClient = MakeShareable(new FSCSEditorViewportClient(BlueprintEditorPtr, PreviewScene, SharedThis(this), ModeTools.Get()));
	ViewportClient->SetRealtime(true);
	ViewportClient->bSetListenerPosition = false;
	ViewportClient->VisibilityDelegate.BindSP(this, &SSCSEditorViewport::IsVisible);

	return ViewportClient.ToSharedRef();
}

TSharedPtr<SWidget> SSCSEditorViewport::BuildViewportToolbar()
{
	// Register the viewport toolbar if another viewport hasn't already (it's shared).
	const FName ViewportToolbarName = "SCSEditor.ViewportToolbar";
	if (!UToolMenus::Get()->IsMenuRegistered(ViewportToolbarName))
	{
		UToolMenu* const ViewportToolbarMenu = UToolMenus::Get()->RegisterMenu(
			ViewportToolbarName, NAME_None /* parent */, EMultiBoxType::SlimHorizontalToolBar
		);

		ViewportToolbarMenu->StyleName = "ViewportToolbar";

		// Add the left-aligned part of the viewport toolbar.
		{
			FToolMenuSection& LeftSection = ViewportToolbarMenu->AddSection("Left");

			LeftSection.AddEntry(UE::UnrealEd::CreateTransformsSubmenu());
			LeftSection.AddEntry(UE::UnrealEd::CreateSnappingSubmenu());
		}

		// Add the right-aligned part of the viewport toolbar.
		{
			FToolMenuSection& RightSection = ViewportToolbarMenu->AddSection("Right");
			RightSection.Alignment = EToolMenuSectionAlign::Last;

			// Add the "Camera" submenu.
			{
				// Add camera options for viewport and show all options
				UE::UnrealEd::FViewportCameraMenuOptions CameraMenuOptions;
				CameraMenuOptions.ShowAll();
				RightSection.AddEntry(UE::UnrealEd::CreateCameraSubmenu(CameraMenuOptions));
				
				const FName SubmenuName = UToolMenus::JoinMenuPaths(ViewportToolbarName, "Camera");
                UToolMenu* const Submenu = UToolMenus::Get()->ExtendMenu(SubmenuName);
                FToolMenuSection& MovementSection = Submenu->FindOrAddSection("Movement");
                MovementSection.AddMenuEntry(FBlueprintEditorCommands::Get().ResetCamera);
			}

			// Add the "View Modes" sub menu.
			{
				// Stay backward-compatible with the old viewport toolbar.
				{
					const FName ParentSubmenuName = "UnrealEd.ViewportToolbar.View";
					// Create our parent menu.
					if (!UToolMenus::Get()->IsMenuRegistered(ParentSubmenuName))
					{
						UToolMenus::Get()->RegisterMenu(ParentSubmenuName);
					}

					// Register our ToolMenu here first, before we create the submenu, so we can set our parent.
					UToolMenus::Get()->RegisterMenu("SCSEditor.ViewportToolbar.ViewModes", ParentSubmenuName);
				}

				RightSection.AddEntry(UE::UnrealEd::CreateViewModesSubmenu());
			}

			// Add the "Show" submenu with all flags, exclude some level viewport specific flags below
			{
				RightSection.AddEntry(UE::UnrealEd::CreateDefaultShowSubmenu());
			}

			// Add the "Performance & Scalability" submenu.
			{
				FToolMenuEntry PerfSubmenu = FToolMenuEntry::InitSubMenu(
					"PerformanceAndScalability",
					LOCTEXT("PerformanceAndScalabilityLabel", "Performance and Scalability"),
					LOCTEXT("PerformanceAndScalabilityTooltip", "Performance and Scalability tools tied to this viewport"),
					FNewToolMenuDelegate::CreateLambda(
						[](UToolMenu* Submenu) -> void
						{
							FToolMenuSection& UnnamedSection = Submenu->FindOrAddSection(NAME_None);
							UnnamedSection.AddEntry(UE::UnrealEd::CreateToggleRealtimeEntry());

							if (const UUnrealEdViewportToolbarContext* Context =
									Submenu->FindContext<UUnrealEdViewportToolbarContext>())
							{
								UnnamedSection.AddEntry(UE::UnrealEd::CreateRemoveRealtimeOverrideEntry(Context->Viewport));
							}
						}
					)
				);
				PerfSubmenu.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Scalability");
				PerfSubmenu.ToolBarData.LabelOverride = FText();
				PerfSubmenu.ToolBarData.ResizeParams.ClippingPriority = 800;
				RightSection.AddEntry(PerfSubmenu);
			}
		}
	}

	FToolMenuContext ViewportToolbarContext;
	{
		ViewportToolbarContext.AppendCommandList(GetCommandList());

		// Add the UnrealEd viewport toolbar context.
		{
			UUnrealEdViewportToolbarContext* const ContextObject = NewObject<UUnrealEdViewportToolbarContext>();
			ContextObject->Viewport = SharedThis(this);

			// Filter unrelated/irrelevant flags for BP editor
			const TArray<FEngineShowFlags::EShowFlag> ExcludedShowFlags
			{
				FEngineShowFlags::EShowFlag::SF_Atmosphere,
				FEngineShowFlags::EShowFlag::SF_BSP,
				FEngineShowFlags::EShowFlag::SF_Cloud,
				FEngineShowFlags::EShowFlag::SF_Fog,
				FEngineShowFlags::EShowFlag::SF_Landscape,
				FEngineShowFlags::EShowFlag::SF_MediaPlanes,
				FEngineShowFlags::EShowFlag::SF_Navigation,
			};
			ContextObject->ExcludedShowMenuFlags.Append(ExcludedShowFlags);

			const TArray<EShowFlagGroup> ExcludedShowGroupFlags
			{
				EShowFlagGroup::SFG_PostProcess,
				EShowFlagGroup::SFG_LightTypes,
				EShowFlagGroup::SFG_LightingComponents,
				EShowFlagGroup::SFG_LightingFeatures,
				EShowFlagGroup::SFG_Lumen,
				EShowFlagGroup::SFG_MegaLights,
				EShowFlagGroup::SFG_Nanite,
				EShowFlagGroup::SFG_Developer,
				EShowFlagGroup::SFG_Visualize,
				EShowFlagGroup::SFG_Advanced,
				EShowFlagGroup::SFG_Custom,
			};
			ContextObject->ExcludedShowMenuGroupFlags.Append(ExcludedShowGroupFlags);

			// Setup the callback to filter available view modes
			ContextObject->IsViewModeSupported = UE::UnrealEd::IsViewModeSupportedDelegate::CreateStatic(
			 	&UE::SCSEditor::Private::IsViewModeSupported
			);

			// Setup the callback to hide/show specific sections
			ContextObject->DoesViewModeMenuShowSection = UE::UnrealEd::DoesViewModeMenuShowSectionDelegate::CreateStatic(
				&UE::SCSEditor::Private::DoesViewModeMenuShowSection
			);

			ViewportToolbarContext.AddObject(ContextObject);
		}
	}

	return UToolMenus::Get()->GenerateWidget(ViewportToolbarName, ViewportToolbarContext);
}

void SSCSEditorViewport::PopulateViewportOverlays(TSharedRef<class SOverlay> Overlay)
{
	SEditorViewport::PopulateViewportOverlays(Overlay);

	// add the feature level display widget
	Overlay->AddSlot()
		.VAlign(VAlign_Bottom)
		.HAlign(HAlign_Right)
		.Padding(5.0f)
		[
			BuildShaderPlatformWidget()
		];
}

void SSCSEditorViewport::BindCommands()
{
	FSCSEditorViewportCommands::Register(); // make sure the viewport specific commands have been registered

	TSharedPtr<FBlueprintEditor> BlueprintEditor = BlueprintEditorPtr.Pin();
	TSharedPtr<SSubobjectEditor> SubobjectEditorPtr = BlueprintEditor->GetSubobjectEditor();
	SSubobjectEditor* SubobjectEditorWidget = SubobjectEditorPtr.Get(); 
	
	// for mac, we have to bind a command that would override the BP-Editor's 
	// "NavigateToParentBackspace" command, because the delete key is the 
	// backspace key for that platform (and "NavigateToParentBackspace" does not 
	// make sense in the viewport window... it blocks the generic delete command)
	// 
	// NOTE: this needs to come before we map any other actions (so it is 
	// prioritized first)

	if(SubobjectEditorWidget)
	{
		CommandList->MapAction(
		    FSCSEditorViewportCommands::Get().DeleteComponent,
		    FExecuteAction::CreateSP(SubobjectEditorWidget, &SSubobjectEditor::OnDeleteNodes),
		    FCanExecuteAction::CreateSP(SubobjectEditorWidget, &SSubobjectEditor::CanDeleteNodes)
		);
		
		CommandList->Append(SubobjectEditorWidget->GetCommandList().ToSharedRef());
	}
	
	CommandList->Append(BlueprintEditor->GetToolkitCommands());
	SEditorViewport::BindCommands();

	const FBlueprintEditorCommands& Commands = FBlueprintEditorCommands::Get();

	BlueprintEditorPtr.Pin()->GetToolkitCommands()->MapAction(
		Commands.EnableSimulation,
		FExecuteAction::CreateSP(this, &SSCSEditorViewport::ToggleIsSimulateEnabled),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(ViewportClient.Get(), &FSCSEditorViewportClient::GetIsSimulateEnabled),
		FIsActionButtonVisible::CreateSP(this, &SSCSEditorViewport::ShouldShowViewportCommands));

	// Toggle camera lock on/off
	CommandList->MapAction(
		Commands.ResetCamera,
		FExecuteAction::CreateSP(ViewportClient.Get(), &FSCSEditorViewportClient::ResetCamera));

	CommandList->MapAction(
		Commands.ShowFloor,
		FExecuteAction::CreateSP(ViewportClient.Get(), &FSCSEditorViewportClient::ToggleShowFloor),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(ViewportClient.Get(), &FSCSEditorViewportClient::GetShowFloor));

	CommandList->MapAction(
		Commands.ShowGrid,
		FExecuteAction::CreateSP(ViewportClient.Get(), &FSCSEditorViewportClient::ToggleShowGrid),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(ViewportClient.Get(), &FSCSEditorViewportClient::GetShowGrid));
}

void SSCSEditorViewport::Invalidate()
{
	ViewportClient->Invalidate();
}

void SSCSEditorViewport::ToggleIsSimulateEnabled()
{
	// Make the viewport visible if the simulation is starting.
	if ( !ViewportClient->GetIsSimulateEnabled() )
	{
		if ( GetDefault<UBlueprintEditorSettings>()->bShowViewportOnSimulate )
		{
			BlueprintEditorPtr.Pin()->GetTabManager()->TryInvokeTab(FBlueprintEditorTabs::SCSViewportID);
		}
	}

	ViewportClient->ToggleIsSimulateEnabled();
}

void SSCSEditorViewport::EnablePreview(bool bEnable)
{
	const FText SystemDisplayName = NSLOCTEXT("BlueprintEditor", "RealtimeOverrideMessage_Blueprints", "the active blueprint mode");
	if(bEnable)
	{
		// Restore the previously-saved realtime setting
		ViewportClient->RemoveRealtimeOverride(SystemDisplayName);
	}
	else
	{
		// Disable and store the current realtime setting. This will bypass real-time rendering in the preview viewport (see UEditorEngine::UpdateSingleViewportClient).
		const bool bShouldBeRealtime = false;
		ViewportClient->AddRealtimeOverride(bShouldBeRealtime, SystemDisplayName);
	}
}

void SSCSEditorViewport::RequestRefresh(bool bResetCamera, bool bRefreshNow)
{
	if(bRefreshNow)
	{
		if(ViewportClient.IsValid())
		{
			ViewportClient->InvalidatePreview(bResetCamera);
		}
	}
	else
	{
		// Defer the update until the next tick. This way we don't accidentally spawn the preview actor in the middle of a transaction, for example.
		if (!bIsActiveTimerRegistered)
		{
			bIsActiveTimerRegistered = true;
			RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SSCSEditorViewport::DeferredUpdatePreview, bResetCamera));
		}
	}
}

void SSCSEditorViewport::OnComponentSelectionChanged()
{
	// When the component selection changes, make sure to invalidate hit proxies to sync with the current selection
	SceneViewport->Invalidate();
}

void SSCSEditorViewport::OnFocusViewportToSelection()
{
	ViewportClient->FocusViewportToSelection();
}

bool SSCSEditorViewport::ShouldShowViewportCommands() const
{
	// Hide if actively debugging
	return !GIntraFrameDebuggingGameThread;
}

bool SSCSEditorViewport::GetIsSimulateEnabled()
{
	return ViewportClient->GetIsSimulateEnabled();
}

void SSCSEditorViewport::SetOwnerTab(TSharedRef<SDockTab> Tab)
{
	OwnerTab = Tab;
}

TSharedPtr<SDockTab> SSCSEditorViewport::GetOwnerTab() const
{
	return OwnerTab.Pin();
}

FReply SSCSEditorViewport::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<SSubobjectEditor> SubobjectEditor = BlueprintEditorPtr.Pin()->GetSubobjectEditor();

	return SubobjectEditor->TryHandleAssetDragDropOperation(DragDropEvent);
}

EActiveTimerReturnType SSCSEditorViewport::DeferredUpdatePreview(double InCurrentTime, float InDeltaTime, bool bResetCamera)
{
	if (ViewportClient.IsValid())
	{
		ViewportClient->InvalidatePreview(bResetCamera);
	}

	bIsActiveTimerRegistered = false;
	return EActiveTimerReturnType::Stop;
}

#undef LOCTEXT_NAMESPACE
