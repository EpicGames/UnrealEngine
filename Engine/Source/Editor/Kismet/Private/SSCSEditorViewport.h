// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Input/Reply.h"
#include "SEditorViewport.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FDragDropEvent;
class SDockTab;
struct FGeometry;

/**
 * Implements the viewport widget that's hosted in the SCS editor tab.
 */
class SSCSEditorViewport : public SEditorViewport
{
public:
	SLATE_BEGIN_ARGS(SSCSEditorViewport){}
		SLATE_ARGUMENT(TWeakPtr<class FBlueprintEditor>, BlueprintEditor)
		SLATE_ARGUMENT(TSharedPtr<class FEditorModeTools>, ModeTools)
	SLATE_END_ARGS()

	/**
	 * Constructs this widget with the given parameters.
	 *
	 * @param InArgs Construction parameters.
	 */
	void Construct(const FArguments& InArgs);

	/**
	 * Destructor.
	 */
	virtual ~SSCSEditorViewport();

	/**
	 * Invalidates the viewport client
	 */
	void Invalidate();

	/**
	 * Sets whether or not the preview should be enabled.
	 *
	 * @param bEnable Whether or not to enable the preview.
	 */
	void EnablePreview(bool bEnable);

	/**
	 * Request a refresh of the preview scene/world. Will recreate actors as needed.
	 *
	 * @param bResetCamera If true, the camera will be reset to its default position based on the preview.
	 * @param bRefreshNow If true, the preview will be refreshed immediately. Otherwise, it will be deferred until the next tick (default behavior).
	 */
	void RequestRefresh(bool bResetCamera = false, bool bRefreshNow = false);

	// SWidget interface
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	// End of SWidget interface

	/**
	 * Called when the selected component changes in the SCS editor.
	 */
	void OnComponentSelectionChanged();

	/**
	 * Focuses the viewport on the currently selected components
	 */
	virtual void OnFocusViewportToSelection() override;

	/**
	 * Returns true if simulation is enabled for the viewport
	 */
	bool GetIsSimulateEnabled();

	void SetOwnerTab(TSharedRef<SDockTab> Tab);

	TSharedPtr<SDockTab> GetOwnerTab() const;

protected:
	/**
	 * Determines if the viewport widget is visible.
	 *
	 * @return true if the viewport is visible; false otherwise.
	 */
	bool IsVisible() const override;

	/**
	 * Returns true if the viewport commands should be shown
	 */
	bool ShouldShowViewportCommands() const;

	/** Called when the simulation toggle command is fired */
	void ToggleIsSimulateEnabled();

	/** SEditorViewport interface */
	virtual TSharedRef<class FEditorViewportClient> MakeEditorViewportClient() override;
	virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
	virtual void PopulateViewportOverlays(TSharedRef<class SOverlay> Overlay) override;
	virtual void BindCommands() override;

private:
	/** One-off active timer to update the preview */
	EActiveTimerReturnType DeferredUpdatePreview(double InCurrentTime, float InDeltaTime, bool bResetCamera);

private:
	/** Pointer back to editor tool (owner) */
	TWeakPtr<class FBlueprintEditor> BlueprintEditorPtr;

	/** Viewport client */
	TSharedPtr<class FSCSEditorViewportClient> ViewportClient;

	/** Whether the active timer (for updating the preview) is registered */
	bool bIsActiveTimerRegistered;

	/** The owner dock tab for this viewport. */
	TWeakPtr<SDockTab> OwnerTab;

	/** Handle to the registered OnPreviewFeatureLevelChanged delegate. */
	FDelegateHandle PreviewFeatureLevelChangedHandle;

	/** Pre-configured mode tools for the viewport (optional) */
	TSharedPtr<class FEditorModeTools> ModeTools;
};
