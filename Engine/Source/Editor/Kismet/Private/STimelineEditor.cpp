// Copyright Epic Games, Inc. All Rights Reserved.


#include "STimelineEditor.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "BlueprintEditor.h"
#include "Components/TimelineComponent.h"
#include "Containers/EnumAsByte.h"
#include "Curves/CurveBase.h"
#include "Curves/CurveFloat.h"
#include "Curves/CurveLinearColor.h"
#include "Curves/CurveVector.h"
#include "Curves/KeyHandle.h"
#include "Curves/RichCurve.h"
#include "Dialogs/DlgPickAssetPath.h"
#include "EdGraph/EdGraphPin.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Blueprint.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/TimelineTemplate.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Framework/Views/ITypedTableView.h"
#include "HAL/PlatformMisc.h"
#include "IAssetTools.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_Timeline.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/CString.h"
#include "Modules/ModuleManager.h"
#include "PropertyCustomizationHelpers.h"
#include "SCurveEditor.h"
#include "SPositiveActionButton.h"
#include "ScopedTransaction.h"
#include "Selection.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/StyleColors.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Textures/SlateIcon.h"
#include "UObject/Class.h"
#include "UObject/GarbageCollection.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/ReflectedTypeAccessors.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"

class FTagMetaData;
class ITableRow;
class STableViewBase;
class SWidget;
struct FGeometry;
struct FKeyEvent;

#define LOCTEXT_NAMESPACE "STimelineEditor"

static TArray<TSharedPtr<FString>> TickGroupNameStrings;
static bool TickGroupNamesInitialized = false;

namespace TimelineEditorHelpers
{
	FTTTrackBase* GetTrackFromTimeline(UTimelineTemplate* InTimeline, TSharedPtr<FTimelineEdTrack> InTrack)
	{
		FTTTrackId TrackId = InTimeline->GetDisplayTrackId(InTrack->DisplayIndex);
		FTTTrackBase::ETrackType TrackType = (FTTTrackBase::ETrackType)TrackId.TrackType;
		if (TrackType == FTTTrackBase::TT_Event)
		{
			if (InTimeline->EventTracks.IsValidIndex(TrackId.TrackIndex))
			{
				return &InTimeline->EventTracks[TrackId.TrackIndex];
			}
		}
		else if (TrackType == FTTTrackBase::TT_FloatInterp)
		{
			if (InTimeline->FloatTracks.IsValidIndex(TrackId.TrackIndex))
			{
				return &InTimeline->FloatTracks[TrackId.TrackIndex];
			}
		}
		else if (TrackType == FTTTrackBase::TT_VectorInterp)
		{
			if (InTimeline->VectorTracks.IsValidIndex(TrackId.TrackIndex))
			{
				return &InTimeline->VectorTracks[TrackId.TrackIndex];
			}
		}
		else if (TrackType == FTTTrackBase::TT_LinearColorInterp)
		{
			if (InTimeline->LinearColorTracks.IsValidIndex(TrackId.TrackIndex))
			{
				return &InTimeline->LinearColorTracks[TrackId.TrackIndex];
			}
		}
		return nullptr;
	}

	FName GetTrackNameFromTimeline(UTimelineTemplate* InTimeline, TSharedPtr<FTimelineEdTrack> InTrack)
	{
		FTTTrackBase* TrackBase = GetTrackFromTimeline(InTimeline, InTrack);
		if (TrackBase)
		{
			return TrackBase->GetTrackName();
		}
		return NAME_None;
	}

	TSubclassOf<UCurveBase> TrackTypeToAllowedClass(FTTTrackBase::ETrackType TrackType)
	{
		switch (TrackType)
		{
		case FTTTrackBase::TT_Event:
		case FTTTrackBase::TT_FloatInterp:
			return UCurveFloat::StaticClass();
		case FTTTrackBase::TT_VectorInterp:
			return UCurveVector::StaticClass();
		case FTTTrackBase::TT_LinearColorInterp:
			return UCurveLinearColor::StaticClass();
		default:
			return UCurveBase::StaticClass();
		}
	}
}

//////////////////////////////////////////////////////////////////////////
// STimelineEdTrack

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void STimelineEdTrack::Construct(const FArguments& InArgs, TSharedPtr<FTimelineEdTrack> InTrack, TSharedPtr<STimelineEditor> InTimelineEd)
{
	Track = InTrack;
	TimelineEdPtr = InTimelineEd;

	ResetExternalCurveInfo();

	// Get the timeline we are editing
	TSharedPtr<STimelineEditor> TimelineEd = TimelineEdPtr.Pin();
	check(TimelineEd.IsValid());
	UTimelineTemplate* TimelineObj = TimelineEd->GetTimeline();
	check(TimelineObj); // We shouldn't have any tracks if there is no track object!

	// Get a pointer to the track this widget is for
	CurveBasePtr = nullptr;
	FTTTrackBase* TrackBase = nullptr;
	bool bDrawCurve = true;

	FTTTrackId TrackId = TimelineObj->GetDisplayTrackId(InTrack->DisplayIndex);
	FTTTrackBase::ETrackType TrackType = (FTTTrackBase::ETrackType)TrackId.TrackType;

	if(TrackType == FTTTrackBase::TT_Event)
	{
		check(TrackId.TrackIndex < TimelineObj->EventTracks.Num());
		FTTEventTrack* EventTrack = &(TimelineObj->EventTracks[TrackId.TrackIndex]);
		CurveBasePtr = EventTrack->CurveKeys;
		TrackBase = EventTrack;
		bDrawCurve = false;
	}
	else if(TrackType == FTTTrackBase::TT_FloatInterp)
	{
		check(TrackId.TrackIndex < TimelineObj->FloatTracks.Num());
		FTTFloatTrack* FloatTrack = &(TimelineObj->FloatTracks[TrackId.TrackIndex]);
		CurveBasePtr = FloatTrack->CurveFloat;
		TrackBase = FloatTrack;
	}
	else if(TrackType == FTTTrackBase::TT_VectorInterp)
	{
		check(TrackId.TrackIndex < TimelineObj->VectorTracks.Num());
		FTTVectorTrack* VectorTrack = &(TimelineObj->VectorTracks[TrackId.TrackIndex]);
		CurveBasePtr = VectorTrack->CurveVector;
		TrackBase = VectorTrack;
	}
	else if(TrackType == FTTTrackBase::TT_LinearColorInterp)
	{
		check(TrackId.TrackIndex < TimelineObj->LinearColorTracks.Num());
		FTTLinearColorTrack* LinearColorTrack = &(TimelineObj->LinearColorTracks[TrackId.TrackIndex]);
		CurveBasePtr = LinearColorTrack->CurveLinearColor;
		TrackBase = LinearColorTrack;
	}

	if( TrackBase && TrackBase->bIsExternalCurve )
	{
		//Update track with external curve info
		UseExternalCurve( CurveBasePtr );
	}

	TSharedRef<STimelineEditor> TimelineRef = TimelineEd.ToSharedRef();
	TSharedPtr<SInlineEditableTextBlock> InlineTextBlock;

	this->ChildSlot
	[
		SNew(SVerticalBox)
		
		// Heading Slot
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop_Hovered"))
			.ForegroundColor(FLinearColor::White)
			[
				SNew(SHorizontalBox)

				// Expander Button
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SCheckBox)
					.IsChecked(this, &STimelineEdTrack::GetIsExpandedState)
					.OnCheckStateChanged(this, &STimelineEdTrack::OnIsExpandedStateChanged)
					.CheckedImage(FAppStyle::GetBrush("TreeArrow_Expanded"))
					.CheckedHoveredImage(FAppStyle::GetBrush("TreeArrow_Expanded_Hovered"))
					.CheckedPressedImage(FAppStyle::GetBrush("TreeArrow_Expanded"))
					.UncheckedImage(FAppStyle::GetBrush("TreeArrow_Collapsed"))
					.UncheckedHoveredImage(FAppStyle::GetBrush("TreeArrow_Collapsed_Hovered"))
					.UncheckedPressedImage(FAppStyle::GetBrush("TreeArrow_Collapsed"))
				]

				// Track Name
				+ SHorizontalBox::Slot()
				.FillWidth(1)
				[
					// Name of track
					SAssignNew(InlineTextBlock, SInlineEditableTextBlock)
					.Text(FText::FromName(TrackBase->GetTrackName()))
					.ToolTipText(LOCTEXT("TrackNameTooltip", "Enter track name"))
					.OnVerifyTextChanged(TimelineRef, &STimelineEditor::OnVerifyTrackNameCommit, TrackBase, this)
					.OnTextCommitted(TimelineRef, &STimelineEditor::OnTrackNameCommitted, TrackBase, this)
				]
			]
		]

		// Content Slot
		+ SVerticalBox::Slot()
		[
			// Box for content visibility
			SNew(SBox)
			.Visibility(this, &STimelineEdTrack::GetContentVisibility)
			[
				SNew(SHorizontalBox)

				// Label Area
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SVerticalBox)
					// External Curve Label
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ExternalCurveLabel", "External Curve"))
						.ColorAndOpacity(FStyleColors::Foreground)
					]

					// External Curve Controls
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2, 0, 0, 4)
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("NoBrush"))
						.ForegroundColor(FStyleColors::Foreground)
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.FillWidth(1)
							[
								SNew(SObjectPropertyEntryBox)
								.AllowedClass(TimelineEditorHelpers::TrackTypeToAllowedClass(TrackType))
								.ObjectPath(this, &STimelineEdTrack::GetExternalCurvePath)
								.OnObjectChanged(FOnSetObject::CreateSP(this, &STimelineEdTrack::OnChooseCurve))
							]

							// Convert to internal curve button
							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(SButton)
								.ButtonStyle( FAppStyle::Get(), "NoBorder" )
								.OnClicked(this, &STimelineEdTrack::OnClickClear)
								.ContentPadding(1.f)
								.ToolTipText(NSLOCTEXT("TimelineEdTrack", "TimelineEdTrack_Clear", "Convert to Internal Curve"))
								[
									SNew(SImage)
									.Image( FAppStyle::GetBrush(TEXT("PropertyWindow.Button_Clear")))
									.ColorAndOpacity(FStyleColors::Foreground)
								]
							]
						]
					]

					// Synchronize curve view checkbox.
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2, 0, 2, 0)
					[
						SNew(SCheckBox)
						.IsChecked(this, &STimelineEdTrack::GetIsCurveViewSynchronizedState)
						.OnCheckStateChanged(this, &STimelineEdTrack::OnIsCurveViewSynchronizedStateChanged)
						.ToolTipText(LOCTEXT("SynchronizeViewToolTip", "Keep the zoom and pan of this curve synchronized with other curves."))
						[
							SNew(STextBlock)
							.Text(LOCTEXT("SynchronizeViewLabel", "Synchronize View"))
							.ColorAndOpacity(FStyleColors::Foreground)
						]
					]

					// Re-ordering timeline tracks.
					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(2, 0, 2, 0)
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(SButton)
								.ButtonStyle( FAppStyle::Get(), "NoBorder" )
								.OnClicked(this, &STimelineEdTrack::OnMoveUp)
								.IsEnabled(this, &STimelineEdTrack::CanMoveUp)
								.ContentPadding(1.f)
								.ToolTipText(NSLOCTEXT("TimelineEdTrack", "TimelineEdTrack_MoveUp", "Move track up list"))
								[
									SNew(SImage)
									.Image( FAppStyle::GetBrush(TEXT("ArrowUp")) )
									.ColorAndOpacity(FStyleColors::Foreground)
								]
							]

							// Convert to internal curve button
							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(SButton)
								.ButtonStyle( FAppStyle::Get(), "NoBorder" )
								.OnClicked(this, &STimelineEdTrack::OnMoveDown)
								.IsEnabled(this, &STimelineEdTrack::CanMoveDown)
								.ContentPadding(1.f)
								.ToolTipText(NSLOCTEXT("TimelineEdTrack", "TimelineEdTrack_MoveDown", "Move track down list"))
								[
									SNew(SImage)
									.Image( FAppStyle::GetBrush(TEXT("ArrowDown")) )
									.ColorAndOpacity(FStyleColors::Foreground)
								]
							]
							+SHorizontalBox::Slot()
							.FillWidth(1)
							.HAlign(HAlign_Left)
							.Padding(2)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("ReorderLabel", "Reorder"))
								.ColorAndOpacity(FStyleColors::Foreground)
							]
						]
				]

				// Graph Area
				+SHorizontalBox::Slot()
				.FillWidth(1)
				[
					SNew(SBorder)
					.VAlign(VAlign_Fill)
					[
						SAssignNew(TrackWidget, SCurveEditor)
						.ViewMinInput(this, &STimelineEdTrack::GetMinInput)
						.ViewMaxInput(this, &STimelineEdTrack::GetMaxInput)
						.ViewMinOutput(this, &STimelineEdTrack::GetMinOutput)
						.ViewMaxOutput(this, &STimelineEdTrack::GetMaxOutput)
						.TimelineLength(TimelineRef, &STimelineEditor::GetTimelineLength)
						.OnSetInputViewRange(this, &STimelineEdTrack::OnSetInputViewRange)
						.OnSetOutputViewRange(this, &STimelineEdTrack::OnSetOutputViewRange)
						.DesiredSize(TimelineRef, &STimelineEditor::GetTimelineDesiredSize)
						.DrawCurve(bDrawCurve)
						.HideUI(false)
						.OnCreateAsset(this, &STimelineEdTrack::OnCreateExternalCurve )
					]
				]
			]
		]
	];

	if( TrackBase )
	{
		bool bZoomToFit = false;
		if((GetMaxInput() == 0) && (GetMinInput() == 0))
		{
			// If the input range has not been set, zoom to fit to set it
			bZoomToFit = true;
		}

		//Inform track widget about the curve and whether it is editable or not.
		TrackWidget->SetZoomToFit(bZoomToFit, bZoomToFit);
		TrackWidget->SetCurveOwner(CurveBasePtr, !TrackBase->bIsExternalCurve);

		// In case the user has disabled auto frame in their settings, make sure to still adjust the zoom if we don't have an input
		// range yet.
		if (!TrackWidget->GetAutoFrame() && bZoomToFit)
		{
			TrackWidget->ZoomToFitVertical();
			TrackWidget->ZoomToFitHorizontal();
		}
	}

	InTrack->OnRenameRequest.BindSP(InlineTextBlock.Get(), &SInlineEditableTextBlock::EnterEditingMode);

}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FString STimelineEdTrack::CreateUniqueCurveAssetPathName()
{
	//Default path
	FString BasePath = FString(TEXT( "/Game/Unsorted" ));

	TSharedRef<STimelineEditor> TimelineRef = TimelineEdPtr.Pin().ToSharedRef();

	//Get curve name from editable text box
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");

	// Create a unique asset name so the user can instantly hit OK if they want to create the new asset
	FString AssetName = TimelineEditorHelpers::GetTrackNameFromTimeline(TimelineEdPtr.Pin()->GetTimeline(), Track).ToString();
	FString PackageName;
	BasePath = BasePath + TEXT("/") + AssetName;
	AssetToolsModule.Get().CreateUniqueAssetName(BasePath, TEXT(""), PackageName, AssetName);

	return PackageName;
}

void STimelineEdTrack::OnCloseCreateCurveWindow()
{
	if(AssetCreationWindow.IsValid())
	{
		//Destroy asset creation dialog
		TSharedPtr<SWindow> ParentWindow = AssetCreationWindow->GetParentWindow();
		AssetCreationWindow->RequestDestroyWindow();
		AssetCreationWindow.Reset();
	}
}

void STimelineEdTrack::OnCreateExternalCurve()
{
	UCurveBase* NewCurveAsset = CreateCurveAsset();
	if( NewCurveAsset )
	{
		//Switch internal to external curve
		SwitchToExternalCurve(NewCurveAsset);
	}
	//Close dialog once switching is complete
	OnCloseCreateCurveWindow();
}

void STimelineEdTrack::SwitchToExternalCurve(UCurveBase* AssetCurvePtr)
{
	if( AssetCurvePtr )
	{
		// Get the timeline we are editing
		TSharedPtr<STimelineEditor> TimelineEd = TimelineEdPtr.Pin();
		check(TimelineEd.IsValid());
		UTimelineTemplate* TimelineObj = TimelineEd->GetTimeline();
		check(TimelineObj); // We shouldn't have any tracks if there is no track object!

		FTTTrackId TrackId = TimelineObj->GetDisplayTrackId(Track->DisplayIndex);
		FTTTrackBase::ETrackType TrackType = (FTTTrackBase::ETrackType)TrackId.TrackType;

		FTTTrackBase* TrackBase = nullptr;
		if(TrackType == FTTTrackBase::TT_Event)
		{
			if(AssetCurvePtr->IsA(UCurveFloat::StaticClass()))
			{
				FTTEventTrack& NewTrack = TimelineObj->EventTracks[ TrackId.TrackIndex ];
				NewTrack.CurveKeys = Cast<UCurveFloat>(AssetCurvePtr);
				TrackBase = &NewTrack;
			}
		}
		else if(TrackType == FTTTrackBase::TT_FloatInterp)
		{
			if(AssetCurvePtr->IsA(UCurveFloat::StaticClass()))
			{
				FTTFloatTrack& NewTrack = TimelineObj->FloatTracks[ TrackId.TrackIndex ];
				NewTrack.CurveFloat = Cast<UCurveFloat>(AssetCurvePtr);
				TrackBase = &NewTrack;
			}
		}
		else if(TrackType == FTTTrackBase::TT_VectorInterp)
		{
			if(AssetCurvePtr->IsA(UCurveVector::StaticClass()))
			{
				FTTVectorTrack& NewTrack = TimelineObj->VectorTracks[ TrackId.TrackIndex ];
				NewTrack.CurveVector = Cast<UCurveVector>(AssetCurvePtr);
				TrackBase = &NewTrack;
			}
		}
		else if(TrackType == FTTTrackBase::TT_LinearColorInterp)
		{
			if(AssetCurvePtr->IsA(UCurveLinearColor::StaticClass()))
			{
				FTTLinearColorTrack& NewTrack = TimelineObj->LinearColorTracks[ TrackId.TrackIndex ];
				NewTrack.CurveLinearColor = Cast<UCurveLinearColor>(AssetCurvePtr);
				TrackBase = &NewTrack;
			}
		}

		if( TrackBase )
		{
			//Flag it as using external curve
			TrackBase->bIsExternalCurve = true;
			TrackWidget->SetCurveOwner( AssetCurvePtr, false );
			CurveBasePtr = AssetCurvePtr;

			UseExternalCurve(CurveBasePtr);
		}
	}
}

void STimelineEdTrack::UseExternalCurve( UObject* AssetObj )
{
	if (AssetObj)
	{
		ExternalCurvePath = AssetObj->GetPathName();
	}
	else
	{
		ResetExternalCurveInfo();
	}
}


void STimelineEdTrack::UseInternalCurve( )
{
	if( CurveBasePtr )
	{
		TSharedPtr<STimelineEditor> TimelineEd = TimelineEdPtr.Pin();
		check(TimelineEd.IsValid());
		UTimelineTemplate* TimelineObj = TimelineEd->GetTimeline();
		check(TimelineObj); // We shouldn't have any tracks if there is no track object!

		FTTTrackId TrackId = TimelineObj->GetDisplayTrackId(Track->DisplayIndex);
		FTTTrackBase::ETrackType TrackType = (FTTTrackBase::ETrackType)TrackId.TrackType;

		FTTTrackBase* TrackBase = nullptr;
		UCurveBase* CurveBase = nullptr;

		if(TrackType == FTTTrackBase::TT_Event)
		{
			FTTEventTrack& NewTrack = TimelineObj->EventTracks[ TrackId.TrackIndex ];

			if(NewTrack.bIsExternalCurve )
			{
				UCurveFloat* SrcCurve = NewTrack.CurveKeys;
				UCurveFloat* DestCurve = Cast<UCurveFloat>(TimelineEd->CreateNewCurve( TrackType) );
				if( SrcCurve && DestCurve )
				{
					//Copy external event curve data to internal curve
					CopyCurveData( &SrcCurve->FloatCurve, &DestCurve->FloatCurve );
					NewTrack.CurveKeys = DestCurve;
					CurveBase = DestCurve;
				}
			}
			TrackBase = &NewTrack;
		}
		else if(TrackType == FTTTrackBase::TT_FloatInterp)
		{
			FTTFloatTrack& NewTrack = TimelineObj->FloatTracks[ TrackId.TrackIndex ];
			if(NewTrack.bIsExternalCurve)
			{
				UCurveFloat* SrcCurve = NewTrack.CurveFloat;
				UCurveFloat* DestCurve = Cast<UCurveFloat>(TimelineEd->CreateNewCurve( TrackType) );
				if( SrcCurve && DestCurve )
				{
					//Copy external float curve data to internal curve
					CopyCurveData( &SrcCurve->FloatCurve, &DestCurve->FloatCurve );
					NewTrack.CurveFloat = DestCurve;
					CurveBase = DestCurve;
				}
			}
			TrackBase = &NewTrack;
		}
		else if(TrackType == FTTTrackBase::TT_VectorInterp)
		{
			FTTVectorTrack& NewTrack = TimelineObj->VectorTracks[ TrackId.TrackIndex ];
			if(NewTrack.bIsExternalCurve )
			{
				UCurveVector* SrcCurve = NewTrack.CurveVector;
				UCurveVector* DestCurve = Cast<UCurveVector>(TimelineEd->CreateNewCurve( TrackType) );
				if( SrcCurve && DestCurve )
				{
					for( int32 i=0; i<3; i++ )
					{
						//Copy external vector curve data to internal curve
						CopyCurveData( &SrcCurve->FloatCurves[i], &DestCurve->FloatCurves[i] );
					}
					NewTrack.CurveVector = DestCurve;
					CurveBase = DestCurve;
				}
			}
			TrackBase = &NewTrack;
		}
		else if(TrackType == FTTTrackBase::TT_LinearColorInterp)
		{
			FTTLinearColorTrack& NewTrack = TimelineObj->LinearColorTracks[ TrackId.TrackIndex ];
			if(NewTrack.bIsExternalCurve )
			{
				UCurveLinearColor* SrcCurve = NewTrack.CurveLinearColor;
				UCurveLinearColor* DestCurve = Cast<UCurveLinearColor>(TimelineEd->CreateNewCurve( TrackType) );
				if( SrcCurve && DestCurve )
				{
					for( int32 i=0; i<4; i++ )
					{
						//Copy external vector curve data to internal curve
						CopyCurveData( &SrcCurve->FloatCurves[i], &DestCurve->FloatCurves[i] );
					}
					NewTrack.CurveLinearColor = DestCurve;
					CurveBase = DestCurve;
				}
			}
			TrackBase = &NewTrack;
		}

		if( TrackBase && CurveBase )
		{
			//Reset flag
			TrackBase->bIsExternalCurve = false;

			TrackWidget->SetCurveOwner( CurveBase );
			CurveBasePtr = CurveBase;

			ResetExternalCurveInfo();
		}
	}
}

FReply STimelineEdTrack::OnClickClear()
{
	UseInternalCurve();
	return FReply::Handled();
}

void STimelineEdTrack::OnChooseCurve(const FAssetData& InObject)
{
	UCurveBase* SelectedObj = Cast<UCurveBase>(InObject.GetAsset());
	if (SelectedObj)
	{
		SwitchToExternalCurve(SelectedObj);
	}
	else
	{
		UseInternalCurve();
	}
}

FString STimelineEdTrack::GetExternalCurvePath( ) const
{
	return ExternalCurvePath;
}

UCurveBase* STimelineEdTrack::CreateCurveAsset()
{
	UCurveBase* AssetCurve = nullptr;

	TSharedPtr<STimelineEditor> TimelineEd = TimelineEdPtr.Pin();
	check(TimelineEd.IsValid());
	UTimelineTemplate* TimelineObj = TimelineEd->GetTimeline();
	check(TimelineObj); // We shouldn't have any tracks if there is no track object!

	FTTTrackId TrackId = TimelineObj->GetDisplayTrackId(Track->DisplayIndex);
	FTTTrackBase::ETrackType TrackType = (FTTTrackBase::ETrackType)TrackId.TrackType;

	if( TrackWidget.IsValid() )
	{
		TSharedRef<SDlgPickAssetPath> NewLayerDlg = 
			SNew(SDlgPickAssetPath)
			.Title(LOCTEXT("CreateExternalCurve", "Create External Curve"))
			.DefaultAssetPath(FText::FromString(CreateUniqueCurveAssetPathName()));

		if (NewLayerDlg->ShowModal() != EAppReturnType::Cancel)
		{
			FString PackageName = NewLayerDlg->GetFullAssetPath().ToString();
			FName AssetName = FName(*NewLayerDlg->GetAssetName().ToString());

			UPackage* Package = CreatePackage( *PackageName);
			
			//Get the curve class type
			TSubclassOf<UCurveBase> CurveType;

			if( TrackType == FTTTrackBase::TT_Event || TrackType == FTTTrackBase::TT_FloatInterp )
			{
				CurveType = UCurveFloat::StaticClass();
			}
			else if( TrackType == FTTTrackBase::TT_LinearColorInterp )
			{
				CurveType = UCurveLinearColor::StaticClass();
			}
			else 
			{
				CurveType = UCurveVector::StaticClass();
			}

			//Create curve object
			UObject* NewObj = TrackWidget->CreateCurveObject( CurveType, Package, AssetName );
			if( NewObj )
			{
				//Copy curve data from current curve to newly create curve
				if(  TrackType == FTTTrackBase::TT_Event || TrackType == FTTTrackBase::TT_FloatInterp )
				{
					UCurveFloat* DestCurve = CastChecked<UCurveFloat>(NewObj);

					AssetCurve = DestCurve;
					UCurveFloat* SourceCurve = CastChecked<UCurveFloat>(CurveBasePtr);

					if( SourceCurve && DestCurve )
					{
						CopyCurveData( &SourceCurve->FloatCurve, &DestCurve->FloatCurve );
					}

					DestCurve->bIsEventCurve = ( TrackType == FTTTrackBase::TT_Event ) ? true : false;
				}
				else if( TrackType == FTTTrackBase::TT_VectorInterp)
				{
					UCurveVector* DestCurve = Cast<UCurveVector>(NewObj);

					AssetCurve = DestCurve;
					UCurveVector* SrcCurve = CastChecked<UCurveVector>(CurveBasePtr);

					if( SrcCurve && DestCurve )
					{
						for( int32 i=0; i<3; i++ )
						{
							CopyCurveData( &SrcCurve->FloatCurves[i], &DestCurve->FloatCurves[i] );
						}
					}
				}
				else if( TrackType == FTTTrackBase::TT_LinearColorInterp)
				{
					UCurveLinearColor* DestCurve = Cast<UCurveLinearColor>(NewObj);

					AssetCurve = DestCurve;
					UCurveLinearColor* SrcCurve = CastChecked<UCurveLinearColor>(CurveBasePtr);

					if( SrcCurve && DestCurve )
					{
						for( int32 i=0; i<4; i++ )
						{
							CopyCurveData( &SrcCurve->FloatCurves[i], &DestCurve->FloatCurves[i] );
						}
					}
				}

				// Set the new objects as the sole selection.
				USelection* SelectionSet = GEditor->GetSelectedObjects();
				SelectionSet->DeselectAll();
				SelectionSet->Select( NewObj );

				// Notify the asset registry
				FAssetRegistryModule::AssetCreated(NewObj);

				// Mark the package dirty...
				Package->GetOutermost()->MarkPackageDirty();
				return AssetCurve;
			}
		}
	}

	return nullptr;
}


void STimelineEdTrack::CopyCurveData( const FRichCurve* SrcCurve, FRichCurve* DestCurve )
{
	if( SrcCurve && DestCurve )
	{
		for (auto It(SrcCurve->GetKeyIterator()); It; ++It)
		{
			const FRichCurveKey& Key = *It;
			FKeyHandle KeyHandle = DestCurve->AddKey(Key.Time, Key.Value);
			DestCurve->GetKey(KeyHandle) = Key;
		}
	}
}

ECheckBoxState STimelineEdTrack::GetIsExpandedState() const
{
	const FTTTrackBase* TrackBase = GetTrackBase();

	return (TrackBase && TrackBase->bIsExpanded) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void STimelineEdTrack::OnIsExpandedStateChanged(ECheckBoxState IsExpandedState)
{
	FTTTrackBase* TrackBase = GetTrackBase();

	if (TrackBase)
	{
		TrackBase->bIsExpanded = IsExpandedState == ECheckBoxState::Checked;
	}

	//recalculate how much space the widgets take up to enable scrolling when needed
	TSharedPtr<STimelineEditor> TimelineEditor = TimelineEdPtr.Pin();
	TimelineEditor->OnTimelineChanged();
}

EVisibility STimelineEdTrack::GetContentVisibility() const
{
	const FTTTrackBase* TrackBase = GetTrackBase();

	return (TrackBase && TrackBase->bIsExpanded) ? EVisibility::Visible : EVisibility::Collapsed;
}

ECheckBoxState  STimelineEdTrack::GetIsCurveViewSynchronizedState() const
{
	const FTTTrackBase* TrackBase = GetTrackBase();

	return (TrackBase && TrackBase->bIsCurveViewSynchronized) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void  STimelineEdTrack::OnIsCurveViewSynchronizedStateChanged(ECheckBoxState IsCurveViewSynchronizedState)
{
	FTTTrackBase* TrackBase = GetTrackBase();

	if (TrackBase)
	{
		TrackBase->bIsCurveViewSynchronized = IsCurveViewSynchronizedState == ECheckBoxState::Checked;
	}

	//local is always up to date, make sure the timeline editor is inited at least once
	TSharedPtr<STimelineEditor> TimelineEditor = TimelineEdPtr.Pin();
	if ((TimelineEditor->GetViewMaxInput() == 0) && (TimelineEditor->GetViewMinInput() == 0))
	{
		//we've never used the shared timeline range, but our local one is always up to date!
		TimelineEditor->SetInputViewRange(LocalInputMin, LocalInputMax);
		TimelineEditor->SetOutputViewRange(LocalOutputMin, LocalOutputMax);
	}
	//only take the timeline editors extents if we are accepting synchronization
	if ((TrackBase && TrackBase->bIsCurveViewSynchronized) || ((LocalInputMax == 0.0f) && (LocalInputMin == 0.0f)))
	{
		LocalInputMin = TimelineEditor->GetViewMinInput();
		LocalInputMax = TimelineEditor->GetViewMaxInput();
		LocalOutputMin = TimelineEditor->GetViewMinOutput();
		LocalOutputMax = TimelineEditor->GetViewMaxOutput();
	}
}

FReply STimelineEdTrack::OnMoveUp()
{
	MoveTrack(-1);

	return FReply::Handled();
}
bool STimelineEdTrack::CanMoveUp() const
{
	return (Track->DisplayIndex > 0);
}
FReply STimelineEdTrack::OnMoveDown()
{
	MoveTrack(1);

	return FReply::Handled();
}

bool STimelineEdTrack::CanMoveDown() const
{
	TSharedPtr<STimelineEditor> TimelineEd = TimelineEdPtr.Pin();
	check(TimelineEd.IsValid());
	UTimelineTemplate* TimelineObj = TimelineEd->GetTimeline();
	check(TimelineObj); // We shouldn't have any tracks if there is no track object!

	return (Track->DisplayIndex < (TimelineObj->GetNumDisplayTracks() - 1));
}

void STimelineEdTrack::MoveTrack(int32 DirectionDelta)
{
	TSharedPtr<STimelineEditor> TimelineEd = TimelineEdPtr.Pin();
	check(TimelineEd.IsValid());

	TimelineEd->OnReorderTracks(Track->DisplayIndex, DirectionDelta);
}


float STimelineEdTrack::GetMinInput() const
{
	const FTTTrackBase* TrackBase = GetTrackBase();
	return (TrackBase && TrackBase->bIsCurveViewSynchronized)
		? TimelineEdPtr.Pin()->GetViewMinInput()
		: LocalInputMin;
}

float STimelineEdTrack::GetMaxInput() const
{
	const FTTTrackBase* TrackBase = GetTrackBase();
	return (TrackBase && TrackBase->bIsCurveViewSynchronized)
		? TimelineEdPtr.Pin()->GetViewMaxInput()
		: LocalInputMax;
}

float STimelineEdTrack::GetMinOutput() const
{
	const FTTTrackBase* TrackBase = GetTrackBase();
	return (TrackBase && TrackBase->bIsCurveViewSynchronized)
		? TimelineEdPtr.Pin()->GetViewMinOutput()
		: LocalOutputMin;
}

float STimelineEdTrack::GetMaxOutput() const
{
	const FTTTrackBase* TrackBase = GetTrackBase();
	return (TrackBase && TrackBase->bIsCurveViewSynchronized)
		? TimelineEdPtr.Pin()->GetViewMaxOutput()
		: LocalOutputMax;
}

void STimelineEdTrack::OnSetInputViewRange(float Min, float Max)
{
	const FTTTrackBase* TrackBase = GetTrackBase();
	if (TrackBase && TrackBase->bIsCurveViewSynchronized)
	{
		TimelineEdPtr.Pin()->SetInputViewRange(Min, Max);
	}
	//always set these in case we go back and forth
	LocalInputMin = Min;
	LocalInputMax = Max;
}

void STimelineEdTrack::OnSetOutputViewRange(float Min, float Max)
{
	const FTTTrackBase* TrackBase = GetTrackBase();
	if (TrackBase && TrackBase->bIsCurveViewSynchronized)
	{
		TimelineEdPtr.Pin()->SetOutputViewRange(Min, Max);
	}
	//always set these in case we go back and forth
	LocalOutputMin = Min;
	LocalOutputMax = Max;
}

void STimelineEdTrack::ResetExternalCurveInfo( )
{
	ExternalCurvePath = FString( TEXT( "None" ) );
}

FTTTrackBase* STimelineEdTrack::GetTrackBase()
{
	TSharedPtr<STimelineEditor> TimelineEd = TimelineEdPtr.Pin();
	check(TimelineEd.IsValid());
	UTimelineTemplate* TimelineObj = TimelineEd->GetTimeline();
	check(TimelineObj); // We shouldn't have any tracks if there is no track object!
	FTTTrackBase* TrackBase = TimelineEditorHelpers::GetTrackFromTimeline(TimelineObj, Track);
	return TrackBase;
}
const FTTTrackBase* STimelineEdTrack::GetTrackBase() const
{
	TSharedPtr<STimelineEditor> TimelineEd = TimelineEdPtr.Pin();
	check(TimelineEd.IsValid());
	UTimelineTemplate* TimelineObj = TimelineEd->GetTimeline();
	check(TimelineObj); // We shouldn't have any tracks if there is no track object!
	FTTTrackBase* TrackBase = TimelineEditorHelpers::GetTrackFromTimeline(TimelineObj, Track);
	return TrackBase;
}

//////////////////////////////////////////////////////////////////////////
// STimelineEditor

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void STimelineEditor::Construct(const FArguments& InArgs, TSharedPtr<FBlueprintEditor> InKismet2, UTimelineTemplate* InTimelineObj)
{
	NewTrackPendingRename = NAME_None;

	Kismet2Ptr = InKismet2;
	TimelineObj = nullptr;

	NominalTimelineDesiredHeight = 300.0f;
	TimelineDesiredSize = FVector2f(128.0f, NominalTimelineDesiredHeight);

	// Leave these uninitialized at first.  We'll zoom to fit the tracks which will set the correct values
	ViewMinInput = 0.f;
	ViewMaxInput = 0.f;
	ViewMinOutput = 0.f;
	ViewMaxOutput = 0.f;

	CommandList = MakeShareable( new FUICommandList );

	CommandList->MapAction( FGenericCommands::Get().Rename,
		FExecuteAction::CreateSP(this, &STimelineEditor::OnRequestTrackRename),
		FCanExecuteAction::CreateSP(this, &STimelineEditor::CanRenameSelectedTrack) );

	CommandList->MapAction( FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &STimelineEditor::OnDeleteSelectedTracks),
		FCanExecuteAction::CreateSP(this, &STimelineEditor::CanDeleteSelectedTracks) );

	// Get TickGroup enum info for the TimelineEditor control panel
	int32 CurrentTickGroupNameStringIndex = 0;
	const UEnum* TickGroupEnum = StaticEnum<ETickingGroup>();
	if (!TickGroupNamesInitialized && TickGroupEnum)
	{
		// Store the TickGroup name info one time, in one place accessible to all TimelineEditors
		TickGroupNameStrings.Empty();
		for (int32 TickGroupIndex = 0; TickGroupIndex < TickGroupEnum->NumEnums() - 1; TickGroupIndex++)
		{
			if (!TickGroupEnum->HasMetaData(TEXT("Hidden"), TickGroupIndex))
			{
				TickGroupNameStrings.Add(MakeShareable(new FString(TickGroupEnum->GetNameStringByIndex(TickGroupIndex))));
			}
		}
		TickGroupNamesInitialized = true;
	}
	if (TickGroupNamesInitialized && InTimelineObj)
	{
		// Set the current index into the TickGroupNameStrings so the ComboBox being set up below can highlight the current value
		FString CurrentTickGroupNameString = TickGroupEnum->GetNameStringByValue((int64)InTimelineObj->TimelineTickGroup);
		CurrentTickGroupNameStringIndex = TickGroupNameStrings.IndexOfByPredicate([CurrentTickGroupNameString](const TSharedPtr<FString> NameString)
		{
			return *NameString.Get() == CurrentTickGroupNameString;
		});
	}
	else
	{
		// If we don't have the ETickingGroup enum available for some reason, don't crash the Editor
		TickGroupNameStrings.Empty();
		TickGroupNameStrings.Add(MakeShareable(new FString(TEXT("EnumNotReady"))));
	}

	this->ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			// Header, shows name of timeline we are editing
			SNew(SBorder)
			. BorderImage( FAppStyle::GetBrush( TEXT("Graph.TitleBackground") ) )
			. HAlign(HAlign_Center)
			.AddMetaData<FTagMetaData>(TEXT("TimelineEditor.Title"))
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding( 10,0 )
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image( FAppStyle::GetBrush(TEXT("GraphEditor.TimelineGlyph")) )
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				. VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font( FCoreStyle::GetDefaultFontStyle("Regular", 14) )
					.ColorAndOpacity( FLinearColor(1,1,1,0.5) )
					.Text( this, &STimelineEditor::GetTimelineName )
				]
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			// Box for holding buttons
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(6.f)
			[
				SNew(SPositiveActionButton)
				.OnGetMenuContent(this, &STimelineEditor::MakeAddButton)
				.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
				.Text(LOCTEXT("Track", "Track"))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.f)
			.VAlign(VAlign_Center)
			[
				// Length label
				SNew(STextBlock)
				.Text( LOCTEXT( "Length", "Length" ) )
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(6.0f, 2.0f, 2.0f, 2.0f))
			.VAlign(VAlign_Center)
			[
				// Length edit box
				SAssignNew(TimelineLengthEdit, SEditableTextBox)
				.Text( this, &STimelineEditor::GetLengthString )
				.OnTextCommitted( this, &STimelineEditor::OnLengthStringChanged )
				.SelectAllTextWhenFocused(true)
				.MinDesiredWidth(64)
				.AddMetaData<FTagMetaData>(TEXT("TimelineEditor.Length"))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.f)
			.VAlign(VAlign_Center)
			[
				// Use last keyframe as length check box
				SAssignNew(UseLastKeyframeCheckBox, SCheckBox)
				.IsChecked( this, &STimelineEditor::IsUseLastKeyframeChecked )
				.OnCheckStateChanged( this, &STimelineEditor::OnUseLastKeyframeChanged )
				.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
				.ToolTipText(LOCTEXT("UseLastKeyframe", "Use Last Keyframe"))
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("TimelineEditor.UseLastKeyframe"))
					.AddMetaData<FTagMetaData>(TEXT("TimelineEditor.UseLastKeyframe"))
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.f)
			.VAlign(VAlign_Center)
			[
				// Play check box
				SAssignNew(PlayCheckBox, SCheckBox)
				.IsChecked( this, &STimelineEditor::IsAutoPlayChecked )
				.OnCheckStateChanged( this, &STimelineEditor::OnAutoPlayChanged )
				.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
				.ToolTipText(LOCTEXT("AutoPlay", "AutoPlay"))
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("TimelineEditor.AutoPlay"))
					.AddMetaData<FTagMetaData>(TEXT("TimelineEditor.AutoPlay"))
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.f)
			.VAlign(VAlign_Center)
			[
				// Loop check box
				SAssignNew(LoopCheckBox, SCheckBox)
				.IsChecked( this, &STimelineEditor::IsLoopChecked )
				.OnCheckStateChanged( this, &STimelineEditor::OnLoopChanged )
				.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
				.ToolTipText(LOCTEXT("Loop", "Loop"))
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("TimelineEditor.Loop"))
					.AddMetaData<FTagMetaData>(TEXT("TimelineEditor.Loop"))
				]
				
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.f)
			.VAlign(VAlign_Center)
			[
				// Replicated check box
				SAssignNew(ReplicatedCheckBox, SCheckBox)
				.IsChecked( this, &STimelineEditor::IsReplicatedChecked )
				.OnCheckStateChanged( this, &STimelineEditor::OnReplicatedChanged )
				.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
				.ToolTipText(LOCTEXT("Replicated", "Replicated"))
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("TimelineEditor.Replicated"))
					.AddMetaData<FTagMetaData>(TEXT("TimelineEditor.Replicated"))
				]
			
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.f)
			.VAlign(VAlign_Center)
			[
				// Ignore Time Dilation check box
				SAssignNew(IgnoreTimeDilationCheckBox, SCheckBox)
				.IsChecked( this, &STimelineEditor::IsIgnoreTimeDilationChecked )
				.OnCheckStateChanged( this, &STimelineEditor::OnIgnoreTimeDilationChanged )
				.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
				.ToolTipText(LOCTEXT("IgnoreTimeDilation", "Ignore Time Dilation"))
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("TimelineEditor.IgnoreTimeDilation"))
					.AddMetaData<FTagMetaData>(TEXT("TimelineEditor.IgnoreTimeDilation"))
				]
			]
			// Tick Group Controls
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("TickGroupLabel", "Tick Group"))
				.AddMetaData<FTagMetaData>(TEXT("TimelineEditor.TickGroup"))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(6.f)
			.VAlign(VAlign_Center)
			[
				SNew(STextComboBox)
				.OptionsSource(&TickGroupNameStrings)
				.InitiallySelectedItem(TickGroupNameStrings[CurrentTickGroupNameStringIndex])
				.OnSelectionChanged(this, &STimelineEditor::OnTimelineTickGroupChanged)
				.ToolTipText(LOCTEXT("TimelineTickGroupDropdownTooltip", "Select the TickGroup you want this timeline to run in.\nTo assign options use context menu on timelines."))
			]
		]
		+SVerticalBox::Slot()
		.FillHeight(1)
		[
			// The list of tracks
			SAssignNew( TrackListView, STimelineEdTrackListType )
			.ListItemsSource( &TrackList )
			.OnGenerateRow( this, &STimelineEditor::MakeTrackWidget )
			.OnItemScrolledIntoView(this, &STimelineEditor::OnItemScrolledIntoView)
			.OnContextMenuOpening(this, &STimelineEditor::MakeContextMenu)
			.SelectionMode(ESelectionMode::SingleToggle)
		]
	];

	TimelineObj = InTimelineObj;
	check(TimelineObj);

	// Initial call to get list built
	OnTimelineChanged();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void STimelineEditor::OnTimelineTickGroupChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo)
{
	if (TickGroupNamesInitialized && TimelineObj && NewValue.IsValid())
	{
		if (const UEnum* TickGroupEnum = StaticEnum<ETickingGroup>())
		{
			ETickingGroup NewTickGroup = (ETickingGroup)TickGroupEnum->GetValueByNameString(*NewValue.Get());
			if (NewTickGroup != TimelineObj->TimelineTickGroup)
			{
				TimelineObj->TimelineTickGroup = NewTickGroup;

				// Mark blueprint as modified
				TSharedPtr<FBlueprintEditor> Kismet2 = Kismet2Ptr.Pin();
				if (UBlueprint* Blueprint = Kismet2->GetBlueprintObj())
				{
					FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
				}
			}
		}
	}
	return;
}

FText STimelineEditor::GetTimelineName() const
{
	if(TimelineObj != nullptr)
	{
		return FText::FromString(TimelineObj->GetVariableName().ToString());
	}
	else
	{
		return LOCTEXT( "NoTimeline", "No Timeline" );
	}
}

float STimelineEditor::GetViewMaxInput() const
{
	return ViewMaxInput;
}

float STimelineEditor::GetViewMinInput() const
{
	return ViewMinInput;
}

float STimelineEditor::GetViewMaxOutput() const
{
	return ViewMaxOutput;
}

float STimelineEditor::GetViewMinOutput() const
{
	return ViewMinOutput;
}

float STimelineEditor::GetTimelineLength() const
{
	return (TimelineObj != nullptr) ? TimelineObj->TimelineLength : 0.f;
}

void STimelineEditor::SetInputViewRange(float InViewMinInput, float InViewMaxInput)
{
	ViewMaxInput = InViewMaxInput;
	ViewMinInput = InViewMinInput;
}

void STimelineEditor::SetOutputViewRange(float InViewMinOutput, float InViewMaxOutput)
{
	ViewMaxOutput = InViewMaxOutput;
	ViewMinOutput = InViewMinOutput;
}

TSharedRef<ITableRow> STimelineEditor::MakeTrackWidget( TSharedPtr<FTimelineEdTrack> Track, const TSharedRef<STableViewBase>& OwnerTable )
{
	check( Track.IsValid() );

	return
	SNew(STableRow< TSharedPtr<FTimelineEdTrack> >, OwnerTable )
	.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TimelineEditor.TrackRowSubtleHighlight"))
	.Padding(FMargin(0, 0, 0, 2))
	[
		SNew(STimelineEdTrack, Track, SharedThis(this))
	];
}

void STimelineEditor::CreateNewTrack(FTTTrackBase::ETrackType Type)
{
	FName TrackName;
	do
	{
		// MakeUniqueObjectName is misleading here since tracks aren't UObjects, although the function
		// will still keep a counter for tracks. This may take a couple tries to find a valid name.
		TrackName = MakeUniqueObjectName(TimelineObj, UTimelineTemplate::StaticClass(), FName(*(LOCTEXT("NewTrack_DefaultName", "NewTrack").ToString())));
	} while (!TimelineObj->IsNewTrackNameValid(TrackName));
		
	TSharedPtr<FBlueprintEditor> Kismet2 = Kismet2Ptr.Pin();
	UBlueprint* Blueprint = Kismet2->GetBlueprintObj();
	UK2Node_Timeline* TimelineNode = FBlueprintEditorUtils::FindNodeForTimeline(Blueprint, TimelineObj);
	UClass* OwnerClass = Blueprint->GeneratedClass;
	check(OwnerClass);

	FText ErrorMessage;

	if (TimelineNode)
	{
		const FScopedTransaction Transaction( LOCTEXT( "TimelineEditor_AddNewTrack", "Add new track" ) );

		TimelineNode->Modify();
		TimelineObj->Modify();

		NewTrackPendingRename = TrackName;
		
		FTTTrackId NewTrackId;
		NewTrackId.TrackType = Type;

		if(Type == FTTTrackBase::TT_Event)
		{
			NewTrackId.TrackIndex = TimelineObj->EventTracks.Num();
				
			FTTEventTrack NewTrack;
			NewTrack.SetTrackName(TrackName, TimelineObj);
			NewTrack.CurveKeys = NewObject<UCurveFloat>(OwnerClass, NAME_None, RF_Public); // Needs to be marked public so that it can be referenced from timeline instances in the level
			NewTrack.CurveKeys->bIsEventCurve = true;
			TimelineObj->EventTracks.Add(NewTrack);
		}
		else if(Type == FTTTrackBase::TT_FloatInterp)
		{
			NewTrackId.TrackIndex = TimelineObj->FloatTracks.Num();

			FTTFloatTrack NewTrack;
			NewTrack.SetTrackName(TrackName, TimelineObj);
			// @hack for using existing curve assets.  need something better!
			NewTrack.CurveFloat = FindFirstObject<UCurveFloat>(*TrackName.ToString(), EFindFirstObjectOptions::NativeFirst | EFindFirstObjectOptions::EnsureIfAmbiguous);
			if (NewTrack.CurveFloat == nullptr)
			{
				NewTrack.CurveFloat = NewObject<UCurveFloat>(OwnerClass, NAME_None, RF_Public);
			}
			TimelineObj->FloatTracks.Add(NewTrack);
		}
		else if(Type == FTTTrackBase::TT_VectorInterp)
		{
			NewTrackId.TrackIndex = TimelineObj->VectorTracks.Num();

			FTTVectorTrack NewTrack;
			NewTrack.SetTrackName(TrackName, TimelineObj);
			NewTrack.CurveVector = NewObject<UCurveVector>(OwnerClass, NAME_None, RF_Public);
			TimelineObj->VectorTracks.Add(NewTrack);
		}
		else if(Type == FTTTrackBase::TT_LinearColorInterp)
		{
			NewTrackId.TrackIndex = TimelineObj->LinearColorTracks.Num();

			FTTLinearColorTrack NewTrack;
			NewTrack.SetTrackName(TrackName, TimelineObj);
			NewTrack.CurveLinearColor = NewObject<UCurveLinearColor>(OwnerClass, NAME_None, RF_Public);
			TimelineObj->LinearColorTracks.Add(NewTrack);
		}

		TimelineObj->AddDisplayTrack(NewTrackId);

		// Refresh the node that owns this timeline template to get new pin
		TimelineNode->ReconstructNode();
		Kismet2->RefreshEditors();

		//rebuild the widgets!
		OnTimelineChanged();
	}
	else
	{
		// invalid node for timeline
		ErrorMessage = LOCTEXT( "InvalidTimelineNodeCreate","Failed to create track. Timeline node is invalid. Please remove timeline node." );
	}

	if (!ErrorMessage.IsEmpty())
	{
		FNotificationInfo Info(ErrorMessage);
		Info.ExpireDuration = 3.0f;
		Info.bUseLargeFont = false;
		TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
		if ( Notification.IsValid() )
		{
			Notification->SetCompletionState( SNotificationItem::CS_Fail );
		}
	}
}

UCurveBase* STimelineEditor::CreateNewCurve(FTTTrackBase::ETrackType Type )
{
	TSharedPtr<FBlueprintEditor> Kismet2 = Kismet2Ptr.Pin();
	UBlueprint* Blueprint = Kismet2->GetBlueprintObj();
	UClass* OwnerClass = Blueprint->GeneratedClass;
	check(OwnerClass);
	UCurveBase* NewCurve = nullptr;
	if(Type == FTTTrackBase::TT_Event)
	{
		NewCurve = NewObject<UCurveFloat>(OwnerClass, NAME_None, RF_Public);
	}
	else if(Type == FTTTrackBase::TT_FloatInterp)
	{
		NewCurve = NewObject<UCurveFloat>(OwnerClass, NAME_None, RF_Public);
	}
	else if(Type == FTTTrackBase::TT_VectorInterp)
	{
		NewCurve = NewObject<UCurveVector>(OwnerClass, NAME_None, RF_Public);
	}
	else if(Type == FTTTrackBase::TT_LinearColorInterp)
	{
		NewCurve = NewObject<UCurveLinearColor>(OwnerClass, NAME_None, RF_Public);
	}

	return NewCurve;
}

bool STimelineEditor::CanDeleteSelectedTracks() const
{
	int32 SelectedItems = TrackListView->GetNumItemsSelected();
	return (SelectedItems == 1);
}

void STimelineEditor::OnDeleteSelectedTracks()
{
	if(TimelineObj != nullptr)
	{
		TSharedPtr<FBlueprintEditor> Kismet2 = Kismet2Ptr.Pin();
		UBlueprint* Blueprint = Kismet2->GetBlueprintObj();
		UK2Node_Timeline* TimelineNode = FBlueprintEditorUtils::FindNodeForTimeline(Blueprint, TimelineObj);

		TArray< TSharedPtr<FTimelineEdTrack> > SelTracks = TrackListView->GetSelectedItems();
		if(SelTracks.Num() == 1)
		{
			if (TimelineNode)
			{
				const FScopedTransaction Transaction( LOCTEXT( "TimelineEditor_DeleteTrack", "Delete track" ) );

				TimelineNode->Modify();
				TimelineObj->Modify();

				TSharedPtr<FTimelineEdTrack> SelTrack = SelTracks[0];
				FTTTrackId TrackId = TimelineObj->GetDisplayTrackId(SelTrack->DisplayIndex);
				FTTTrackBase::ETrackType TrackType = (FTTTrackBase::ETrackType)TrackId.TrackType;

				TimelineObj->RemoveDisplayTrack(SelTrack->DisplayIndex);

				if (TrackType == FTTTrackBase::TT_Event)
				{
					TimelineObj->EventTracks.RemoveAt(TrackId.TrackIndex);
				}
				else if (TrackType == FTTTrackBase::TT_FloatInterp)
				{
					TimelineObj->FloatTracks.RemoveAt(TrackId.TrackIndex);
				}
				else if (TrackType == FTTTrackBase::TT_VectorInterp)
				{
					TimelineObj->VectorTracks.RemoveAt(TrackId.TrackIndex);
				}
				else if (TrackType == FTTTrackBase::TT_LinearColorInterp)
				{
					TimelineObj->LinearColorTracks.RemoveAt(TrackId.TrackIndex);
				}

				// Refresh the node that owns this timeline template to remove pin
				TimelineNode->ReconstructNode();
				Kismet2->RefreshEditors();

				//rebuild the widgets!
				OnTimelineChanged();
				TrackListView->RebuildList();
			}
			else
			{
				FNotificationInfo Info( LOCTEXT( "InvalidTimelineNodeDestroy","Failed to destroy track. Timeline node is invalid. Please remove timeline node." ) );
				Info.ExpireDuration = 3.0f;
				Info.bUseLargeFont = false;
				TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
				if ( Notification.IsValid() )
				{
					Notification->SetCompletionState( SNotificationItem::CS_Fail );
				}
			}
		}
	}
}

UTimelineTemplate* STimelineEditor::GetTimeline()
{
	return TimelineObj;
}

void STimelineEditor::OnTimelineChanged()
{
	TrackList.Empty();

	TSharedPtr<FTimelineEdTrack> NewlyCreatedTrack;

	// If we have a timeline,
	if(TimelineObj != nullptr)
	{
		// Iterate over tracks and create entries in the array that drives the list widget
		for (int32 i = 0; i < TimelineObj->GetNumDisplayTracks(); ++i)
		{
			FTTTrackId TrackId = TimelineObj->GetDisplayTrackId(i);

			TSharedRef<FTimelineEdTrack> Track = FTimelineEdTrack::Make(i);
			TrackList.Add(Track);

			FTTTrackBase* TrackBase = TimelineEditorHelpers::GetTrackFromTimeline(TimelineObj, Track);
			if (TrackBase->GetTrackName() == NewTrackPendingRename)
			{
				NewlyCreatedTrack = Track;
			}
		}
	}

	TrackListView->RequestListRefresh();

	TrackListView->RequestScrollIntoView(NewlyCreatedTrack);
}

void STimelineEditor::OnItemScrolledIntoView( TSharedPtr<FTimelineEdTrack> InTrackNode, const TSharedPtr<ITableRow>& InWidget )
{
	if(NewTrackPendingRename != NAME_None)
	{
		InTrackNode->OnRenameRequest.ExecuteIfBound();
		NewTrackPendingRename = NAME_None;
	}
}

ECheckBoxState STimelineEditor::IsAutoPlayChecked() const
{
	return (TimelineObj && TimelineObj->bAutoPlay) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void STimelineEditor::OnAutoPlayChanged(ECheckBoxState NewType)
{
	if(TimelineObj)
	{
		TimelineObj->bAutoPlay = (NewType == ECheckBoxState::Checked) ? true : false;

		// Refresh the node that owns this timeline template to cache play status
		TSharedPtr<FBlueprintEditor> Kismet2 = Kismet2Ptr.Pin();
		UBlueprint* Blueprint = Kismet2->GetBlueprintObj();

		UK2Node_Timeline* TimelineNode = FBlueprintEditorUtils::FindNodeForTimeline(Blueprint, TimelineObj);
		if (TimelineNode)
		{
			TimelineNode->bAutoPlay = TimelineObj->bAutoPlay;

			// Mark blueprint as modified
			FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
		}
	}
}


ECheckBoxState STimelineEditor::IsLoopChecked() const
{
	return (TimelineObj && TimelineObj->bLoop) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void STimelineEditor::OnLoopChanged(ECheckBoxState NewType)
{
	if(TimelineObj)
	{
		TimelineObj->bLoop = (NewType == ECheckBoxState::Checked) ? true : false;

		// Refresh the node that owns this timeline template to cache play status
		TSharedPtr<FBlueprintEditor> Kismet2 = Kismet2Ptr.Pin();
		UBlueprint* Blueprint = Kismet2->GetBlueprintObj();

		UK2Node_Timeline* TimelineNode = FBlueprintEditorUtils::FindNodeForTimeline(Blueprint, TimelineObj);
		if (TimelineNode)
		{
			TimelineNode->bLoop = TimelineObj->bLoop;

			// Mark blueprint as modified
			FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
		}
	}
}

ECheckBoxState STimelineEditor::IsReplicatedChecked() const
{
	return (TimelineObj && TimelineObj->bReplicated) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void STimelineEditor::OnReplicatedChanged(ECheckBoxState NewType)
{
	if(TimelineObj)
	{
		TimelineObj->bReplicated = (NewType == ECheckBoxState::Checked) ? true : false;

		// Refresh the node that owns this timeline template to cache replicated status
		TSharedPtr<FBlueprintEditor> Kismet2 = Kismet2Ptr.Pin();
		UBlueprint* Blueprint = Kismet2->GetBlueprintObj();

		UK2Node_Timeline* TimelineNode = FBlueprintEditorUtils::FindNodeForTimeline(Blueprint, TimelineObj);
		if (TimelineNode)
		{
			TimelineNode->bReplicated = TimelineObj->bReplicated;

			// Mark blueprint as modified
			FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
		}
	}
}

ECheckBoxState STimelineEditor::IsUseLastKeyframeChecked() const
{
	return (TimelineObj && TimelineObj->LengthMode == ETimelineLengthMode::TL_LastKeyFrame) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void STimelineEditor::OnUseLastKeyframeChanged(ECheckBoxState NewType)
{
	if(TimelineObj)
	{
		TimelineObj->LengthMode = (NewType == ECheckBoxState::Checked) ? ETimelineLengthMode::TL_LastKeyFrame : ETimelineLengthMode::TL_TimelineLength;

		// Mark blueprint as modified
		FBlueprintEditorUtils::MarkBlueprintAsModified(Kismet2Ptr.Pin()->GetBlueprintObj());
	}
}


ECheckBoxState STimelineEditor::IsIgnoreTimeDilationChecked() const
{
	return (TimelineObj && TimelineObj->bIgnoreTimeDilation) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void STimelineEditor::OnIgnoreTimeDilationChanged(ECheckBoxState NewType)
{
	if (TimelineObj)
	{
		TimelineObj->bIgnoreTimeDilation = (NewType == ECheckBoxState::Checked) ? true : false;

		// Refresh the node that owns this timeline template to cache play status
		TSharedPtr<FBlueprintEditor> Kismet2 = Kismet2Ptr.Pin();
		UBlueprint* Blueprint = Kismet2->GetBlueprintObj();

		// Mark blueprint as modified
		FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);

		UK2Node_Timeline* TimelineNode = FBlueprintEditorUtils::FindNodeForTimeline(Blueprint, TimelineObj);
		if (TimelineNode)
		{
			TimelineNode->bIgnoreTimeDilation = TimelineObj->bIgnoreTimeDilation;
		}
	}
}

FText STimelineEditor::GetLengthString() const
{
	FString LengthString(TEXT("0.0"));
	if(TimelineObj != nullptr)
	{
		LengthString = FString::Printf(TEXT("%.2f"), TimelineObj->TimelineLength);
	}
	return FText::FromString(LengthString);
}

void STimelineEditor::OnLengthStringChanged(const FText& NewString, ETextCommit::Type CommitInfo)
{
	bool bCommitted = (CommitInfo == ETextCommit::OnEnter) || (CommitInfo == ETextCommit::OnUserMovedFocus);
	if(TimelineObj != nullptr && bCommitted)
	{
		float NewLength = FCString::Atof( *NewString.ToString() );
		if(NewLength > KINDA_SMALL_NUMBER)
		{
			TimelineObj->TimelineLength = NewLength;

			// Mark blueprint as modified
			FBlueprintEditorUtils::MarkBlueprintAsModified(Kismet2Ptr.Pin()->GetBlueprintObj());
		}
	}
}

bool STimelineEditor::OnVerifyTrackNameCommit(const FText& TrackName, FText& OutErrorMessage, FTTTrackBase* TrackBase, STimelineEdTrack* Track )
{
	FName RequestedName(  *TrackName.ToString() );
	bool bValid(true);

	if(TrackName.IsEmpty())
	{
		OutErrorMessage = LOCTEXT( "NameMissing_Error", "You must provide a name." );
		bValid = false;
	}
	else if(TrackBase->GetTrackName() != RequestedName && 
		false == TimelineObj->IsNewTrackNameValid(RequestedName))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("TrackName"), TrackName);
		OutErrorMessage = FText::Format(LOCTEXT("AlreadyInUse", "\"{TrackName}\" is already in use."), Args);
		bValid = false;
	}
	else
	{
		TSharedPtr<FBlueprintEditor> Kismet2 = Kismet2Ptr.Pin();
		UBlueprint* Blueprint = Kismet2->GetBlueprintObj();
		UK2Node_Timeline* TimelineNode = FBlueprintEditorUtils::FindNodeForTimeline(Blueprint, TimelineObj);
		if (TimelineNode)
		{
			for(TArray<UEdGraphPin*>::TIterator PinIt(TimelineNode->Pins);PinIt;++PinIt)
			{
				UEdGraphPin* Pin = *PinIt;

				if (Pin->PinName == RequestedName)
				{
					FFormatNamedArguments Args;
					Args.Add(TEXT("TrackName"), TrackName);
					OutErrorMessage = FText::Format(LOCTEXT("PinAlreadyInUse", "\"{TrackName}\" is already in use as a default pin!"), Args);
					bValid = false;
					break;
				}
			}
		}
	}

	return bValid;
}

void STimelineEditor::OnTrackNameCommitted( const FText& StringName, ETextCommit::Type /*CommitInfo*/, FTTTrackBase* TrackBase, STimelineEdTrack* Track )
{
	FName RequestedName( *StringName.ToString() );
	if( TimelineObj->IsNewTrackNameValid(RequestedName))
	{	
		TimelineObj->Modify();
		TSharedPtr<FBlueprintEditor> Kismet2 = Kismet2Ptr.Pin();
		UBlueprint* Blueprint = Kismet2->GetBlueprintObj();

		UK2Node_Timeline* TimelineNode = FBlueprintEditorUtils::FindNodeForTimeline(Blueprint, TimelineObj);
		
		if (TimelineNode)
		{
			// Start looking from the bottom of the list of pins, where user defined ones are stored.
			// It should not be possible to name pins to be the same as default pins, 
			// but in the case (fixes broken nodes) that they happen to be the same, this protects them
			for (int32 PinIdx = TimelineNode->Pins.Num() - 1; PinIdx >= 0; --PinIdx)
			{
				UEdGraphPin* Pin = TimelineNode->Pins[PinIdx];
			
				if (Pin->PinName == TrackBase->GetTrackName())
				{
					Pin->Modify();
					Pin->PinName = RequestedName;
					break;
				}
			}

			TrackBase->SetTrackName(RequestedName, TimelineObj);

			Kismet2->RefreshEditors();
			OnTimelineChanged();
		}
	}
}

void STimelineEditor::OnReorderTracks(int32 DisplayIndex, int32 DirectionDelta)
{
	if (TimelineObj != nullptr)
	{
		const FScopedTransaction Transaction(LOCTEXT("TimelineEditor_DeleteTrack", "Delete track"));

		TSharedPtr<FBlueprintEditor> Kismet2 = Kismet2Ptr.Pin();
		UBlueprint* Blueprint = Kismet2->GetBlueprintObj();
		UK2Node_Timeline* TimelineNode = FBlueprintEditorUtils::FindNodeForTimeline(Blueprint, TimelineObj);

		TimelineNode->Modify();
		TimelineObj->Modify();

		TimelineObj->MoveDisplayTrack(DisplayIndex, DirectionDelta);

		// Refresh the node that owns this timeline template to remove pin
		TimelineNode->ReconstructNode();
		Kismet2->RefreshEditors();
	}
}

bool STimelineEditor::IsCurveAssetSelected() const
{
	// Note: Cannot call GetContentBrowserSelectionClasses() during serialization and GC due to its use of FindObject()
	if(!UE::IsSavingPackage() && !IsGarbageCollecting())
	{
		TArray<UClass*> SelectionList;
		GEditor->GetContentBrowserSelectionClasses(SelectionList);

		for( int i=0; i<SelectionList.Num(); i++ )
		{
			UClass* Item = SelectionList[i];
			if( Item->IsChildOf(UCurveBase::StaticClass()))
			{
				return true;
			}
		}
	}
	
	return false;
}


void STimelineEditor::CreateNewTrackFromAsset()
{
	FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();
	UCurveBase* SelectedObj = GEditor->GetSelectedObjects()->GetTop<UCurveBase>();

	TSharedPtr<FBlueprintEditor> Kismet2 = Kismet2Ptr.Pin();
	UBlueprint* Blueprint = Kismet2->GetBlueprintObj();
	UK2Node_Timeline* TimelineNode = FBlueprintEditorUtils::FindNodeForTimeline(Blueprint, TimelineObj);
		
	if( SelectedObj && TimelineNode )
	{
		const FScopedTransaction Transaction( LOCTEXT( "TimelineEditor_CreateFromAsset", "Add new track from asset" ) );

		TimelineNode->Modify();
		TimelineObj->Modify();

		const FName TrackName = SelectedObj->GetFName();

		if(SelectedObj->IsA( UCurveFloat::StaticClass() ) )
		{
			UCurveFloat* FloatCurveObj = CastChecked<UCurveFloat>(SelectedObj);
			if( FloatCurveObj->bIsEventCurve )
			{
				FTTEventTrack NewEventTrack;
				NewEventTrack.SetTrackName(TrackName, TimelineObj);
				NewEventTrack.CurveKeys = CastChecked<UCurveFloat>(SelectedObj);
				NewEventTrack.bIsExternalCurve = true;

				TimelineObj->EventTracks.Add(NewEventTrack);
			}
			else
			{
				FTTFloatTrack NewFloatTrack;
				NewFloatTrack.SetTrackName(TrackName, TimelineObj);
				NewFloatTrack.CurveFloat = CastChecked<UCurveFloat>(SelectedObj);
				NewFloatTrack.bIsExternalCurve = true;

				TimelineObj->FloatTracks.Add(NewFloatTrack);
			}
		}
		else if(SelectedObj->IsA( UCurveVector::StaticClass() ))
		{
			FTTVectorTrack NewTrack;
			NewTrack.SetTrackName(TrackName, TimelineObj);
			NewTrack.CurveVector = CastChecked<UCurveVector>(SelectedObj);
			NewTrack.bIsExternalCurve = true;
			TimelineObj->VectorTracks.Add(NewTrack);
		}
		else if(SelectedObj->IsA( UCurveLinearColor::StaticClass() ))
		{
			FTTLinearColorTrack NewTrack;
			NewTrack.SetTrackName(TrackName, TimelineObj);
			NewTrack.CurveLinearColor = CastChecked<UCurveLinearColor>(SelectedObj);
			NewTrack.bIsExternalCurve = true;
			TimelineObj->LinearColorTracks.Add(NewTrack);
		}

		// Refresh the node that owns this timeline template to get new pin
		TimelineNode->ReconstructNode();
		Kismet2->RefreshEditors();
	}
}

bool STimelineEditor::CanRenameSelectedTrack() const
{
	return TrackListView->GetNumItemsSelected() == 1;
}

void STimelineEditor::OnRequestTrackRename() const
{
	check(TrackListView->GetNumItemsSelected() == 1);

	TrackListView->GetSelectedItems()[0]->OnRenameRequest.Execute();
}

FReply STimelineEditor::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	if(CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

TSharedPtr< SWidget > STimelineEditor::MakeContextMenu() const
{
	// Build up the menu
	FMenuBuilder MenuBuilder( true, CommandList );
	{
		MenuBuilder.AddMenuEntry( FGenericCommands::Get().Rename );
		MenuBuilder.AddMenuEntry( FGenericCommands::Get().Delete );
	}

	{
		TSharedRef<SWidget> SizeSlider = SNew(SSlider)
			.Value(this, &STimelineEditor::GetSizeScaleValue)
			.OnValueChanged(const_cast<STimelineEditor*>(this), &STimelineEditor::SetSizeScaleValue);

		MenuBuilder.AddWidget(SizeSlider, LOCTEXT("TimelineEditorVerticalSize", "Height"));
	}

	return MenuBuilder.MakeWidget();
}


TSharedRef<SWidget> STimelineEditor::MakeAddButton()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddFloatTrack", "Add Float Track"),
		LOCTEXT("AddFloatTrackToolTip", "Adds a Float Track."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "TimelineEditor.AddFloatTrack"),
		FUIAction(FExecuteAction::CreateRaw(this, &STimelineEditor::CreateNewTrack, FTTTrackBase::TT_FloatInterp)));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddVectorTrack", "Add Vector Track"),
		LOCTEXT("AddVectorTrackToolTip", "Adds a Vector Track."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "TimelineEditor.AddVectorTrack"),
		FUIAction(FExecuteAction::CreateRaw(this, &STimelineEditor::CreateNewTrack, FTTTrackBase::TT_VectorInterp)));
	
	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddEventTrack", "Add Event Track"),
		LOCTEXT("AddEventTrackToolTip", "Adds an Event Track."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "TimelineEditor.AddEventTrack"),
		FUIAction(FExecuteAction::CreateRaw(this, &STimelineEditor::CreateNewTrack, FTTTrackBase::TT_Event)));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddColorTrack", "Add Color Track"),
		LOCTEXT("AddColorTrackToolTip", "Adds a Color Track."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "TimelineEditor.AddColorTrack"),
		FUIAction(FExecuteAction::CreateRaw(this, &STimelineEditor::CreateNewTrack, FTTTrackBase::TT_LinearColorInterp)));

	FUIAction AddCurveAssetAction(FExecuteAction::CreateRaw(this, &STimelineEditor::CreateNewTrackFromAsset), FCanExecuteAction::CreateRaw(this, &STimelineEditor::IsCurveAssetSelected));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddExternalAsset", "Add Selected Curve Asset"),
		LOCTEXT("AddExternalAssetToolTip", "Add the currently selected curve asset."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "TimelineEditor.AddCurveAssetTrack"),
		AddCurveAssetAction);

	return MenuBuilder.MakeWidget();
}

FVector2D STimelineEditor::GetTimelineDesiredSize() const
{
	return FVector2D{ TimelineDesiredSize };
}

void STimelineEditor::SetSizeScaleValue(float NewValue)
{
	TimelineDesiredSize.Y = NominalTimelineDesiredHeight * (1.0f + NewValue * 5.0f);
	TrackListView->RequestListRefresh();
}

float STimelineEditor::GetSizeScaleValue() const
{
	return ((TimelineDesiredSize.Y / NominalTimelineDesiredHeight) - 1.0f) / 5.0f;
}

#undef LOCTEXT_NAMESPACE
