// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Engine/TimelineTemplate.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Math/Vector2D.h"
#include "Misc/Optional.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class FBlueprintEditor;
class FUICommandList;
class ITableRow;
class SCheckBox;
class SEditableTextBox;
class SInlineEditableTextBlock;
class STableViewBase;
class SWidget;
class SWindow;
class UCurveBase;
class UObject;
struct FAssetData;
struct FGeometry;
struct FKeyEvent;
struct FRichCurve;

//////////////////////////////////////////////////////////////////////////
// FTimelineEdTrack

/** Represents a single track on the timeline */
class FTimelineEdTrack
{
public:

public:
	/** The index of the curve (due to re-ordering) */
	int32 DisplayIndex;

	/** Trigger when a rename is requested on the track */
	FSimpleDelegate OnRenameRequest;

public:
	static TSharedRef<FTimelineEdTrack> Make(int32 DisplayIndex)
	{
		return MakeShareable(new FTimelineEdTrack(DisplayIndex));
	}

private:
	/** Hidden constructor, always use Make above */
	FTimelineEdTrack(int32 InDisplayIndex)
		: DisplayIndex(InDisplayIndex)
	{
	}

	/** Hidden constructor, always use Make above */
	FTimelineEdTrack() {}
};

//////////////////////////////////////////////////////////////////////////
// STimelineEdTrack

/** Widget for drawing a single track */
class STimelineEdTrack : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( STimelineEdTrack ){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FTimelineEdTrack> InTrack, TSharedPtr<class STimelineEditor> InTimelineEd);
private:
	/** Pointer to the underlying track information */
	TSharedPtr<FTimelineEdTrack> Track;

	/** Pointer back to the timeline editor widget */
	TWeakPtr<class STimelineEditor> TimelineEdPtr;

	/** Pointer to track widget for drawing keys */
	TSharedPtr<class SCurveEditor> TrackWidget;

	/**	Pointer to a window which prompts user to save internal curve as an external asset */
	TSharedPtr<SWindow> AssetCreationWindow;

	/**	Pointer to the curve */
	UCurveBase* CurveBasePtr;

	/** String to display external curve path as tooltip*/
	FString ExternalCurvePath;

	/** The local curve input min to use when the this tracks curve view isn't synchronized. */
	float LocalInputMin;

	/** The local curve input max to use when the this tracks curve view isn't synchronized. */
	float LocalInputMax;

	/** The local curve output min to use when the this tracks curve view isn't synchronized. */
	float LocalOutputMin;

	/** The local curve output max to use when the this tracks curve view isn't synchronized. */
	float LocalOutputMax;

	/**Function to destroy popup window*/
	void OnCloseCreateCurveWindow();

	/** Function to create curve asset inside the user specified package*/
	UCurveBase* CreateCurveAsset();

	/** Callback function to initiate the external curve asset creation process*/
	void OnCreateExternalCurve();

	/** Creates default asset path*/
	FString CreateUniqueCurveAssetPathName();

	/** Get the current external curve path*/
	FString GetExternalCurvePath( ) const;

	/** Function to replace internal curve with an external curve*/
	void SwitchToExternalCurve(UCurveBase* AssetCurvePtr);

	/** Function to update track with external curve reference*/
	void UseExternalCurve( UObject* AssetObj );

	/** Function to convert external curve to an internal curve*/
	void UseInternalCurve( );

	/** Callback function to replace external curve with an internal curve*/
	FReply OnClickClear();

	/**Function to reset external curve info*/
	void ResetExternalCurveInfo( );

	/** Function to copy data from one curve to another*/
	static void CopyCurveData( const FRichCurve* SrcCurve, FRichCurve* DestCurve );

	/** Gets whether this track is expanded. */
	ECheckBoxState GetIsExpandedState() const;
	/** Callback for when the expanded state for this track changes. */
	void OnIsExpandedStateChanged(ECheckBoxState IsExpandedState);

	/** Gets whether or not the content of this track should be visible, based on whether or not it's expanded. */
	EVisibility GetContentVisibility() const;

	/** Gets a check box state representing whether or not this track's curve view is synchronized with other tracks. */
	ECheckBoxState GetIsCurveViewSynchronizedState() const;
	/** Callback for when the check box state representing whether or not this track's curve view is synchronized with other tracks changes. */
	void OnIsCurveViewSynchronizedStateChanged(ECheckBoxState IsCurveViewSynchronized);

	/** Moves selected track up in the track list */
	FReply OnMoveUp();
	/** Checks if you can move the selected track up */
	bool CanMoveUp() const;

	/** Moves selected track down in the track list */
	FReply OnMoveDown();
	/** Checks if you can move the selected track down */
	bool CanMoveDown() const;

	void MoveTrack(int32 DirectionDelta);

	/** Get the minimum input for the curve view. */
	float GetMinInput() const;
	/** Get the maximum input for the curve view. */
	float GetMaxInput() const;

	/** Get the minimum output for the curve view. */
	float GetMinOutput() const;
	/** Get the maximum output for the curve view. */
	float GetMaxOutput() const;

	/** Callback to set the input view range for the curve editor. */
	void OnSetInputViewRange(float Min, float Max);
	/** Callback to set the output view range for the curve editor. */
	void OnSetOutputViewRange(float Min, float Max);
	/** Callback when the user picks a curve from the asset picker for a track */
	void OnChooseCurve(const FAssetData& InObject);

	//helper function to make getting expanded and synchronized state easier
	FTTTrackBase* GetTrackBase();
	const FTTTrackBase* GetTrackBase() const;

public:
	/** Inline block for changing name of track */
	TSharedPtr<SInlineEditableTextBlock> InlineNameBlock;
};

//////////////////////////////////////////////////////////////////////////
// STimelineEditor

/** Type used for list widget of tracks */
typedef SListView< TSharedPtr<FTimelineEdTrack> > STimelineEdTrackListType;

/** Overall timeline editing widget */
class STimelineEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( STimelineEditor ){}
	SLATE_END_ARGS()

private:
	/** List widget used for showing tracks */
	TSharedPtr<STimelineEdTrackListType> TrackListView;

	/** Underlying array of tracks, used by TrackListView */
	TArray< TSharedPtr<FTimelineEdTrack> > TrackList;

	/** Pointer back to owning Kismet 2 tool */
	TWeakPtr<FBlueprintEditor> Kismet2Ptr;

	/** Text box for editing length of timeline */
	TSharedPtr<SEditableTextBox> TimelineLengthEdit;

	/** If we want the timeline to loop */
	TSharedPtr<SCheckBox> LoopCheckBox;

	/** If we want the timeline to replicate */
	TSharedPtr<SCheckBox> ReplicatedCheckBox;

	/** If we want the timeline to auto-play */
	TSharedPtr<SCheckBox> PlayCheckBox;

	/** If we want the timeline to play to the full specified length, or just to the last keyframe of its curves */
	TSharedPtr<SCheckBox> UseLastKeyframeCheckBox;

	/** If we want the timeline to replicate */
	TSharedPtr<SCheckBox> IgnoreTimeDilationCheckBox;

	/** Pointer to the timeline object we are editing */
	UTimelineTemplate* TimelineObj;

	/** Minimum input shown for tracks */
	float ViewMinInput;

	/** Maximum input shown for tracks */
	float ViewMaxInput;

	/** Minimum output shown for tracks */
	float ViewMinOutput;

	/** Maximum output shown for tracks */
	float ViewMaxOutput;

	/** The default name of the last track created, used to identify which track needs to be renamed. */
	FName NewTrackPendingRename;

	/** The commandlist for the Timeline editor. */
	TSharedPtr< FUICommandList > CommandList;

	/** The current desired size of the timeline */
	FVector2f TimelineDesiredSize;

	/** The nominal desired height of a single timeline track at 1.0x height */
	float NominalTimelineDesiredHeight;
public:
	/** Get the timeline object that we are currently editing */
	UTimelineTemplate* GetTimeline();

	/** Called when the timeline changes to get the editor to refresh its state */
	void OnTimelineChanged();

	void Construct(const FArguments& InArgs, TSharedPtr<FBlueprintEditor> InKismet2, UTimelineTemplate* InTimelineObj);

	float GetViewMinInput() const;
	float GetViewMaxInput() const;
	float GetViewMinOutput() const;
	float GetViewMaxOutput() const;

	/** Return length of timeline */
	float GetTimelineLength() const;

	void SetInputViewRange(float InViewMinInput, float InViewMaxInput);
	void SetOutputViewRange(float InViewMinOutput, float InViewMaxOutput);

	/** When user attempts to commit the name of a track*/
	bool OnVerifyTrackNameCommit(const FText& TrackName, FText& OutErrorMessage, FTTTrackBase* TrackBase, STimelineEdTrack* Track );
	/** When user commits the name of a track*/
	void OnTrackNameCommitted(const FText& Name, ETextCommit::Type CommitInfo, FTTTrackBase* TrackBase, STimelineEdTrack* Track );

	/**Create curve object based on curve type*/
	UCurveBase* CreateNewCurve(FTTTrackBase::ETrackType Type );

	void OnReorderTracks(int32 DisplayIndex, int32 DirectionDelta);

	/** Gets the desired size for timelines */
	FVector2D GetTimelineDesiredSize() const;

	// SWidget interface
	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	// End of SWidget interface
private:
	/** Used by list view to create a track widget from the track item struct */
	TSharedRef<ITableRow> MakeTrackWidget( TSharedPtr<FTimelineEdTrack> Track, const TSharedRef<STableViewBase>& OwnerTable );
	/** Add a new track to the timeline */
	void CreateNewTrack(FTTTrackBase::ETrackType Type );

	/** Checks if the user can delete the selected tracks */
	bool CanDeleteSelectedTracks() const;
	/** Deletes the currently selected tracks */
	void OnDeleteSelectedTracks();

	/** Get the name of the timeline we are editing */
	FText GetTimelineName() const;

	/** Get state of autoplay box*/
	ECheckBoxState IsAutoPlayChecked() const;
	/** Handle play box being changed */
	void OnAutoPlayChanged(ECheckBoxState NewType);

	/** Get state of loop box */
	ECheckBoxState IsLoopChecked() const;
	/** Handle loop box being changed */
	void OnLoopChanged(ECheckBoxState NewType);

	/** Get state of replicated box */
	ECheckBoxState IsReplicatedChecked() const;
	/** Handle loop box being changed */
	void OnReplicatedChanged(ECheckBoxState NewType);

	/** Get state of the use last keyframe checkbox */
	ECheckBoxState IsUseLastKeyframeChecked() const;
	/** Handle toggling between use last keyframe and use length */
	void OnUseLastKeyframeChanged(ECheckBoxState NewType);

	/** Get state of replicated box */
	ECheckBoxState IsIgnoreTimeDilationChecked() const;
	/** Handle loop box being changed */
	void OnIgnoreTimeDilationChanged(ECheckBoxState NewType);

	/** Get current length of timeline as string */
	FText GetLengthString() const;
	/** Handle length string being changed */
	void OnLengthStringChanged(const FText& NewString, ETextCommit::Type CommitInfo);

	/** Function to check whether a curve asset is selected in content browser in order to enable "Add Curve Asset" button */
	bool IsCurveAssetSelected() const;

	/** Create new track from curve asset */
	void CreateNewTrackFromAsset();

	/** Callback when a track item is scrolled into view */
	void OnItemScrolledIntoView( TSharedPtr<FTimelineEdTrack> InTrackNode, const TSharedPtr<ITableRow>& InWidget );

	/** Callback when the TimelineTickGroup is changed via Editor controls */
	void OnTimelineTickGroupChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo);

	/** Checks if you can rename the selected track */
	bool CanRenameSelectedTrack() const;
	/** Informs the selected track that the user wants to rename it */
	void OnRequestTrackRename() const;

	/** Creates the right click context menu for the track list */
	TSharedPtr< SWidget > MakeContextMenu() const;

	void SetSizeScaleValue(float NewValue);
	float GetSizeScaleValue() const;

	TSharedRef<SWidget> MakeAddButton();
};

