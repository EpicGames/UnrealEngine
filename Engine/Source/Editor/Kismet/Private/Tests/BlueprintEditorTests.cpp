// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/AutomationTest.h"
#include "Modules/ModuleManager.h"
#include "UObject/Class.h"
#include "UObject/Package.h"
#include "Misc/PackageName.h"
#include "Framework/Commands/InputChord.h"
#include "Templates/SubclassOf.h"
#include "EdGraph/EdGraphPin.h"
#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"
#include "Engine/Blueprint.h"
#include "Components/StaticMeshComponent.h"
#include "EdGraph/EdGraph.h"
#include "Factories/BlueprintFactory.h"
#include "Particles/ParticleSystem.h"
#include "Engine/StaticMesh.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "AssetRegistry/AssetData.h"
#include "Editor.h"
#include "UObject/SavePackage.h"

#include "Kismet2/KismetEditorUtilities.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphSchema_K2_Actions.h"
#include "K2Node_Event.h"
#include "K2Node_CallFunction.h"
#include "K2Node_AddComponent.h"
#include "K2Node_CustomEvent.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_VariableGet.h"
#include "K2Node_VariableSet.h"
#include "Engine/SCS_Node.h"
#include "BlueprintEditorModes.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "ComponentAssetBroker.h"
#include "AssetRegistry/ARFilter.h"

#include "ScopedTransaction.h"
#include "ObjectTools.h"

// Automation
#include "AssetRegistry/AssetRegistryModule.h"
#include "Tests/AutomationTestSettings.h"
#include "Tests/AutomationEditorCommon.h"
#include "Tests/AutomationEditorPromotionCommon.h"
#include "Subsystems/AssetEditorSubsystem.h"

#if WITH_DEV_AUTOMATION_TESTS

#define LOCTEXT_NAMESPACE "BlueprintEditorPromotionTests"

DEFINE_LOG_CATEGORY_STATIC(LogBlueprintEditorPromotionTests, Log, All);

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FBlueprintEditorPromotionTest, "System.Promotion.Editor.Blueprint Editor", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

/**
* Helper functions used by the blueprint editor promotion automation test
*/
namespace BlueprintEditorPromotionUtils
{
	/**
	* Constants
	*/
	static const FString BlueprintNameString = TEXT("BlueprintEditorPromotionBlueprint");
	static const FName BlueprintStringVariableName(TEXT("MyStringVariable"));

	/**
	* Gets the full path to the folder on disk
	*/
	static FString GetFullPath()
	{
		return FPackageName::FilenameToLongPackageName(FPaths::ProjectContentDir() + TEXT("BlueprintEditorPromotionTest"));
	}

	/**
	* Helper class to track once a certain time has passed
	*/
	class FDelayHelper
	{
	public:
		/**
		* Constructor
		*/
		FDelayHelper() :
			bIsRunning(false),
			StartTime(0.f),
			Duration(0.f)
		{
		}

		/**
		* Returns if the delay is still running
		*/
		bool IsRunning()
		{
			return bIsRunning;
		}

		/**
		* Sets the helper state to not running
		*/
		void Reset()
		{
			bIsRunning = false;
		}

		/**
		* Starts the delay timer
		*
		* @param InDuration - How long to delay for in seconds
		*/
		void Start(double InDuration)
		{
			bIsRunning = true;
			StartTime = FPlatformTime::Seconds();
			Duration = InDuration;
		}

		/**
		* Returns true if the desired amount of time has passed
		*/
		bool IsComplete()
		{
			if (IsRunning())
			{
				const double CurrentTime = FPlatformTime::Seconds();
				return CurrentTime - StartTime >= Duration;
			}
			else
			{
				return false;
			}
		}

	private:
		/** If true, this delay timer is active */
		bool bIsRunning;
		/** The time the delay started */
		double StartTime;
		/** How long the timer is for */
		double Duration;
	};

	/**
	* Sends the AssetEditor->SaveAsset UI command
	*/
	static void SendBlueprintResetViewCommand()
	{
		const FString Context = TEXT("BlueprintEditor");
		const FString Command = TEXT("ResetCamera");

		FInputChord CurrentSaveChord = FEditorPromotionTestUtilities::GetOrSetUICommand(Context, Command);

		const FName FocusWidgetType(TEXT("SSCSEditorViewport"));
		FEditorPromotionTestUtilities::SendCommandToCurrentEditor(CurrentSaveChord, FocusWidgetType);
	}

	/**
	* Compiles the blueprint
	*
	* @param InBlueprint - The blueprint to compile
	*/
	static void CompileBlueprint(UBlueprint* InBlueprint)
	{
		FBlueprintEditorUtils::RefreshAllNodes(InBlueprint);

		FKismetEditorUtilities::CompileBlueprint(InBlueprint, EBlueprintCompileOptions::SkipGarbageCollection);
		if (InBlueprint->Status == EBlueprintStatus::BS_UpToDate)
		{
			UE_LOGF(LogBlueprintEditorPromotionTests, Display, "Blueprint compiled successfully (%ls)", *InBlueprint->GetName());
		}
		else if (InBlueprint->Status == EBlueprintStatus::BS_UpToDateWithWarnings)
		{
			UE_LOGF(LogBlueprintEditorPromotionTests, Display, "Blueprint compiled successfully with warnings(%ls)", *InBlueprint->GetName());
		}
		else if (InBlueprint->Status == EBlueprintStatus::BS_Error)
		{
			UE_LOGF(LogBlueprintEditorPromotionTests, Display, "Blueprint failed to compile (%ls)", *InBlueprint->GetName());
		}
		else
		{
			UE_LOGF(LogBlueprintEditorPromotionTests, Error, "Blueprint is in an unexpected state after compiling (%ls)", *InBlueprint->GetName());
		}
	}

	/**
	* Creates a blueprint component based off the supplied asset
	*
	* @param InBlueprint - The blueprint to modify
	* @param InAsset - The asset to use for the component
	*/
	static USCS_Node* CreateBlueprintComponent(UBlueprint* InBlueprint, UObject* InAsset)
	{
		IAssetEditorInstance* OpenEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(InBlueprint, true);
		FBlueprintEditor* CurrentBlueprintEditor = (FBlueprintEditor*)OpenEditor;
		TSubclassOf<UActorComponent> ComponentClass = FComponentAssetBrokerage::GetPrimaryComponentForAsset(InAsset->GetClass());

		USCS_Node* NewNode = InBlueprint->SimpleConstructionScript->CreateNode(ComponentClass);
		// Assign the asset to the template
		FComponentAssetBrokerage::AssignAssetToComponent(NewNode->ComponentTemplate, InAsset);

		// Add node to the SCS
		TArray<USCS_Node*> AllNodes = InBlueprint->SimpleConstructionScript->GetAllNodes();
		USCS_Node* RootNode = AllNodes.Num() > 0 ? AllNodes[0] : NULL;
		if (!RootNode || (RootNode == InBlueprint->SimpleConstructionScript->GetDefaultSceneRootNode()))
		{
			//New Root
			InBlueprint->SimpleConstructionScript->AddNode(NewNode);
		}
		else
		{
			//Add as a child
			RootNode->AddChildNode(NewNode);
		}

		// Recompile skeleton because of the new component we added
		FKismetEditorUtilities::GenerateBlueprintSkeleton(InBlueprint, true);

		CurrentBlueprintEditor->UpdateSubobjectPreview(true);

		return NewNode;
	}

	/**
	* Sets a new component as the root
	*
	* @param InBlueprint - The blueprint to modify
	* @param NewRoot - The new root
	*/
	static void SetComponentAsRoot(UBlueprint* InBlueprint, USCS_Node* NewRoot)
	{
		// @FIXME: Current usages doesn't guarantee NewRoot is valid!!! Check first!
		//Get all the construction script nodes
		TArray<USCS_Node*> AllNodes = InBlueprint->SimpleConstructionScript->GetAllNodes();

		USCS_Node* OldRootNode = AllNodes[0];
		USCS_Node* OldParent = NULL;

		//Find old parent
		for (int32 NodeIndex = 0; NodeIndex < AllNodes.Num(); ++NodeIndex)
		{
			if (AllNodes[NodeIndex]->ChildNodes.Contains(NewRoot))
			{
				OldParent = AllNodes[NodeIndex];
				break;
			}
		}

		check(OldParent);

		//Remove the new root from its old parent and 
		OldParent->RemoveChildNode(NewRoot);
		NewRoot->Modify();
		NewRoot->AttachToName = NAME_None;

		//Remove the old root, add the new root, and attach the old root as a child
		InBlueprint->SimpleConstructionScript->RemoveNode(OldRootNode);
		InBlueprint->SimpleConstructionScript->AddNode(NewRoot);
		NewRoot->AddChildNode(OldRootNode);
	}

	/**
	* Removes a blueprint component from the simple construction script
	*
	* @param InBlueprint - The blueprint to modify
	* @param InNode - The node of the component to remove
	*/
	static void RemoveBlueprintComponent(UBlueprint* InBlueprint, USCS_Node* InNode)
	{
		if (InNode != NULL)
		{
			// Remove node from SCS tree
			InNode->GetSCS()->RemoveNodeAndPromoteChildren(InNode);

			// Clear the delegate
			InNode->SetOnNameChanged(FSCSNodeNameChanged());
		}
	}

	/**
	* Creates a new graph node from a given template
	*
	* @param NodeTemplate - The template to use for the node
	* @param InGraph - The graph to create the new node in
	* @param GraphLocation - The location to place the node
	* @param ConnectPin - The pin to connect the node to
	*/
	static UEdGraphNode* CreateNewGraphNodeFromTemplate(UK2Node* NodeTemplate, UEdGraph* InGraph, const FVector2f& GraphLocation, UEdGraphPin* ConnectPin = NULL)
	{
		TSharedPtr<FEdGraphSchemaAction_K2NewNode> Action = TSharedPtr<FEdGraphSchemaAction_K2NewNode>(new FEdGraphSchemaAction_K2NewNode(FText::GetEmpty(), FText::GetEmpty(), FText::GetEmpty(), 0));
		Action->NodeTemplate = NodeTemplate;

		return Action->PerformAction(InGraph, ConnectPin, GraphLocation, false);
	}

	/**
	* Creates an AddComponent action to the blueprint graph
	*
	* @param InBlueprint - The blueprint to modify
	* @param InGraph - The blueprint graph to use
	* @param InAsset - The asset to use
	*/
	static UEdGraphNode* CreateAddComponentActionNode(UBlueprint* InBlueprint, UEdGraph* InGraph, UObject* InAsset)
	{
		UEdGraph* TempOuter = NewObject<UEdGraph>((UObject*)InBlueprint);
		TempOuter->SetFlags(RF_Transient);

		const FScopedTransaction PropertyChanged(LOCTEXT("AddedGraphNode", "Added a graph node"));
		InGraph->Modify();

		// Make an add component node
		UK2Node_CallFunction* CallFuncNode = NewObject<UK2Node_AddComponent>(TempOuter);
		UFunction* AddComponentFn = FindFieldChecked<UFunction>(AActor::StaticClass(), UK2Node_AddComponent::GetAddComponentFunctionName());
		CallFuncNode->FunctionReference.SetFromField<UFunction>(AddComponentFn, FBlueprintEditorUtils::IsActorBased(InBlueprint));

		UEdGraphNode* NewNode = CreateNewGraphNodeFromTemplate(CallFuncNode, InGraph, FVector2f(200.0f, 0.0f));

		TSubclassOf<UActorComponent> ComponentClass = InAsset ? FComponentAssetBrokerage::GetPrimaryComponentForAsset(InAsset->GetClass()) : NULL;
		if ((NewNode != NULL) && (InBlueprint != NULL))
		{
			UK2Node_AddComponent* AddCompNode = CastChecked<UK2Node_AddComponent>(NewNode);

			ensure(NULL != Cast<UBlueprintGeneratedClass>(InBlueprint->GeneratedClass));
			// Then create a new template object, and add to array in
			UActorComponent* NewTemplate = NewObject<UActorComponent>(InBlueprint->GeneratedClass, ComponentClass, NAME_None, RF_ArchetypeObject | RF_Public);
			InBlueprint->ComponentTemplates.Add(NewTemplate);

			// Set the name of the template as the default for the TemplateName param
			UEdGraphPin* TemplateNamePin = AddCompNode->GetTemplateNamePinChecked();
			if (TemplateNamePin)
			{
				TemplateNamePin->DefaultValue = NewTemplate->GetName();
			}

			// Set the return type to be the type of the template
			UEdGraphPin* ReturnPin = AddCompNode->GetReturnValuePin();
			if (ReturnPin)
			{
				ReturnPin->PinType.PinSubCategoryObject = *ComponentClass;
			}

			// Set the asset
			if (InAsset != NULL)
			{
				FComponentAssetBrokerage::AssignAssetToComponent(NewTemplate, InAsset);
			}

			AddCompNode->ReconstructNode();
		}

		FBlueprintEditorUtils::MarkBlueprintAsModified(InBlueprint);
		return NewNode;
	}

	/**
	* Creates a SetStaticMesh action in the blueprint graph
	*
	* @param InBlueprint - The blueprint to modify
	* @param InGraph - The blueprint graph to use
	*/
	static UEdGraphNode* AddSetStaticMeshNode(UBlueprint* InBlueprint, UEdGraph* InGraph)
	{
		UEdGraph* TempOuter = NewObject<UEdGraph>((UObject*)InBlueprint);
		TempOuter->SetFlags(RF_Transient);

		// Make a call function template
		UK2Node_CallFunction* CallFuncNode = NewObject<UK2Node_CallFunction>(TempOuter);
		static FName PrintStringFunctionName(TEXT("SetStaticMesh"));
		UFunction* DelayFn = FindFieldChecked<UFunction>(UStaticMeshComponent::StaticClass(), PrintStringFunctionName);
		CallFuncNode->FunctionReference.SetFromField<UFunction>(DelayFn, false);

		return CreateNewGraphNodeFromTemplate(CallFuncNode, InGraph, FVector2f(850.0f, 0.0f));
	}

	/**
	* Connects two nodes using the supplied pin names
	*
	* @param NodeA - The first node to connect
	* @param PinAName - The name of the pin on the first node
	* @param NodeB - The second node to connect
	* @param PinBName - The name of the pin on the second node
	*/
	static void ConnectGraphNodes(UEdGraphNode* NodeA, const FName& PinAName, UEdGraphNode* NodeB, const FName& PinBName)
	{
		const FScopedTransaction PropertyChanged(LOCTEXT("ConnectedNode", "Connected graph nodes"));
		NodeA->GetGraph()->Modify();

		UEdGraphPin* PinA = NodeA->FindPin(PinAName);
		UEdGraphPin* PinB = NodeB->FindPin(PinBName);

		if (PinA && PinB)
		{
			PinA->MakeLinkTo(PinB);
		}
		else
		{
			UE_LOGF(LogBlueprintEditorPromotionTests, Error, "Could not connect pins %ls and %ls ", *PinAName.ToString(), *PinBName.ToString());
		}
	}
	
	/**
	* Promotes a pin to a variable
	*
	* @param InBlueprint - The blueprint to modify
	* @param Node - The node that owns the pin
	* @param PinName - The name of the pin to promote
	*/
	static void PromotePinToVariable(UBlueprint* InBlueprint, UEdGraphNode* Node, const FName& PinName)
	{
		IAssetEditorInstance* OpenEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(InBlueprint, true);
		FBlueprintEditor* CurrentBlueprintEditor = (FBlueprintEditor*)OpenEditor;

		UEdGraphPin* PinToPromote = Node->FindPin(PinName);
		CurrentBlueprintEditor->DoPromoteToVariable2f(InBlueprint, PinToPromote, true);
	}

	/**
	* Creates a ReceiveBeginPlay event node
	*
	* @param InBlueprint - The blueprint to modify
	* @param InGraph - The graph to use for the new node
	*/
	static UEdGraphNode* CreatePostBeginPlayEvent(UBlueprint* InBlueprint, UEdGraph* InGraph)
	{
		UEdGraph* TempOuter = NewObject<UEdGraph>((UObject*)InBlueprint);
		TempOuter->SetFlags(RF_Transient);

		// Make an add component node
		UK2Node_Event* NewEventNode = NewObject<UK2Node_Event>(TempOuter);
		NewEventNode->EventReference.SetExternalMember(FName(TEXT("ReceiveBeginPlay")), AActor::StaticClass());
		NewEventNode->bOverrideFunction = true;

		//Check for existing events
		UK2Node_Event* ExistingEvent = FBlueprintEditorUtils::FindOverrideForFunction(InBlueprint, NewEventNode->EventReference.GetMemberParentClass(NewEventNode->GetBlueprintClassFromNode()), NewEventNode->EventReference.GetMemberName());

		if (!ExistingEvent)
		{
			return CreateNewGraphNodeFromTemplate(NewEventNode, InGraph, FVector2f(200.0f, 0.0f));
		}
		return ExistingEvent;
	}

	/**
	* Creates a custom event node
	*
	* @param InBlueprint - The blueprint to modify
	* @param InGraph - The graph to use for the new node
	* @param EventName - The name of the event
	*/
	static UEdGraphNode* CreateCustomEvent(UBlueprint* InBlueprint, UEdGraph* InGraph, const FString& EventName)
	{
		UEdGraph* TempOuter = NewObject<UEdGraph>((UObject*)InBlueprint);
		TempOuter->SetFlags(RF_Transient);

		// Make an add component node
		UK2Node_CustomEvent* NewEventNode = NewObject<UK2Node_CustomEvent>(TempOuter);
		NewEventNode->CustomFunctionName = "EventName";

		return CreateNewGraphNodeFromTemplate(NewEventNode, InGraph, FVector2f(1200.0f, 0.0f));
	}

	/**
	* Creates a node template for a UKismetSystemLibrary function
	*
	* @param NodeOuter - The outer to use for the template
	* @param InGraph - The function to use for the node
	*/
	static UK2Node* CreateKismetFunctionTemplate(UObject* NodeOuter, const FName& FunctionName)
	{
		// Make a call function template
		UK2Node_CallFunction* CallFuncNode = NewObject<UK2Node_CallFunction>(NodeOuter);
		UFunction* Function = FindFieldChecked<UFunction>(UKismetSystemLibrary::StaticClass(), FunctionName);
		CallFuncNode->FunctionReference.SetFromField<UFunction>(Function, false);
		return CallFuncNode;
	}

	/**
	* Creates a delay node
	*
	* @param InBlueprint - The blueprint to modify
	* @param InGraph - The graph to use for the new node
	* @param ConnectPin - The pin to connect the new node to
	*/
	static UEdGraphNode* AddDelayNode(UBlueprint* InBlueprint, UEdGraph* InGraph, UEdGraphPin* ConnectPin = NULL)
	{
		UEdGraph* TempOuter = NewObject<UEdGraph>((UObject*)InBlueprint);
		TempOuter->SetFlags(RF_Transient);

		const FScopedTransaction PropertyChanged(LOCTEXT("AddedGraphNode", "Added a graph node"));
		InGraph->Modify();

		// Make a call function template
		static FName DelayFunctionName(TEXT("Delay"));
		UK2Node* CallFuncNode = CreateKismetFunctionTemplate(TempOuter, DelayFunctionName);

		//Create the node
		return CreateNewGraphNodeFromTemplate(CallFuncNode, InGraph, FVector2f(400.0f, 0.0f), ConnectPin);
	}

	/**
	* Creates a PrintString node
	*
	* @param InBlueprint - The blueprint to modify
	* @param InGraph - The graph to use for the new node
	* @param ConnectPin - The pin to connect the new node to
	*/
	static UEdGraphNode* AddPrintStringNode(UBlueprint* InBlueprint, UEdGraph* InGraph, UEdGraphPin* ConnectPin = NULL)
	{
		UEdGraph* TempOuter = NewObject<UEdGraph>((UObject*)InBlueprint);
		TempOuter->SetFlags(RF_Transient);

		// Make a call function template
		static FName PrintStringFunctionName(TEXT("PrintString"));
		UK2Node* CallFuncNode = CreateKismetFunctionTemplate(TempOuter, PrintStringFunctionName);

		return CreateNewGraphNodeFromTemplate(CallFuncNode, InGraph, FVector2f(680.0f, 0.0f), ConnectPin);
	}

	/**
	* Creates a call function node
	*
	* @param InBlueprint - The blueprint to modify
	* @param InGraph - The graph to use for the new node
	* @param FunctionName - The name of the function to call
	* @param ConnectPin - The pin to connect the new node to
	*/
	static UEdGraphNode* AddCallFunctionGraphNode(UBlueprint* InBlueprint, UEdGraph* InGraph, const FName& FunctionName, UEdGraphPin* ConnectPin = NULL)
	{
		UEdGraph* TempOuter = NewObject<UEdGraph>((UObject*)InBlueprint);
		TempOuter->SetFlags(RF_Transient);

		// Make a call function template
		UK2Node_CallFunction* CallFuncNode = NewObject<UK2Node_CallFunction>(TempOuter);
		CallFuncNode->FunctionReference.SetSelfMember(FunctionName);

		return CreateNewGraphNodeFromTemplate(CallFuncNode, InGraph, FVector2f(1200.0f, 0.0f), ConnectPin);
	}

	/**
	* Creates Get or Set node
	*
	* @param InBlueprint - The blueprint to modify
	* @param InGraph - The graph to use for the new node
	* @param VarName - The name of the variable to use
	* @param bGet - If true, create a Get node.  If false, create a Set node.
	* @param XOffset - How far to offset the node in the graph
	*/
	static UEdGraphNode* AddGetSetNode(UBlueprint* InBlueprint, UEdGraph* InGraph, const FString& VarName, bool bGet, float XOffset = 0.f)
	{
		const FScopedTransaction PropertyChanged(LOCTEXT("AddedGraphNode", "Added a graph node"));
		InGraph->Modify();

		FEdGraphSchemaAction_K2NewNode NodeInfo;
		// Create get or set node, depending on whether we clicked on an input or output pin
		UK2Node_Variable* TemplateNode = NULL;
		if (bGet)
		{
			TemplateNode = NewObject<UK2Node_VariableGet>();
		}
		else
		{
			TemplateNode = NewObject<UK2Node_VariableSet>();
		}

		TemplateNode->VariableReference.SetSelfMember(FName(*VarName));
		NodeInfo.NodeTemplate = TemplateNode;

		return NodeInfo.PerformAction(InGraph, NULL, FVector2f(XOffset, 130.0f), true);
	}

	/**
	* Sets the default value for a pin
	*
	* @param Node - The node that owns the pin to set
	* @param PinName - The name of the pin
	* @param PinValue - The new default value
	*/
	static void SetPinDefaultValue(UEdGraphNode* Node, const FName PinName, const FString& PinValue)
	{
		UEdGraphPin* Pin = Node->FindPin(PinName);
		Pin->DefaultValue = PinValue;
	}

	/**
	* Sets the default object for a pin
	*
	* @param Node - The node that owns the pin to set
	* @param PinName - The name of the pin
	* @param PinObject - The new default object
	*/
	static void SetPinDefaultObject(UEdGraphNode* Node, const FName PinName, UObject* PinObject)
	{
		UEdGraphPin* Pin = Node->FindPin(PinName);
		Pin->DefaultObject = PinObject;
	}

	/**
	* Adds a string member variable to a blueprint
	*
	* @param InBlueprint - The blueprint to modify
	* @param VariableName - The name of the new string variable
	*/
	static void AddStringMemberValue(UBlueprint* InBlueprint, const FName& VariableName)
	{
		FEdGraphPinType StringPinType(UEdGraphSchema_K2::PC_String, NAME_None, nullptr, EPinContainerType::None, false, FEdGraphTerminalType());
		FBlueprintEditorUtils::AddMemberVariable(InBlueprint, VariableName, StringPinType);
	}

	/**
	* Creates a new function graph
	*
	* @param InBlueprint - The blueprint to modify
	* @param FunctionName - The function name to use for the new graph
	*/
	static UEdGraph* CreateNewFunctionGraph(UBlueprint* InBlueprint, const FName& FunctionName)
	{
		UEdGraph* NewGraph = FBlueprintEditorUtils::CreateNewGraph(InBlueprint, FunctionName, UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());
		FBlueprintEditorUtils::AddFunctionGraph<UClass>(InBlueprint, NewGraph, /*bIsUserCreated=*/ true, NULL);
		return NewGraph;
	}
}


/**
* Container for items related to the blueprint editor test
*/
namespace BlueprintEditorPromotionTestHelper
{
	/**
	* The main build promotion test class
	*/
	struct FBlueprintEditorPromotionTestHelper
	{
		/** Pointer to running automation test instance */
		FBlueprintEditorPromotionTest* Test;
		
		/** Function definition for the test stage functions */
		typedef bool(BlueprintEditorPromotionTestHelper::FBlueprintEditorPromotionTestHelper::*TestStageFn)();

		/** List of test stage functions */
		TArray<TestStageFn> TestStages;
		TArray<FString> StageNames;

		/** The index of the test stage we are on */
		int32 CurrentStage;

		/** Pointer to the active editor world */
		UWorld* CurrentWorld;

		/** Particle System loaded from Automation Settings for Blueprint Pass */
		UParticleSystem* LoadedParticleSystem;

		/** Meshes to use for the blueprint */
		UStaticMesh* FirstBlueprintMesh;
		UStaticMesh* SecondBlueprintMesh;

		/** Objects created by the Blueprint stages */
		UBlueprint* BlueprintObject;
		UPackage* BlueprintPackage;
		UEdGraph* CustomGraph;
		USCS_Node* MeshNode;
		USCS_Node* OtherMeshNode;
		USCS_Node* PSNode;
		UEdGraphNode* AddMeshNode;
		UEdGraphNode* PostBeginPlayEventNode;
		UEdGraphNode* DelayNode;
		UEdGraphNode* SetNode;
		UEdGraphNode* GetNode;
		UEdGraphNode* PrintNode;
		UEdGraphNode* SetStaticMeshNode;
		UEdGraphNode* CustomEventNode;
		UEdGraphNode* AddParticleSystemNode;
		UEdGraphNode* CallFunctionNode;

		/** Delay helper */
		BlueprintEditorPromotionUtils::FDelayHelper DelayHelper;

		/** List of skipped tests */
		TArray<FString> SkippedTests;

		/** summary logs to display at the end */
		TArray<FString> SummaryLines;

		/** Keeps track of errors generated by building the map */
		int32 BuildStartErrorCount;

		int32 SectionSuccessCount;
		int32 SectionTestCount;

		int32 SectionErrorCount;
		int32 SectionWarningCount;
		int32 SectionLogCount;

#define ADD_TEST_STAGE(FuncName,StageName) \
	TestStages.Add(&BlueprintEditorPromotionTestHelper::FBlueprintEditorPromotionTestHelper::FuncName); \
	StageNames.Add(StageName); 

		/**
		* Constructor
		*/
		FBlueprintEditorPromotionTestHelper()
			: CurrentStage(0)
		{
			FMemory::Memzero(this, sizeof(FBlueprintEditorPromotionTestHelper));

			ADD_TEST_STAGE(Cleanup, TEXT("Pre-start cleanup"));
			ADD_TEST_STAGE(Setup, TEXT("Setup"));
			ADD_TEST_STAGE(Blueprint_CreateNewBlueprint_Part1, TEXT("Create a new Blueprint"));
			ADD_TEST_STAGE(Blueprint_CreateNewBlueprint_Part2, TEXT("Create a new Blueprint"));
			ADD_TEST_STAGE(Blueprint_DataOnlyBlueprint_Part1, TEXT("Data-only Blueprint"));
			ADD_TEST_STAGE(Blueprint_DataOnlyBlueprint_Part2, TEXT("Data-only Blueprint"));
			ADD_TEST_STAGE(Blueprint_DataOnlyBlueprint_Part3, TEXT("Data-only Blueprint"));
			ADD_TEST_STAGE(Blueprint_ComponentsMode_Part1, TEXT("Components Mode"));
			ADD_TEST_STAGE(Blueprint_ComponentsMode_Part2, TEXT("Components Mode"));
			ADD_TEST_STAGE(Blueprint_ConstructionScript, TEXT("Construction Script"));
			ADD_TEST_STAGE(Blueprint_PromoteVariable_Part1, TEXT("Variable from Component Mode 1"));
			ADD_TEST_STAGE(Blueprint_PromoteVariable_Part2, TEXT("Variable from Component Mode 2"));
			//ADD_TEST_STAGE(Blueprint_PromoteVariable_Part3, TEXT("Variable from Component Mode 3"));
			ADD_TEST_STAGE(Blueprint_EventGraph, TEXT("Event Graph"));
			ADD_TEST_STAGE(Blueprint_CustomVariable, TEXT("Custom Variables"));
			ADD_TEST_STAGE(Blueprint_UsingVariables, TEXT("Using Variables"));
			ADD_TEST_STAGE(Blueprint_RenameCustomEvent, TEXT("Renaming Custom Event"));
			ADD_TEST_STAGE(Blueprint_NewFunctions, TEXT("New Function"));
			ADD_TEST_STAGE(Blueprint_CompleteBlueprint, TEXT("Completing the Blueprint"));
			ADD_TEST_STAGE(Cleanup, TEXT("Teardown"));
		}

		/**
		* Runs the current test stage
		*/
		bool Update()
		{
			Test->PushContext(StageNames[CurrentStage]);
			bool bStageComplete = (this->*TestStages[CurrentStage])();
			Test->PopContext();

			if (bStageComplete)
			{
				CurrentStage++;
			}

			return CurrentStage >= TestStages.Num();
		}

	private:

		bool Cleanup()
		{
			//Make sure no editors are open
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllAssetEditors();

			//remove all assets in the build package
			// Load the asset registry module
			IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();

			// Form a filter from the paths
			FARFilter Filter;
			Filter.bRecursivePaths = true;
			new (Filter.PackagePaths) FName(*FEditorPromotionTestUtilities::GetGamePath());

			// Query for a list of assets in the selected paths
			TArray<FAssetData> AssetList;
			AssetRegistry.GetAssets(Filter, AssetList);

			// Clear and try to delete all assets
			for (int32 AssetIdx = 0; AssetIdx < AssetList.Num(); ++AssetIdx)
			{
				Test->AddInfo(*FString::Printf(TEXT("Removing asset: %s"), *AssetList[AssetIdx].AssetName.ToString()));
				if (AssetList[AssetIdx].IsAssetLoaded())
				{
					UObject* LoadedAsset = AssetList[AssetIdx].GetAsset();
					AssetRegistry.AssetDeleted(LoadedAsset);

					bool bSuccessful = ObjectTools::DeleteSingleObject(LoadedAsset, false);

					//If we failed to delete this object manually clear any references and try again
					if (!bSuccessful)
					{
						//Clear references to the object so we can delete it
						FAutomationEditorCommonUtils::NullReferencesToObject(LoadedAsset);

						bSuccessful = ObjectTools::DeleteSingleObject(LoadedAsset, false);
					}
				}
			}

			Test->AddInfo(*FString::Printf(TEXT("Clearing Path: %s"), *FEditorPromotionTestUtilities::GetGamePath()));
			AssetRegistry.RemovePath(FEditorPromotionTestUtilities::GetGamePath());

			//Remove the directory
			bool bEnsureExists = false;
			bool bDeleteEntireTree = true;
			FString PackageDirectory = FPaths::ProjectContentDir() / TEXT("BuildPromotionTest");
			IFileManager::Get().DeleteDirectory(*PackageDirectory, bEnsureExists, bDeleteEntireTree);
			Test->AddInfo(*FString::Printf(TEXT("Deleting Folder: %s"), *PackageDirectory));

			return true;
		}

		bool Setup()
		{
			//Make sure we have the required assets
			UAutomationTestSettings const* AutomationTestSettings = GetDefault<UAutomationTestSettings>();
			check(AutomationTestSettings);
			FAssetData AssetData;

			const FString FirstMeshPath = AutomationTestSettings->BlueprintEditorPromotionTest.FirstMeshPath.FilePath;
			if (FirstMeshPath.Len() > 0)
			{
				AssetData = FAutomationEditorCommonUtils::GetAssetDataFromPackagePath(FirstMeshPath);
				FirstBlueprintMesh = Cast<UStaticMesh>(AssetData.GetAsset());
			}

			const FString SecondMeshPath = AutomationTestSettings->BlueprintEditorPromotionTest.SecondMeshPath.FilePath;
			if (SecondMeshPath.Len() > 0)
			{
				AssetData = FAutomationEditorCommonUtils::GetAssetDataFromPackagePath(SecondMeshPath);
				SecondBlueprintMesh = Cast<UStaticMesh>(AssetData.GetAsset());
			}

			const FString ParticleSystemPath = AutomationTestSettings->BlueprintEditorPromotionTest.DefaultParticleAsset.FilePath;
			if (ParticleSystemPath.Len() > 0)
			{
				AssetData = FAutomationEditorCommonUtils::GetAssetDataFromPackagePath(ParticleSystemPath);
				LoadedParticleSystem = Cast<UParticleSystem>(AssetData.GetAsset());
			}

			if (!(FirstBlueprintMesh && SecondBlueprintMesh && LoadedParticleSystem))
			{
				SkippedTests.Add(TEXT("All Blueprint tests. (Missing a required mesh or particle system)"));
				if (FirstMeshPath.IsEmpty() || SecondMeshPath.IsEmpty())
				{
					Test->AddInfo(TEXT("SKIPPING BLUEPRINT TESTS.  FirstMeshPath or SecondMeshPath not configured in AutomationTestSettings."));
				}
				else
				{
					Test->AddWarning(TEXT("SKIPPING BLUEPRINT TESTS.  Invalid FirstMeshPath or SecondMeshPath in AutomationTestSettings, or particle system was not created."));
				}
			}

			return true;
		}

		/**
		* Blueprint Test Stage: Create a new Blueprint (Part 1)
		*    Creates a new actor based blueprint and opens the editor
		*/
		bool Blueprint_CreateNewBlueprint_Part1()
		{
			// Exit early if any of the required assets are missing
			if (!(FirstBlueprintMesh && SecondBlueprintMesh && LoadedParticleSystem))
			{
				return true;
			}

			UBlueprintFactory* Factory = NewObject<UBlueprintFactory>();
			Factory->ParentClass = AActor::StaticClass();

			const FString PackageName = FEditorPromotionTestUtilities::GetGamePath() + TEXT("/") + BlueprintEditorPromotionUtils::BlueprintNameString;
			BlueprintPackage = CreatePackage( *PackageName);
			EObjectFlags Flags = RF_Public | RF_Standalone;

			UObject* ExistingBlueprint = FindObject<UBlueprint>(BlueprintPackage, *BlueprintEditorPromotionUtils::BlueprintNameString);
			Test->TestNull(TEXT("Blueprint asset does not already exist (delete blueprint and restart editor)"), ExistingBlueprint);
			// Exit early since test will not be valid with pre-existing assets
			if (ExistingBlueprint)
			{
				return true;
			}

			BlueprintObject = Cast<UBlueprint>(Factory->FactoryCreateNew(UBlueprint::StaticClass(), BlueprintPackage, FName(*BlueprintEditorPromotionUtils::BlueprintNameString), Flags, NULL, GWarn));
			Test->TestNotNull(TEXT("Created new Actor-based blueprint"), BlueprintObject);
			if (BlueprintObject)
			{
				// Update asset registry and mark package dirty
				FAssetRegistryModule::AssetCreated(BlueprintObject);
				BlueprintPackage->MarkPackageDirty();

				Test->AddInfo(TEXT("Opening the blueprint editor for the first time"));
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(BlueprintObject);
			}

			return true;
		}

		/**
		* Blueprint Test Stage: Create a new Blueprint (Part 2)
		*    Checks that the blueprint editor opened in the correct mode
		*/
		bool Blueprint_CreateNewBlueprint_Part2()
		{
			if (BlueprintObject)
			{
				IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(BlueprintObject, true);
				IBlueprintEditor* BlueprintEditor = (IBlueprintEditor*)AssetEditor;
				Test->TestTrue(TEXT("Opened correct initial editor"), BlueprintEditor->GetCurrentMode() != FBlueprintEditorApplicationModes::BlueprintDefaultsMode);
			}

			return true;
		}

		/**
		* Blueprint Test Stage: Data-only Blueprint (Part 1)
		*    Closes the blueprint editor
		*/
		bool Blueprint_DataOnlyBlueprint_Part1()
		{
			if (BlueprintObject)
			{
				Test->AddInfo(TEXT("Closing the blueprint editor"));
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllAssetEditors();
			}
			return true;
		}

		/**
		* Blueprint Test Stage: Data-only Blueprint (Part 2)
		*    Re opens the blueprint editor
		*/
		bool Blueprint_DataOnlyBlueprint_Part2()
		{
			if (BlueprintObject)
			{
				Test->AddInfo(TEXT("Opening the blueprint editor for the second time"));
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(BlueprintObject);
			}
			return true;
		}

		/**
		* Blueprint Test Stage: Data-only Blueprint (Part 3)
		*    Checks that the editor opened in the Defaults mode
		*/
		bool Blueprint_DataOnlyBlueprint_Part3()
		{
			if (BlueprintObject)
			{
				IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(BlueprintObject, true);
				IBlueprintEditor* BlueprintEditor = (IBlueprintEditor*)AssetEditor;
				
				bool bCorrectEditorOpened = BlueprintEditor->GetCurrentMode() == FBlueprintEditorApplicationModes::BlueprintDefaultsMode;
				Test->TestTrue(TEXT("Correct defaults editor opened"), bCorrectEditorOpened);

				if (bCorrectEditorOpened)
				{
					Test->AddInfo(TEXT("Switching to components mode"));
					BlueprintEditor->SetCurrentMode(FBlueprintEditorApplicationModes::BlueprintComponentsMode);
				}
			}
			return true;
		}

		/**
		* Blueprint Test Stage:  Components Mode (Part 1)
		*   Adds 3 new components to the blueprint, changes the root component, renames the components, and compiles the blueprint
		*/
		bool Blueprint_ComponentsMode_Part1()
		{
			if (BlueprintObject)
			{
				IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(BlueprintObject, true);
				IBlueprintEditor* BlueprintEditor = (IBlueprintEditor*)AssetEditor;

				MeshNode = BlueprintEditorPromotionUtils::CreateBlueprintComponent(BlueprintObject, FirstBlueprintMesh);
				Test->TestNotNull(TEXT("First mesh component added"), MeshNode);

				OtherMeshNode = BlueprintEditorPromotionUtils::CreateBlueprintComponent(BlueprintObject, SecondBlueprintMesh);
				Test->TestNotNull(TEXT("Second mesh component added"), OtherMeshNode);

				PSNode = BlueprintEditorPromotionUtils::CreateBlueprintComponent(BlueprintObject, LoadedParticleSystem);
				Test->TestNotNull(TEXT("Particle system component added"), PSNode);

				//Set the Particle System as the root
				BlueprintEditorPromotionUtils::SetComponentAsRoot(BlueprintObject, PSNode);
				Test->TestTrue(TEXT("Particle system set as root"), PSNode->IsRootNode());

				//Set Names
				const FName MeshName(TEXT("FirstMesh"));
				FBlueprintEditorUtils::RenameComponentMemberVariable(BlueprintObject, MeshNode, MeshName);
				Test->AddInfo(TEXT("Renamed the first mesh component to FirstMesh"));

				const FName OtherMeshName(TEXT("SecondMesh"));
				FBlueprintEditorUtils::RenameComponentMemberVariable(BlueprintObject, OtherMeshNode, OtherMeshName);
				Test->AddInfo(TEXT("Renamed the second mesh component to SecondMesh"));

				const FName PSName(TEXT("ParticleSys"));
				FBlueprintEditorUtils::RenameComponentMemberVariable(BlueprintObject, PSNode, PSName);
				Test->AddInfo(TEXT("Renamed the particle system component to ParticleSys"));

				BlueprintEditorPromotionUtils::CompileBlueprint(BlueprintObject);

				Test->AddInfo(TEXT("Switched to graph editing mode"));
				BlueprintEditor->SetCurrentMode(FBlueprintEditorApplicationModes::StandardBlueprintEditorMode);
			}
			return true;
		}

		/**
		* Blueprint Test Stage: Components Mode (Part 2)
		*    Removes the 3 components added before and compiles the blueprint
		*/
		bool Blueprint_ComponentsMode_Part2()
		{
			if (BlueprintObject)
			{
				IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(BlueprintObject, true);
				IBlueprintEditor* BlueprintEditor = (IBlueprintEditor*)AssetEditor;

				//FEditorPromotionTestUtilities::TakeScreenshot(TEXT("BlueprintComponentVariables"), true);

				Test->AddInfo(TEXT("Switched to components mode"));
				BlueprintEditor->SetCurrentMode(FBlueprintEditorApplicationModes::BlueprintComponentsMode);

				BlueprintEditorPromotionUtils::RemoveBlueprintComponent(BlueprintObject, MeshNode);
				BlueprintEditorPromotionUtils::RemoveBlueprintComponent(BlueprintObject, OtherMeshNode);
				BlueprintEditorPromotionUtils::RemoveBlueprintComponent(BlueprintObject, PSNode);

				//There should only be the scene component left
				Test->TestTrue(TEXT("Successfully removed blueprint components"), BlueprintObject->SimpleConstructionScript->GetAllNodes().Num() == 1);

				MeshNode = NULL;
				OtherMeshNode = NULL;
				PSNode = NULL;

				Test->AddInfo(TEXT("Switched to graph mode"));
				BlueprintEditor->SetCurrentMode(FBlueprintEditorApplicationModes::StandardBlueprintEditorMode);

				BlueprintEditorPromotionUtils::CompileBlueprint(BlueprintObject);
			}
			return true;
		}

		/**
		* Blueprint Test Stage: Construction Script
		*    Adds an AddStaticMeshComponent to the construction graph and links it to the entry node
		*/
		bool Blueprint_ConstructionScript()
		{
			if (BlueprintObject)
			{
				IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(BlueprintObject, true);
				FBlueprintEditor* BlueprintEditor = (FBlueprintEditor*)AssetEditor;

				UEdGraph* ConstructionGraph = FBlueprintEditorUtils::FindUserConstructionScript(BlueprintObject);
				BlueprintEditor->OpenGraphAndBringToFront(ConstructionGraph);

				AddMeshNode = BlueprintEditorPromotionUtils::CreateAddComponentActionNode(BlueprintObject, ConstructionGraph, FirstBlueprintMesh);
				Test->TestNotNull(TEXT("Add Static Mesh Component node created"), AddMeshNode);

				GEditor->UndoTransaction();
				Test->TestTrue(TEXT("Undo add component node completed"), ConstructionGraph->Nodes.Num() == 0 || ConstructionGraph->Nodes[ConstructionGraph->Nodes.Num() - 1] != AddMeshNode);
				
				GEditor->RedoTransaction();
				Test->TestTrue(TEXT("Redo add component node completed"), ConstructionGraph->Nodes.Num() > 0 && ConstructionGraph->Nodes[ConstructionGraph->Nodes.Num() - 1] == AddMeshNode);

				TArray<UK2Node_FunctionEntry*> EntryNodes;
				ConstructionGraph->GetNodesOfClass(EntryNodes);
				UEdGraphNode* EntryNode = EntryNodes.Num() > 0 ? EntryNodes[0] : NULL;
				Test->TestNotNull(TEXT("Found entry node to connect Add Static Mesh to"), EntryNode);
				if (EntryNode)
				{
					BlueprintEditorPromotionUtils::ConnectGraphNodes(AddMeshNode, UEdGraphSchema_K2::PN_Execute, EntryNode, UEdGraphSchema_K2::PN_Then);

					UEdGraphPin* EntryOutPin = EntryNode->FindPin(UEdGraphSchema_K2::PN_Then);
					UEdGraphPin* AddStaticMeshInPin = AddMeshNode->FindPin(UEdGraphSchema_K2::PN_Execute);

					Test->TestTrue(TEXT("Connected entry node to Add Static Mesh node"), EntryOutPin->LinkedTo.Contains(AddStaticMeshInPin));

					GEditor->UndoTransaction();
					Test->TestTrue(TEXT("Undo connection to Add Static Mesh Node succeeded"), EntryOutPin->LinkedTo.Num() == 0);
					
					GEditor->RedoTransaction();
					Test->TestTrue(TEXT("Redo connection to Add Static Mesh Node succeeded"), EntryOutPin->LinkedTo.Contains(AddStaticMeshInPin));
				}

				BlueprintEditorPromotionUtils::CompileBlueprint(BlueprintObject);
			}
			return true;
		}

		/**
		* Saves the blueprint stored in BlueprintObject
		*/
		void SaveBlueprint()
		{
			if (BlueprintObject && BlueprintPackage)
			{
				BlueprintPackage->SetDirtyFlag(true);
				BlueprintPackage->FullyLoad();
				const FString PackagePath = FEditorPromotionTestUtilities::GetGamePath() + TEXT("/") + BlueprintEditorPromotionUtils::BlueprintNameString;
				FSavePackageArgs SaveArgs;
				SaveArgs.TopLevelFlags = RF_Standalone;
				SaveArgs.Error = GLog;
				bool bBlueprintSaved = UPackage::SavePackage(BlueprintPackage, nullptr,
					*FPackageName::LongPackageNameToFilename(PackagePath, FPackageName::GetAssetPackageExtension()),
					SaveArgs);
				Test->TestTrue(*FString::Printf(TEXT("Blueprint saved successfully (%s)"), *BlueprintObject->GetName()), bBlueprintSaved);
			}
		}

		/**
		* Blueprint Test Stage: Variable from component mode (Part 1)
		*    Promotes the return pin of the AddStaticMeshNode to a variable and then renames it
		*/
		bool Blueprint_PromoteVariable_Part1()
		{
			if (BlueprintObject)
			{
				IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(BlueprintObject, true);
				FBlueprintEditor* BlueprintEditor = (FBlueprintEditor*)AssetEditor;

				BlueprintEditorPromotionUtils::PromotePinToVariable(BlueprintObject, AddMeshNode, UEdGraphSchema_K2::PN_ReturnValue);

				Test->AddInfo(TEXT("Promoted the return pin on the add mesh node to a variable"));

				const FName OldVarName(TEXT("NewVar")); // Default variable name
				const FName NewVarName(TEXT("MyMesh"));
				FBlueprintEditorUtils::RenameMemberVariable(BlueprintObject, OldVarName, NewVarName);
				Test->TestNotEqual(TEXT("New variable was renamed"), FBlueprintEditorUtils::FindMemberVariableGuidByName(BlueprintObject, OldVarName), FBlueprintEditorUtils::FindMemberVariableGuidByName(BlueprintObject, NewVarName));
				
				BlueprintEditorPromotionUtils::CompileBlueprint(BlueprintObject);

				Test->AddInfo(TEXT("Switched to graph mode"));
				BlueprintEditor->SetCurrentMode(FBlueprintEditorApplicationModes::StandardBlueprintEditorMode);
			}
			return true;
		}

		bool Blueprint_PromoteVariable_Part2()
		{
			if (BlueprintObject)
			{
				IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(BlueprintObject, true);
				FBlueprintEditor* BlueprintEditor = (FBlueprintEditor*)AssetEditor;

				//FEditorPromotionTestUtilities::TakeScreenshot(TEXT("BlueprintMeshVariable"), true);

				Test->AddInfo(TEXT("Switched to components mode"));
				BlueprintEditor->SetCurrentMode(FBlueprintEditorApplicationModes::BlueprintComponentsMode);

				BlueprintEditorPromotionUtils::SendBlueprintResetViewCommand();
			}
			return true;
		}

		///**
		//* Blueprint Test Stage: Variable from component mode (Part 3)
		//*    Takes a screenshot of the mesh variable
		//*/
		//bool Blueprint_PromoteVariable_Part3()
		//{
		//	if (BlueprintObject)
		//	{
		//		FEditorPromotionTestUtilities::TakeScreenshot(TEXT("BlueprintComponent"), true);
		//	}
		//	return true;
		//}

		/**
		* Blueprint Test Stage: Event Graph
		*    Adds a ReceiveBeginPlay and Delay node to the event graph 
		*/
		bool Blueprint_EventGraph()
		{
			if (BlueprintObject)
			{
				IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(BlueprintObject, true);
				FBlueprintEditor* BlueprintEditor = (FBlueprintEditor*)AssetEditor;

				UEdGraph* EventGraph = FBlueprintEditorUtils::FindEventGraph(BlueprintObject);
				BlueprintEditor->OpenGraphAndBringToFront(EventGraph);
				Test->AddInfo(TEXT("Opened the event graph"));

				PostBeginPlayEventNode = BlueprintEditorPromotionUtils::CreatePostBeginPlayEvent(BlueprintObject, EventGraph);
				Test->TestNotNull(TEXT("Created EventBeginPlay node"), PostBeginPlayEventNode);
				
				UEdGraphPin* PlayThenPin = PostBeginPlayEventNode->FindPin(UEdGraphSchema_K2::PN_Then);
				
				DelayNode = BlueprintEditorPromotionUtils::AddDelayNode(BlueprintObject, EventGraph, PlayThenPin);
				Test->TestNotNull(TEXT("Created Delay node"), DelayNode);

				GEditor->UndoTransaction();
				Test->TestTrue(TEXT("Undo adding Delay node succeeded"), EventGraph->Nodes.Num() == 0 || EventGraph->Nodes[EventGraph->Nodes.Num() - 1] != DelayNode);

				GEditor->RedoTransaction();
				Test->TestTrue(TEXT("Redo adding Delay node succeeded"), EventGraph->Nodes.Num() > 0 && EventGraph->Nodes[EventGraph->Nodes.Num() - 1] == DelayNode);
				
				// Update Delay node's Duration pin with new default value
				const FName DelayDurationPinName = TEXT("Duration");
				const FString NewDurationDefaultValue = TEXT("2.0");

				BlueprintEditorPromotionUtils::SetPinDefaultValue(DelayNode, DelayDurationPinName, NewDurationDefaultValue);				
				Test->TestEqual(TEXT("Delay node default duration set to 2.0 seconds"), DelayNode->FindPin(DelayDurationPinName)->DefaultValue, NewDurationDefaultValue);
				
				BlueprintEditorPromotionUtils::CompileBlueprint(BlueprintObject);
			}

			return true;
		}

		/**
		* Blueprint Test Stage: Custom Variable
		*    Creates a custom string variable and adds Get/Set nodes for it
		*/
		bool Blueprint_CustomVariable()
		{
			if (BlueprintObject)
			{
				UEdGraph* EventGraph = FBlueprintEditorUtils::FindEventGraph(BlueprintObject);

				Test->AddInfo(TEXT("Added a string member variable"));
				BlueprintEditorPromotionUtils::AddStringMemberValue(BlueprintObject, BlueprintEditorPromotionUtils::BlueprintStringVariableName);

				SetNode = BlueprintEditorPromotionUtils::AddGetSetNode(BlueprintObject, EventGraph, BlueprintEditorPromotionUtils::BlueprintStringVariableName.ToString(), false);
				Test->TestNotNull(TEXT("Added Set node for string variable"), SetNode);

				GEditor->UndoTransaction();
				Test->TestTrue(TEXT("Undo adding Set node succeeded"), EventGraph->Nodes.Num() == 0 || EventGraph->Nodes[EventGraph->Nodes.Num() - 1] != SetNode);

				GEditor->RedoTransaction();
				Test->TestTrue(TEXT("Redo adding Set node succeeded"), EventGraph->Nodes.Num() > 0 && EventGraph->Nodes[EventGraph->Nodes.Num() - 1] == SetNode);
				
				GetNode = BlueprintEditorPromotionUtils::AddGetSetNode(BlueprintObject, EventGraph, BlueprintEditorPromotionUtils::BlueprintStringVariableName.ToString(), true, 400);
				Test->TestNotNull(TEXT("Added Get node for string variable"), GetNode);
				
				GEditor->UndoTransaction();
				Test->TestTrue(TEXT("Undo adding Get node succeeded"), EventGraph->Nodes.Num() == 0 || EventGraph->Nodes[EventGraph->Nodes.Num() - 1] != GetNode);

				GEditor->RedoTransaction();
				Test->TestTrue(TEXT("Redo adding Get node succeeded"), EventGraph->Nodes.Num() > 0 && EventGraph->Nodes[EventGraph->Nodes.Num() - 1] == GetNode);

				EventGraph->RemoveNode(SetNode);
				Test->TestFalse(TEXT("Set node removed from EventGraph"), EventGraph->Nodes.Contains(SetNode));
				SetNode = NULL;
			}
			return true;
		}

		/**
		* Blueprint Test Stage:  Using Variables
		*   Adds a PrintString and SetStaticMesh then connects all the existing nodes
		*/
		bool Blueprint_UsingVariables()
		{
			if (BlueprintObject)
			{
				const bool bVariableIsHidden = false;
				FBlueprintEditorUtils::SetBlueprintOnlyEditableFlag(BlueprintObject, BlueprintEditorPromotionUtils::BlueprintStringVariableName, bVariableIsHidden);
				Test->AddInfo(TEXT("Exposed the blueprint string variable"));

				UEdGraph* EventGraph = FBlueprintEditorUtils::FindEventGraph(BlueprintObject);
				PrintNode = BlueprintEditorPromotionUtils::AddPrintStringNode(BlueprintObject, EventGraph);
				Test->TestNotNull(TEXT("Added Print String node"), PrintNode);
				
				//Connect Get to printstring
				UEdGraphPin* GetVarPin = GetNode->FindPin(BlueprintEditorPromotionUtils::BlueprintStringVariableName);
				UEdGraphPin* InStringPin = PrintNode->FindPin(TEXT("InString"));
				GetVarPin->MakeLinkTo(InStringPin);
				Test->TestTrue(TEXT("Connected string variable Get node to the Print String node"), GetVarPin->LinkedTo.Contains(InStringPin));


				//Connect Delay to PrintString
				UEdGraphPin* DelayExecPin = DelayNode->FindPin(UEdGraphSchema_K2::PN_Then);
				UEdGraphPin* PrintStringPin = PrintNode->FindPin(UEdGraphSchema_K2::PN_Execute);
				DelayExecPin->MakeLinkTo(PrintStringPin);
				Test->TestTrue(TEXT("Connected Delay nod to Print String node"), DelayExecPin->LinkedTo.Contains(PrintStringPin));

				const FName MyMeshVarName(TEXT("MyMesh"));
				GetNode = BlueprintEditorPromotionUtils::AddGetSetNode(BlueprintObject, EventGraph, MyMeshVarName.ToString(), true, 680);
				Test->TestNotNull(TEXT("Added Get node for MyMesh variable"), GetNode);
				
				SetStaticMeshNode = BlueprintEditorPromotionUtils::AddSetStaticMeshNode(BlueprintObject, EventGraph);
				Test->TestNotNull(TEXT("Added Set Static Mesh node"), SetStaticMeshNode);
				
 				UEdGraphPin* GetExecPin = GetNode->FindPin(TEXT("MyMesh"));
				UEdGraphPin* SetStaticMeshSelfPin = SetStaticMeshNode->FindPin(UEdGraphSchema_K2::PN_Self);
				GetExecPin->MakeLinkTo(SetStaticMeshSelfPin);
				Test->TestTrue(TEXT("Connected Get MyMesh node to Set Static Mesh node"), GetExecPin->LinkedTo.Contains(SetStaticMeshSelfPin));

				UEdGraphPin* SetStaticMeshMeshPin = SetStaticMeshNode->FindPin(TEXT("NewMesh"));
				SetStaticMeshMeshPin->DefaultObject = SecondBlueprintMesh;
				Test->TestEqual(*FString::Printf(TEXT("Set Static Mesh default mesh updated to %s"), *SecondBlueprintMesh->GetName()), Cast<UStaticMesh>(SetStaticMeshMeshPin->DefaultObject), SecondBlueprintMesh);

				//Connect SetStaticMeshMesh to PrintString
				UEdGraphPin* PrintStringThenPin = PrintNode->FindPin(UEdGraphSchema_K2::PN_Then);
				UEdGraphPin* SetStaticMeshExecPin = SetStaticMeshNode->FindPin(UEdGraphSchema_K2::PN_Execute);
				PrintStringThenPin->MakeLinkTo(SetStaticMeshExecPin);
				Test->TestTrue(TEXT("Connected Print String node to Set Static Mesh node"), PrintStringThenPin->LinkedTo.Contains(SetStaticMeshExecPin));
			}
			return true;
		}

		/**
		* Blueprint Test Stage: Renaming custom event
		*    Creates, renames, and then removes a custom event node
		*/
		bool Blueprint_RenameCustomEvent()
		{
			if (BlueprintObject)
			{
				UEdGraph* EventGraph = FBlueprintEditorUtils::FindEventGraph(BlueprintObject);
				CustomEventNode = BlueprintEditorPromotionUtils::CreateCustomEvent(BlueprintObject, EventGraph, TEXT("NewEvent"));
				Test->TestNotNull(TEXT("Custom event node created"), CustomEventNode);
				if (CustomEventNode)
				{
					//Rename the event
					const FString NewEventNodeName = TEXT("RenamedEvent");
					CustomEventNode->Rename(*NewEventNodeName);
					Test->TestEqual(TEXT("Custom event rename succeeded"), CustomEventNode->GetName(), NewEventNodeName);
					
					EventGraph->RemoveNode(CustomEventNode);
					Test->TestFalse(TEXT("Blueprint EventGraph does not contain removed custom event node"), EventGraph->Nodes.Contains(CustomEventNode));
					CustomEventNode = NULL;
				}
			}

			return true;
		}

		/**
		* Blueprint Test Stage: New function
		*    Creates a new function graph and then hooks up a new AddParticleSystem inside it
		*/
		bool Blueprint_NewFunctions()
		{
			if (BlueprintObject)
			{
				CustomGraph = BlueprintEditorPromotionUtils::CreateNewFunctionGraph(BlueprintObject, TEXT("NewFunction"));
				Test->TestNotNull(TEXT("Created new function graph"), CustomGraph);

				AddParticleSystemNode = BlueprintEditorPromotionUtils::CreateAddComponentActionNode(BlueprintObject, CustomGraph, LoadedParticleSystem);
				Test->TestNotNull(TEXT("Created Add Particle System node"), AddParticleSystemNode);

				UEdGraphPin* ExecutePin = AddParticleSystemNode ? AddParticleSystemNode->FindPin(UEdGraphSchema_K2::PN_Execute) : NULL;

				//Find the input for the function graph
				TArray<UK2Node_FunctionEntry*> EntryNodes;
				CustomGraph->GetNodesOfClass(EntryNodes);
				UEdGraphNode* EntryNode = EntryNodes.Num() > 0 ? EntryNodes[0] : NULL;
				if (EntryNode && ExecutePin)
				{
					UEdGraphPin* EntryPin = EntryNode->FindPin(UEdGraphSchema_K2::PN_Then);
					EntryPin->MakeLinkTo(ExecutePin);
					Test->TestTrue(TEXT("Connected Add Particle System node to entry node"), EntryPin->LinkedTo.Contains(ExecutePin));
				}

				BlueprintEditorPromotionUtils::CompileBlueprint(BlueprintObject);
			}
			return true;
		}

		/**
		* Blueprint Test Stage: Completing the blueprint
		*    Adds a CallFunction node to call the custom function created in the previous step.
		*/
		bool Blueprint_CompleteBlueprint()
		{
			if (BlueprintObject)
			{
				UEdGraph* EventGraph = FBlueprintEditorUtils::FindEventGraph(BlueprintObject);
				UEdGraphPin* SetStaticMeshThenPin = SetStaticMeshNode->FindPin(UEdGraphSchema_K2::PN_Then);
				CallFunctionNode = BlueprintEditorPromotionUtils::AddCallFunctionGraphNode(BlueprintObject, EventGraph, TEXT("NewFunction"), SetStaticMeshThenPin);
				Test->TestNotNull(TEXT("Created Call Function node"), CallFunctionNode);
				if (CallFunctionNode)
				{
					Test->TestTrue(TEXT("Connected Set Static Mesh node to Call Function node"), SetStaticMeshThenPin->LinkedTo.Contains(CallFunctionNode->FindPin(UEdGraphSchema_K2::PN_Execute)));
				}

				BlueprintEditorPromotionUtils::CompileBlueprint(BlueprintObject);

				SaveBlueprint();
			}
			return true;
		}
	};
}

/**
* Latent command to run the main build promotion test
*/
DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FRunPromotionTestCommand, TSharedPtr<BlueprintEditorPromotionTestHelper::FBlueprintEditorPromotionTestHelper>, BlueprintEditorTestInfo);
bool FRunPromotionTestCommand::Update()
{
	return BlueprintEditorTestInfo->Update();
}

/**
* Automation test that handles the blueprint editor promotion process
*/
bool FBlueprintEditorPromotionTest::RunTest(const FString& Parameters)
{
	TSharedPtr<BlueprintEditorPromotionTestHelper::FBlueprintEditorPromotionTestHelper> BuildPromotionTest = MakeShareable(new BlueprintEditorPromotionTestHelper::FBlueprintEditorPromotionTestHelper());
	BuildPromotionTest->Test = this;
	ADD_LATENT_AUTOMATION_COMMAND(FRunPromotionTestCommand(BuildPromotionTest));
	return true;
}


#undef LOCTEXT_NAMESPACE

#endif //WITH_DEV_AUTOMATION_TESTS
