// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserDefinedEnumEditor.h"
#include "Editor.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/AppStyle.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailDragDropHandler.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "SPositiveActionButton.h"
#include "Styling/ToolBarStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#include "PropertyCustomizationHelpers.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Docking/SDockTab.h"
#include "IDocumentation.h"
#include "STextPropertyEditableTextBox.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "UserDefinedEnumEditor"

const FName FUserDefinedEnumEditor::EnumeratorsTabId( TEXT( "UserDefinedEnum_EnumeratorEditor" ) );
const FName FUserDefinedEnumEditor::UserDefinedEnumEditorAppIdentifier( TEXT( "UserDefinedEnumEditorApp" ) );

/** Allows STextPropertyEditableTextBox to edit a user defined enum entry */
class FEditableTextUserDefinedEnum : public IEditableTextProperty
{
public:
	FEditableTextUserDefinedEnum(UUserDefinedEnum* InTargetEnum, const int32 InEnumeratorIndex)
		: TargetEnum(InTargetEnum)
		, EnumeratorIndex(InEnumeratorIndex)
		, bCausedChange(false)
	{
	}

	virtual bool IsMultiLineText() const override
	{
		return false;
	}

	virtual bool IsPassword() const override
	{
		return false;
	}

	virtual bool IsReadOnly() const override
	{
		return false;
	}

	virtual bool IsDefaultValue() const override
	{
		return false;
	}

	virtual FText GetToolTipText() const override
	{
		return FText::GetEmpty();
	}

	virtual int32 GetNumTexts() const override
	{
		return 1;
	}

	virtual FText GetText(const int32 InIndex) const override
	{
		check(InIndex == 0);
		return TargetEnum->GetDisplayNameTextByIndex(EnumeratorIndex);
	}

	virtual void SetText(const int32 InIndex, const FText& InText) override
	{
		check(InIndex == 0);
		TGuardValue<bool> CausingChange(bCausedChange, true);
		FEnumEditorUtils::SetEnumeratorDisplayName(TargetEnum, EnumeratorIndex, InText);
	}

	virtual bool IsValidText(const FText& InText, FText& OutErrorMsg) const override
	{
		bool bValidName = true;

		bool bUnchangedName = (InText.ToString().Equals(TargetEnum->GetDisplayNameTextByIndex(EnumeratorIndex).ToString()));
		if (InText.IsEmpty())
		{
			OutErrorMsg = LOCTEXT("NameMissingError", "You must provide a name.");
			bValidName = false;
		}
		else if (!FEnumEditorUtils::IsEnumeratorDisplayNameValid(TargetEnum, EnumeratorIndex, InText))
		{
			OutErrorMsg = FText::Format(LOCTEXT("NameInUseError", "'{0}' is already in use."), InText);
			bValidName = false;
		}

		return bValidName && !bUnchangedName;
	}

#if USE_STABLE_LOCALIZATION_KEYS
	virtual void GetStableTextId(const int32 InIndex, const ETextPropertyEditAction InEditAction, const FString& InTextSource, const FString& InProposedNamespace, const FString& InProposedKey, FString& OutStableNamespace, FString& OutStableKey) const override
	{
		check(InIndex == 0);
		return StaticStableTextId(TargetEnum, InEditAction, InTextSource, InProposedNamespace, InProposedKey, OutStableNamespace, OutStableKey);
	}
#endif // USE_STABLE_LOCALIZATION_KEYS

	bool CausedChange() const
	{
		return bCausedChange;
	}

private:
	/** The user defined enum being edited */
	UUserDefinedEnum* TargetEnum;

	/** Index of enumerator entry */
	int32 EnumeratorIndex;

	/** Set while we are invoking a change to the user defined enum */
	bool bCausedChange;
};



/** Allows STextPropertyEditableTextBox to edit the tooltip metadata for a user defined enum entry */
class FEditableTextUserDefinedEnumTooltip : public IEditableTextProperty
{
public:
	FEditableTextUserDefinedEnumTooltip(UUserDefinedEnum* InTargetEnum, const int32 InEnumeratorIndex)
		: TargetEnum(InTargetEnum)
		, EnumeratorIndex(InEnumeratorIndex)
		, bCausedChange(false)
	{
	}

	virtual bool IsMultiLineText() const override
	{
		return true;
	}

	virtual bool IsPassword() const override
	{
		return false;
	}

	virtual bool IsReadOnly() const override
	{
		return false;
	}

	virtual bool IsDefaultValue() const override
	{
		return false;
	}

	virtual FText GetToolTipText() const override
	{
		return FText::GetEmpty();
	}

	virtual int32 GetNumTexts() const override
	{
		return 1;
	}

	virtual FText GetText(const int32 InIndex) const override
	{
		check(InIndex == 0);
		return TargetEnum->GetToolTipTextByIndex(EnumeratorIndex);
	}

	virtual void SetText(const int32 InIndex, const FText& InText) override
	{
		check(InIndex == 0);
		TGuardValue<bool> CausingChange(bCausedChange, true);
		//@TODO: Metadata is not transactional right now, so we cannot transact a tooltip edit
		// const FScopedTransaction Transaction(NSLOCTEXT("EnumEditor", "SetEnumeratorTooltip", "Set Description"));
		TargetEnum->Modify();
		TargetEnum->SetMetaData(TEXT("ToolTip"), *InText.ToString(), EnumeratorIndex);
	}

	virtual bool IsValidText(const FText& InText, FText& OutErrorMsg) const override
	{
		return true;
	}

#if USE_STABLE_LOCALIZATION_KEYS
	virtual void GetStableTextId(const int32 InIndex, const ETextPropertyEditAction InEditAction, const FString& InTextSource, const FString& InProposedNamespace, const FString& InProposedKey, FString& OutStableNamespace, FString& OutStableKey) const override
	{
		check(InIndex == 0);
		return StaticStableTextId(TargetEnum, InEditAction, InTextSource, InProposedNamespace, InProposedKey, OutStableNamespace, OutStableKey);
	}
#endif // USE_STABLE_LOCALIZATION_KEYS

	bool CausedChange() const
	{
		return bCausedChange;
	}

private:
	/** The user defined enum being edited */
	UUserDefinedEnum* TargetEnum;

	/** Index of enumerator entry */
	int32 EnumeratorIndex;

	/** Set while we are invoking a change to the user defined enum */
	bool bCausedChange;
};


/** Drag-and-drop operation that stores data about the source enumerator being dragged */
class FUserDefinedEnumIndexDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FUserDefinedEnumIndexDragDropOp, FDecoratedDragDropOp);

	FUserDefinedEnumIndexDragDropOp(UUserDefinedEnum* InTargetEnum, int32 InEnumeratorIndex)
		: TargetEnum(InTargetEnum)
		, EnumeratorIndex(InEnumeratorIndex)
	{
		check(InTargetEnum);
		check(InEnumeratorIndex >= 0 && InEnumeratorIndex < InTargetEnum->NumEnums());

		EnumDisplayText = InTargetEnum->GetDisplayNameTextByIndex(InEnumeratorIndex);
		MouseCursor = EMouseCursor::GrabHandClosed;
	}

	void Init()
	{
		SetValidTarget(false);
		SetupDefaults();
		Construct();
	}

	void SetValidTarget(bool IsValidTarget)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("EnumeratorName"), EnumDisplayText);

		if (IsValidTarget)
		{
			CurrentHoverText = FText::Format(LOCTEXT("MoveEnumeratorHere", "Move '{EnumeratorName}' Here"), Args);
			CurrentIconBrush = FAppStyle::GetBrush("Graph.ConnectorFeedback.OK");
		}
		else
		{
			CurrentHoverText = FText::Format(LOCTEXT("CannotMoveEnumeratorHere", "Cannot Move '{EnumeratorName}' Here"), Args);
			CurrentIconBrush = FAppStyle::GetBrush("Graph.ConnectorFeedback.Error");
		}
	}

	UUserDefinedEnum* GetTargetEnum() const
	{
		return TargetEnum;
	}

	int32 GetEnumeratorIndex() const
	{
		return EnumeratorIndex;
	}

private:
	UUserDefinedEnum* TargetEnum;
	int32 EnumeratorIndex;
	FText EnumDisplayText;
};


/** Handler for customizing the drag-and-drop behavior for enum index rows, allowing enumerators to be reordered */
class FUserDefinedEnumIndexDragDropHandler : public IDetailDragDropHandler
{
public:
	FUserDefinedEnumIndexDragDropHandler(UUserDefinedEnum* InTargetEnum, int32 InEnumeratorIndex)
		: TargetEnum(InTargetEnum)
		, EnumeratorIndex(InEnumeratorIndex)
	{
		check(InTargetEnum);
		check(InEnumeratorIndex >= 0 && InEnumeratorIndex < InTargetEnum->NumEnums());
	}

	virtual TSharedPtr<FDragDropOperation> CreateDragDropOperation() const override
	{
		TSharedPtr<FUserDefinedEnumIndexDragDropOp> DragOp = MakeShared<FUserDefinedEnumIndexDragDropOp>(TargetEnum, EnumeratorIndex);
		DragOp->Init();
		return DragOp;
	}

	/** Compute new target index for use with FEnumEditorUtils::MoveEnumeratorInUserDefinedEnum based on drop zone (above vs below) */
	static int32 ComputeNewIndex(int32 OriginalIndex, int32 DropOntoIndex, EItemDropZone DropZone)
	{
		check(DropZone != EItemDropZone::OntoItem);

		int32 NewIndex = DropOntoIndex;
		if (DropZone == EItemDropZone::BelowItem)
		{
			// If the drop zone is below, then we actually move it to the next item's index
			NewIndex++;
		}
		if (OriginalIndex < NewIndex)
		{
			// If the item is moved down the list, then all the other elements below it are shifted up one
			NewIndex--;
		}

		return ensure(NewIndex >= 0) ? NewIndex : 0;
	}

	virtual bool AcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone) const override
	{
		const TSharedPtr<FUserDefinedEnumIndexDragDropOp> DragOp = DragDropEvent.GetOperationAs<FUserDefinedEnumIndexDragDropOp>();
		if (!DragOp.IsValid() || DragOp->GetTargetEnum() != TargetEnum || DropZone == EItemDropZone::OntoItem)
		{
			return false;
		}

		const int32 NewIndex = ComputeNewIndex(DragOp->GetEnumeratorIndex(), EnumeratorIndex, DropZone);
		FEnumEditorUtils::MoveEnumeratorInUserDefinedEnum(TargetEnum, DragOp->GetEnumeratorIndex(), NewIndex);
		return true;
	}

	virtual TOptional<EItemDropZone> CanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone) const override
	{
		const TSharedPtr<FUserDefinedEnumIndexDragDropOp> DragOp = DragDropEvent.GetOperationAs<FUserDefinedEnumIndexDragDropOp>();
		if (!DragOp.IsValid() || DragOp->GetTargetEnum() != TargetEnum)
		{
			return TOptional<EItemDropZone>();
		}

		// We're reordering, so there's no logical interpretation for dropping directly onto another enum.
		// Just change it to a drop-above in this case.
		const EItemDropZone OverrideZone = (DropZone == EItemDropZone::BelowItem) ? EItemDropZone::BelowItem : EItemDropZone::AboveItem;
		const int32 NewIndex = ComputeNewIndex(DragOp->GetEnumeratorIndex(), EnumeratorIndex, OverrideZone);

		// Make sure that the new index is valid *and* that it represents an actual move from the current position.
		if (NewIndex < 0 || NewIndex >= TargetEnum->NumEnums() || NewIndex == DragOp->GetEnumeratorIndex())
		{
			return TOptional<EItemDropZone>();
		}

		DragOp->SetValidTarget(true);
		return OverrideZone;
	}

private:
	UUserDefinedEnum* TargetEnum;
	int32 EnumeratorIndex;
};


void FUserDefinedEnumEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_UserDefinedEnumEditor", "User-Defined Enum Editor"));

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner( EnumeratorsTabId, FOnSpawnTab::CreateSP(this, &FUserDefinedEnumEditor::SpawnEnumeratorsTab) )
		.SetDisplayName( LOCTEXT("EnumeratorEditor", "Enumerators") )
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Enum_16x"));
}

void FUserDefinedEnumEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner( EnumeratorsTabId );
}

void FUserDefinedEnumEditor::InitEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UUserDefinedEnum* EnumToEdit)
{
	TargetEnum = EnumToEdit;

	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout( "Standalone_UserDefinedEnumEditor_Layout_v3" )
	->AddArea
	(
		FTabManager::NewPrimaryArea() ->SetOrientation(Orient_Vertical)
		->Split
		(
			FTabManager::NewSplitter()
			->Split
			(
				FTabManager::NewStack()
				->AddTab( EnumeratorsTabId, ETabState::OpenedTab )
				->SetHideTabWell(true)
			)
		)
	);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor( Mode, InitToolkitHost, UserDefinedEnumEditorAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, EnumToEdit );

	TSharedPtr<FExtender> Extender = MakeShared<FExtender>();
	Extender->AddToolBarExtension("Asset", EExtensionHook::After, GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateSP(this, &FUserDefinedEnumEditor::FillToolbar));
	AddToolbarExtender(Extender);
	RegenerateMenusAndToolbars();

	// @todo toolkit world centric editing
	/*if (IsWorldCentricAssetEditor())
	{
		SpawnToolkitTab(GetToolbarTabId(), FString(), EToolkitTabSpot::ToolBar);
		const FString TabInitializationPayload(TEXT(""));	
		SpawnToolkitTab( EnumeratorsTabId, TabInitializationPayload, EToolkitTabSpot::Details );
	}*/

	//
}

TSharedRef<SDockTab> FUserDefinedEnumEditor::SpawnEnumeratorsTab(const FSpawnTabArgs& Args)
{
	check( Args.GetTabId() == EnumeratorsTabId );

	UUserDefinedEnum* EditedEnum = NULL;
	const auto& EditingObjs = GetEditingObjects();
	if (EditingObjs.Num())
	{
		EditedEnum = Cast<UUserDefinedEnum>(EditingObjs[ 0 ]);
	}

	// Create a property view
	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.ColumnWidth = 0.85f;

	PropertyView = EditModule.CreateDetailView( DetailsViewArgs );

	FOnGetDetailCustomizationInstance LayoutEnumDetails = FOnGetDetailCustomizationInstance::CreateStatic(&FEnumDetails::MakeInstance);
	PropertyView->RegisterInstancedCustomPropertyLayout(UUserDefinedEnum::StaticClass(), LayoutEnumDetails);

	PropertyView->SetObject(EditedEnum);

	return SNew(SDockTab)
		.Label( LOCTEXT("EnumeratorEditor", "Enumerators") )
		.TabColorScale( GetTabColorScale() )
		[
			PropertyView.ToSharedRef()
		];
}

void FUserDefinedEnumEditor::FillToolbar(FToolBarBuilder& ToolbarBuilder)
{
	const FToolBarStyle& ToolBarStyle = ToolbarBuilder.GetStyleSet()->GetWidgetStyle<FToolBarStyle>(ToolbarBuilder.GetStyleName());

	ToolbarBuilder.BeginSection("Enumerators");
	ToolbarBuilder.AddWidget(
		SNew(SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Fill)
		.Padding(ToolBarStyle.ButtonPadding)
		[
			SNew(SPositiveActionButton)
			.Text(LOCTEXT("AddEnumeratorButtonText", "Add Enumerator"))
			.OnClicked(this, &FUserDefinedEnumEditor::OnAddNewEnumerator)
		]);
	ToolbarBuilder.EndSection();
}

FReply FUserDefinedEnumEditor::OnAddNewEnumerator()
{
	if (!ensure(TargetEnum.IsValid()))
	{
		return FReply::Handled();
	}

	FEnumEditorUtils::AddNewEnumeratorForUserDefinedEnum(TargetEnum.Get());
	return FReply::Handled();
}

FUserDefinedEnumEditor::~FUserDefinedEnumEditor()
{
}

FName FUserDefinedEnumEditor::GetToolkitFName() const
{
	return FName("EnumEditor");
}

FText FUserDefinedEnumEditor::GetBaseToolkitName() const
{
	return LOCTEXT( "AppLabel", "Enum Editor" );
}

FText FUserDefinedEnumEditor::GetToolkitName() const
{
	if (1 == GetEditingObjects().Num())
	{
		return FAssetEditorToolkit::GetToolkitName();
	}
	return GetBaseToolkitName();
}

FText FUserDefinedEnumEditor::GetToolkitToolTipText() const
{
	if (1 == GetEditingObjects().Num())
	{
		return FAssetEditorToolkit::GetToolkitToolTipText();
	}
	return GetBaseToolkitName();
}

FString FUserDefinedEnumEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("UDEnumWorldCentricTabPrefix", "Enum ").ToString();
}

FLinearColor FUserDefinedEnumEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor( 0.5f, 0.0f, 0.0f, 0.5f );
}

void FEnumDetails::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{
	const TArray<TWeakObjectPtr<UObject>>& Objects = DetailLayout.GetSelectedObjects();
	check(Objects.Num() > 0);

	if (Objects.Num() == 1)
	{
		TargetEnum = CastChecked<UUserDefinedEnum>(Objects[0].Get());
		TSharedRef<IPropertyHandle> PropertyHandle = DetailLayout.GetProperty(FName("Names"), UEnum::StaticClass());

		const FString DocLink = TEXT("Shared/Editors/BlueprintEditor/EnumDetails");

		DetailLayout.EditCategory("Description"); // make Description category appear before Enumerators

		IDetailCategoryBuilder& InputsCategory = DetailLayout.EditCategory("Enumerators", LOCTEXT("EnumDetailsEnumerators", "Enumerators"));

		Layout = MakeShareable( new FUserDefinedEnumLayout(TargetEnum.Get()) );
		InputsCategory.AddCustomBuilder( Layout.ToSharedRef() );

		TSharedPtr<SToolTip> BitmaskFlagsTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("BitmaskFlagsTooltip", "When enabled, this enumeration can be used as a set of explicitly-named bitmask flags. Each enumerator's value will correspond to the index of the bit (flag) in the mask."), nullptr, DocLink, TEXT("Bitmask Flags"));

		InputsCategory.AddCustomRow(LOCTEXT("BitmaskFlagsAttributeLabel", "Bitmask Flags"), true)
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BitmaskFlagsAttributeLabel", "Bitmask Flags"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.ToolTip(BitmaskFlagsTooltip)
			.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FEnumDetails::OnGetBitmaskFlagsAttributeState)
			.OnCheckStateChanged(this, &FEnumDetails::OnBitmaskFlagsAttributeStateChanged)
			.ToolTip(BitmaskFlagsTooltip)
		];
	}
}

FEnumDetails::FEnumDetails()
	: TargetEnum(nullptr)
{
	GEditor->RegisterForUndo(this);
}

FEnumDetails::~FEnumDetails()
{
	GEditor->UnregisterForUndo( this );
}

void FEnumDetails::OnForceRefresh()
{
	if (Layout.IsValid())
	{
		Layout->Refresh();
	}
}

void FEnumDetails::PostUndo(bool bSuccess)
{
	OnForceRefresh();
}

void FEnumDetails::PreChange(const class UUserDefinedEnum* Enum, FEnumEditorUtils::EEnumEditorChangeInfo Info)
{
}

void FEnumDetails::PostChange(const class UUserDefinedEnum* Enum, FEnumEditorUtils::EEnumEditorChangeInfo Info)
{
	if (Enum && (TargetEnum.Get() == Enum))
	{
		OnForceRefresh();
	}
}

ECheckBoxState FEnumDetails::OnGetBitmaskFlagsAttributeState() const
{
	return FEnumEditorUtils::IsEnumeratorBitflagsType(TargetEnum.Get()) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FEnumDetails::OnBitmaskFlagsAttributeStateChanged(ECheckBoxState InNewState)
{
	FEnumEditorUtils::SetEnumeratorBitflagsTypeState(TargetEnum.Get(), InNewState == ECheckBoxState::Checked);
}

bool FUserDefinedEnumLayout::CausedChange() const
{
	for (const TWeakPtr<class FUserDefinedEnumIndexLayout>& Child : Children)
	{
		if (Child.IsValid() && Child.Pin()->CausedChange())
		{
			return true;
		}
	}
	return false;
}

void FUserDefinedEnumLayout::GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder )
{
	const int32 EnumToShowNum = FMath::Max<int32>(0, TargetEnum->NumEnums() - 1);
	Children.Reset(EnumToShowNum);
	for (int32 EnumIdx = 0; EnumIdx < EnumToShowNum; ++EnumIdx)
	{
		TSharedRef<class FUserDefinedEnumIndexLayout> EnumIndexLayout = MakeShareable(new FUserDefinedEnumIndexLayout(TargetEnum.Get(), EnumIdx) );
		ChildrenBuilder.AddCustomBuilder(EnumIndexLayout);
		Children.Add(EnumIndexLayout);
	}
}


bool FUserDefinedEnumIndexLayout::CausedChange() const
{
	return (DisplayNameEditor.IsValid() && DisplayNameEditor->CausedChange()) || (TooltipEditor.IsValid() && TooltipEditor->CausedChange());
}

void FUserDefinedEnumIndexLayout::GenerateHeaderRowContent( FDetailWidgetRow& NodeRow )
{
	DisplayNameEditor = MakeShared<FEditableTextUserDefinedEnum>(TargetEnum, EnumeratorIndex);

	TooltipEditor = MakeShared<FEditableTextUserDefinedEnumTooltip>(TargetEnum, EnumeratorIndex);

	const bool bIsEditable = !DisplayNameEditor->IsReadOnly();

	TSharedRef< SWidget > ClearButton = PropertyCustomizationHelpers::MakeEmptyButton(
		FSimpleDelegate::CreateSP(this, &FUserDefinedEnumIndexLayout::OnEnumeratorRemove),
		LOCTEXT("RemoveEnumToolTip", "Remove enumerator"));
	ClearButton->SetEnabled(bIsEditable);

	NodeRow
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("EnumDisplayNameLabel", "Display Name"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.FillWidth(1.0f)
			[
				SNew(STextPropertyEditableTextBox, DisplayNameEditor.ToSharedRef())
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]

			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(36, 0, 12, 0)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("EnumTooltipLabel", "Description"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]

			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.FillWidth(1.0f)
			[
				SNew(STextPropertyEditableTextBox, TooltipEditor.ToSharedRef())
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2, 0)
			.VAlign(VAlign_Center)
			[
				ClearButton
			]
		]
		.DragDropHandler(MakeShared<FUserDefinedEnumIndexDragDropHandler>(TargetEnum, EnumeratorIndex));
}

void FUserDefinedEnumIndexLayout::OnEnumeratorRemove()
{
	FEnumEditorUtils::RemoveEnumeratorFromUserDefinedEnum(TargetEnum, EnumeratorIndex);
}

#undef LOCTEXT_NAMESPACE
