// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintEditorModule.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EditorUndoClient.h"
#include "Engine/UserDefinedEnum.h"
#include "HAL/Platform.h"
#include "IDetailCustomNodeBuilder.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Kismet2/EnumEditorUtils.h"
#include "Math/Color.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/IToolkit.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API KISMET_API

class FDetailWidgetRow;
class FEditableTextUserDefinedEnum;
class FEditableTextUserDefinedEnumTooltip;
class FSpawnTabArgs;
class FToolBarBuilder;
class IDetailChildrenBuilder;
class IDetailLayoutBuilder;
class SDockTab;

class FUserDefinedEnumEditor : public IUserDefinedEnumEditor
{
	/** App Identifier.*/
	static UE_API const FName UserDefinedEnumEditorAppIdentifier;

	/**	The tab ids for all the tabs used */
	static UE_API const FName EnumeratorsTabId;
	
	/** Property viewing widget */
	TSharedPtr<class IDetailsView> PropertyView;

public:
	/**
	 * Edits the specified enum
	 *
	 * @param	Mode					Asset editing mode for this editor (standalone or world-centric)
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	EnumToEdit				The user defined enum to edit
	 */
	UE_API void InitEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UUserDefinedEnum* EnumToEdit);

	/** Destructor */
	UE_API virtual ~FUserDefinedEnumEditor();

	/** IToolkit interface */
	UE_API virtual FName GetToolkitFName() const override;
	UE_API virtual FText GetBaseToolkitName() const override;
	UE_API virtual FText GetToolkitName() const override;
	UE_API virtual FText GetToolkitToolTipText() const override;
	UE_API virtual FString GetWorldCentricTabPrefix() const override;
	UE_API virtual FLinearColor GetWorldCentricTabColorScale() const override;

	UE_API virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	UE_API virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

protected:
	UE_API TSharedRef<SDockTab> SpawnEnumeratorsTab(const FSpawnTabArgs& Args);

private:
	UE_API void FillToolbar(FToolBarBuilder& ToolbarBuilder);

	/** Handles new enum element request */
	UE_API FReply OnAddNewEnumerator();

private:
	TWeakObjectPtr<UUserDefinedEnum> TargetEnum;
};

/** Details customization for functions and graphs selected in the MyBlueprint panel */
class FEnumDetails : public IDetailCustomization, FEnumEditorUtils::INotifyOnEnumChanged, FEditorUndoClient
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FEnumDetails);
	}

	FEnumDetails();
	~FEnumDetails();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailLayout ) override;

	/** Forces a refresh on the details customization */
	void OnForceRefresh();

	/** FEnumEditorUtils::INotifyOnEnumChanged */
	virtual void PreChange(const class UUserDefinedEnum* Enum, FEnumEditorUtils::EEnumEditorChangeInfo Info) override;
	virtual void PostChange(const class UUserDefinedEnum* Enum, FEnumEditorUtils::EEnumEditorChangeInfo Info) override;

	/** FEditorUndoClient Interface */
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }

private:
	/** Handles the optional bitmask flags attribute */
	ECheckBoxState OnGetBitmaskFlagsAttributeState() const;
	void OnBitmaskFlagsAttributeStateChanged(ECheckBoxState InNewState);

private:
	TSharedPtr<class FUserDefinedEnumLayout> Layout;
	/** The target node that this argument is on */
	TWeakObjectPtr<UUserDefinedEnum> TargetEnum;
};

/** Custom struct for each group of arguments in the function editing details */
class FUserDefinedEnumLayout : public IDetailCustomNodeBuilder, public TSharedFromThis<FUserDefinedEnumLayout>
{
public:
	FUserDefinedEnumLayout(UUserDefinedEnum* InTargetEnum)
		: TargetEnum(InTargetEnum)
	{}

	void Refresh()
	{
		OnRebuildChildren.ExecuteIfBound();
	}

	bool CausedChange() const;

private:
	/** IDetailCustomNodeBuilder Interface*/
	virtual void SetOnRebuildChildren( FSimpleDelegate InOnRegenerateChildren ) override { OnRebuildChildren = InOnRegenerateChildren; }
	virtual void GenerateHeaderRowContent( FDetailWidgetRow& NodeRow ) override {}
	virtual void GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder ) override;
	virtual void Tick( float DeltaTime ) override {}
	virtual bool RequiresTick() const override { return false; }
	virtual FName GetName() const override { return NAME_None; }
	virtual bool InitiallyCollapsed() const override { return false; }

private:
	FSimpleDelegate OnRebuildChildren;
	/** The target node that this argument is on */
	TWeakObjectPtr<UUserDefinedEnum> TargetEnum;

	TArray<TWeakPtr<class FUserDefinedEnumIndexLayout>> Children;
};

/** Custom struct for each group of arguments in the function editing details */
class FUserDefinedEnumIndexLayout : public IDetailCustomNodeBuilder, public TSharedFromThis<FUserDefinedEnumIndexLayout>
{
public:
	FUserDefinedEnumIndexLayout(UUserDefinedEnum* InTargetEnum, int32 InEnumeratorIndex)
		: TargetEnum(InTargetEnum)
		, EnumeratorIndex(InEnumeratorIndex)
	{}

	bool CausedChange() const;

private:
	/** IDetailCustomNodeBuilder Interface*/
	virtual void SetOnRebuildChildren( FSimpleDelegate InOnRegenerateChildren ) override {};
	virtual void GenerateHeaderRowContent( FDetailWidgetRow& NodeRow ) override;
	virtual void GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder ) override {};
	virtual void Tick( float DeltaTime ) override {}
	virtual bool RequiresTick() const override { return false; }
	virtual FName GetName() const override { return NAME_None; }
	virtual bool InitiallyCollapsed() const override { return false; }

private:
	/** Deletes the enumerator */
	void OnEnumeratorRemove();

private:
	/** The target node that this argument is on */
	UUserDefinedEnum* TargetEnum;

	/** Index of enumerator */
	int32 EnumeratorIndex;

	/** The editable text interface for the display name data */
	TSharedPtr<FEditableTextUserDefinedEnum> DisplayNameEditor;

	/** The editable text interface for the tooltip data */
	TSharedPtr<FEditableTextUserDefinedEnumTooltip> TooltipEditor;
};

#undef UE_API
