// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserDefinedStructureEditor.h"

#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DetailsViewArgs.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraphSchema_K2.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GenericPlatform/ICursor.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailCustomNodeBuilder.h"
#include "IDetailCustomization.h"
#include "IDetailDragDropHandler.h"
#include "IDetailsView.h"
#include "Input/DragAndDrop.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/StructureEditorUtils.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Guid.h"
#include "Misc/NotifyHook.h"
#include "Misc/Optional.h"
#include "Modules/ModuleManager.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "SPinTypeSelector.h"
#include "SPositiveActionButton.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/ToolBarStyle.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/StructOnScope.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UserDefinedStructure/UserDefinedStructEditorData.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"

class UObject;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "StructureEditor"

///////////////////////////////////////////////////////////////////////////////////////
// FDefaultValueDetails

class FDefaultValueDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance(TWeakPtr<class FStructureDefaultValueView> InDefaultValueView, TSharedPtr<FStructOnScope> InStructData)
	{
		return MakeShareable(new FDefaultValueDetails(InDefaultValueView, InStructData));
	}

	FDefaultValueDetails(TWeakPtr<class FStructureDefaultValueView> InDefaultValueView, TSharedPtr<FStructOnScope> InStructData)
		: DefaultValueView(InDefaultValueView)
		, StructData(InStructData)
	{}

	~FDefaultValueDetails()
	{
	}

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(class IDetailLayoutBuilder& DetailLayout) override;

	/** Callback when finished changing properties to export the default value from the property to where strings are stored */
	void OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent);

private:
	TWeakObjectPtr<UUserDefinedStruct> UserDefinedStruct;
	TWeakPtr<class FStructureDefaultValueView> DefaultValueView;
	TSharedPtr<FStructOnScope> StructData;
	IDetailLayoutBuilder* DetailLayoutPtr;
};

void FDefaultValueDetails::CustomizeDetails(class IDetailLayoutBuilder& DetailLayout) 
{
	DetailLayoutPtr = &DetailLayout;
	const TArray<TWeakObjectPtr<UObject>>& Objects = DetailLayout.GetSelectedObjects();
	check(Objects.Num() > 0);

	if (Objects.Num() == 1)
	{
		UserDefinedStruct = CastChecked<UUserDefinedStruct>(Objects[0].Get());

		if (TSharedPtr<const IDetailsView> DetailsView = DetailLayout.GetDetailsViewSharedPtr())
		{
			DetailsView->OnFinishedChangingProperties().AddSP(this, &FDefaultValueDetails::OnFinishedChangingProperties);
		}

		IDetailCategoryBuilder& StructureCategory = DetailLayout.EditCategory("DefaultValues", LOCTEXT("DefaultValues", "Default Values"));

		for (TFieldIterator<FProperty> PropertyIter(UserDefinedStruct.Get()); PropertyIter; ++PropertyIter)
		{
			StructureCategory.AddExternalStructureProperty(StructData, (*PropertyIter)->GetFName());
		}
	}
}

/////////////////////////////////
// FStructureDefaultValueView

class FStructureDefaultValueView : public FStructureEditorUtils::INotifyOnStructChanged, public TSharedFromThis<FStructureDefaultValueView>, public FNotifyHook
{
public:
	FStructureDefaultValueView(UUserDefinedStruct* EditedStruct) 
		: UserDefinedStruct(EditedStruct)
		, PropertyChangeRecursionGuard(0)
	{
	}

	void Initialize()
	{
		StructData = MakeShareable(new FStructOnScope(UserDefinedStruct.Get()));
		UserDefinedStruct.Get()->InitializeDefaultValue(StructData->GetStructMemory());
		StructData->SetPackage(UserDefinedStruct->GetOutermost());

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		FDetailsViewArgs ViewArgs;
		ViewArgs.bAllowSearch = false;
		ViewArgs.bHideSelectionTip = false;
		ViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		ViewArgs.NotifyHook = this;

		DetailsView = PropertyModule.CreateDetailView(ViewArgs);
		TWeakPtr< FStructureDefaultValueView > LocalWeakThis = SharedThis(this);
		FOnGetDetailCustomizationInstance LayoutStructDetails = FOnGetDetailCustomizationInstance::CreateStatic(&FDefaultValueDetails::MakeInstance, LocalWeakThis, StructData);
		DetailsView->RegisterInstancedCustomPropertyLayout(UUserDefinedStruct::StaticClass(), LayoutStructDetails);
		DetailsView->SetObject(UserDefinedStruct.Get());
	}

	virtual ~FStructureDefaultValueView()
	{
	}

	UUserDefinedStruct* GetUserDefinedStruct()
	{
		return UserDefinedStruct.Get();
	}

	TSharedPtr<class SWidget> GetWidget()
	{
		return DetailsView;
	}

	virtual void PreChange(const class UUserDefinedStruct* Struct, FStructureEditorUtils::EStructureEditorChangeInfo Info) override
	{
		// No need to destroy the struct data if only the default values are changing
		if (Info != FStructureEditorUtils::DefaultValueChanged)
		{
			StructData->Destroy();
			DetailsView->SetObject(nullptr);
			DetailsView->OnFinishedChangingProperties().Clear();
		}
	}

	virtual void PostChange(const class UUserDefinedStruct* Struct, FStructureEditorUtils::EStructureEditorChangeInfo Info) override
	{
		// If change is due to default value, then struct data was not destroyed (see PreChange) and therefore does not need to be re-initialized
		if (Info != FStructureEditorUtils::DefaultValueChanged)
		{
			StructData->Initialize(UserDefinedStruct.Get());

			// Force the set object call because we may be called multiple times in a row if more than one struct was changed at the same time
			DetailsView->SetObject(UserDefinedStruct.Get(), true);
		}

		UserDefinedStruct.Get()->InitializeDefaultValue(StructData->GetStructMemory());
	}

	// FNotifyHook interface
	virtual void NotifyPreChange( FProperty* PropertyAboutToChange ) override
	{
		++PropertyChangeRecursionGuard;
	}

	virtual void NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override
	{
		--PropertyChangeRecursionGuard;
	}
	// End of FNotifyHook interface

	/** Returns TRUE when property changes are complete, according to recursion counts */
	bool IsPropertyChangeComplete() { return PropertyChangeRecursionGuard == 0; }
private:

	/** Struct on scope data that is being viewed in the details panel */
	TSharedPtr<FStructOnScope> StructData;

	/** Details view being used for viewing the struct */
	TSharedPtr<class IDetailsView> DetailsView;

	/** User defined struct that is being represented */
	const TWeakObjectPtr<UUserDefinedStruct> UserDefinedStruct;

	/** Manages recursion in property changing, to ensure we only compile the structure when all properties are done changing */
	int32 PropertyChangeRecursionGuard;
};

///////////////////////////////////////////////////////////////////////////////////////
// FDefaultValueDetails

void FDefaultValueDetails::OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent)
{
	if (DefaultValueView.Pin()->IsPropertyChangeComplete())
	{
		UStruct* OwnerStruct = PropertyChangedEvent.MemberProperty->GetOwnerStruct();

		check(PropertyChangedEvent.MemberProperty && OwnerStruct);

		if ( ensure(OwnerStruct == UserDefinedStruct.Get()) )
		{
			const FProperty* DirectProperty = PropertyChangedEvent.MemberProperty;
			while (DirectProperty && !DirectProperty->GetOwner<const UUserDefinedStruct>())
			{
				DirectProperty = DirectProperty->GetOwner<const FProperty>();
			}
			ensure(nullptr != DirectProperty);

			if (DirectProperty)
			{
				FString DefaultValueString;
				bool bDefaultValueSet = false;
				{
					if (StructData.IsValid() && StructData->IsValid())
					{
						bDefaultValueSet = FBlueprintEditorUtils::PropertyValueToString(DirectProperty, StructData->GetStructMemory(), DefaultValueString, OwnerStruct);
					}
				}

				const FGuid VarGuid = FStructureEditorUtils::GetGuidForProperty(DirectProperty);
				if (bDefaultValueSet && VarGuid.IsValid())
				{
					FStructureEditorUtils::ChangeVariableDefaultValue(UserDefinedStruct.Get(), VarGuid, DefaultValueString);
				}
			}
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////
// FUserDefinedStructureDetails

class FUserDefinedStructureDetails : public IDetailCustomization, FStructureEditorUtils::INotifyOnStructChanged
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance(TWeakPtr<FUserDefinedStructureEditor> InStructureEditor)
	{
		return MakeShareable(new FUserDefinedStructureDetails(InStructureEditor));
	}

	FUserDefinedStructureDetails(TWeakPtr<FUserDefinedStructureEditor> InStructureEditor)
		: StructureEditor(InStructureEditor)
	{
	}

	~FUserDefinedStructureDetails()
	{
	}

	UUserDefinedStruct* GetUserDefinedStruct()
	{
		return UserDefinedStruct.Get();
	}

	struct FStructVariableDescription* FindStructureFieldByGuid(FGuid Guid)
	{
		if (auto Struct = GetUserDefinedStruct())
		{
			return FStructureEditorUtils::GetVarDesc(Struct).FindByPredicate(FStructureEditorUtils::FFindByGuidHelper<FStructVariableDescription>(Guid));
		}
		return NULL;
	}

	const TWeakPtr<FUserDefinedStructureEditor>& GetStructureEditor() const
	{
		return StructureEditor;
	}

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(class IDetailLayoutBuilder& DetailLayout) override;

	/** FStructureEditorUtils::INotifyOnStructChanged */
	virtual void PreChange(const class UUserDefinedStruct* Struct, FStructureEditorUtils::EStructureEditorChangeInfo Info) override {}
	virtual void PostChange(const class UUserDefinedStruct* Struct, FStructureEditorUtils::EStructureEditorChangeInfo Info) override;

private:
	TWeakObjectPtr<UUserDefinedStruct> UserDefinedStruct;
	TSharedPtr<class FUserDefinedStructureLayout> Layout;
	TWeakPtr<FUserDefinedStructureEditor> StructureEditor;
};

///////////////////////////////////////////////////////////////////////////////////////
// FUserDefinedStructureEditor

const FName FUserDefinedStructureEditor::MemberVariablesTabId( TEXT( "UserDefinedStruct_MemberVariablesEditor" ) );
const FName FUserDefinedStructureEditor::DefaultValuesTabId(TEXT("UserDefinedStruct_DefaultValuesEditor"));
const FName FUserDefinedStructureEditor::UserDefinedStructureEditorAppIdentifier( TEXT( "UserDefinedStructEditorApp" ) );

void FUserDefinedStructureEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_UserDefinedStructureEditor", "User-Defined Structure Editor"));

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner( MemberVariablesTabId, FOnSpawnTab::CreateSP(this, &FUserDefinedStructureEditor::SpawnStructureTab) )
		.SetDisplayName( LOCTEXT("MemberVariablesEditor", "Structure") )
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Kismet.Tabs.Variables"));

	InTabManager->RegisterTabSpawner( DefaultValuesTabId, FOnSpawnTab::CreateSP(this, &FUserDefinedStructureEditor::SpawnStructureDefaultValuesTab))
		.SetDisplayName(LOCTEXT("DefaultValuesEditor", "Default Values"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Details"));
}

void FUserDefinedStructureEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner( MemberVariablesTabId );
}

void FUserDefinedStructureEditor::InitEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UUserDefinedStruct* Struct)
{
	UserDefinedStruct = Struct;
	InitialPinType = FEdGraphPinType(UEdGraphSchema_K2::PC_Boolean, NAME_None, nullptr, EPinContainerType::None, false, FEdGraphTerminalType());

	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout( "Standalone_UserDefinedStructureEditor_Layout_v3" )
	->AddArea
	(
		FTabManager::NewPrimaryArea() 
		->SetOrientation(Orient_Vertical)
		->Split
		(
			FTabManager::NewSplitter()
			->Split
			(
				FTabManager::NewStack()
				->AddTab( MemberVariablesTabId, ETabState::OpenedTab )
				->AddTab( DefaultValuesTabId, ETabState::OpenedTab )
				->SetForegroundTab(MemberVariablesTabId)
			)
		)
	);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor( Mode, InitToolkitHost, UserDefinedStructureEditorAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, Struct );

	TSharedPtr<FExtender> Extender = MakeShared<FExtender>();
	Extender->AddToolBarExtension("Asset", EExtensionHook::After, GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateSP(this, &FUserDefinedStructureEditor::FillToolbar));
	AddToolbarExtender(Extender);
	RegenerateMenusAndToolbars();
}

void FUserDefinedStructureEditor::SetInitialPinType(FEdGraphPinType PinType)
{
	InitialPinType = PinType;
}

FUserDefinedStructureEditor::~FUserDefinedStructureEditor()
{
}

FName FUserDefinedStructureEditor::GetToolkitFName() const
{
	return FName("UserDefinedStructureEditor");
}

FText FUserDefinedStructureEditor::GetBaseToolkitName() const
{
	return LOCTEXT( "AppLabel", "Struct Editor" );
}

FText FUserDefinedStructureEditor::GetToolkitName() const
{
	if (1 == GetEditingObjects().Num())
	{
		return FAssetEditorToolkit::GetToolkitName();
	}
	return GetBaseToolkitName();
}

FText FUserDefinedStructureEditor::GetToolkitToolTipText() const
{
	if (1 == GetEditingObjects().Num())
	{
		return FAssetEditorToolkit::GetToolkitToolTipText();
	}
	return GetBaseToolkitName();
}

FString FUserDefinedStructureEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("UDStructWorldCentricTabPrefix", "Struct ").ToString();
}

FLinearColor FUserDefinedStructureEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor( 0.0f, 0.0f, 1.0f, 0.5f );
}

TSharedRef<SDockTab> FUserDefinedStructureEditor::SpawnStructureTab(const FSpawnTabArgs& Args)
{
	check( Args.GetTabId() == MemberVariablesTabId );

	UUserDefinedStruct* EditedStruct = NULL;
	const auto& EditingObjs = GetEditingObjects();
	if (EditingObjs.Num())
	{
		EditedStruct = Cast<UUserDefinedStruct>(EditingObjs[ 0 ]);
	}

	// Create a property view
	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bShowOptions = false;
	PropertyView = EditModule.CreateDetailView(DetailsViewArgs);
	TWeakPtr<FUserDefinedStructureEditor> LocalWeakThis = SharedThis(this);
	FOnGetDetailCustomizationInstance LayoutStructDetails = FOnGetDetailCustomizationInstance::CreateStatic(&FUserDefinedStructureDetails::MakeInstance, LocalWeakThis);
	PropertyView->RegisterInstancedCustomPropertyLayout(UUserDefinedStruct::StaticClass(), LayoutStructDetails);
	PropertyView->SetObject(EditedStruct);

	return SNew(SDockTab)
		.Label( LOCTEXT("UserDefinedStructureEditor", "Structure") )
		.TabColorScale( GetTabColorScale() )
		[
			PropertyView.ToSharedRef()
		];
}

TSharedRef<SDockTab> FUserDefinedStructureEditor::SpawnStructureDefaultValuesTab(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == DefaultValuesTabId);

	UUserDefinedStruct* EditedStruct = NULL;
	const auto& EditingObjs = GetEditingObjects();
	if (EditingObjs.Num())
	{
		EditedStruct = Cast<UUserDefinedStruct>(EditingObjs[0]);
	}

	DefaultValueView = MakeShareable(new FStructureDefaultValueView(EditedStruct));
	DefaultValueView->Initialize();

	return SNew(SDockTab)
		.Label(LOCTEXT("UserDefinedStructureDefaultValuesEditor", "Default Values"))
		.TabColorScale(GetTabColorScale())
		[
			DefaultValueView->GetWidget().ToSharedRef()
		];
}

void FUserDefinedStructureEditor::FillToolbar(FToolBarBuilder& ToolbarBuilder)
{
	const FToolBarStyle& ToolBarStyle = ToolbarBuilder.GetStyleSet()->GetWidgetStyle<FToolBarStyle>(ToolbarBuilder.GetStyleName());

	ToolbarBuilder.BeginSection("UserDefinedStructure");

	TSharedPtr<SLayeredImage> CompileStatusImage;
	ToolbarBuilder.AddWidget(
		SNew(SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding(ToolBarStyle.ButtonPadding)
		[
			SAssignNew(CompileStatusImage, SLayeredImage)
			.Image(FAppStyle::Get().GetBrush("Blueprint.CompileStatus.Background"))
			.ToolTipText(this, &FUserDefinedStructureEditor::OnGetStatusTooltip)
		]);
	CompileStatusImage->AddLayer(TAttribute<const FSlateBrush*>::CreateSP(this, &FUserDefinedStructureEditor::OnGetStructureStatus));

	ToolbarBuilder.AddWidget(
		SNew(SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Fill)
		.Padding(ToolBarStyle.ButtonPadding)
		[
			SNew(SPositiveActionButton)
			.Text(LOCTEXT("AddStructVariable", "Add Variable"))
			.ToolTipText(LOCTEXT("AddStructVariableToolTip", "Adds a new member variable to the end of this structure"))
			.OnClicked(this, &FUserDefinedStructureEditor::OnAddNewField)
		]);

	ToolbarBuilder.EndSection();
}

FReply FUserDefinedStructureEditor::OnAddNewField()
{
	if (UserDefinedStruct.IsValid())
	{
		FStructureEditorUtils::AddVariable(UserDefinedStruct.Get(), InitialPinType);

		// Ensure the member variables tab is topmost so the user can edit the newly-added variable
		InvokeTab(MemberVariablesTabId);
	}

	return FReply::Handled();
}

const FSlateBrush* FUserDefinedStructureEditor::OnGetStructureStatus() const
{
	if (UserDefinedStruct.IsValid())
	{
		switch (UserDefinedStruct->Status.GetValue())
		{
		case EUserDefinedStructureStatus::UDSS_Error:
			return FAppStyle::Get().GetBrush("Blueprint.CompileStatus.Overlay.Error");
		case EUserDefinedStructureStatus::UDSS_UpToDate:
			return FAppStyle::Get().GetBrush("Blueprint.CompileStatus.Overlay.Good");
		default:
			return FAppStyle::Get().GetBrush("Blueprint.CompileStatus.Overlay.Unknown");
		}
	}
	return nullptr;
}

FText FUserDefinedStructureEditor::OnGetStatusTooltip() const
{
	if (UserDefinedStruct.IsValid())
	{
		switch (UserDefinedStruct->Status.GetValue())
		{
		case EUserDefinedStructureStatus::UDSS_Error:
			return FText::FromString(UserDefinedStruct->ErrorMessage);
		default:
			return LOCTEXT("GoodToGo_Status", "Good to go");
		}
	}
	return FText::GetEmpty();
}

///////////////////////////////////////////////////////////////////////////////////////
// FUserDefinedStructureLayout

//Represents single structure (List of fields)
class FUserDefinedStructureLayout : public IDetailCustomNodeBuilder, public TSharedFromThis<FUserDefinedStructureLayout>
{
public:
	FUserDefinedStructureLayout(TWeakPtr<class FUserDefinedStructureDetails> InStructureDetails)
		: StructureDetails(InStructureDetails)
	{}

	void OnChanged()
	{
		OnRegenerateChildren.ExecuteIfBound();
	}

	FText OnGetTooltipText() const
	{
		auto StructureDetailsSP = StructureDetails.Pin();
		if (StructureDetailsSP.IsValid())
		{
			if (auto Struct = StructureDetailsSP->GetUserDefinedStruct())
			{
				return FText::FromString(FStructureEditorUtils::GetTooltip(Struct));
			}
		}
		return FText();
	}

	void OnTooltipCommitted(const FText& NewText, ETextCommit::Type InTextCommit)
	{
		auto StructureDetailsSP = StructureDetails.Pin();
		if (StructureDetailsSP.IsValid())
		{
			if (auto Struct = StructureDetailsSP->GetUserDefinedStruct())
			{
				FStructureEditorUtils::ChangeTooltip(Struct, NewText.ToString());
			}
		}
	}

	/** IDetailCustomNodeBuilder Interface*/
	virtual void SetOnRebuildChildren( FSimpleDelegate InOnRegenerateChildren ) override 
	{
		OnRegenerateChildren = InOnRegenerateChildren;
	}
	virtual void GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder ) override;

	virtual void GenerateHeaderRowContent( FDetailWidgetRow& NodeRow ) override {}

	virtual void Tick( float DeltaTime ) override {}
	virtual bool RequiresTick() const override { return false; }
	virtual FName GetName() const override 
	{ 
		auto StructureDetailsSP = StructureDetails.Pin();
		if(StructureDetailsSP.IsValid())
		{
			if(auto Struct = StructureDetailsSP->GetUserDefinedStruct())
			{
				return Struct->GetFName();
			}
		}
		return NAME_None; 
	}
	virtual bool InitiallyCollapsed() const override { return false; }

private:
	TWeakPtr<class FUserDefinedStructureDetails> StructureDetails;
	FSimpleDelegate OnRegenerateChildren;
};

///////////////////////////////////////////////////////////////////////////////////////
// FUserDefinedStructureFieldDragDropOp

/** Provides information about the source row (single field) being dragged */
class FUserDefinedStructureFieldDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FUserDefinedStructureFieldDragDropOp, FDecoratedDragDropOp);

	FUserDefinedStructureFieldDragDropOp(TWeakPtr<FUserDefinedStructureDetails> InStructureDetails, const FGuid& InFieldGuid)
		: StructureDetails(InStructureDetails)
		, FieldGuid(InFieldGuid)
	{
		MouseCursor = EMouseCursor::GrabHandClosed;
		if (TSharedPtr<FUserDefinedStructureDetails> StructureDetailsSP = InStructureDetails.Pin())
		{
			VariableFriendlyName = FStructureEditorUtils::GetVariableFriendlyName(StructureDetailsSP->GetUserDefinedStruct(), FieldGuid);
		}
	}

	void Init()
	{
		SetValidTarget(false);
		SetupDefaults();
		Construct();
	}

	void SetValidTarget(bool IsValidTarget)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("StructVariableName"), FText::FromString(VariableFriendlyName));

		if (IsValidTarget)
		{
			CurrentHoverText = FText::Format(LOCTEXT("MoveVariableHere", "Move '{StructVariableName}' Here"), Args);
			CurrentIconBrush = FAppStyle::Get().GetBrush("Graph.ConnectorFeedback.OK");
		}
		else
		{
			CurrentHoverText = FText::Format(LOCTEXT("CannotMoveVariableHere", "Cannot Move '{StructVariableName}' Here"), Args);
			CurrentIconBrush = FAppStyle::Get().GetBrush("Graph.ConnectorFeedback.Error");
		}
	}

	const TWeakPtr<FUserDefinedStructureDetails>& GetStructureDetails() const
	{
		return StructureDetails;
	}

	const FGuid& GetFieldGuid() const
	{
		return FieldGuid;
	}

private:
	TWeakPtr<FUserDefinedStructureDetails> StructureDetails;
	FGuid FieldGuid;
	FString VariableFriendlyName;
};

///////////////////////////////////////////////////////////////////////////////////////
// FUserDefinedStructureFieldDragDropHandler

/** Handles drag-and-drop (as both source and target) for a single field's widget row */
class FUserDefinedStructureFieldDragDropHandler : public IDetailDragDropHandler
{
public:
	FUserDefinedStructureFieldDragDropHandler(TWeakPtr<FUserDefinedStructureDetails> InStructureDetails, const FGuid& InFieldGuid)
		: StructureDetails(InStructureDetails)
		, FieldGuid(InFieldGuid)
	{
	}

	virtual TSharedPtr<FDragDropOperation> CreateDragDropOperation() const override
	{
		TSharedPtr<FUserDefinedStructureFieldDragDropOp> DragOp = MakeShared<FUserDefinedStructureFieldDragDropOp>(StructureDetails, FieldGuid);
		DragOp->Init();
		return DragOp;
	}

	virtual TOptional<EItemDropZone> CanAcceptDrop(const FDragDropEvent& DragDropSource, EItemDropZone DropZone) const override
	{
		const TSharedPtr<FUserDefinedStructureFieldDragDropOp> DragOp = DragDropSource.GetOperationAs<FUserDefinedStructureFieldDragDropOp>();
		if (!DragOp.IsValid())
		{
			return TOptional<EItemDropZone>();
		}

		// Struct must match between drag source and drop target
		const TSharedPtr<FUserDefinedStructureDetails> OtherStructureDetailsSP = DragOp->GetStructureDetails().Pin();
		const TSharedPtr<FUserDefinedStructureDetails> MyStructureDetailsSP = StructureDetails.Pin();
		if (!OtherStructureDetailsSP.IsValid() || !MyStructureDetailsSP.IsValid() || OtherStructureDetailsSP->GetUserDefinedStruct() != MyStructureDetailsSP->GetUserDefinedStruct())
		{
			DragOp->SetValidTarget(false);
			return TOptional<EItemDropZone>();
		}

		// Struct fields must be moved above or below, so don't allow dropping directly onto a row
		const EItemDropZone OverrideZone = (DropZone == EItemDropZone::BelowItem) ? EItemDropZone::BelowItem : EItemDropZone::AboveItem;
		const FStructureEditorUtils::EMovePosition MovePosition = (OverrideZone == EItemDropZone::BelowItem) ? FStructureEditorUtils::PositionBelow : FStructureEditorUtils::PositionAbove;
		if (!FStructureEditorUtils::CanMoveVariable(MyStructureDetailsSP->GetUserDefinedStruct(), DragOp->GetFieldGuid(), FieldGuid, MovePosition))
		{
			DragOp->SetValidTarget(false);
			return TOptional<EItemDropZone>();
		}

		DragOp->SetValidTarget(true);
		return OverrideZone;
	}

	virtual bool AcceptDrop(const FDragDropEvent& DragDropSource, EItemDropZone DropZone) const override
	{
		const TSharedPtr<FUserDefinedStructureFieldDragDropOp> DragOp = DragDropSource.GetOperationAs<FUserDefinedStructureFieldDragDropOp>();
		if (!DragOp.IsValid())
		{
			return false;
		}

		// Struct must match between drag source and drop target
		const TSharedPtr<FUserDefinedStructureDetails> OtherStructureDetailsSP = DragOp->GetStructureDetails().Pin();
		const TSharedPtr<FUserDefinedStructureDetails> MyStructureDetailsSP = StructureDetails.Pin();
		if (!OtherStructureDetailsSP.IsValid() || !MyStructureDetailsSP.IsValid() || OtherStructureDetailsSP->GetUserDefinedStruct() != MyStructureDetailsSP->GetUserDefinedStruct())
		{
			return false;
		}

		const FStructureEditorUtils::EMovePosition MovePosition = (DropZone == EItemDropZone::BelowItem) ? FStructureEditorUtils::PositionBelow : FStructureEditorUtils::PositionAbove;
		return FStructureEditorUtils::MoveVariable(MyStructureDetailsSP->GetUserDefinedStruct(), DragOp->GetFieldGuid(), FieldGuid, MovePosition);
	}

private:
	TWeakPtr<FUserDefinedStructureDetails> StructureDetails;
	FGuid FieldGuid;
};

///////////////////////////////////////////////////////////////////////////////////////
// FUserDefinedStructureFieldLayout

//Represents single field
class FUserDefinedStructureFieldLayout : public IDetailCustomNodeBuilder, public TSharedFromThis<FUserDefinedStructureFieldLayout>
{
public:
	FUserDefinedStructureFieldLayout(TWeakPtr<class FUserDefinedStructureDetails> InStructureDetails, TWeakPtr<class FUserDefinedStructureLayout> InStructureLayout, FGuid InFieldGuid)
		: StructureDetails(InStructureDetails)
		, StructureLayout(InStructureLayout)
		, FieldGuid(InFieldGuid)
	{}

	void OnChanged()
	{
		OnRegenerateChildren.ExecuteIfBound();
	}

	FText OnGetNameText() const
	{
		auto StructureDetailsSP = StructureDetails.Pin();
		if(StructureDetailsSP.IsValid())
		{
			return FText::FromString(FStructureEditorUtils::GetVariableFriendlyName(StructureDetailsSP->GetUserDefinedStruct(), FieldGuid));
		}
		return FText::GetEmpty();
	}

	void OnNameTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit)
	{
		auto StructureDetailsSP = StructureDetails.Pin();
		if(StructureDetailsSP.IsValid())
		{
			const FString NewNameStr = NewText.ToString();
			FStructureEditorUtils::RenameVariable(StructureDetailsSP->GetUserDefinedStruct(), FieldGuid, NewNameStr);
		}
	}

	FEdGraphPinType OnGetPinInfo() const
	{
		auto StructureDetailsSP = StructureDetails.Pin();
		if(StructureDetailsSP.IsValid())
		{
			if(const FStructVariableDescription* FieldDesc = StructureDetailsSP->FindStructureFieldByGuid(FieldGuid))
			{
				return FieldDesc->ToPinType();
			}
		}
		return FEdGraphPinType();
	}

	void PinInfoChanged(const FEdGraphPinType& PinType)
	{
		auto StructureDetailsSP = StructureDetails.Pin();
		if(StructureDetailsSP.IsValid())
		{
			if (FStructureEditorUtils::ChangeVariableType(StructureDetailsSP->GetUserDefinedStruct(), FieldGuid, PinType))
			{
				if (TSharedPtr<FUserDefinedStructureEditor> StructureEditorSP = StructureDetailsSP->GetStructureEditor().Pin())
				{
					StructureEditorSP->SetInitialPinType(PinType);
				}
			}
			else
			{
				FNotificationInfo NotificationInfo(LOCTEXT("VariableTypeChange_FailureNotification", "Variable type change failed (the selected type may not be compatible with this struct). See log for details."));
				NotificationInfo.ExpireDuration = 5.0f;
				FSlateNotificationManager::Get().AddNotification(NotificationInfo);
			}
		}
	}

	void OnPrePinInfoChange(const FEdGraphPinType& PinType)
	{

	}

	void OnRemovField()
	{
		auto StructureDetailsSP = StructureDetails.Pin();
		if(StructureDetailsSP.IsValid())
		{
			FStructureEditorUtils::RemoveVariable(StructureDetailsSP->GetUserDefinedStruct(), FieldGuid);
		}
	}

	bool IsRemoveButtonEnabled()
	{
		auto StructureDetailsSP = StructureDetails.Pin();
		if (StructureDetailsSP.IsValid())
		{
			if (auto UDStruct = StructureDetailsSP->GetUserDefinedStruct())
			{
				return (FStructureEditorUtils::GetVarDesc(UDStruct).Num() > 1);
			}
		}
		return false;
	}

	FText OnGetTooltipText() const
	{
		auto StructureDetailsSP = StructureDetails.Pin();
		if (StructureDetailsSP.IsValid())
		{
			if (const FStructVariableDescription* FieldDesc = StructureDetailsSP->FindStructureFieldByGuid(FieldGuid))
			{
				return FText::FromString(FieldDesc->ToolTip);
			}
		}
		return FText();
	}

	void OnTooltipCommitted(const FText& NewText, ETextCommit::Type InTextCommit)
	{
		auto StructureDetailsSP = StructureDetails.Pin();
		if (StructureDetailsSP.IsValid())
		{
			FStructureEditorUtils::ChangeVariableTooltip(StructureDetailsSP->GetUserDefinedStruct(), FieldGuid, NewText.ToString());
		}
	}

	ECheckBoxState OnGetEditableOnBPInstanceState() const
	{
		auto StructureDetailsSP = StructureDetails.Pin();
		if (StructureDetailsSP.IsValid())
		{
			if (const FStructVariableDescription* FieldDesc = StructureDetailsSP->FindStructureFieldByGuid(FieldGuid))
			{
				return !FieldDesc->bDontEditOnInstance ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
		}
		return ECheckBoxState::Undetermined;
	}

	void OnEditableOnBPInstanceCommitted(ECheckBoxState InNewState)
	{
		auto StructureDetailsSP = StructureDetails.Pin();
		if (StructureDetailsSP.IsValid())
		{
			FStructureEditorUtils::ChangeEditableOnBPInstance(StructureDetailsSP->GetUserDefinedStruct(), FieldGuid, ECheckBoxState::Unchecked != InNewState);
		}
	}

	ECheckBoxState OnGetSaveGameState() const
	{
		auto StructureDetailsSP = StructureDetails.Pin();
		if (StructureDetailsSP.IsValid())
		{
			if (const FStructVariableDescription* FieldDesc = StructureDetailsSP->FindStructureFieldByGuid(FieldGuid))
			{
				return FieldDesc->bEnableSaveGame ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
		}
		return ECheckBoxState::Undetermined;
	}

	void OnSaveGameCommitted(ECheckBoxState InNewState)
	{
		auto StructureDetailsSP = StructureDetails.Pin();
		if (StructureDetailsSP.IsValid() && (ECheckBoxState::Undetermined != InNewState))
		{
			FStructureEditorUtils::ChangeSaveGameEnabled(StructureDetailsSP->GetUserDefinedStruct(), FieldGuid, InNewState == ECheckBoxState::Checked);
		}
	}

	// Multi-line text
	EVisibility IsMultiLineTextOptionVisible() const
	{
		auto StructureDetailsSP = StructureDetails.Pin();
		if (StructureDetailsSP.IsValid())
		{
			return FStructureEditorUtils::CanEnableMultiLineText(StructureDetailsSP->GetUserDefinedStruct(), FieldGuid) ? EVisibility::Visible : EVisibility::Collapsed;
		}
		return EVisibility::Collapsed;
	}

	ECheckBoxState OnGetMultiLineTextEnabled() const
	{
		auto StructureDetailsSP = StructureDetails.Pin();
		if (StructureDetailsSP.IsValid())
		{
			return FStructureEditorUtils::IsMultiLineTextEnabled(StructureDetailsSP->GetUserDefinedStruct(), FieldGuid) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
		return ECheckBoxState::Undetermined;
	}

	void OnMultiLineTextEnabledCommitted(ECheckBoxState InNewState)
	{
		auto StructureDetailsSP = StructureDetails.Pin();
		if (StructureDetailsSP.IsValid() && (ECheckBoxState::Undetermined != InNewState))
		{
			FStructureEditorUtils::ChangeMultiLineTextEnabled(StructureDetailsSP->GetUserDefinedStruct(), FieldGuid, ECheckBoxState::Checked == InNewState);
		}
	}

	// 3D widget
	EVisibility Is3dWidgetOptionVisible() const
	{
		auto StructureDetailsSP = StructureDetails.Pin();
		if (StructureDetailsSP.IsValid())
		{
			return FStructureEditorUtils::CanEnable3dWidget(StructureDetailsSP->GetUserDefinedStruct(), FieldGuid) ? EVisibility::Visible : EVisibility::Collapsed;
		}
		return EVisibility::Collapsed;
	}

	ECheckBoxState OnGet3dWidgetEnabled() const
	{
		auto StructureDetailsSP = StructureDetails.Pin();
		if (StructureDetailsSP.IsValid())
		{
			return FStructureEditorUtils::Is3dWidgetEnabled(StructureDetailsSP->GetUserDefinedStruct(), FieldGuid) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
		return ECheckBoxState::Undetermined;
	}

	void On3dWidgetEnabledCommitted(ECheckBoxState InNewState)
	{
		auto StructureDetailsSP = StructureDetails.Pin();
		if (StructureDetailsSP.IsValid() && (ECheckBoxState::Undetermined != InNewState))
		{
			FStructureEditorUtils::Change3dWidgetEnabled(StructureDetailsSP->GetUserDefinedStruct(), FieldGuid, ECheckBoxState::Checked == InNewState);
		}
	}

	// Value Range
	EVisibility IsValueRangeOptionVisible() const
	{
		if (const TSharedPtr<FUserDefinedStructureDetails> StructureDetailsSP = StructureDetails.Pin())
		{
			return FStructureEditorUtils::CanEditValueRange(StructureDetailsSP->GetUserDefinedStruct(), FieldGuid)
				? EVisibility::Visible
				: EVisibility::Collapsed;
		}
		return EVisibility::Collapsed;
	}

	// Meta Data
	FText OnGetMetaData(const FName Key) const
	{
		const TSharedPtr<FUserDefinedStructureDetails> Details = StructureDetails.Pin();
		if (!Details)
		{
			return FText();
		}

		const FString* Value = FStructureEditorUtils::GetMetaData(Details->GetUserDefinedStruct(), FieldGuid, Key);
		return Value ? FText::FromString(*Value) : FText();
	}

	void OnMetaDataCommitted(const FText& NewText, ETextCommit::Type, const FName Key)
	{
		if (const TSharedPtr<FUserDefinedStructureDetails> StructureDetailsSP = StructureDetails.Pin())
		{
			FStructureEditorUtils::SetMetaData(StructureDetailsSP->GetUserDefinedStruct(), FieldGuid, Key, NewText.ToString());
		}
	}

	/** IDetailCustomNodeBuilder Interface*/
	virtual void SetOnRebuildChildren( FSimpleDelegate InOnRegenerateChildren ) override 
	{
		OnRegenerateChildren = InOnRegenerateChildren;
	}

	EVisibility GetErrorIconVisibility()
	{
		auto StructureDetailsSP = StructureDetails.Pin();
		if(StructureDetailsSP.IsValid())
		{
			auto FieldDesc = StructureDetailsSP->FindStructureFieldByGuid(FieldGuid);
			if (FieldDesc && FieldDesc->bInvalidMember)
			{
				return EVisibility::Visible;
			}
		}

		return EVisibility::Collapsed;
	}

	void GetFilteredVariableTypeTree( TArray< TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo> >& TypeTree, ETypeTreeFilter TypeTreeFilter) const
	{
		auto K2Schema = GetDefault<UEdGraphSchema_K2>();
		auto StructureDetailsSP = StructureDetails.Pin();
		if(StructureDetailsSP.IsValid() && K2Schema)
		{
			K2Schema->GetVariableTypeTree(TypeTree, TypeTreeFilter);
		}
	}

	virtual void GenerateHeaderRowContent( FDetailWidgetRow& NodeRow ) override 
	{
		auto K2Schema = GetDefault<UEdGraphSchema_K2>();

		TSharedPtr<SImage> ErrorIcon;

		const float ValueContentWidth = 200.0f;

		NodeRow
		.NameContent()
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SAssignNew(ErrorIcon, SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.Error") )
				.ToolTipText(LOCTEXT("MemberVariableErrorToolTip", "Member variable is invalid"))
			]

			+SHorizontalBox::Slot()
			.FillWidth(1)
			.VAlign(VAlign_Center)
			[
				SNew(SEditableTextBox)
				.Text( this, &FUserDefinedStructureFieldLayout::OnGetNameText )
				.OnTextCommitted( this, &FUserDefinedStructureFieldLayout::OnNameTextCommitted )
				.Font( IDetailLayoutBuilder::GetDetailFont() )
			]
		]
		.ValueContent()
		.MaxDesiredWidth(ValueContentWidth)
		.MinDesiredWidth(ValueContentWidth)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			[
				SNew(SPinTypeSelector, FGetPinTypeTree::CreateSP(this, &FUserDefinedStructureFieldLayout::GetFilteredVariableTypeTree))
				.TargetPinType(this, &FUserDefinedStructureFieldLayout::OnGetPinInfo)
				.OnPinTypePreChanged(this, &FUserDefinedStructureFieldLayout::OnPrePinInfoChange)
				.OnPinTypeChanged(this, &FUserDefinedStructureFieldLayout::PinInfoChanged)
				.Schema(K2Schema)
				.TypeTreeFilter(ETypeTreeFilter::None)
				.Font( IDetailLayoutBuilder::GetDetailFont() )
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				PropertyCustomizationHelpers::MakeEmptyButton(
					FSimpleDelegate::CreateSP(this, &FUserDefinedStructureFieldLayout::OnRemovField),
					LOCTEXT("RemoveVariable", "Remove member variable"),
					TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FUserDefinedStructureFieldLayout::IsRemoveButtonEnabled)))
			]
		]
		.DragDropHandler(MakeShared<FUserDefinedStructureFieldDragDropHandler>(StructureDetails, FieldGuid))
		;

		if (ErrorIcon.IsValid())
		{
			ErrorIcon->SetVisibility(
				TAttribute<EVisibility>::Create(
					TAttribute<EVisibility>::FGetter::CreateSP(
						this, &FUserDefinedStructureFieldLayout::GetErrorIconVisibility)));
		}
	}

	virtual void GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder ) override 
	{
		ChildrenBuilder.AddCustomRow(LOCTEXT("Tooltip", "Tooltip"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Tooltip", "Tooltip"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SEditableTextBox)
			.Text(this, &FUserDefinedStructureFieldLayout::OnGetTooltipText)
			.OnTextCommitted(this, &FUserDefinedStructureFieldLayout::OnTooltipCommitted)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];

		ChildrenBuilder.AddCustomRow(LOCTEXT("EditableOnInstance", "EditableOnInstance"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Editable", "Editable"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.ToolTipText(LOCTEXT("EditableOnBPInstance", "Variable can be edited on an instance of a Blueprint."))
			.OnCheckStateChanged(this, &FUserDefinedStructureFieldLayout::OnEditableOnBPInstanceCommitted)
			.IsChecked(this, &FUserDefinedStructureFieldLayout::OnGetEditableOnBPInstanceState)
		];

		ChildrenBuilder.AddCustomRow(LOCTEXT("SaveGame", "Save Game"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("SaveGameText", "Save Game"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.ToolTipText(LOCTEXT("SaveGame_Tooltip", "Should variable be serialized for saved games"))
			.OnCheckStateChanged(this, &FUserDefinedStructureFieldLayout::OnSaveGameCommitted)
			.IsChecked(this, &FUserDefinedStructureFieldLayout::OnGetSaveGameState)
		];

		ChildrenBuilder.AddCustomRow(LOCTEXT("MultiLineText", "Multi-line Text"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("MultiLineText", "Multi-line Text"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.ToolTipText(LOCTEXT("MultiLineTextToolTip", "Should this property allow multiple lines of text to be entered?"))
			.OnCheckStateChanged(this, &FUserDefinedStructureFieldLayout::OnMultiLineTextEnabledCommitted)
			.IsChecked(this, &FUserDefinedStructureFieldLayout::OnGetMultiLineTextEnabled)
		]
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FUserDefinedStructureFieldLayout::IsMultiLineTextOptionVisible)));

		ChildrenBuilder.AddCustomRow(LOCTEXT("3dWidget", "3D Widget"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("3dWidget", "3D Widget"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.OnCheckStateChanged(this, &FUserDefinedStructureFieldLayout::On3dWidgetEnabledCommitted)
			.IsChecked(this, &FUserDefinedStructureFieldLayout::OnGet3dWidgetEnabled)
		]
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FUserDefinedStructureFieldLayout::Is3dWidgetOptionVisible)));

		ChildrenBuilder.AddCustomRow(LOCTEXT("ClampRange", "Value Range"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ValueRangeLabel", "Value Range"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNew(SEditableTextBox)
				.OnTextCommitted(this, &FUserDefinedStructureFieldLayout::OnMetaDataCommitted, FStructVariableMetaData::ClampMin)
				.Text(this, &FUserDefinedStructureFieldLayout::OnGetMetaData, FStructVariableMetaData::ClampMin)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Min .. Max Separator", " .. "))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			+SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNew(SEditableTextBox)
				.OnTextCommitted(this, &FUserDefinedStructureFieldLayout::OnMetaDataCommitted, FStructVariableMetaData::ClampMax)
				.Text(this, &FUserDefinedStructureFieldLayout::OnGetMetaData, FStructVariableMetaData::ClampMax)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		]
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FUserDefinedStructureFieldLayout::IsValueRangeOptionVisible)));

		ChildrenBuilder.AddCustomRow(LOCTEXT("UIRange", "Slider Range"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("SliderRangeLabel", "Slider Range"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNew(SEditableTextBox)
				.OnTextCommitted(this, &FUserDefinedStructureFieldLayout::OnMetaDataCommitted, FStructVariableMetaData::UIMin)
				.Text(this, &FUserDefinedStructureFieldLayout::OnGetMetaData, FStructVariableMetaData::UIMin)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Min .. Max Separator", " .. "))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			+SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNew(SEditableTextBox)
				.OnTextCommitted(this, &FUserDefinedStructureFieldLayout::OnMetaDataCommitted, FStructVariableMetaData::UIMax)
				.Text(this, &FUserDefinedStructureFieldLayout::OnGetMetaData, FStructVariableMetaData::UIMax)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		]
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FUserDefinedStructureFieldLayout::IsValueRangeOptionVisible)));
	}

	virtual void Tick( float DeltaTime ) override {}
	virtual bool RequiresTick() const override { return false; }
	virtual FName GetName() const override { return FName(*FieldGuid.ToString()); }
	virtual bool InitiallyCollapsed() const override { return true; }

private:
	TWeakPtr<class FUserDefinedStructureDetails> StructureDetails;

	TWeakPtr<class FUserDefinedStructureLayout> StructureLayout;

	FGuid FieldGuid;

	FSimpleDelegate OnRegenerateChildren;
};

///////////////////////////////////////////////////////////////////////////////////////
// FUserDefinedStructureLayout
void FUserDefinedStructureLayout::GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder ) 
{
	ChildrenBuilder.AddCustomRow(FText::GetEmpty())
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Tooltip", "Tooltip"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.MinDesiredWidth(400.0f)
		[
			SNew(SEditableTextBox)
			.Text(this, &FUserDefinedStructureLayout::OnGetTooltipText)
			.OnTextCommitted(this, &FUserDefinedStructureLayout::OnTooltipCommitted)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];

	auto StructureDetailsSP = StructureDetails.Pin();
	if(StructureDetailsSP.IsValid())
	{
		if(auto Struct = StructureDetailsSP->GetUserDefinedStruct())
		{
			auto& VarDescArrayRef = FStructureEditorUtils::GetVarDesc(Struct);
			for (int32 Index = 0; Index < VarDescArrayRef.Num(); ++Index)
			{
				auto& VarDesc = VarDescArrayRef[Index];
				TSharedRef<class FUserDefinedStructureFieldLayout> VarLayout = MakeShareable(new FUserDefinedStructureFieldLayout(StructureDetails,  SharedThis(this), VarDesc.VarGuid));
				ChildrenBuilder.AddCustomBuilder(VarLayout);
			}
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////
// FUserDefinedStructureLayout

/** IDetailCustomization interface */
void FUserDefinedStructureDetails::CustomizeDetails(class IDetailLayoutBuilder& DetailLayout) 
{
	const TArray<TWeakObjectPtr<UObject>>& Objects = DetailLayout.GetSelectedObjects();
	check(Objects.Num() > 0);

	if (Objects.Num() == 1)
	{
		UserDefinedStruct = CastChecked<UUserDefinedStruct>(Objects[0].Get());

		IDetailCategoryBuilder& StructureCategory = DetailLayout.EditCategory("Structure", LOCTEXT("StructureCategory", "Structure"));
		Layout = MakeShareable(new FUserDefinedStructureLayout(SharedThis(this)));
		StructureCategory.AddCustomBuilder(Layout.ToSharedRef());
	}
}

/** FStructureEditorUtils::INotifyOnStructChanged */
void FUserDefinedStructureDetails::PostChange(const class UUserDefinedStruct* Struct, FStructureEditorUtils::EStructureEditorChangeInfo Info)
{
	if (Struct && (GetUserDefinedStruct() == Struct))
	{
		if (Layout.IsValid())
		{
			Layout->OnChanged();
		}
	}
}

#undef LOCTEXT_NAMESPACE
