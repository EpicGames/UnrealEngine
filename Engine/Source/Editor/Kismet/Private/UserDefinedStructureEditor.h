// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintEditorModule.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraphPin.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/IToolkit.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FSpawnTabArgs;
class FToolBarBuilder;
class SDockTab;
class UUserDefinedStruct;
struct FSlateBrush;

class FUserDefinedStructureEditor : public IUserDefinedStructureEditor
{
	/** App Identifier.*/
	static const FName UserDefinedStructureEditorAppIdentifier;

	/**	The tab ids for all the tabs used */
	static const FName MemberVariablesTabId;
	static const FName DefaultValuesTabId;
	
	/** Property viewing widget */
	TSharedPtr<class IDetailsView> PropertyView;
	TSharedPtr<class FStructureDefaultValueView> DefaultValueView;
public:
	/**
	 * Edits the specified enum
	 *
	 * @param	Mode					Asset editing mode for this editor (standalone or world-centric)
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	EnumToEdit				The user defined enum to edit
	 */
	void InitEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, class UUserDefinedStruct* EnumToEdit);

	/** Sets the pin type for new struct members added via the Add Variable toolbar button */
	void SetInitialPinType(FEdGraphPinType PinType);

	/** Destructor */
	virtual ~FUserDefinedStructureEditor();

	/** IToolkit interface */
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FText GetToolkitName() const override;
	virtual FText GetToolkitToolTipText() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;

	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

protected:
	TSharedRef<SDockTab> SpawnStructureTab(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnStructureDefaultValuesTab(const FSpawnTabArgs& Args);

private:
	void FillToolbar(FToolBarBuilder& ToolbarBuilder);

	/** Handles adding a new member variable from the toolbar button */
	FReply OnAddNewField();

	/** Returns the overlay image to indicate the compile status */
	const FSlateBrush* OnGetStructureStatus() const;

	/** Returns the tooltip describing the compile status */
	FText OnGetStatusTooltip() const;

private:
	TWeakObjectPtr<UUserDefinedStruct> UserDefinedStruct;

	/** Cached value of the last pin type the user selected, used as the initial value for new struct members */
	FEdGraphPinType InitialPinType;
};

