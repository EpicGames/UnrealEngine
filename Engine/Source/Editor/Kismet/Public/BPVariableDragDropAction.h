// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintEditor.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Engine/Blueprint.h"
#include "GraphEditorDragDropAction.h"
#include "Input/DragAndDrop.h"
#include "Input/Reply.h"
#include "Math/Vector2D.h"
#include "MyBlueprintItemDragDropAction.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API KISMET_API

class UBlueprint;
class UEdGraph;
class UEdGraphNode;
struct FEdGraphSchemaAction;
struct FSlateBrush;
struct FSlateColor;

/** DragDropAction class for dropping a Variable onto a graph */
class FKismetVariableDragDropAction : public FMyBlueprintItemDragDropAction
{
public:
	DRAG_DROP_OPERATOR_TYPE(FKismetVariableDragDropAction, FMyBlueprintItemDragDropAction)

	// FGraphEditorDragDropAction interface
	UE_API virtual void HoverTargetChanged() override;
	UE_API virtual FReply DroppedOnPin(const FVector2f& ScreenPosition, const FVector2f& GraphPosition) override;
	UE_API virtual FReply DroppedOnNode(const FVector2f& ScreenPosition, const FVector2f& GraphPosition) override;
	UE_API virtual FReply DroppedOnPanel(const TSharedRef< class SWidget >& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph) override;
	// End of FGraphEditorDragDropAction

	static TSharedRef<FKismetVariableDragDropAction> New(TSharedPtr<FEdGraphSchemaAction> InAction, FName InVariableName, UStruct* InVariableSource, FNodeCreationAnalytic AnalyticCallback)
	{
		TSharedRef<FKismetVariableDragDropAction> Operation = MakeShareable(new FKismetVariableDragDropAction);
		Operation->VariableName = InVariableName;
		Operation->VariableSource = InVariableSource;
		Operation->AnalyticCallback = AnalyticCallback;
		Operation->SourceAction = InAction;
		Operation->Construct();
		return Operation;
	}

	FProperty* GetVariableProperty()
	{
		if (VariableSource.IsValid() && VariableName != NAME_None)
		{
			return FindFProperty<FProperty>(VariableSource.Get(), VariableName);
		}
		return nullptr;
	}

protected:
	 /** Construct a FKismetVariableDragDropAction */
	UE_API FKismetVariableDragDropAction();

	/** Structure for required node construction parameters */
	struct FNodeConstructionParams
	{
		FDeprecateSlateVector2D GraphPosition;
		UEdGraph* Graph;
		FName VariableName;
		TWeakObjectPtr<UStruct> VariableSource;
	};

	// FGraphSchemaActionDragDropAction interface
	UE_API virtual void GetDefaultStatusSymbol(const FSlateBrush*& PrimaryBrushOut, FSlateColor& IconColorOut, FSlateBrush const*& SecondaryBrushOut, FSlateColor& SecondaryColorOut) const override;
	// End of FGraphSchemaActionDragDropAction interface

	// FMyBlueprintItemDragDropAction interface
	UE_API virtual UBlueprint* GetSourceBlueprint() const override;
	// End of FMyBlueprintItemDragDropAction interface

	/** Called when user selects to create a Getter for the variable */
	static UE_API void MakeGetter(FNodeConstructionParams InParams);
	/** Called when user selects to create a Setter for the variable */
	static UE_API void MakeSetter(FNodeConstructionParams InParams);
	/** Called too check if we can execute a setter on a given property */
	static UE_API bool CanExecuteMakeSetter(FNodeConstructionParams InParams, FProperty* InVariableProperty);

	/**
	 * Test new variable type against existing links for node and get any links that will break
	 *
	 * @param	Node						The node with existing links
	 * @param	NewVariableProperty			the property for the new variable type 
	 * @param	OutBroken						All of the links which are NOT compatible with the new type
	 */
	UE_API void GetLinksThatWillBreak(UEdGraphNode* Node, FProperty* NewVariableProperty, TArray<class UEdGraphPin*>& OutBroken);

	/** Indicates if replacing the variable node, with the new property will require any links to be broken*/
	bool WillBreakLinks( UEdGraphNode* Node, FProperty* NewVariableProperty ) 
	{
		TArray<class UEdGraphPin*> BadLinks;
		GetLinksThatWillBreak(Node,NewVariableProperty,BadLinks);
		return BadLinks.Num() > 0;
	}

	/**
	 * Checks if the property can be dropped in a graph
	 *
	 * @param InVariableProperty		The variable property to check with
	 * @param InGraph					The graph to check against placing the variable
	 */
	UE_API bool CanVariableBeDropped(const FProperty* InVariableProperty, const UEdGraph& InGraph) const;


	/** Returns the local variable's scope, if any */
	UE_API UStruct* GetLocalVariableScope() const;

protected:
	/** Name of variable being dragged */
	FName VariableName;
	/** Scope this variable belongs to */
	TWeakObjectPtr<UStruct> VariableSource;
};

#undef UE_API
