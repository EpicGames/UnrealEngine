// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphSchema.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

#define UE_API KISMET_API

class FBlueprintActionFilter;
class FBlueprintEditor;

namespace FBlueprintActionMenuBuilderImpl
{
	// internal types, forward declared to hide implementation details
	struct FMenuSectionDefinition;
	struct FMenuItemListAddHelper;
};
class  FBlueprintActionFilter;
class  FBlueprintEditor;
struct FBlueprintActionContext;
struct FBlueprintActionInfo;

/**
 * Responsible for constructing a list of viable blueprint actions. Runs the 
 * blueprint actions database through a filter and spawns a series of 
 * FBlueprintActionMenuItems for actions that pass. Takes care of generating the 
 * each menu item's category/name/etc.
 */
struct FBlueprintActionMenuBuilder : public FGraphActionListBuilderBase
{
public:
	/** Flags used to configure the builder. */
	enum EConfigFlags
	{
		// If set, the builder will be configured to process the registered
		// action set over time, rather than process the entire set in a
		// single blocking frame.
		UseTimeSlicing		= (1<<0),

		// The default configuration for this builder type.
		DefaultConfig		= 0,
	};

	/** Flags used to customize specific sections of the menu. */
	enum ESectionFlag
	{
		// Rolls properties into a single menu item that will spawn a drag-drop
		// menu for users to pick a node type from.
		ConsolidatePropertyActions = (1<<0),

		// Rolls bound node spawners into a single menu entry that will spawn 
		// multiple nodes, each bound to a single binding.
		ConsolidateBoundActions    = (1<<1),

		// Will clear all action categories (except the section's root) 
		FlattenCategoryHierarcy    = (1<<2),
	};
	
public:
	/**
	 * Default constructor.
	 * 
	 * @param Flags	- Optional configuration flags.
	 */
	UE_API FBlueprintActionMenuBuilder(EConfigFlags Flags = EConfigFlags::DefaultConfig);
	
	// FGraphActionListBuilderBase interface
	UE_API virtual void Empty() override;
	// End FGraphActionListBuilderBase interface
	
	/**
	 * Some action menus require multiple sections. One option is to create 
	 * multiple FBlueprintActionMenuBuilders and append them together, but that
	 * can be unperformant (each builder will run through the entire database
	 * separately)... This method provides an alternative, where you can specify
	 * a separate filter/heading/ordering for a sub-section of the menu.
	 *
	 * @param  Filter	 The filter you want applied to this section of the menu.
	 * @param  Heading	 The root category for this section of the menu (can be left blank).
	 * @param  MenuOrder The sort order to assign this section of the menu (higher numbers come first).
	 * @param  Flags	 Set of ESectionFlags to customize this menu section.
	 */
	UE_API void AddMenuSection(FBlueprintActionFilter const& Filter, FText const& Heading = FText::GetEmpty(), int32 MenuOrder = 0, uint32 const Flags = 0);
	
	/**
	 * Regenerates the entire menu list from the cached menu sections. Filters 
	 * and adds action items from the blueprint action database (as defined by 
	 * the MenuSections list).
	 */
	UE_API void RebuildActionList();

	/** Returns the current number of actions that are still pending */
	UE_API int32 GetNumPendingActions() const;

	/** Processes any actions that may be added asynchronously or across multiple frames. Returns true if one or more actions were added into the list. */
	UE_API bool ProcessPendingActions();

	/** Returns the normalized completion state when processing pending actions (e.g. for a status indicator) */
	UE_API float GetPendingActionsProgress() const;

protected:
	// Adds menu items for the given database action.
	UE_API void MakeMenuItems(FBlueprintActionInfo& InAction);

	// Called when all menu items have been constructed.
	UE_API void BuildCompleted();

private:
	/** 
	 * Defines all the separate sections of the menu (filter, sort order, etc.).
	 * Defined as a TSharedRef<> so as to hide the implementation details (keep 
	 * this API clean).
	 */
	TArray< TSharedRef<FBlueprintActionMenuBuilderImpl::FMenuSectionDefinition> > MenuSections;

	/**
	 * Defines a utility that assists with building the list of action menu items
	 * for each menu section based on a set of Blueprint action descriptor records.
	 */
	TSharedPtr<FBlueprintActionMenuBuilderImpl::FMenuItemListAddHelper> MenuItemListAddHelper;

	/** If enabled, actions will be added to the pending list rather than processed immediately. */
	bool bUsePendingActionList;
};

ENUM_CLASS_FLAGS(FBlueprintActionMenuBuilder::EConfigFlags);

#undef UE_API
