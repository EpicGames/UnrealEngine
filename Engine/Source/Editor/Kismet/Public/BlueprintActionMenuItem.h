// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintNodeBinder.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphSchema.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateColor.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"

#include "BlueprintActionMenuItem.generated.h"

#define UE_API KISMET_API

class FReferenceCollector;
class UBlueprintNodeSpawner;
class UEdGraph;
class UEdGraphNode;
class UEdGraphPin;
struct FBlueprintActionContext;
struct FBlueprintActionUiSpec;
struct FSlateBrush;

/**
 * Wrapper around a UBlueprintNodeSpawner, which takes care of specialized
 * node spawning. This class should not be sub-classed, any special handling
 * should be done inside a UBlueprintNodeSpawner subclass, which will be
 * invoked from this class (separated to divide ui and node-spawning).
 */
USTRUCT()
struct FBlueprintActionMenuItem : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY();
	static FName StaticGetTypeId() { static FName const TypeId("FBlueprintActionMenuItem"); return TypeId; }

public:	
	/** Constructors */
	FBlueprintActionMenuItem(UBlueprintNodeSpawner const* NodeSpawner = nullptr) : Action(NodeSpawner), IconTint(FLinearColor::White), IconBrush(nullptr) {}
	UE_API FBlueprintActionMenuItem(UBlueprintNodeSpawner const* NodeSpawner, FBlueprintActionUiSpec const& UiDef, IBlueprintNodeBinder::FBindingSet const& Bindings = IBlueprintNodeBinder::FBindingSet(), FText InNodeCategory = FText(), int32 InGrouping = 0);
	
	// FEdGraphSchemaAction interface
	virtual FName         GetTypeId() const final { return StaticGetTypeId(); }
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) final;
	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode = true) final;
	UE_API virtual void          AddReferencedObjects(FReferenceCollector& Collector) final;
	// End FEdGraphSchemaAction interface

	/** @return */
	UE_API UBlueprintNodeSpawner const* GetRawAction() const;

	UE_API void AppendBindings(const FBlueprintActionContext& Context, IBlueprintNodeBinder::FBindingSet const& BindingSet);

	/**
	 * Retrieves the icon brush for this menu entry (to be displayed alongside
	 * in the menu).
	 *
	 * @param  ColorOut	An output parameter that's filled in with the color to tint the brush with.
	 * @return An slate brush to be used for this menu item in the action menu.
	 */
	UE_API FSlateBrush const* GetMenuIcon(FSlateColor& ColorOut);

	/** Utility struct for pairing documentation page names with excerpt names */
	struct FDocExcerptRef
	{
		UE_API bool IsValid() const;

		FString DocLink;
		FString DocExcerptName;		
	};
	/**  */
	UE_API const FDocExcerptRef& GetDocumentationExcerpt() const;

private:
	/** Specialized node-spawner, that comprises the action portion of this menu entry. */
	TObjectPtr<const UBlueprintNodeSpawner> Action;
	/** Tint to return along with the icon brush. */
	FSlateColor IconTint;
	/** Brush that should be used for the icon on this menu item. */
	FSlateBrush const* IconBrush;
	/** */
	IBlueprintNodeBinder::FBindingSet Bindings;
	/** References the documentation page/excerpt pertaining to the node this will spawn */
	FDocExcerptRef DocExcerptRef;
};

#undef UE_API
