// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintActionFilter.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphSchema.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"

#include "BlueprintActionMenuUtils.generated.h"

class UClass;
class UK2Node;
struct FBlueprintActionContext;
struct FBlueprintActionMenuBuilder;
struct FEdGraphSchemaAction;

UENUM()
namespace EContextTargetFlags
{
	enum Type : int
	{
		TARGET_Blueprint			= 0x00000001 UMETA(DisplayName="This Blueprint", ToolTip="Include functions and variables that belong to this Blueprint."),
		TARGET_SubComponents		= 0x00000002 UMETA(DisplayName="Components", ToolTip="Include functions that belong to components of this Blueprint and/or the other target classes."),
		TARGET_NodeTarget			= 0x00000004 UMETA(DisplayName="Node Target", ToolTip="Include functions and variables that belong to the same class that the pin's node does."),
		TARGET_PinObject			= 0x00000008 UMETA(DisplayName="Pin Type Class", ToolTip="Include functions and variables that belong to this pin type."),
		TARGET_SiblingPinObjects	= 0x00000010 UMETA(DisplayName="Other Object Outputs", ToolTip="Include functions and variables that belong to any of this node's output types."),
		TARGET_BlueprintLibraries	= 0x00000020 UMETA(DisplayName="Libraries", ToolTip="Include static functions that are globally accessible (belonging to function/macro libraries, etc.)."),
		TARGET_NonImportedTypes		= 0x00000040 UMETA(DisplayName="Non-Imported Types", ToolTip="Include functions from loaded types that are not imported by this Blueprint."),

		// +1 to the last flag (so we can easily iterate these flags)
		ContextTargetFlagsEnd UMETA(Hidden),
	};
}

struct FBlueprintActionMenuUtils
{
	/**
	 * A centralized utility function for constructing blueprint palette menus.
	 * Rolls the supplied Context and FilterClass into a filter that's used to
	 * construct the palette.
	 *
	 * @param  Context		Contains the blueprint that the palette is for.
	 * @param  FilterClass	If not null, then this specifies the class whose members we want listed (and nothing else).
	 * @param  MenuOut		The structure that will be populated with palette menu items.
	 */
	KISMET_API static void MakePaletteMenu(FBlueprintActionContext const& Context, UClass* FilterClass, FBlueprintActionMenuBuilder& MenuOut);
	
	/**
	 * A centralized utility function for constructing blueprint context menus.
	 * Rolls the supplied Context and SelectedProperties into a series of 
	 * filters that're used to construct the menu.
	 *
	 * @param  Context				Contains the blueprint/graph/pin that the menu is for.
	 * @param  bIsContextSensitive	
	 * @param  ClassTargetMask		
	 * @param  MenuOut				The structure that will be populated with context menu items.
	 */
	KISMET_API static void MakeContextMenu(FBlueprintActionContext const& Context, bool bIsContextSensitive, uint32 ClassTargetMask, FBlueprintActionMenuBuilder& MenuOut);

	/**
	 * A centralized utility function for constructing the blueprint favorites
	 * menu. Takes the palette menu building code and folds in an additional 
	 * "IsFavorited" filter check.
	 *
	 * @param  Context	Contains the blueprint that the palette is for.
	 * @param  MenuOut	The structure that will be populated with favorite menu items.
	 */
	KISMET_API static void MakeFavoritesMenu(FBlueprintActionContext const& Context, FBlueprintActionMenuBuilder& MenuOut);

	/**
	 * A number of different palette actions hold onto node-templates in different 
	 * ways. This handles most of those cases and looks to extract said node- 
	 * template from the specified action.
	 * 
	 * @param  PaletteAction	The action you want a node-template for.
	 * @return A pointer to the extracted node (NULL if the action doesn't have one, or we don't support the specific action type yet)
	 */
	KISMET_API static const UK2Node* ExtractNodeTemplateFromAction(const FEdGraphSchemaAction& PaletteAction);
	KISMET_API static const UK2Node* ExtractNodeTemplateFromAction(const TSharedPtr<FEdGraphSchemaAction>& PaletteAction);
};

