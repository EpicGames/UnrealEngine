// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreTypes.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"
#include "UObject/TopLevelAssetPath.h"

class FText;
class UBlueprint;
class UClass;
class UObject;
struct FAssetData;

/**
 * Interface used to define how to interact with a blueprint within an asset
 */
class IBlueprintAssetHandler
{
public:
	virtual ~IBlueprintAssetHandler() = default;

	/**
	 * Retrieve the blueprint from the specified asset object
	 *
	 *@param InAsset         The asset object to retrieve the blueprint from
	 *@return The blueprint contained within the specified asset, or nullptr if non exists
	 */
	virtual UBlueprint* RetrieveBlueprint(UObject* InAsset) const = 0;


	/**
	 * Check whether the specified asset registry data contains a blueprint
	 *
	 *@param InAssetData     The asset object to retrieve the blueprint from
	 *@return True if the asset contains a blueprint, false otherwise
	 */
	virtual bool AssetContainsBlueprint(const FAssetData& InAssetData) const = 0;
};


/**
 * Singleton class that marshals different blueprint asset handlers for different asset class types
 */
class FBlueprintAssetHandler
{
public:

	/**
	 * Retrieve the singleton instance of this class
	 */
	KISMET_API static FBlueprintAssetHandler& Get();


	/**
	 * Get all the currently registered class names
	 */
	TArrayView<const FTopLevelAssetPath> GetRegisteredClassNames() const
	{
		return ClassNames;
	}


	/**
	 * Register an asset for the specified class name
	 * @note: Any assets whose class is a child of the specified class will use this handler (unless there is a more specific handler registered)
	 *
	 * @param ClassName      The name of the class this handler relates to.
	 */
	template<typename HandlerType>
	void RegisterHandler(FTopLevelAssetPath ClassPathName)
	{
		RegisterHandler(ClassPathName, MakeUnique<HandlerType>());
	}


	/**
	 * Register an asset for the specified class name
	 * @note: Any assets whose class is a child of the specified class will use this handler (unless there is a more specific handler registered)
	 *
	 * @param ClassPathName  The path name of the class this handler relates to.
	 * @param InHandler      The implementation of the handler to use
	 */
	KISMET_API void RegisterHandler(FTopLevelAssetPath ClassPathName, TUniquePtr<IBlueprintAssetHandler>&& InHandler);


	/**
	 * Find a handler that applies to the specified class
	 *
	 * @param InClass        The class to find a handler for
	 * @return A valid asset handler, or nullptr if none exists for this class
	 */
	KISMET_API const IBlueprintAssetHandler* FindHandler(const UClass* InClass) const;


private:

	/** Private constructor - use singleton accessor (::Get) */
	FBlueprintAssetHandler();

	/** Unsorted array of class names, one per-handler below */
	TArray<FTopLevelAssetPath> ClassNames;

	/** Array of handlers that relate to the class names above */
	TArray<TUniquePtr<IBlueprintAssetHandler>> Handlers;
};
