// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CoreMinimal.h" // for DLLEXPORT (KISMET_API)
#include "CoreTypes.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"

#define UE_API KISMET_API

class FCompilerResultsLog;
class FProperty;
class FString;
class UBlueprint;
class UBlueprintCompilerExtension;
class UClass;
struct FCompilerNativizationOptions;
struct FUObjectSerializeContext;

struct FBPCompileRequest
{
	explicit KISMET_API FBPCompileRequest(UBlueprint* InBPToCompile, EBlueprintCompileOptions InCompileOptions, FCompilerResultsLog* InClientResultsLog);

	// BP that needs to be compiled:
	TObjectPtr<UBlueprint> BPToCompile;

	// Legacy options for blueprint compilation:
	EBlueprintCompileOptions CompileOptions;
	
	// Clients can give us a results log if they want to parse or display it themselves, otherwise
	// we will use a transient one:
	FCompilerResultsLog* ClientResultsLog;
};

struct FBlueprintCompilationManager
{
	static UE_API void Initialize();
	static UE_API void Shutdown();

	/**
	 * Compiles all blueprints that have been placed in the compilation queue. 
	 * ObjLoaded is a list of objects that need to be PostLoaded by the linker,
	 * when changing CDOs we will replace objects in this list. It is not a list
	 * of objects the compilation manager has loaded. The compilation manager
	 * will not load data while processing the compilation queue)
	 */
	static UE_API void FlushCompilationQueue(FUObjectSerializeContext* InLoadContext);
	
	/**
	 * Flushes the compilation queue and finishes reinstancing
	 */
	static UE_API void FlushCompilationQueueAndReinstance(const bool bFindAndReplaceCDOReferences = false);

	/**
	 * Immediately compiles the blueprint, no expectation that related blueprints be subsequently compiled.
	 * It will be significantly more efficient to queue blueprints and then flush the compilation queue
	 * if there are several blueprints that require compilation (e.g. typical case on PIE):
	 */
	static UE_API void CompileSynchronously(const FBPCompileRequest& Request);

	/**
	 * Adds a newly loaded blueprint to the compilation queue
	 */
	static UE_API void NotifyBlueprintLoaded(UBlueprint* BPLoaded);

	/**
	 * Adds a blueprint to the compilation queue - useful for batch compilation
	 */
	static UE_API void QueueForCompilation(UBlueprint* BP);

	/** Returns true when UBlueprint::GeneratedClass members are up to date */
	static UE_API bool IsGeneratedClassLayoutReady();
	
	/** 
	 * Returns the Default Value associated with ForClass::Property, if ForClass is currently 
	 * being compiled this function can look at the old version of the CDO and read the default
	 * value from there
	 */
	static UE_API bool GetDefaultValue(const UClass* ForClass, const FProperty* Property, FString& OutDefaultValueAsString);

	/**
	 * Safely reparents all child classes of every Key in OldClassToNewClass to the class in 
	 * the corresponding Value. Typically this means every child type will be reinstanced - although
	 * reinstancing could be avoided when layouts match.
	 */
	static UE_API void ReparentHierarchies(const TMap<UClass*, UClass*>& OldClassToNewClass);

	/** 
	 * Registers a blueprint compiler extension - anytime a blueprint of the provided type is compiled
	 * the extension will be activated. Note that because editor initialization may require blueprint
	 * compilation there may be blueprints compiled before the extension is registed unless special
	 * care has been taken.
	 */
	static UE_API void RegisterCompilerExtension(TSubclassOf<UBlueprint> BlueprintType, UBlueprintCompilerExtension* Extension);
	
	/** Safely updates all instances using the provided UUserDefinedStruct so that it can be mutated (inc. deletion) */
	static UE_API void ReinstanceStruct(class UUserDefinedStruct* Struct);
private:
	UE_API FBlueprintCompilationManager();
};

#undef UE_API
