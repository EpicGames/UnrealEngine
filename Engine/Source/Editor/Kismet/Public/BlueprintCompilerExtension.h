// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "KismetCompiler.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "BlueprintCompilerExtension.generated.h"

#define UE_API KISMET_API

class FKismetCompilerContext;
class UEdGraph;

USTRUCT()
struct FBlueprintCompiledData
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<TObjectPtr<UEdGraph>> IntermediateGraphs;
};

UCLASS(MinimalAPI, Abstract)
class UBlueprintCompilerExtension : public UObject
{
	GENERATED_BODY()
public:
	UE_API UBlueprintCompilerExtension(const FObjectInitializer& ObjectInitializer);

	UE_API void BlueprintCompiled(const FKismetCompilerContext& CompilationContext, const FBlueprintCompiledData& Data);

protected:
	/** 
	 * Override this if you're interested in running logic after class layout has been
	 * generated, but before bytecode and member variables have been 
	 */
	virtual void ProcessBlueprintCompiled(const FKismetCompilerContext& CompilationContext, const FBlueprintCompiledData& Data) {}
};

#undef UE_API
