// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintActionFilter.h"
#include "BlueprintEditor.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphSchema.h"
#include "HAL/Platform.h"
#include "Math/Vector2D.h"
#include "Styling/SlateBrush.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"

#include "BlueprintDragDropMenuItem.generated.h"

#define UE_API KISMET_API

class FDragDropOperation;
class FReferenceCollector;
class FText;
class UBlueprintNodeSpawner;
class UEdGraph;
class UEdGraphNode;
class UEdGraphPin;
struct FBlueprintActionContext;
struct FSlateBrush;
struct FSlateColor;

/**
 * At certain times we want a single menu entry that represents a set of 
 * UBlueprintNodeSpawners (generally when all those UBlueprintNodeSpawners wrap 
 * the same UField). We do this to keep the menu less jumbled, and instead
 * use the drag/drop action to present a sub-menu to the user (so they can pick 
 * the the node type that they want). We do this with both delegates and 
 * variable nodes (where the user can pick a getter vs. a setter, etc.).
 *
 * This class represents those "consolidated" actions, and essentially serves 
 * as a FDragDropOperation spawner. It wraps a single UBlueprintNodeSpawner (any
 * one of the set that it is supposed to represent), that it uses to determine
 * the proper FDragDropOperation.
 */
USTRUCT()
struct FBlueprintDragDropMenuItem : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY();
	
public:
	static FName StaticGetTypeId() { static FName const TypeId("FBlueprintDragDropMenuItem"); return TypeId; }
	/** Default constructor for USTRUCT() purposes. */
	FBlueprintDragDropMenuItem() {}

	/**
	 * Sole constructor, which ensures that the RepresentativeAction member is
	 * set (and valid).
	 * 
	 * @param  SampleAction	One of the node-spawners that this "consolidated" menu item represents (shouldn't matter which one).
	 * @param  MenuGrouping	Defines the order in which this entry is listed in the menu.
	 */
	UE_API FBlueprintDragDropMenuItem(FBlueprintActionContext const& Context, UBlueprintNodeSpawner const* SampleAction, int32 MenuGrouping, FText InNodeCategory, FText InMenuDesc, FText InToolTip );
	
	// FEdGraphSchemaAction interface
	virtual FName GetTypeId() const final { return StaticGetTypeId(); }
	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode = true) override;
	UE_API virtual void  AddReferencedObjects(FReferenceCollector& Collector) override;
	// End FEdGraphSchemaAction interface

	/**
	 * Retrieves the icon brush for this menu entry (to be displayed alongside
	 * in the menu).
	 *
	 * @param  ColorOut	An output parameter that's filled in with the color to tint the brush with.
	 * @return An slate brush to be used for this menu item in the action menu.
	 */
	UE_API FSlateBrush const* GetMenuIcon(FSlateColor& ColorOut);

	/** */
	UE_API void AppendAction(UBlueprintNodeSpawner const* Action);

	/** @return   */
	UE_API UBlueprintNodeSpawner const* GetSampleAction() const;

	/** @return   */
	UE_API TSet<UBlueprintNodeSpawner const*> const& GetActionSet() const;

	/**
	 * Attempts to create a certain drag/drop action with respect to the 
	 * set UBlueprintNodeSpawner.
	 * 
	 * @param  AnalyticsDelegate	The analytics callback to assign the drag/drop operation.
	 * @return An empty TSharedPtr if we failed to determine the FDragDropOperation type to use, else a newly instantiated drag/drop op.
	 */
	UE_API TSharedPtr<FDragDropOperation> OnDragged(FNodeCreationAnalytic AnalyticsDelegate) const;

private:
	/** An arbitrary member of the node-spawner subset that this menu entry represents. */
	TSet<TObjectPtr<const UBlueprintNodeSpawner>> ActionSet;
};

#undef UE_API
