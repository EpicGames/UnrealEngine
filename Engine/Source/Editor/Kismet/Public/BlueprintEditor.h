// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintEditorModule.h"
#include "Blueprint/BlueprintSectionID.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EditorUndoClient.h"
#include "Engine/Blueprint.h"
#include "FindInBlueprints.h"
#include "Framework/Application/IMenu.h"
#include "Framework/Commands/InputChord.h"
#include "GraphEditor.h"
#include "Input/Reply.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Merge.h"
#include "Misc/Guid.h"
#include "Misc/NotifyHook.h"
#include "Misc/Optional.h"
#include "Math/Vector2D.h"
#include "PreviewScene.h"
#include "SKismetInspector.h"
#include "SourceControlOperations.h"
#include "Stats/Stats.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "Templates/UnrealTemplate.h"
#include "Tickable.h"
#include "TickableEditorObject.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Toolkits/IToolkit.h"
#include "Toolkits/IToolkitHost.h"
#include "Types/SlateEnums.h"
#include "UObject/GCObject.h"
#include "UObject/NameTypes.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

#define UE_API KISMET_API

class AActor;
class FBlueprintEditorToolbar;
class FBlueprintNamespaceHelper;
class FEditorViewportSelectability;
class FEditorViewportSelectabilityBridge;
class FProperty;
class FReferenceCollector;
class FSCSEditorTreeNode;
class FSubobjectEditorTreeNode;
class FUICommandList;
class IMessageLogListing;
class INameValidatorInterface;
class ISCSEditorCustomization;
class SBlueprintBookmarks;
class SBlueprintPalette;
class SDockTab;
class SFindInBlueprints;
class SKismetInspector;
class SMyBlueprint;
class SReplaceNodeReferences;
class SSCSEditorViewport;
class SSubobjectEditor;
class SWidget;
class UActorComponent;
class UBlueprintEditorOptions;
class UBlueprintEditorProjectSettings;
class UBlueprintEditorSettings;
class UClass;
class UEdGraph;
class UEdGraphNode;
class UEdGraphSchema;
class UK2Node_Event;
class UK2Node_FunctionEntry;
class ULevelStreaming;
class UObject;
class USceneComponent;
class UStruct;
class UToolMenu;
class UToolMenu;
class UUserDefinedEnum;
class UUserDefinedStruct;
struct FEdGraphSchemaAction;
struct FPropertyChangedEvent;
struct FSlateBrush;
struct FSlateColor;
struct FSubobjectData;
struct FToolMenuContext;
struct Rect;

/////////////////////////////////////////////////////
// FCustomDebugObjectEntry - Used to pass a custom debug object override around

struct FCustomDebugObject
{
public:
	// Custom object to include, regardless of the current debugging World
	UObject* Object;

	// Override for the object name (if not empty)
	FString NameOverride;

public:
	FCustomDebugObject()
		: Object(nullptr)
	{
	}

	FCustomDebugObject(UObject* InObject, const FString& InLabel)
		: Object(InObject)
		, NameOverride(InLabel)
	{
	}
};

/////////////////////////////////////////////////////
// FSelectionDetailsSummoner

#define LOCTEXT_NAMESPACE "BlueprintEditor"

struct FSelectionDetailsSummoner : public FWorkflowTabFactory
{
public:
	UE_API FSelectionDetailsSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp);

	UE_API virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	UE_API virtual TSharedRef<SDockTab> SpawnTab(const FWorkflowTabSpawnInfo& Info) const override;

	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override
	{
		return LOCTEXT("SelectionDetailsTooltip", "The Details tab allows you see and edit properties of whatever is selected.");
	}
};

/////////////////////////////////////////////////////
// FComponentEventConstructionData

/** The structure used to construct the "Add Event" menu entries */
struct FComponentEventConstructionData
{
	// The name of the event handler to create.
	FName VariableName;
	// The template component that the handler applies to.
	TWeakObjectPtr<UObject> Component;
};

/** The delegate that the caller must supply to BuildComponentActionsSubMenu that returns the currently selected items */
DECLARE_DELEGATE_OneParam(FGetSelectedObjectsDelegate, TArray<FComponentEventConstructionData>&);

/** Delegate for Node Creation Analytics */
DECLARE_DELEGATE(FNodeCreationAnalytic);

/** Describes user actions that created new node */
namespace ENodeCreateAction
{
	enum Type
	{
		MyBlueprintDragPlacement,
		PaletteDragPlacement,
		GraphContext,
		PinContext,
		Keymap
	};
}

/////////////////////////////////////////////////////
// FBlueprintEditor

/** Main Kismet asset editor */
class FBlueprintEditor : public IBlueprintEditor, public FGCObject, public FNotifyHook, public 
FTickableEditorObject, public FEditorUndoClient, public FNoncopyable
{
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSetPinVisibility, SGraphEditor::EPinVisibility);

	/** A record of a warning generated by a disallowed pin connection attempt */
	struct FDisallowedPinConnection
	{
		FName PinTypeCategoryA;
		FName PinTypeCategoryB;

		uint8 bPinIsArrayA:1;
		uint8 bPinIsReferenceA:1;
		uint8 bPinIsWeakPointerA:1;

		uint8 bPinIsArrayB:1;
		uint8 bPinIsReferenceB:1;
		uint8 bPinIsWeakPointerB:1;
	};

public:
	//~ Begin IToolkit Interface
	UE_API virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	//~ End IToolkit Interface

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnModeSet, FName);
	FOnModeSet& OnModeSet() { return OnModeSetData; }
	UE_API virtual void SetCurrentMode(FName NewMode) override;

public:
	/**
	 * Edits the specified blueprint
	 *
	 * @param	Mode					Asset editing mode for this editor (standalone or world-centric)
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	InBlueprints			The blueprints to edit
	 * @param	bShouldOpenInDefaultsMode	If true, the editor will open in defaults editing mode
	 * @param	InSCSEditorUICustomization	UI customizations for the SCS editor inside the blueprint editor
	 */
	UE_API void InitBlueprintEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost >& InitToolkitHost, const TArray<UBlueprint*>& InBlueprints, bool bShouldOpenInDefaultsMode, TSharedPtr<ISCSEditorUICustomization> InSCSEditorUICustomization = nullptr);

public:
	//~ Begin FAssetEditorToolkit Interface
	UE_API virtual bool OnRequestClose(EAssetEditorCloseReason InCloseReason) override;
	UE_API virtual void OnClose() override;
	UE_API virtual void AddViewportOverlayWidget(TSharedRef<SWidget> InViewportOverlayWidget, int32 ZOrder = INDEX_NONE) override;
	UE_API virtual void RemoveViewportOverlayWidget(TSharedRef<SWidget> InViewportOverlayWidget) override;
	UE_API virtual void CreateEditorModeManager() override;
	// End of FAssetEditorToolkit

	//~ Begin IToolkit Interface
	UE_API virtual FName GetToolkitContextFName() const override;
	UE_API virtual FName GetToolkitFName() const override;
	UE_API virtual FText GetBaseToolkitName() const override;
	UE_API virtual FText GetToolkitName() const override;
	UE_API virtual FText GetToolkitToolTipText() const override;
	UE_API virtual FString GetWorldCentricTabPrefix() const override;
	UE_API virtual FLinearColor GetWorldCentricTabColorScale() const override;
	UE_API virtual bool IsBlueprintEditor() const override;
	//~ End IToolkit Interface

	//~ Begin FGCObject Interface
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	UE_API virtual FString GetReferencerName() const override;
	//~ End FGCObject Interface

	//~ Begin IBlueprintEditor Interface
	UE_API virtual void RefreshEditors(ERefreshBlueprintEditorReason::Type Reason = ERefreshBlueprintEditorReason::UnknownReason) override;
	UE_API virtual void RefreshMyBlueprint();
	UE_API virtual void RefreshInspector();
	UE_API virtual void AddToSelection(UEdGraphNode* InNode) override;
	UE_API virtual void JumpToHyperlink(const UObject* ObjectReference, bool bRequestRename = false) override;
	UE_API virtual void JumpToPin(const class UEdGraphPin* Pin) override;
	UE_API virtual void SummonSearchUI(bool bSetFindWithinBlueprint, FString NewSearchTerms = FString(), bool bSelectFirstResult = false) override;
	UE_API virtual void SummonFindAndReplaceUI() override;
	UE_API virtual UEdGraph* GetFocusedGraph() const override;
	UE_API virtual TSharedPtr<SGraphEditor> OpenGraphAndBringToFront(UEdGraph* Graph, bool bSetFocus = true) override;
	UE_API virtual TArray<TSharedPtr<FSubobjectEditorTreeNode>> GetSelectedSubobjectEditorTreeNodes() const override;
	UE_API virtual TSharedPtr<FSubobjectEditorTreeNode> FindAndSelectSubobjectEditorTreeNode(const UActorComponent* InComponent, bool IsCntrlDown) override;
	UE_API virtual int32 GetNumberOfSelectedNodes() const override;
	UE_API virtual void AnalyticsTrackNodeEvent(UBlueprint* Blueprint, UEdGraphNode *GraphNode, bool bNodeDelete = false) const override;
	UE_API void AnalyticsTrackCompileEvent(UBlueprint* Blueprint, int32 NumErrors, int32 NumWarnings) const;
	virtual TSharedPtr<class IClassViewerFilter> GetImportedClassViewerFilter() const override { return ImportedClassViewerFilter; }
	UE_API virtual void GetPinTypeSelectorFilters(TArray<TSharedPtr<class IPinTypeSelectorFilter>>& OutFilters) const override;
	UE_API virtual bool IsNonImportedObject(const UObject* InObject) const;
	UE_API virtual bool IsNonImportedObject(const FSoftObjectPath& InObject) const;
	//~ End IBlueprintEditor Interface

	//~ Begin FTickableEditorObject Interface
	UE_API virtual void Tick(float DeltaTime) override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
	UE_API virtual TStatId GetStatId() const override;
	//~ End FTickableEditorObject Interface

public:
	UE_API FBlueprintEditor();

	UE_API virtual ~FBlueprintEditor();

	/** Add context objects for menus and toolbars */
	UE_API virtual void InitToolMenuContext(FToolMenuContext& MenuContext) override;

	/** Check the Node Title is visible */
	UE_API bool IsNodeTitleVisible(const UEdGraphNode* Node, bool bRequestRename);

	/** Pan the view to center on a particular node */
	UE_API void JumpToNode(const class UEdGraphNode* Node, bool bRequestRename = false);

	/** Returns a pointer to the Blueprint object we are currently editing, as long as we are editing exactly one */
	UE_API virtual UBlueprint* GetBlueprintObj() const;

	/**	Returns whether the editor is currently editing a single blueprint object */
	UE_API bool IsEditingSingleBlueprint() const;

	/** Ensures the blueprint has keyboard focus. */
	UE_API void SetKeyboardFocus();

	/** Getters for the various Kismet2 widgets */
	TSharedRef<SKismetInspector> GetInspector() const { return Inspector.ToSharedRef(); }
	TSharedRef<SKismetInspector> GetDefaultEditor() const { return DefaultEditor.ToSharedRef(); }
	UE_API TSharedRef<SBlueprintPalette> GetPalette();
	TSharedRef<SBlueprintBookmarks> GetBookmarksWidget() const { return BookmarksWidget.ToSharedRef(); }
	TSharedRef<SWidget> GetCompilerResults() const { return CompilerResults.ToSharedRef(); }
	TSharedRef<SFindInBlueprints> GetFindResults() const { return FindResults.ToSharedRef(); }

	/** Getters for the various optional Kismet2 widgets */	
	TSharedPtr<SSubobjectEditor> GetSubobjectEditor() const { return SubobjectEditor; }
	TSharedPtr<SSCSEditorViewport> GetSubobjectViewport() const { return SubobjectViewport; }
	TSharedPtr<SMyBlueprint> GetMyBlueprintWidget() const { return MyBlueprintWidget; }
	TSharedPtr<SReplaceNodeReferences> GetReplaceReferencesWidget() const { return ReplaceReferencesWidget; }

	/**
	 * Provides access to the preview actor.
	 */
	UE_API AActor* GetPreviewActor() const;

	/**
	 * Provides access to the preview scene.
	 */
	FPreviewScene* GetPreviewScene()
	{
		return &PreviewScene;
	}

	/**
	* Creates/updates the preview actor for the given blueprint.
	*
	* @param InBlueprint			The Blueprint to create or update the preview for.
	* @param bInForceFullUpdate	Force a full update to respawn actors.
	*/
	UE_API void UpdatePreviewActor(UBlueprint* InBlueprint, bool bInForceFullUpdate = false);

	/**
	* Destroy the Blueprint preview.
	*/
	UE_API void DestroyPreview();

	TSharedPtr<class FBlueprintEditorToolbar> GetToolbarBuilder() { return Toolbar; }

	/** @return the documentation location for this editor */
	UE_API virtual FString GetDocumentationLink() const override;

	/**	Returns whether the edited blueprint has components */
	UE_API bool CanAccessComponentsMode() const;

	/** Returns if we are currently closing the editor */
	UE_API bool IsEditorClosing() const;

	// @todo This is a hack for now until we reconcile the default toolbar with application modes
	UE_API void RegisterToolbarTab(const TSharedRef<class FTabManager>& TabManager);
	/** Throw a simple message into the log */
	UE_API void LogSimpleMessage(const FText& MessageText);

	/** Dumps messages to the compiler log, with an option to force it to display/come to front */
	UE_API void DumpMessagesToCompilerLog(const TArray<TSharedRef<class FTokenizedMessage>>& Messages, bool bForceMessageDisplay);

	/** Returns true if PIE is active */
	UE_API bool IsPlayInEditorActive() const;

	/** Get the currently selected set of nodes */
	UE_API FGraphPanelSelectionSet GetSelectedNodes() const;

	/** Returns the currently selected node if there is a single node selected (if there are multiple nodes selected or none selected, it will return nullptr) */
	UE_API UEdGraphNode* GetSingleSelectedNode() const;

	/** Save the current set of edited objects in the LastEditedObjects array so it will be opened next time we open K2 */
	UE_API void SaveEditedObjectState();

	/** Create new tab for each element of LastEditedObjects array */
	UE_API void RestoreEditedObjectState();

	// Request a save of the edited object state
	// This is used to delay it by one frame when triggered by a tab being closed, so it can finish closing before remembering the new state
	UE_API void RequestSaveEditedObjectState();

	/** Get the visible bounds of the given graph node */
	UE_API void GetBoundsForNode(const UEdGraphNode* InNode, class FSlateRect& OutRect, float InPadding) const;

	/** Gets the focused graph current view bookmark ID */
	UE_API void GetViewBookmark(FGuid& BookmarkId);

	/** Gets the focused graph view location/zoom amount */
	UE_API void GetViewLocation(FVector2D& Location, float& ZoomAmount);

	/** Sets the focused graph view location/zoom amount */
	UE_API void SetViewLocation(const FVector2D& Location, float ZoomAmount, const FGuid& BookmarkId = FGuid());

	/** Returns whether a graph is editable or not */
	UE_API virtual bool IsEditable(UEdGraph* InGraph) const;
	/** Determines if the graph is ReadOnly, this differs from editable in that it is never expected to be edited and is in a read-only state */
	UE_API bool IsGraphReadOnly(UEdGraph* InGraph) const;

	/** Used to determine the visibility of the graph's instruction text. */
	UE_API float GetInstructionTextOpacity(UEdGraph* InGraph) const;

	/** Returns true if in editing mode */
	UE_API virtual bool InEditingMode() const;

	/** Returns true if able to compile */
	UE_API virtual bool IsCompilingEnabled() const;

	/** Returns true if the parent class of the Blueprint being edited is also a Blueprint */
	UE_API bool IsParentClassABlueprint() const;

	/** Returns true if the parent class of the Blueprint being edited is an editable Blueprint */
	UE_API bool IsParentClassAnEditableBlueprint() const;

	/** Returns true if the parent class of the Blueprint being edited is native */
	UE_API bool IsParentClassNative() const;

	/** Returns true if the parent class is native and the link to it's header can be shown*/
	UE_API virtual bool IsNativeParentClassCodeLinkEnabled() const;

	/** Handles opening the header file of native parent class */
	UE_API void OnEditParentClassNativeCodeClicked();

	/** Returns: "(<NativeParentClass>.h)" */
	UE_API FText GetTextForNativeParentClassHeaderLink() const;

	/** Determines visibility of the native parent class manipulation buttons on the menu bar overlay */
	UE_API EVisibility GetNativeParentClassButtonsVisibility() const;

	/** Determines visibility of the standard parent class label on the menu bar overlay */
	UE_API EVisibility GetParentClassNameVisibility() const;

	/** Returns our PIE Status - SIMULATING / SERVER / CLIENT */
	UE_API FText GetPIEStatus() const;

	/**
	 * Util for finding a glyph for a graph
	 *
	 * @param Graph - The graph to evaluate
	 * @param bInLargeIcon - if true the icon returned is 22x22 pixels, else it is 16x16
	 * @return An appropriate brush to use to represent the graph, if the graph is an unknown type the function will return the default "function" glyph
	 */
	static UE_API const FSlateBrush* GetGlyphForGraph(const UEdGraph* Graph, bool bInLargeIcon = false);

	/**
	 * Util for finding a glyph and color for a variable.
	 *
	 * @param VarScope			Scope to find the variable in
	 * @param VarName			Name of variable
	 * @param IconColorOut		The resulting color for the glyph
	 * @param SecondaryBrushOut The resulting secondary glyph brush (used for Map types)
	 * @param SecondaryColorOut The resulting secondary color for the glyph (used for Map types)
	 * @return					The resulting glyph brush
	 */
	static UE_API FSlateBrush const* GetVarIconAndColor(const UStruct* VarScope, FName VarName, FSlateColor& IconColorOut, FSlateBrush const*& SecondaryBrushOut, FSlateColor& SecondaryColorOut);

	/**
	 * Util for finding a glyph and color for a variable.
	 *
	 * @param Property       The variable's property
	 * @param IconColorOut      The resulting color for the glyph
	 * @param SecondaryBrushOut The resulting secondary glyph brush (used for Map types)
	 * @param SecondaryColorOut The resulting secondary color for the glyph (used for Map types)
	 * @return					The resulting glyph brush
	 */
	static UE_API FSlateBrush const* GetVarIconAndColorFromProperty(const FProperty* Property, FSlateColor& IconColorOut, FSlateBrush const*& SecondaryBrushOut, FSlateColor& SecondaryColorOut);

	/**
	* Util for finding a glyph and color for a variable.
	*
	* @param PinType       The variable's pin type
	* @param IconColorOut      The resulting color for the glyph
	* @param SecondaryBrushOut The resulting secondary glyph brush (used for Map types)
	* @param SecondaryColorOut The resulting secondary color for the glyph (used for Map types)
	* @return					The resulting glyph brush
	*/
	static UE_API FSlateBrush const* GetVarIconAndColorFromPinType(const FEdGraphPinType& PinType, FSlateColor& IconColorOut, FSlateBrush const*& SecondaryBrushOut, FSlateColor& SecondaryColorOut);

	/** Overridable function for determining if the current mode can script */
	UE_API virtual bool IsInAScriptingMode() const;

	/** Called when Compile button is clicked */
	UE_API virtual void Compile();

	/** Helper functions used to construct/operate-on the "Save on Compile" command */
	UE_API bool IsSaveOnCompileEnabled() const;

	/** Calls the above function, but returns an FReply::Handled(). Used in SButtons */
	UE_API virtual FReply Compile_OnClickWithReply();

	/** Called when the refresh all nodes button is clicked */
	UE_API void RefreshAllNodes_OnClicked();

	UE_API EVisibility IsDebuggerVisible() const;

	UE_API virtual void OnActiveTabChanged(TSharedPtr<SDockTab> PreviouslyActive, TSharedPtr<SDockTab> NewlyActivated);

	/** Called when a token in a log message is clicked */
	UE_API void LogToken_OnClicked(const class IMessageToken& Token);

	UE_API virtual void FocusInspectorOnGraphSelection(const TSet<class UObject*>& NewSelection, bool bForceRefresh = false);

	/** Variable list window calls this after it is updated */
	UE_API void VariableListWasUpdated();

	/** Virtual override point for editing defaults; allowing more derived editors to edit something else */
	UE_API virtual void StartEditingDefaults(bool bAutoFocus = true, bool bForceRefresh = false);

	// Called by the blueprint editing app mode to focus the appropriate tabs, etc...
	UE_API void SetupViewForBlueprintEditingMode();

	// Ensures the blueprint is up to date
	UE_API void EnsureBlueprintIsUpToDate(UBlueprint* BlueprintObj);

	// Should be called when initializing any editor built off this foundation
	UE_API void CommonInitialization(const TArray<UBlueprint*>& InitBlueprints, bool bShouldOpenInDefaultsMode);

	// Other types of BP editors that don't use BP VM/function libs can choose to not load BP function libraries, for example: Control Rig Blueprints
	virtual bool ShouldLoadBPLibrariesFromAssetRegistry() { return true; }

	// Should be called when initializing an editor that has a blueprint, after layout (tab spawning) is done
	UE_API void PostLayoutBlueprintEditorInitialization();

	/** Called when graph editor focus is changed */
	UE_API virtual void OnGraphEditorFocused(const TSharedRef<class SGraphEditor>& InGraphEditor);

	/** Called when the graph editor tab is backgrounded */
	UE_API virtual void OnGraphEditorBackgrounded(const TSharedRef<SGraphEditor>& InGraphEditor);

	/** Enable/disable the subobject editor preview viewport */
	UE_API void EnableSubobjectPreview(bool bEnable);

	/** Refresh the preview viewport to reflect changes in the subobject */
	UE_API void UpdateSubobjectPreview(bool bUpdateNow = false);

	///////////////////////////////////////////////////////
	// Subobject Editor Callbacks
	/** Delegate invoked when the Subobject editor needs to obtain the object context for editing */
	UE_API UObject* GetSubobjectEditorObjectContext() const;

	/** Delegate invoked when the selection is changed in the subobject editor widget */
	UE_API virtual void OnSelectionUpdated(const TArray<TSharedPtr<class FSubobjectEditorTreeNode>>& SelectedNodes);

	/** Delegate invoked when an item is double clicked in the subobject editor widget */
	UE_API virtual void OnComponentDoubleClicked(TSharedPtr<class FSubobjectEditorTreeNode> Node);

	/** Delegate invoked when a new subobject is added in the subobject editor widget */
	UE_API virtual void OnComponentAddedToBlueprint(const FSubobjectData& NewSubobjectData);

	/** Pin visibility accessors */
	UE_API void SetPinVisibility(SGraphEditor::EPinVisibility Visibility);
	bool GetPinVisibility(SGraphEditor::EPinVisibility Visibility) const { return PinVisibility == Visibility; }

	/** Reparent the current blueprint */
	UE_API void ReparentBlueprint_Clicked();
	UE_API virtual bool ReparentBlueprint_IsVisible() const;
	UE_API void ReparentBlueprint_NewParentChosen(UClass* ChosenClass);

	/** Utility function to handle all steps required to rename a newly added action */
	UE_API void RenameNewlyAddedAction(FName InActionName);

	/** Adds a new variable to this blueprint */
	UE_API void OnAddNewVariable();
	FReply OnAddNewVariable_OnClick() { OnAddNewVariable(); return FReply::Handled(); }

	/** Checks if adding a local variable is allowed in the focused graph */
	UE_API virtual bool CanAddNewLocalVariable() const;

	/** Adds a new local variable to the focused function graph */
	UE_API virtual void OnAddNewLocalVariable();

	/** Pastes a new local variable to the focused function graph */
	UE_API virtual void OnPasteNewLocalVariable(const FBPVariableDescription& VariableDescription);

	/** Create a variant for a function */
	virtual void AddNewFunctionVariant(const UEdGraph* InOriginalFunction) {}

	// Type of new document/graph being created by a menu item
	enum ECreatedDocumentType
	{
		CGT_NewVariable,
		CGT_NewFunctionGraph,
		CGT_NewMacroGraph,
		CGT_NewAnimationLayer,
		CGT_NewEventGraph,
		CGT_NewLocalVariable
	};

	/** Called when New Function button is clicked */
	UE_API virtual void NewDocument_OnClicked(ECreatedDocumentType GraphType);
	FReply NewDocument_OnClick(ECreatedDocumentType GraphType) { NewDocument_OnClicked(GraphType); return FReply::Handled(); }

	/** Called when New Delegate button is clicked */
	UE_API void OnAddNewDelegate();
	UE_API bool AddNewDelegateIsVisible() const;

	// Called to see if the new document menu items is visible for this type
	virtual bool IsSectionVisible(NodeSectionID::Type InSectionID) const { return true; }
	UE_API virtual bool NewDocument_IsVisibleForType(ECreatedDocumentType GraphType) const;
	EVisibility NewDocument_GetVisibilityForType(ECreatedDocumentType GraphType) const
	{
		return NewDocument_IsVisibleForType(GraphType) ? EVisibility::Visible : EVisibility::Collapsed;
	}

	static UE_API FName SelectionState_MyBlueprint;
	static UE_API FName SelectionState_Components;
	static UE_API FName SelectionState_Graph;
	static UE_API FName SelectionState_ClassSettings;
	static UE_API FName SelectionState_ClassDefaults;

	/** Gets or sets the flag for context sensitivity in the graph action menu */
	bool& GetIsContextSensitive() { return bIsActionMenuContextSensitive; }

	/** Gets the UI selection state of this editor */
	FName GetUISelectionState() const { return CurrentUISelection; }
	UE_API void SetUISelectionState(FName SelectionOwner);

	UE_API virtual void ClearSelectionStateFor(FName SelectionOwner);

	/** Handles spawning a graph node in the current graph using the passed in chord */
	UE_DEPRECATED(5.6, "Please use the version of the function accepting FVector2f.")
	UE_API virtual FReply OnSpawnGraphNodeByShortcut(FInputChord InChord, const FVector2D& InPosition, UEdGraph* InGraph) UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION;
	UE_API virtual FReply OnSpawnGraphNodeByShortcut(FInputChord InChord, const FVector2f& InPosition, UEdGraph* InGraph);

	/**
	 * Perform the actual promote to variable action on the given pin in the given blueprint.
	 *
	 * @param	InBlueprint				The blueprint in which to create the variable.
	 * @param	InTargetPin				The pin on which to base the variable.
	 * @param	bInToMemberVariable		TRUE if attempting to create a member variable, FALSE if the variable should be local
	 * @param   InOptionalLocation		Where the new node should be placed. If null, a fixed offset from the parent node will be used.
	 */
	UE_DEPRECATED(5.6, "Use the version of this function accepting a TOptional")
	UE_API void DoPromoteToVariable(UBlueprint* InBlueprint, UEdGraphPin* InTargetPin, bool bInToMemberVariable, const FVector2D* InOptionalLocation = nullptr);
	UE_API void DoPromoteToVariable2f(UBlueprint* InBlueprint, UEdGraphPin* InTargetPin, bool bInToMemberVariable, TOptional<FVector2f> InOptionalLocation = TOptional<FVector2f>());

	/** Called when node is spawned by keymap */
	UE_API void OnNodeSpawnedByKeymap();

	/** Update Node Creation mechanisms for analytics */
	UE_API void UpdateNodeCreationStats(const ENodeCreateAction::Type CreateAction);

	/** Sets customizations for the BP editor details panel. */
	UE_API void SetDetailsCustomization(TSharedPtr<class FDetailsViewObjectFilter> DetailsObjectFilter, TSharedPtr<class IDetailRootObjectCustomization> DetailsRootCustomization);

	/** Sets subobject editor UI customization */
	UE_API void SetSubobjectEditorUICustomization(TSharedPtr<class ISCSEditorUICustomization> InSCSEditorUICustomization);

	/**
	 * Register a customization for interacting with the subobject editor
	 * @param	InComponentName			The name of the component to customize behavior for
	 * @param	InCustomization			The customization instance to use
	 */
	UE_API void RegisterSCSEditorCustomization(const FName& InComponentName, TSharedPtr<class ISCSEditorCustomization> InCustomization);

	/**
	 * Unregister a previously registered customization for interacting with the subobject editor
	 * @param	InComponentName			The name of the component to customize behavior for
	 */
	UE_API void UnregisterSCSEditorCustomization(const FName& InComponentName);

	/** Forces the merge tool to be shown */
	UE_API void CreateMergeToolTab();
	UE_API void CreateMergeToolTab(const UBlueprint* BaseBlueprint, const UBlueprint* RemoteBlueprint, const FOnMergeResolved& ResolutionCallback);

	/** Closes the merge tool, rather than simply hiding it */
	UE_API void CloseMergeTool();

	/** Dumps the current blueprint search index to a JSON file for debugging purposes */
	UE_API void OnGenerateSearchIndexForDebugging();

	/** Dumps the currently-cached index data for the blueprint to a file for debugging */
	UE_API void OnDumpCachedIndexDataForBlueprint();

	/**
	 * Check to see if we can customize the subobject editor for the passed-in scene component
	 * @param	InComponentToCustomize	The component to check to see if a customization exists
	 * @return an subobject editor customization instance, if one exists.
	 */
	UE_API TSharedPtr<class ISCSEditorCustomization> CustomizeSubobjectEditor(const USceneComponent* InComponentToCustomize) const;

	/** Adds to a list of custom objects for debugging beyond what will automatically be found/used */
	virtual void GetCustomDebugObjects(TArray<FCustomDebugObject>& DebugList) const { }

	/** If returns true only the custom debug list will be used */
	virtual bool OnlyShowCustomDebugObjects() const { return false; }

	/** Can be overloaded to customize the labels in the debug filter */
	virtual FString GetCustomDebugObjectLabel(UObject* ObjectBeingDebugged) const { return FString(); }

	/** Called when a node's title is committed for a rename */
	UE_API void OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged);

	/** Called by a graph title bar to get any extra information the editor would like to display */
	UE_API virtual FText GetGraphDecorationString(UEdGraph* InGraph) const;

	/** Gets the display name of a graph */
	static UE_API FText GetGraphDisplayName(const UEdGraph* Graph);

	/** Checks to see if the provided graph is contained within the current blueprint */
	UE_API bool IsGraphInCurrentBlueprint(const UEdGraph* InGraph) const;

	/** Get the context to use from the Blueprint type */
	static UE_API FName GetContextFromBlueprintType(EBlueprintType InType);

	/* Selects an item in "My Blueprint" by name. */
	UE_API void SelectGraphActionItemByName(const FName& ItemName, ESelectInfo::Type SelectInfo = ESelectInfo::Direct, int32 SectionId = INDEX_NONE, bool bIsCategory = false);

	/** Handle when the debug object is changed in the UI */
	virtual void HandleSetObjectBeingDebugged(UObject* InObject) {}

	/** Adds a new bookmark node to the Blueprint that's being edited. */
	UE_API FBPEditorBookmarkNode* AddBookmark(const FText& DisplayName, const FEditedDocumentInfo& BookmarkInfo, bool bSharedBookmark = false);

	/** Renames the bookmark node with the given ID. */
	UE_API void RenameBookmark(const FGuid& BookmarkNodeId, const FText& NewName);

	/** Removes the bookmark node with the given ID. */
	UE_API void RemoveBookmark(const FGuid& BookmarkNodeId, bool bRefreshUI = true);

	/** Gets the default schema for this editor */
	TSubclassOf<UEdGraphSchema> GetDefaultSchema() const { return GetDefaultSchemaClass(); }
	
	/** Imports the given namespace into the editor. This may trigger a load event for additional macro and/or function library assets if not already loaded. */
	UE_API void ImportNamespace(const FString& InNamespace);

	/** Parameters for the extended ImportNamespaceEx() method */
	struct FImportNamespaceExParameters
	{
		/** Whether this is an automatic or explicit (i.e. user-initiated) action. */
		bool bIsAutoImport;

		/** Callback to use for any post-import actions. Will not be invoked if nothing is imported. */
		FSimpleDelegate OnPostImportCallback;

		/** One or more unique namespace identifiers to be imported as part of the single transaction. */
		TSet<FString> NamespacesToImport;

		/** Default ctor (for initialization). */
		FImportNamespaceExParameters()
		{
			// Treat as auto-import by default (implies that the editor will auto-refresh the details view).
			bIsAutoImport = true;
		}
	};

	/** Imports a set of namespace(s) into the editor. This may trigger a load event for additional macro and/or function library assets if not already loaded. */
	UE_API void ImportNamespaceEx(const FImportNamespaceExParameters& InParams);

	/** Removes the given namespace from the editor's current import context. However, this will NOT unload any associated macro and/or function library assets. */
	UE_API void RemoveNamespace(const FString& InNamespace);

	/** Selects a local variable to load in the details panel. */
	virtual bool SelectLocalVariable(const UEdGraph* Graph, const FName& VariableName) { return false; }

	/** Duplicates an EdGraphNode using selection and copy / paste functionality */
	UE_API void SelectAndDuplicateNode(UEdGraphNode* InNode);

protected:

	/** Called during initialization of the blueprint editor to register commands and extenders. */
	UE_API virtual void InitalizeExtenders();

	/** Called during initialization of the blueprint editor to register any application modes. */
	UE_API virtual void RegisterApplicationModes(const TArray<UBlueprint*>& InBlueprints, bool bShouldOpenInDefaultsMode, bool bNewlyCreated = false);

	// Updates the selected object used by the stand lone defaults editor widget.
	UE_API void RefreshStandAloneDefaultsEditor();

	// Zooming to fit the entire graph
	UE_API void ZoomToWindow_Clicked();
	UE_API bool CanZoomToWindow() const;

	// Zooming to fit the current selection
	UE_API void ZoomToSelection_Clicked();
	UE_API bool CanZoomToSelection() const;

	// Navigating into/out of graphs
	UE_API void NavigateToParentGraph();
	UE_API void NavigateToParentGraphByDoubleClick();
	UE_API bool CanNavigateToParentGraph() const;
	UE_API void NavigateToChildGraph();
	UE_API bool CanNavigateToChildGraph() const;

	/** Determines visibility of the find parent class in content browser button on the menu bar overlay */
	UE_API EVisibility GetFindParentClassVisibility() const;

	/** Determines visibility of the edit parent class button on the menu bar overlay */
	UE_API EVisibility GetEditParentClassVisibility() const;

	/** Recreates the overlay on the menu bar */
	UE_API virtual void PostRegenerateMenusAndToolbars() override;

	/** Returns the name of the Blueprint's parent class */
	UE_API FText GetParentClassNameText() const;

	/** Returns the native name of the Blueprint's parent class */
	UE_API FText GetParentClassNativeNameText() const;

	/** Handler for "Find parent class in CB" button */
	UE_API FReply OnFindParentClassInContentBrowserClicked();

	/** Handler for "Edit parent class" button */
	UE_API FReply OnEditParentClassClicked();

	/** Called to start a quick find (focus the search box in the explorer tab) */
	UE_API void FindInBlueprint_Clicked();

	// Is the main details panel currently showing 'Global options' (e.g., class metadata)?
	UE_API bool IsDetailsPanelEditingGlobalOptions() const;

	/** Edit the class settings aka Blueprint global options */
	UE_API void EditGlobalOptions_Clicked();

	// Is the main details panel currently showing 'Class defaults' (Note: Has nothing to do with the standalone class defaults panel)?
	UE_API bool IsDetailsPanelEditingClassDefaults() const;

	/** Edit the class defaults */
	UE_API void EditClassDefaults_Clicked();

	/** Called to undo the last action */
	UE_API void UndoGraphAction();

	/** Whether or not we can perform an undo of the last transacted action */
	UE_API bool CanUndoGraphAction() const;

	/** Called to redo the last undone action */
	UE_API void RedoGraphAction();

	/** Whether or not we can redo an undone action */
	UE_API bool CanRedoGraphAction() const;
	
	/** Called when the selection changes in the GraphEditor */
	UE_API virtual void OnSelectedNodesChangedImpl(const TSet<class UObject*>& NewSelection);

	/** Called when the selection changes in the GraphEditor */
	void OnSelectedNodesChanged(const TSet<class UObject*>& NewSelection) { OnSelectedNodesChangedImpl(NewSelection); }

	/** Called when an actor is dropped onto the graph editor */
	UE_API void OnGraphEditorDropActor(const TArray< TWeakObjectPtr<AActor> >& Actors, UEdGraph* Graph, const FVector2f& DropLocation);

	/** Called when a streaming level is dropped onto the graph editor */
	UE_API void OnGraphEditorDropStreamingLevel(const TArray< TWeakObjectPtr<ULevelStreaming> >& Levels, UEdGraph* Graph, const FVector2f& DropLocation);

	/** Called to create context menu when right-clicking on graph */
	UE_DEPRECATED(5.6, "Please use the version of the function taking FVector2f.")
	UE_API FActionMenuContent OnCreateGraphActionMenu(UEdGraph* InGraph, const FVector2D& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed);
	UE_API FActionMenuContent OnCreateGraphActionMenu(UEdGraph* InGraph, const FVector2f& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed);

	/** Called from graph context menus when they close to tell the editor why they closed */
	UE_API void OnGraphActionMenuClosed(bool bActionExecuted, bool bContextSensitiveChecked, bool bGraphPinContext);

	/** Called when the Blueprint we are editing has changed */
	UE_API virtual void OnBlueprintChangedImpl(UBlueprint* InBlueprint, bool bIsJustBeingCompiled = false);

	/** Called when the Blueprint we are editing has changed, forwards to impl */
	void OnBlueprintChanged(UBlueprint* InBlueprint) { return OnBlueprintChangedImpl(InBlueprint); }

	UE_API void OnBlueprintCompiled(UBlueprint* InBlueprint);

	/** Handles the unloading of Blueprints (by closing the editor, if it operating on the Blueprint being unloaded)*/
	UE_API void OnBlueprintUnloaded(UBlueprint* InBlueprint);

	/** Called when a property change is about to be propagated to instances of the Blueprint */
	UE_API void OnPreObjectPropertyChanged(UObject* InObject, const FEditPropertyChain& EditPropertyChain);

	/** Called after a property change has been propagated to instances of the Blueprint */
	UE_API void OnPostObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& PropertyChangedEvent);

	//@TODO: Should the breakpoint/watch modification operations be whole-blueprint, or current-graph?

	/** Deletes all breakpoints for the blueprint being edited */
	UE_API void ClearAllBreakpoints();

	/** Disables all breakpoints for the blueprint being edited */
	UE_API void DisableAllBreakpoints();

	/** Enables all breakpoints for the blueprint being edited */
	UE_API void EnableAllBreakpoints();

	/** Clears all watches associated with the blueprint being edited */
	UE_API void ClearAllWatches();

	UE_API bool HasAnyBreakpoints() const;
	UE_API bool HasAnyEnabledBreakpoints() const;
	UE_API bool HasAnyDisabledBreakpoints() const;
	UE_API bool HasAnyWatches() const;

	/** Spawns a new blueprint debugger tab */
	UE_API void OpenBlueprintDebugger();
	UE_API bool CanOpenBlueprintDebugger() const;

	// Utility helper to get the currently hovered pin in the currently visible graph, or nullptr if there isn't one
	UE_API UEdGraphPin* GetCurrentlySelectedPin() const;

	// UI Action functionality
	UE_API void OnPromoteToVariable(bool bInToMemberVariable);
	UE_API bool CanPromoteToVariable(bool bInToMemberVariable) const;

	UE_API void OnSplitStructPin();
	UE_API bool CanSplitStructPin() const;

	UE_API void OnRecombineStructPin();
	UE_API bool CanRecombineStructPin() const;

	UE_API void OnAddExecutionPin();
	UE_API bool CanAddExecutionPin() const;

	UE_API void OnInsertExecutionPinBefore();
	UE_API void OnInsertExecutionPinAfter();
	UE_API void OnInsertExecutionPin(EPinInsertPosition Position);
	UE_API bool CanInsertExecutionPin() const;

	UE_API void OnRemoveExecutionPin();
	UE_API bool CanRemoveExecutionPin() const;

	UE_API void OnRemoveThisStructVarPin();
	UE_API bool CanRemoveThisStructVarPin() const;

	UE_API void OnRemoveOtherStructVarPins();
	UE_API bool CanRemoveOtherStructVarPins() const;

	UE_API void OnRestoreAllStructVarPins();
	UE_API bool CanRestoreAllStructVarPins() const;

	UE_API void OnResetPinToDefaultValue();
	UE_API bool CanResetPinToDefaultValue() const;

	UE_API void OnAddOptionPin();
	UE_API bool CanAddOptionPin() const;

	UE_API void OnRemoveOptionPin();
	UE_API bool CanRemoveOptionPin() const;

	/** Functions for handling the changing of the pin's type (PinCategory, PinSubCategory, etc) */
	UE_API FEdGraphPinType OnGetPinType(UEdGraphPin* SelectedPin) const;
	UE_API void OnChangePinType();
	UE_API void OnChangePinTypeFinished(const FEdGraphPinType& PinType, UEdGraphPin* SelectedPin);
	UE_API bool CanChangePinType() const;

	UE_API void OnAddParentNode();
	UE_API bool CanAddParentNode() const;
	
	UE_API void OnCreateMatchingFunction();
	UE_API bool CanCreateMatchingFunction() const;

	UE_API void OnEnableBreakpoint();
	UE_API bool CanEnableBreakpoint() const;

	UE_API void OnToggleBreakpoint();
	UE_API bool CanToggleBreakpoint() const;

	UE_API void OnDisableBreakpoint();
	UE_API bool CanDisableBreakpoint() const;

	UE_API void OnAddBreakpoint();
	UE_API bool CanAddBreakpoint() const;

	UE_API void OnRemoveBreakpoint();
	UE_API bool CanRemoveBreakpoint() const;

	UE_API void OnCollapseNodes();
	UE_API bool CanCollapseNodes() const;

	UE_API void OnCollapseSelectionToFunction();
	UE_API bool CanCollapseSelectionToFunction() const;

	UE_API void OnCollapseSelectionToMacro();
	UE_API bool CanCollapseSelectionToMacro() const;

	UE_API void OnPromoteSelectionToFunction();
	UE_API bool CanPromoteSelectionToFunction() const;

	UE_API void OnPromoteSelectionToMacro();
	UE_API bool CanPromoteSelectionToMacro() const;

	UE_API void OnExpandNodes();
	UE_API bool CanExpandNodes() const;

	UE_API void OnOpenSelectedNodesInNewTab();
	UE_API bool CanOpenSelectedNodesInNewTab() const;

	/**
	* Move the given set of nodes to an average spot near the Source position
	* 
	* @param AverageNodes					The nodes to move
	* @param SourcePos						The source position used to average the nodes around
	* @param bExpandedNodesNeedUniqueGuid	If true then a new Guid will be generated for each node in the set
	*/
	UE_API void MoveNodesToAveragePos(TSet<UEdGraphNode*>& AverageNodes, FVector2D SourcePos, bool bExpandedNodesNeedUniqueGuid = false) const;

	UE_API void OnConvertFunctionToEvent();

public:
	/** Converts the given function entry node to an event if it passes validation */
	UE_API bool ConvertFunctionIfValid(UK2Node_FunctionEntry* FuncEntryNode);

	/** Converts the given event node to a function graph on this blueprint if it passes validation */
	UE_API bool ConvertEventIfValid(UK2Node_Event* EventToConv);

protected:
	/** 
	* Callback function for the context menu on a node to determine if a function 
	* could possibly be converted to an event
	*/
	UE_API bool CanConvertFunctionToEvent() const;

	/*
	* Given a function node, move all nodes from the function out of the function graph,
	* create an event with the same function name, and connect nodes to that event. 
	* 
	* @param SelectedCallFunctionNode	The function node to convert to an event
	*/
	UE_API void ConvertFunctionToEvent(UK2Node_FunctionEntry* SelectedCallFunctionNode);

	UE_API void OnConvertEventToFunction();
	UE_API void ConvertEventToFunction(UK2Node_Event* SelectedEventNode);
	UE_API bool CanConvertEventToFunction() const;

	/**
	* Get all connected nodes and add them to a set of out nodes
	* 
	* @param Node		The node to get all connects to 
	* @return	TArray of all connected nodes
	*/
	UE_API TArray<UEdGraphNode*> GetAllConnectedNodes(UEdGraphNode* const Node) const;

	UE_API void OnAlignTop();
	UE_API void OnAlignMiddle();
	UE_API void OnAlignBottom();
	UE_API void OnAlignLeft();
	UE_API void OnAlignCenter();
	UE_API void OnAlignRight();

	UE_API void OnStraightenConnections();

	UE_API void OnDistributeNodesH();
	UE_API void OnDistributeNodesV();

	UE_API void OnStackNodesH();
	UE_API void OnStackNodesV();

	UE_API void SelectAllNodes();
	UE_API bool CanSelectAllNodes() const;

	UE_API virtual void DeleteSelectedNodes();
	UE_API virtual bool CanDeleteNodes() const;

	/**
	* Given a node, make connections from anything connected to it's input pin to
	* anything connected to it's "then" pins. Only works on impure nodes with single exec/then pins.
	* 
	* @param Node	The node to reconnect
	*/
	UE_API void ReconnectExecPins(class UK2Node* Node);

	UE_API void DeleteSelectedDuplicatableNodes();

	UE_API virtual void CutSelectedNodes();
	UE_API virtual bool CanCutNodes() const;

	UE_API virtual void CopySelectedNodes();
	UE_API virtual bool CanCopyNodes() const;

	/** Paste on graph at specific location */
	UE_API virtual void PasteNodesHere(class UEdGraph* DestinationGraph, const FVector2f& GraphLocation) override;

	/** Paste Variable Definition or Nodes */
	UE_API virtual void PasteGeneric();
	UE_API virtual bool CanPasteGeneric() const;

	UE_API virtual void PasteNodes();
	UE_API virtual bool CanPasteNodes() const override;

	UE_API void DuplicateNodes();
	UE_API bool CanDuplicateNodes() const;

	UE_API void OnSelectReferenceInLevel();
	UE_API bool CanSelectReferenceInLevel() const;

	UE_API void OnAssignReferencedActor();
	UE_API bool CanAssignReferencedActor() const;

	UE_API virtual void OnStartWatchingPin();
	UE_API virtual bool CanStartWatchingPin() const;

	UE_API virtual void OnStopWatchingPin();
	UE_API virtual bool CanStopWatchingPin() const;

	UE_API void ToggleSaveIntermediateBuildProducts();
	UE_API bool GetSaveIntermediateBuildProducts() const;

	UE_API void OnListObjectsReferencedByClass();
	UE_API void OnListObjectsReferencedByBlueprint();
	UE_API void OnRepairCorruptedBlueprint();

	UE_API void OnNodeDoubleClicked(UEdGraphNode* Node);

	UE_API virtual void OnEditTabClosed(TSharedRef<SDockTab> Tab);

	UE_API virtual bool GetBoundsForSelectedNodes(class FSlateRect& Rect, float Padding) override;

	/**
	 * Pulls out the pins to use as a template when collapsing a selection to a function with a custom event involved.
	 *
	 * @param InCustomEvent				The custom event used as a template
	 * @param InGatewayNode				The node replacing the selection of nodes
	 * @param InEntryNode				The entry node in the graph
	 * @param InResultNode				The result node in the graph
	 * @param InCollapsableNodes		The selection of nodes being collapsed
	 */
	UE_API void ExtractEventTemplateForFunction(class UK2Node_CustomEvent* InCustomEvent, UEdGraphNode* InGatewayNode, class UK2Node_EditablePinBase* InEntryNode, class UK2Node_EditablePinBase* InResultNode, TSet<UEdGraphNode*>& InCollapsableNodes);

	/**
	 * Collapses a selection of nodes into a graph for composite, function, or macro nodes.
	 *
	 * @param InGatewayNode				The node replacing the selection of nodes
	 * @param InEntryNode				The entry node in the graph
	 * @param InResultNode				The result node in the graph
	 * @param InSourceGraph				The graph the selection is from
	 * @param InDestinationGraph		The destination graph to move the selected nodes to
	 * @param InCollapsableNodes		The selection of nodes being collapsed
	 */
	UE_API void CollapseNodesIntoGraph(UEdGraphNode* InGatewayNode, class UK2Node_EditablePinBase* InEntryNode, class UK2Node_EditablePinBase* InResultNode, UEdGraph* InSourceGraph, UEdGraph* InDestinationGraph, TSet<UEdGraphNode*>& InCollapsableNodes, bool bCanDiscardEmptyReturnNode, bool bCanHaveWeakObjPtrParam);

	/** Called when a selection of nodes are being collapsed into a sub-graph */
	UE_API void CollapseNodes(TSet<class UEdGraphNode*>& InCollapsableNodes);

	/** Called when a selection of nodes are being collapsed into a function */
	UE_API UEdGraph* CollapseSelectionToFunction(TSharedPtr<SGraphEditor> InRootGraph, TSet<class UEdGraphNode*>& InCollapsableNodes, UEdGraphNode*& OutFunctionNode);

	/** Called when a selection of nodes are being collapsed into a macro */
	UE_API UEdGraph* CollapseSelectionToMacro(TSharedPtr<SGraphEditor> InRootGraph, TSet<class UEdGraphNode*>& InCollapsableNodes, UEdGraphNode*& OutMacroNode);

	/**
	 * Called when a selection of nodes is being collapsed into a function
	 *
	 * @param InSelection		The selection to check
	 *
	 * @return					Returns TRUE if the selection can be promoted to a function
	 */
	UE_API bool CanCollapseSelectionToFunction(TSet<class UEdGraphNode*>& InSelection) const;

	/**
	 * Called when a selection of nodes is being collapsed into a macro
	 *
	 * @param InSelection		The selection to check
	 *
	 * @return					Returns TRUE if the selection can be promoted to a macro
	 */
	UE_API bool CanCollapseSelectionToMacro(TSet<class UEdGraphNode*>& InSelection) const;

	/**
	 * Expands passed in node */
	static UE_API void ExpandNode(UEdGraphNode* InNodeToExpand, UEdGraph* InSourceGraph, TSet<UEdGraphNode*>& OutExpandedNodes);

	/**
	* Move every node from the source graph to the destination graph. Add Each node that is moved to the OutExpandedNodes set.
	* If the source graph is a function graph, keep track of the entry and result nodes in the given Out Parameters. 
	*
	* @param SourceNodes		Nodes to move
	* @param DestinationGraph	Graph to move nodes to
	* @param OutExpandedNodes	Set of each node that was moved from the source to destination graph
	* @param OutEntry			Pointer to the function entry node
	* @param OutResult			Pointer to the function result node
	* @param bIsCollapsedGraph	Whether or not the source graph is collapsed
	**/
	static UE_API void MoveNodesToGraph(TArray<UEdGraphNode*>& SourceNodes, UEdGraph* DestinationGraph, TSet<UEdGraphNode*>& OutExpandedNodes, UEdGraphNode** OutEntry, UEdGraphNode** OutResult, const bool bIsCollapsedGraph = false);

	/** Start editing the defaults for this blueprint */
	UE_API void OnStartEditingDefaultsClicked();

	/** Creates the widgets that go into the tabs (note: does not create the tabs themselves) **/
	UE_API virtual void CreateDefaultTabContents(const TArray<UBlueprint*>& InBlueprints);

	/** Create Default Commands **/
	UE_API virtual void CreateDefaultCommands();

	/** Called when DeleteUnusedVariables button is clicked */
	UE_API void DeleteUnusedVariables_OnClicked();

	/** Called when Find In Blueprints menu is opened is clicked */
	UE_API void FindInBlueprints_OnClicked();

	//~ Begin FNotifyHook Interface
	UE_API virtual void NotifyPreChange( FProperty* PropertyAboutToChange ) override;
	UE_API virtual void NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;
	//~ End FNotifyHook Interface

	/** Callback to determine visibility of the public view checkbox in the Defaults editor */
	UE_API bool ShouldShowPublicViewControl() const;

	/** Callback when properties have finished being handled */
	UE_API virtual void OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent);

	/** On starting to rename node */
	UE_API void OnRenameNode();
	UE_API bool CanRenameNodes() const;

	/** Called when a node's title is being committed for a rename so it can be verified */
	UE_API bool OnNodeVerifyTitleCommit(const FText& NewText, UEdGraphNode* NodeBeingChanged, FText& OutErrorMessage);

	/**Load macro & function blueprint libraries from asset registry*/
	UE_API void LoadLibrariesFromAssetRegistry();

	//~ Begin FEditorUndoClient Interface
	UE_API virtual void	PostUndo(bool bSuccess) override;
	UE_API virtual void	PostRedo(bool bSuccess) override;
	// End of FEditorUndoClient

	/** Setup all the events that the graph editor can handle */
	UE_API virtual void SetupGraphEditorEvents(UEdGraph* InGraph, SGraphEditor::FGraphEditorEvents& InEvents);

	/** Get the graph appearance of the currently focused graph */
	UE_API FGraphAppearanceInfo GetCurrentGraphAppearance() const;

	/** Get the graph appearance of a specific graph, GetCurrentGraphAppearance() uses the currently focused graph. */
	UE_API virtual FGraphAppearanceInfo GetGraphAppearance(class UEdGraph* InGraph) const;

	/** Whenever new graphs need to be created it will use this schema by default. */
	UE_API virtual TSubclassOf<UEdGraphSchema> GetDefaultSchemaClass() const;

	/** Attempts to invoke the details tab if it's currently possible to. */
	UE_API void TryInvokingDetailsTab(bool bFlash = true);

	/** Handles activation of a graph editor "quick jump" command */
	UE_API void OnGraphEditorQuickJump(int32 BookmarkIndex);

	/** Binds a graph editor "quick jump" to the command at the given index */
	UE_API void SetGraphEditorQuickJump(int32 BookmarkIndex);

	/** Unbinds a graph editor "quick jump" from the command at the given index */
	UE_API void ClearGraphEditorQuickJump(int32 BookmarkIndex);

	/** Unbinds all graph editor "quick jump" commands */
	UE_API void ClearAllGraphEditorQuickJumps();

	/** Create a graph title bar widget */
	UE_API TSharedRef<SWidget> CreateGraphTitleBarWidget(TSharedRef<FTabInfo> InTabInfo, UEdGraph* InGraph);

private:

	/** Returns true if modules can be recompiled */
	static UE_API bool CanRecompileModules();

	/* User wants to edit tunnel via function editor */
	UE_API void OnEditTunnel();

protected:
	
	/* Create comment node on graph */
	UE_API virtual void OnCreateComment();
	
	/** Create custom event node on graph */
	UE_API virtual void OnCreateCustomEvent();

	// Create new graph editor widget for the supplied document container
	UE_API virtual TSharedRef<SGraphEditor> CreateGraphEditorWidget(TSharedRef<class FTabInfo> InTabInfo, class UEdGraph* InGraph);

private:

	/** Helper to move focused graph when clicking on graph breadcrumb */
	UE_API void OnChangeBreadCrumbGraph( class UEdGraph* InGraph);

	/** Function to check whether the give graph is a subgraph */
	static UE_API bool IsASubGraph( const class UEdGraph* GraphPtr );

	/** Creates the Subobject Editor tree component view and the Subobject Viewport. */
	UE_API void CreateSubobjectEditors();

	/** Callback when a token is clicked on in the compiler results log */
	UE_API void OnLogTokenClicked(const TSharedRef<class IMessageToken>& Token);

	/** Helper function to navigate the current tab */
	UE_API void NavigateTab(FDocumentTracker::EOpenDocumentCause InCause);

	/** Find all references of the selected node. */
	UE_API void OnFindReferences(bool bSearchAllBlueprints, const EGetFindReferenceSearchStringFlags Flags);

	/** Checks if we can currently find all references of the node selection. */
	UE_API bool CanFindReferences();

	/** Called when the user generates a warning tooltip because a connection was invalid */
	UE_API void OnDisallowedPinConnection(const class UEdGraphPin* PinA, const class UEdGraphPin* PinB);

	/**
	 * Checks to see if the focused Graph-Editor is focused on a animation graph or not.
	 * 
	 * @return True if GraphEdPtr's current graph is an animation graph, false if not.
	 */
	UE_API bool IsEditingAnimGraph() const;

	/** Returns whether the currently focused graph is editable or not */
	UE_API bool IsFocusedGraphEditable() const;

	/** Called when the user wants to jump to a node's definition */
	UE_API void OnGoToDefinition();

	/** Checks to see if it is possible to jump to the selected node's definition. */
	UE_API bool CanGoToDefinition() const;

	/** Open documentation for the selected node class */
	UE_API void OnGoToDocumentation();

	/** Can we open documentation for the selected node */
	UE_API bool CanGoToDocumentation();

	/** Util to try and get doc link for the currently selected node */
	UE_API FString GetDocLinkForSelectedNode();

	/** Util to try and get the base URL for the doc link for the currently selected node */
	UE_API FString GetDocLinkBaseUrlForSelectedNode();

	/** Set the enabled state for currently-selected nodes */
	UE_API void OnSetEnabledStateForSelectedNodes(ENodeEnabledState NewState);

	/** Returns the appropriate check box state representing whether or not the selected nodes are enabled */
	UE_API ECheckBoxState GetEnabledCheckBoxStateForSelectedNodes();

	/**
	 * Load editor settings from disk (docking state, window pos/size, option state, etc).
	 */
	UE_API virtual void LoadEditorSettings();

	/**
	 * Saves editor settings to disk (docking state, window pos/size, option state, etc).
	 */
	UE_API virtual void SaveEditorSettings();

	/** Attempt to match the given enabled state for currently-selected nodes */
	UE_API ECheckBoxState CheckEnabledStateForSelectedNodes(ENodeEnabledState CheckState);

	/** Handle undo/redo */
	UE_API void HandleUndoTransaction(const class FTransaction* Transaction);

public://@TODO

	UE_API virtual bool TransactionObjectAffectsBlueprint(UObject* InTransactedObject);

	TSharedPtr<FDocumentTracker> DocumentManager;
	
	/** Update all nodes' unrelated states when the graph has changed */
	UE_API void UpdateNodesUnrelatedStatesAfterGraphChange();

	/** Whether event graphs are allowed to be displayed/created in a blueprint */
	UE_API virtual bool AreEventGraphsAllowed() const;

	/** Whether macros are allowed  to be displayed/created in a blueprint */
	UE_API virtual bool AreMacrosAllowed() const;

	/** Whether delegates are allowed  to be displayed/created in a blueprint */
	UE_API virtual bool AreDelegatesAllowed() const;

	UE_API FEditorViewportSelectabilityBridge* GetViewportSelectabilityBridge();

protected:

	/** Should intermediate build products be saved when recompiling? */
	bool bSaveIntermediateBuildProducts;

	/** True if the current blueprint is in the process of being reparented */
	bool bIsReparentingBlueprint;

	/** Flags if this blueprint editor should close on its next tick. */
	bool bPendingDeferredClose;

	/** Currently focused graph editor */
	TWeakPtr<class SGraphEditor> FocusedGraphEdPtr;
	
	/** Factory that spawns graph editors; used to look up all tabs spawned by it. */
	TWeakPtr<FDocumentTabFactory> GraphEditorTabFactoryPtr;

	/** User-defined enumerators to keep loaded */
	TSet<TWeakObjectPtr<UUserDefinedEnum>> UserDefinedEnumerators;

	/** User-defined structures to keep loaded */
	TSet<TWeakObjectPtr<UUserDefinedStruct>> UserDefinedStructures;
	
	/** Macro/function libraries to keep loaded */
	TArray<TObjectPtr<UBlueprint>> StandardLibraries;

	/** Subobject Editor */
	TSharedPtr<SSubobjectEditor> SubobjectEditor;

	/** Viewport widget */
	TSharedPtr<SSCSEditorViewport> SubobjectViewport;

	/** Node inspector widget */
	TSharedPtr<class SKismetInspector> Inspector;

	/** defaults inspector widget */
	TSharedPtr<class SKismetInspector> DefaultEditor;

	/** Palette of all classes with funcs/vars */
	TSharedPtr<class SBlueprintPalette> Palette;

	/** Blueprint bookmark editing widget */
	TSharedPtr<class SBlueprintBookmarks> BookmarksWidget;

	/** All of this blueprints' functions and variables */
	TSharedPtr<class SMyBlueprint> MyBlueprintWidget;
	
	/** Widget for replacing node references */
	TSharedPtr<class SReplaceNodeReferences> ReplaceReferencesWidget;

	/** Compiler results log, with the log listing that it reflects */
	TSharedPtr<class SWidget> CompilerResults;
	TSharedPtr<class IMessageLogListing> CompilerResultsListing;
	
	/** Limited cache of last compile results */
	int32 CachedNumWarnings;
	int32 CachedNumErrors;
	
	/** Find results log as well as the search filter */
	TSharedPtr<class SFindInBlueprints> FindResults;

	/** Merge tool - WeakPtr because it's owned by the GlobalTabManager */
	TWeakPtr<class SDockTab> MergeTool;
	/** Merge tool - Delegate to call when the merge tool is closed. */
	FOnMergeResolved OnMergeResolved;

	/** Reference to owner of the current popup */
	TWeakPtr<class SWindow> NameEntryPopupWindow;

	/** Reference to helper object to validate names in popup */
	TSharedPtr<class INameValidatorInterface> NameEntryValidator;

	/** Reference to owner of the pin type change popup */
	TWeakPtr<class IMenu> PinTypeChangeMenu;

	/** The toolbar builder class */
	TSharedPtr<class FBlueprintEditorToolbar> Toolbar;

	/** Imported namespace helper utility object */
	TSharedPtr<FBlueprintNamespaceHelper> ImportedNamespaceHelper;

	/** Filter used to restrict class viewer widgets in the editor context to imported namespaces only */
	TSharedPtr<class IClassViewerFilter> ImportedClassViewerFilter;

	/** Filter used to restrict pin type selector widgets in the editor context to imported namespaces only */
	TSharedPtr<class IPinTypeSelectorFilter> ImportedPinTypeSelectorFilter;

	/** Filter used to restrict pin type selector widgets to those which permissions allow */
	TSharedPtr<class IPinTypeSelectorFilter> PermissionsPinTypeSelectorFilter;

	/** Filter used to compose multiple pin type filters together */
	TSharedPtr<class IPinTypeSelectorFilter> CompositePinTypeSelectorFilter;

	/** Pending set of namespaces to be auto-imported on the next tick */
	TSet<FString> DeferredNamespaceImports;
	
	FOnSetPinVisibility OnSetPinVisibility;

	/** Has someone requested a deferred update of the saved document state? */
	bool bRequestedSavingOpenDocumentState;

	/** Did we update the blueprint when it opened */
	bool bBlueprintModifiedOnOpen;

	/** Whether to hide unused pins or not */
	SGraphEditor::EPinVisibility PinVisibility;

	/** Whether the graph action menu should be sensitive to the pins dragged off of */
	bool bIsActionMenuContextSensitive;
	
	/** The current UI selection state of this editor */
	FName CurrentUISelection;

	/** Whether we are already in the process of closing this editor */
	bool bEditorMarkedAsClosed;

	/** Blueprint preview scene */
	FPreviewScene PreviewScene;

	/** The preview actor representing the current preview */
	mutable TWeakObjectPtr<AActor> PreviewActorPtr;

	/** If true, fade out nodes which are unrelated to the selected nodes automatically. */
	bool bHideUnrelatedNodes;

	/** Lock the current fade state of each node */
	bool bLockNodeFadeState;

	/** If a regular node (not a comment node) has been selected */
	bool bSelectRegularNode;

	/** True if the editor was opened in defaults mode */
	bool bWasOpenedInDefaultsMode;

	/** Focus nodes which are related to the selected nodes */
	UE_API void ResetAllNodesUnrelatedStates();
	UE_API void CollectExecUpstreamNodes(UEdGraphNode* CurrentNode, TArray<UEdGraphNode*>& CollectedNodes);
	UE_API void CollectExecDownstreamNodes(UEdGraphNode* CurrentNode, TArray<UEdGraphNode*>& CollectedNodes);
	UE_API void CollectPureDownstreamNodes(UEdGraphNode* CurrentNode, TArray<UEdGraphNode*>& CollectedNodes);
	UE_API void CollectPureUpstreamNodes(UEdGraphNode* CurrentNode, TArray<UEdGraphNode*>& CollectedNodes);
	UE_API void HideUnrelatedNodes();

	/** Register Menus */
	UE_API void RegisterMenus();

public:
	/** Make nodes which are unrelated to the selected nodes fade out */
	UE_API virtual void ToggleHideUnrelatedNodes();
	UE_API virtual bool IsToggleHideUnrelatedNodesChecked() const;

	/** Make a drop down menu to control the opacity of unrelated nodes */
	UE_API TSharedRef<SWidget> MakeHideUnrelatedNodesOptionsMenu();
	UE_API TOptional<float> HandleUnrelatedNodesOpacityBoxValue() const;
	UE_API void HandleUnrelatedNodesOpacityBoxChanged(float NewOpacity);
	UE_API void OnLockNodeStateCheckStateChanged(ECheckBoxState NewCheckedState);


public:
	//@TODO: To be moved/merged
	UE_API TSharedPtr<SDockTab> OpenDocument(const UObject* DocumentID, FDocumentTracker::EOpenDocumentCause Cause);

	/** Finds the tab associated with the specified asset, and closes if it is open */
	UE_API void CloseDocumentTab(const UObject* DocumentID);

	// Finds any open tabs containing the specified document and adds them to the specified array; returns true if at least one is found
	UE_API bool FindOpenTabsContainingDocument(const UObject* DocumentID, /*inout*/ TArray< TSharedPtr<SDockTab> >& Results);

public:
	/** Broadcasts a notification whenever the editor needs associated controls to refresh */
	DECLARE_EVENT ( FBlueprintEditor, FOnRefreshEvent );
	FOnRefreshEvent& OnRefresh() { return RefreshEvent; }

	/** function used by the SMyBlueprint to determine if an action matches a name.
	  * This happens during selection of a custom action ( a graph, a variable etc )
	  */
	virtual bool OnActionMatchesName(FEdGraphSchemaAction* InAction, const FName& InName) const { return false; }

private:
	/** Notification used whenever the editor wants associated controls to refresh. */
	FOnRefreshEvent RefreshEvent;

	/** Broadcast notification for associated controls to update */
	void BroadcastRefresh() { RefreshEvent.Broadcast(); }

	/** Command list for the graph editor */
	TSharedPtr<FUICommandList> GraphEditorCommands;

	/** Structure to contain editor usage analytics */
	struct FAnalyticsStatistics
	{
		/** Stats collected about graph action menu usage */
		int32 GraphActionMenusNonCtxtSensitiveExecCount;
		int32 GraphActionMenusCtxtSensitiveExecCount;
		int32 GraphActionMenusCancelledCount;

		/** Stats collection about user node creation actions */
		int32 MyBlueprintNodeDragPlacementCount;
		int32 PaletteNodeDragPlacementCount;
		int32 NodeGraphContextCreateCount;
		int32 NodePinContextCreateCount;
		int32 NodeKeymapCreateCount;
		int32 NodePasteCreateCount;

		/** New node instance information */
		struct FNodeDetails
		{
			FName NodeClass;
			int32 Instances;
		};

		/** Stats collected about warning tooltips */
		TArray<FDisallowedPinConnection> GraphDisallowedPinConnections;
	};

	/** analytics statistics for the Editor */ 
	FAnalyticsStatistics AnalyticsStats;

	/** UI customizations for the SCS editor inside the blueprint editor */
	TSharedPtr<ISCSEditorUICustomization> SCSEditorUICustomization;

	/** Customizations for the Subobject editor */
	TMap<FName, TSharedPtr<ISCSEditorCustomization>> SubobjectEditorCustomizations;

	/** Whether the current project is C++ or blueprint based */
	bool bCodeBasedProject;

	/** Delegates that are fired when the blueprint editor changes modes */
	FOnModeSet OnModeSetData;

	/** When set, flags which graph has a action menu currently open (if null, no graphs do). */
	UEdGraph* HasOpenActionMenu;
	/** Used to nicely fade instruction text, when the context menu is opened. */
	float InstructionsFadeCountdown;

	/** Handle to the registered OnActiveTabChanged delegate */
	FDelegateHandle OnActiveTabChangedDelegateHandle;

	/** Delegate handle registered for when settings change */
	FDelegateHandle BlueprintEditorSettingsChangedHandle;
	FDelegateHandle BlueprintProjectSettingsChangedHandle;

	enum class ESafeToModifyDuringPIEStatus
	{
		Unknown,
		Safe,
		NotSafe
	};

	mutable ESafeToModifyDuringPIEStatus ModifyDuringPIEStatus = ESafeToModifyDuringPIEStatus::Unknown;

	// Allow derived editors to add command mappings 
	virtual void OnCreateGraphEditorCommands(TSharedPtr<FUICommandList> GraphEditorCommandsList) {}

	UE_API virtual void OnBlueprintProjectSettingsChanged(UObject*, struct FPropertyChangedEvent&);
	UE_API virtual void OnBlueprintEditorPreferencesChanged(UObject*, struct FPropertyChangedEvent&);

	TUniquePtr<FEditorViewportSelectabilityBridge> ViewportSelectabilityBridge;
};

#undef LOCTEXT_NAMESPACE

#undef UE_API
