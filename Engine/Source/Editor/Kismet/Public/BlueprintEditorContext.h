// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "BlueprintEditorContext.generated.h"

#define UE_API KISMET_API

class UBlueprint;
class FBlueprintEditor;

UCLASS(MinimalAPI)
class UBlueprintEditorToolMenuContext : public UObject
{
	GENERATED_BODY()
public:

	UFUNCTION(BlueprintCallable, Category="Tool Menus")
	UE_API UBlueprint* GetBlueprintObj() const;

	TWeakPtr<FBlueprintEditor> BlueprintEditor;
};

#undef UE_API
