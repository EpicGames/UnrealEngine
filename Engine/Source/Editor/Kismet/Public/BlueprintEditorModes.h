// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintEditor.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "WorkflowOrientedApp/ApplicationMode.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

#define UE_API KISMET_API

class AActor;
class FBlueprintEditor;
class FTabManager;
class UActorComponent;

struct FBlueprintEditorApplicationModes
{
	// Mode identifiers
	static UE_API const FName StandardBlueprintEditorMode;
	static UE_API const FName BlueprintDefaultsMode;
	static UE_API const FName BlueprintComponentsMode;
	static UE_API const FName BlueprintInterfaceMode;
	static UE_API const FName BlueprintMacroMode;
	static FText GetLocalizedMode( const FName InMode )
	{
		static TMap< FName, FText > LocModes;

		if (LocModes.Num() == 0)
		{
			LocModes.Add( StandardBlueprintEditorMode, NSLOCTEXT("BlueprintEditor", "StandardBlueprintEditorMode", "Graph") );
			LocModes.Add( BlueprintDefaultsMode, NSLOCTEXT("BlueprintEditor", "BlueprintDefaultsMode", "Defaults") );
			LocModes.Add( BlueprintComponentsMode, NSLOCTEXT("BlueprintEditor", "BlueprintComponentsMode", "Components") );
			LocModes.Add( BlueprintInterfaceMode, NSLOCTEXT("BlueprintEditor", "BlueprintInterfaceMode", "Interface") );
			LocModes.Add( BlueprintMacroMode, NSLOCTEXT("BlueprintEditor", "BlueprintMacroMode", "Macro") );
		}

		check( InMode != NAME_None );
		const FText* OutDesc = LocModes.Find( InMode );
		check( OutDesc );
		return *OutDesc;
	}
private:
	FBlueprintEditorApplicationModes() {}
};



class FBlueprintEditorApplicationMode : public FApplicationMode
{
public:
	UE_API FBlueprintEditorApplicationMode(TSharedPtr<class FBlueprintEditor> InBlueprintEditor, FName InModeName, FText(*GetLocalizedMode)(const FName), const bool bRegisterViewport = true, const bool bRegisterDefaultsTab = true);

	UE_API virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;
	UE_API virtual void PreDeactivateMode() override;
	UE_API virtual void PostActivateMode() override;
public:

protected:
	TWeakPtr<FBlueprintEditor> MyBlueprintEditor;

	// Set of spawnable tabs in blueprint editing mode
	FWorkflowAllowedTabSet BlueprintEditorTabFactories;

	// Set of spawnable tabs useful in derived classes, even without a blueprint
	FWorkflowAllowedTabSet CoreTabFactories;

	// Set of spawnable tabs only usable in blueprint editing mode (not useful in Persona, etc...)
	FWorkflowAllowedTabSet BlueprintEditorOnlyTabFactories;
};


class FBlueprintDefaultsApplicationMode : public FApplicationMode
{
public:
	UE_API FBlueprintDefaultsApplicationMode(TSharedPtr<class FBlueprintEditor> InBlueprintEditor);

	UE_API virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;
	UE_API virtual void PostActivateMode() override;
public:

protected:
	TWeakPtr<FBlueprintEditor> MyBlueprintEditor;

	// Set of spawnable tabs in Class Defaults mode
	FWorkflowAllowedTabSet BlueprintDefaultsTabFactories;
};


class FBlueprintComponentsApplicationMode : public FApplicationMode
{
public:
	UE_API FBlueprintComponentsApplicationMode(TSharedPtr<class FBlueprintEditor> InBlueprintEditor);

	UE_API virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;
	UE_API virtual void PreDeactivateMode() override;
	UE_API virtual void PostActivateMode() override;

	TSharedPtr<FBlueprintEditor> GetBlueprintEditor() const { return MyBlueprintEditor.Pin(); }
	UE_API AActor* GetPreviewActor() const;
protected:
	TWeakPtr<FBlueprintEditor> MyBlueprintEditor;

	// Set of spawnable tabs in this mode
	FWorkflowAllowedTabSet BlueprintComponentsTabFactories;

	TArray<TWeakObjectPtr<const UActorComponent>> CachedComponentSelection;
};

class FBlueprintInterfaceApplicationMode : public FApplicationMode
{
public:
	UE_API FBlueprintInterfaceApplicationMode(TSharedPtr<class FBlueprintEditor> InBlueprintEditor, FName InModeName = FBlueprintEditorApplicationModes::BlueprintInterfaceMode, FText(*GetLocalizedMode)(const FName) = FBlueprintEditorApplicationModes::GetLocalizedMode);

	UE_API virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;
	UE_API virtual void PreDeactivateMode() override;
	UE_API virtual void PostActivateMode() override;
public:

protected:
	TWeakPtr<FBlueprintEditor> MyBlueprintEditor;

	// Set of spawnable tabs in this mode
	FWorkflowAllowedTabSet BlueprintInterfaceTabFactories;
};

class FBlueprintMacroApplicationMode : public FApplicationMode
{
public:
	UE_API FBlueprintMacroApplicationMode(TSharedPtr<class FBlueprintEditor> InBlueprintEditor);

	UE_API virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;
	UE_API virtual void PreDeactivateMode() override;
	UE_API virtual void PostActivateMode() override;
public:

protected:
	TWeakPtr<FBlueprintEditor> MyBlueprintEditor;

	// Set of spawnable tabs in this mode
	FWorkflowAllowedTabSet BlueprintMacroTabFactories;
};

class FBlueprintEditorUnifiedMode : public FApplicationMode
{
public:
	UE_API FBlueprintEditorUnifiedMode(TSharedPtr<class FBlueprintEditor> InBlueprintEditor, FName InModeName, FText(*GetLocalizedMode)( const FName ), const bool bRegisterViewport = true);

	UE_API virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;
	UE_API virtual void PreDeactivateMode() override;
	UE_API virtual void PostActivateMode() override;
public:

protected:
	TWeakPtr<FBlueprintEditor> MyBlueprintEditor;

	// Set of spawnable tabs in blueprint editing mode
	FWorkflowAllowedTabSet BlueprintEditorTabFactories;

	// Set of spawnable tabs useful in derived classes, even without a blueprint
	FWorkflowAllowedTabSet CoreTabFactories;

	// Set of spawnable tabs only usable in blueprint editing mode (not useful in Persona, etc...)
	FWorkflowAllowedTabSet BlueprintEditorOnlyTabFactories;
};

#undef UE_API
