// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprint/BlueprintSectionID.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Engine/Blueprint.h"
#include "Framework/Commands/UICommandList.h"
#include "K2Node_EditablePinBase.h"
#include "Math/Vector2D.h"
#include "Misc/Optional.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "Templates/UniquePtr.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Toolkits/IToolkit.h"
#include "UObject/NameTypes.h"
#include "UObject/SoftObjectPath.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"

#define UE_API KISMET_API

class FFBlueprintEditorSCSEditorUICustomization;
class ISCSEditorUICustomization;
class FBlueprintEditor;
class FExtender;
class FFieldClass;
class FKismetCompilerContext;
class FLayoutExtender;
class FSCSEditorTreeNode;
class FSubobjectEditorTreeNode;
class FUICommandList;
class FWorkflowAllowedTabSet;
class IBlueprintEditor;
class IDetailCustomization;
class IToolkitHost;
class SWidget;
class UEdGraphNode;
class UEdGraphPin;
class UEdGraphSchema;
class UK2Node_EditablePinBase;
class UObject;
class UUserDefinedEnum;
class UUserDefinedEnum;
class UUserDefinedStruct;
class UUserDefinedStruct;
struct FBlueprintDebugger;
struct Rect;

namespace UE::BlueprintEditor
{

/**
 * Describes a Blueprint category that should always be visible, even when it contains no entries.
 * Registered programmatically via FBlueprintEditorModule.
 */
struct FAdditionalCategory
{
	/** Display name of the category. */
	FText Category;

	/**
	 * Only apply this category for blueprints whose ParentClass is a child of one of these classes.
	 * If empty, applies to all blueprints.
	 */
	TArray<FSoftClassPath> ClassFilters;

	struct FPlaceholder
	{
		/** Display text of the placeholder entry. */
		FText MenuDescription;

		/** Tooltip of the placeholder entry. */
		FText Tooltip;

		/** Section of the placeholder entry. */
		NodeSectionID::Type SectionID = NodeSectionID::NONE;
	};

	/** Show placeholder row when the category is empty. */
	TArray<FPlaceholder> Placeholders;
};

} // namespace UE::BlueprintEditor

/** Delegate used to customize variable display */
DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<IDetailCustomization>, FOnGetVariableCustomizationInstance, TSharedPtr<IBlueprintEditor> /*BlueprintEditor*/);

/** Delegate used to customize local variable display */
DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<IDetailCustomization>, FOnGetLocalVariableCustomizationInstance, TSharedPtr<IBlueprintEditor> /*BlueprintEditor*/);

/** Delegate used to customize function display */
DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<IDetailCustomization>, FOnGetFunctionCustomizationInstance, TSharedPtr<IBlueprintEditor> /*BlueprintEditor*/);

/** Delegate used to customize graph display */
DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<IDetailCustomization>, FOnGetGraphCustomizationInstance, TSharedPtr<IBlueprintEditor> /*BlueprintEditor*/);

/** Describes the reason for Refreshing the editor */
namespace ERefreshBlueprintEditorReason
{
	enum Type
	{
		BlueprintCompiled,
		PostUndo,
		UnknownReason
	};
}

/**
 * Enum editor public interface
 */
class IUserDefinedEnumEditor : public FAssetEditorToolkit
{
};

/**
 * Enum editor public interface
 */
class IUserDefinedStructureEditor : public FAssetEditorToolkit
{
};

/**
 * Blueprint editor public interface
 */
class IBlueprintEditor : public FWorkflowCentricApplication
{
public:
	virtual void JumpToHyperlink(const UObject* ObjectReference, bool bRequestRename) = 0;
	virtual void JumpToPin(const UEdGraphPin* PinToFocusOn) = 0;

	/** Invokes the search UI and sets the mode and search terms optionally */
	virtual void SummonSearchUI(bool bSetFindWithinBlueprint, FString NewSearchTerms = FString(), bool bSelectFirstResult = false) = 0;

	/** Invokes the Find and Replace UI */
	virtual void SummonFindAndReplaceUI() = 0;

	/** Returns the currently focused graph context, or NULL if no graph is available. */
	virtual UEdGraph* GetFocusedGraph() const = 0;

	/** Tries to open the specified graph and bring it's document to the front (note: this can return NULL) */
	virtual TSharedPtr<class SGraphEditor> OpenGraphAndBringToFront(class UEdGraph* Graph, bool bSetFocus = true) = 0;

	virtual void RefreshEditors(ERefreshBlueprintEditorReason::Type Reason = ERefreshBlueprintEditorReason::UnknownReason) = 0;

	virtual void RefreshMyBlueprint() = 0;

	virtual void RefreshInspector() = 0;

	virtual void AddToSelection(UEdGraphNode* InNode) = 0;

	virtual bool CanPasteNodes() const= 0;

	UE_DEPRECATED(5.6, "Slate positions are represented in FVector2f. Please update to use the function acceping FVector2f.")
	UE_API virtual void PasteNodesHere(class UEdGraph* Graph, const FVector2D& Location) UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION { PasteNodesHere(Graph, UE::Slate::CastToVector2f(Location)); };
	virtual void PasteNodesHere(class UEdGraph* Graph, const FVector2f& Location) = 0;

	virtual bool GetBoundsForSelectedNodes(class FSlateRect& Rect, float Padding ) = 0;

	/** Util to get the currently selected Subobject editor tree Nodes */
	virtual TArray<TSharedPtr<FSubobjectEditorTreeNode>> GetSelectedSubobjectEditorTreeNodes() const = 0;

	/** Get number of currently selected nodes in the SCS editor tree */
	virtual int32 GetNumberOfSelectedNodes() const = 0;

	/** Find and select a specific SCS editor tree node associated with the given component */
	virtual TSharedPtr<FSubobjectEditorTreeNode> FindAndSelectSubobjectEditorTreeNode(const class UActorComponent* InComponent, bool IsCntrlDown) = 0;

	/** Used to track node create/delete events for Analytics */
	virtual void AnalyticsTrackNodeEvent( UBlueprint* Blueprint, UEdGraphNode *GraphNode, bool bNodeDelete = false ) const = 0;

	/** Return the class viewer filter associated with the current set of imported namespaces within this editor context. Default is NULL (no filter). */
	virtual TSharedPtr<class IClassViewerFilter> GetImportedClassViewerFilter() const { return nullptr; }

	/** Get all the the pin type selector filters within this editor context. */
	virtual void GetPinTypeSelectorFilters(TArray<TSharedPtr<class IPinTypeSelectorFilter>>& OutFilters) const {}
	
	/** Return whether the given object falls outside the scope of the current set of imported namespaces within this editor context. Default is FALSE (imported). */
	virtual bool IsNonImportedObject(const UObject* InObject) const { return false; }

	/** Return whether the given object (referenced by path) falls outside the scope of the current set of imported namespaces within this editor context. Default is FALSE (imported). */
	virtual bool IsNonImportedObject(const FSoftObjectPath& InObject) const { return false; }
};

DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<class ISCSEditorCustomization>, FSCSEditorCustomizationBuilder, TSharedRef< IBlueprintEditor > /* InBlueprintEditor */);

/**
 * The blueprint editor module provides the blueprint editor application.
 */
class FBlueprintEditorModule : public IModuleInterface,
	public IHasMenuExtensibility
{
public:
	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// End of IModuleInterface interface

	/**
	 * Creates an instance of a Kismet editor object.  Only virtual so that it can be called across the DLL boundary.
	 *
	 * Note: This function should not be called directly, use one of the following instead:
	 *	- FKismetEditorUtilities::BringKismetToFocusAttentionOnObject
	 *  - GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset
	 *
	 * @param	Mode					Mode that this editor should operate in
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	Blueprint				The blueprint object to start editing
	 * @param	bShouldOpenInDefaultsMode	If true, the editor will open in defaults editing mode
	 *
	 * @return	Interface to the new Blueprint editor
	 */
	virtual TSharedRef<IBlueprintEditor> CreateBlueprintEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UBlueprint* Blueprint, bool bShouldOpenInDefaultsMode = false);
	virtual TSharedRef<IBlueprintEditor> CreateBlueprintEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, const TArray< UBlueprint* >& BlueprintsToEdit, bool bShouldOpenInDefaultsMode = true);

	/** Get all blueprint editor instances */
	virtual TArray<TSharedRef<IBlueprintEditor>> GetBlueprintEditors() const;

	/**
	 * Creates an instance of a Enum editor object.
	 *
	 * @param	Mode					Mode that this editor should operate in
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	UDEnum					The user-defined Enum to start editing
	 *
	 * @return	Interface to the new Enum editor
	 */
	virtual TSharedRef<IUserDefinedEnumEditor> CreateUserDefinedEnumEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UUserDefinedEnum* UDEnum);

	/**
	 * Creates an instance of a Structure editor object.
	 *
	 * @param	Mode					Mode that this editor should operate in
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	UDEnum					The user-defined structure to start editing
	 *
	 * @return	Interface to the new Struct editor
	 */
	virtual TSharedRef<IUserDefinedStructureEditor> CreateUserDefinedStructEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UUserDefinedStruct* UDStruct);

	/** Gets the extensibility managers for outside entities to extend blueprint editor's menus and toolbars */
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override { return MenuExtensibilityManager; }

	/**  */
	DECLARE_EVENT_TwoParams(FBlueprintEditorModule, FBlueprintMenuExtensionEvent, TSharedPtr<FExtender>, UBlueprint*);
	FBlueprintMenuExtensionEvent& OnGatherBlueprintMenuExtensions() { return GatherBlueprintMenuExtensions; }

	DECLARE_EVENT_ThreeParams(IBlueprintEditor, FOnRegisterTabs, FWorkflowAllowedTabSet&, FName /** ModeName */, TSharedPtr<FBlueprintEditor>);
	FOnRegisterTabs& OnRegisterTabsForEditor() { return RegisterTabsForEditor; }

	DECLARE_EVENT_OneParam(IBlueprintEditor, FOnRegisterLayoutExtensions, FLayoutExtender&);
	FOnRegisterLayoutExtensions& OnRegisterLayoutExtensions() { return RegisterLayoutExtensions; }

	/** Sets customizations for the BP editor details panel. */
	virtual void SetDetailsCustomization(TSharedPtr<class FDetailsViewObjectFilter> InDetailsObjectFilter, TSharedPtr<class IDetailRootObjectCustomization> InDetailsRootCustomization);

	/** Sets SCS editor UI customization */
	UE_DEPRECATED(5.7, "Please use AddCustomization instead.")
	virtual void SetSubobjectEditorUICustomization(TSharedPtr<ISCSEditorUICustomization> InSCSEditorUICustomization);

	/** Adds the SCS editor UI customization to the array of customizations to be invoked */
	virtual void AddCustomization(TSharedPtr<ISCSEditorUICustomization> InSCSEditorUICustomization);

	/** Removes the SCS editor UI customization from the array of customizations to be invoked */
	virtual void RemoveCustomization(TSharedPtr<ISCSEditorUICustomization> InSCSEditorUICustomization);

	/**
	 * Register a customization for interacting with the SCS editor 
	 * @param	InComponentName			The name of the component to customize behavior for
	 * @param	InCustomizationBuilder	The delegate used to create customization instances
	 */
	virtual void RegisterSCSEditorCustomization(const FName& InComponentName, FSCSEditorCustomizationBuilder InCustomizationBuilder);

	/** 
	 * Unregister a previously registered customization for interacting with the SCS editor 
	 * @param	InComponentName			The name of the component to customize behavior for
	 */
	virtual void UnregisterSCSEditorCustomization(const FName& InComponentName);

	/** 
	 * Register a customization for for Blueprint variables
	 * @param	InStruct				The type of the variable to create the customization for
	 * @param	InOnGetDetailCustomization	The delegate used to create customization instances
	 */
	virtual FDelegateHandle RegisterVariableCustomization(FFieldClass* InFieldClass, FOnGetVariableCustomizationInstance InOnGetVariableCustomization);

	/**
	 * Unregister a previously registered customization for BP variables
	 * @param	InStruct				The type to create the customization for
	 * @param	InHandle				The handle returned by RegisterVariableCustomization
	 */
	virtual void UnregisterVariableCustomization(FFieldClass* InFieldClass, FDelegateHandle InHandle);

	/** 
	 * Register a customization for for Blueprint local variables
	 * @param	InFieldClass				The type of the variable to create the customization for
	 * @param	InOnGetLocalVariableCustomization	The delegate used to create customization instances
	 */
	virtual FDelegateHandle RegisterLocalVariableCustomization(FFieldClass* InFieldClass, FOnGetLocalVariableCustomizationInstance InOnGetLocalVariableCustomization);

	/** 
	 * Unregister a previously registered customization for BP local variables
	 * @param	InFieldClass				The type to create the customization for
	 * @param	InHandle				The handle returned by RegisterLocalVariableCustomization
	 */
	virtual void UnregisterLocalVariableCustomization(FFieldClass* InFieldClass, FDelegateHandle InHandle);

	/** 
	 * Register a customization for for Blueprint graphs
	 * @param	InGraphSchema				The schema of the graph to create the customization for
	 * @param	InOnGetDetailCustomization	The delegate used to create customization instances
	 */
	virtual void RegisterGraphCustomization(const UEdGraphSchema* InGraphSchema, FOnGetGraphCustomizationInstance InOnGetGraphCustomization);

	/** 
	 * Unregister a previously registered customization for BP graphs
	 * @param	InGraphSchema				The schema of the graph to create the customization for
	 */
	virtual void UnregisterGraphCustomization(const UEdGraphSchema* InGraphSchema);

	/**
	 * Register a customization for for Blueprint functions
	 * @param	InStruct				The type of the pin to create the customization for
	 * @param	InOnGetFunctionCustomization	The delegate used to create customization instances
	 */
	virtual FDelegateHandle RegisterFunctionCustomization(TSubclassOf<UK2Node_EditablePinBase> InFieldClass, FOnGetFunctionCustomizationInstance InOnGetFunctionCustomization);

	/**
	 * Unregister a previously registered customization for BP functions
	 * @param	InStruct				The type to create the customization for
	 * @param	InHandle				The handle returned by UnregisterFunctionCustomization
	 */
	virtual void UnregisterFunctionCustomization(TSubclassOf<UK2Node_EditablePinBase> InFieldClass, FDelegateHandle InHandle);


	/** 
	 * Build a set of details customizations for the passed-in type, if possible.
	 * @param	InStruct				The type to create the customization for
	 * @param	InBlueprintEditor		The Blueprint Editor the customization will be created for
	 */
	virtual TArray<TSharedPtr<IDetailCustomization>> CustomizeVariable(FFieldClass* InFieldClass, TSharedPtr<IBlueprintEditor> InBlueprintEditor);

	/** 
	 * Build a set of details customizations for graphs with the passed-in schema, if possible.
	 * @param	InGraphSchema			The schema to create the customization for
	 * @param	InBlueprintEditor		The Blueprint Editor the customization will be created for
	 */
	virtual TArray<TSharedPtr<IDetailCustomization>> CustomizeGraph(const UEdGraphSchema* InGraphSchema, TSharedPtr<IBlueprintEditor> InBlueprintEditor);

	/** 
	 * Build a set of details customizations for function with the passed-in type, if possible.
	 * @param	InFunctionClass		The type to create the customization for
	 * @param	InBlueprintEditor		The Blueprint Editor the customization will be created for
	 */
	virtual TArray<TSharedPtr<IDetailCustomization>> CustomizeFunction(TSubclassOf<UK2Node_EditablePinBase> InFunctionClass, TSharedPtr<IBlueprintEditor> InBlueprintEditor);

	/** Delegate for binding functions to be called when the blueprint editor finishes getting created */
	DECLARE_EVENT_OneParam( FBlueprintEditorModule, FBlueprintEditorOpenedEvent, EBlueprintType );
	FBlueprintEditorOpenedEvent& OnBlueprintEditorOpened() { return BlueprintEditorOpened; }

	/** 
	 * Exposes a way for other modules to fold in their own Blueprint editor 
	 * commands (folded in with other BP editor commands, when the editor is 
	 * first opened).
	 */
	virtual const TSharedRef<FUICommandList> GetsSharedBlueprintEditorCommands() const { return SharedBlueprintEditorCommands.ToSharedRef(); }

	/** Returns a reference to the Blueprint Debugger state object */
	const TUniquePtr<FBlueprintDebugger>& GetBlueprintDebugger() const { return BlueprintDebugger; }

	/** Opaque handle returned by RegisterAdditionalBlueprintCategory. Pass to Unregister to remove. */
	struct FAdditionalCategoryHandle
	{
		FAdditionalCategoryHandle() = default;

		bool IsValid() const
		{
			return Value != InvalidValue;
		}

		bool operator==(const FAdditionalCategoryHandle& Other) const
		{
			return Other.Value == Value;
		}

	private:
		friend class FBlueprintEditorModule;

		static constexpr uint32 InvalidValue = MAX_uint32;
		explicit FAdditionalCategoryHandle(uint32 InValue)
			: Value(InValue)
		{}

		uint32 Value = InvalidValue;
	};

	/** Register an additional category. Returns a handle for unregistration. */
	UE_API FAdditionalCategoryHandle RegisterAdditionalBlueprintCategory(UE::BlueprintEditor::FAdditionalCategory InCategory);

	/** Unregister a previously registered category by handle. */
	UE_API void UnregisterAdditionalBlueprintCategory(FAdditionalCategoryHandle Handle);

	/** Returns all additional categories. */
	UE_API TArray<UE::BlueprintEditor::FAdditionalCategory> GetAdditionalBlueprintCategories() const;

	/** Returns the additional categories for the Blueprint. */
	UE_API TArray<UE::BlueprintEditor::FAdditionalCategory> GetAdditionalBlueprintCategories(TNotNull<const UBlueprint*> Blueprint) const;

private:
	/** Loads from ini a list of all events that should be auto created for Blueprints of a specific class */
	void PrepareAutoGeneratedDefaultEvents();

private:
	/** List of all blueprint editors that were created. */
	TArray<TWeakPtr<FBlueprintEditor>> BlueprintEditors;

	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;

	//
	FBlueprintMenuExtensionEvent GatherBlueprintMenuExtensions;

	/** Event called to allow external clients to register additional tabs for the specified editor */
	FOnRegisterTabs RegisterTabsForEditor;
	FOnRegisterLayoutExtensions RegisterLayoutExtensions;

	// Event to be called when the blueprint editor is opened
	FBlueprintEditorOpenedEvent BlueprintEditorOpened;

	/** Customizations for the SCS editor */
	TMap<FName, FSCSEditorCustomizationBuilder> SCSEditorCustomizations;

	/** Customizations for Blueprint variables */
	TMultiMap<FFieldClass*, FOnGetVariableCustomizationInstance> VariableCustomizations;

	/** Customizations for Blueprint local variables */
	TMultiMap<FFieldClass*, FOnGetLocalVariableCustomizationInstance> LocalVariableCustomizations;

	/** Customizations for Blueprint graphs */
	TMap<const UEdGraphSchema*, FOnGetGraphCustomizationInstance> GraphCustomizations;

	/** Customizations for Blueprint functions */
	TMultiMap<TSubclassOf<UK2Node_EditablePinBase>, FOnGetFunctionCustomizationInstance> FunctionCustomizations;

	/** Root customization for the BP editor details panel. */
	TSharedPtr<class IDetailRootObjectCustomization> DetailsRootCustomization;

	/** Filter used to determine the set of objects shown in the BP editor details panel. */
	TSharedPtr<class FDetailsViewObjectFilter> DetailsObjectFilter;

	/** UI customizations for the SCS editor inside the blueprint editor */
	TSharedPtr<FFBlueprintEditorSCSEditorUICustomization> SCSEditorUICustomization;

	/** 
	 * A command list that can be passed around and isn't bound to an instance 
	 * of the blueprint editor. 
	 */
	TSharedPtr<FUICommandList> SharedBlueprintEditorCommands;

	/** Handle to a registered LevelViewportContextMenuBlueprintExtender delegate */
	FDelegateHandle LevelViewportContextMenuBlueprintExtenderDelegateHandle;

	/** Reference to keep our custom configuration panel alive */
	TSharedPtr<SWidget> ConfigurationPanel;

	/** Blueprint debugger state - refactor into SBlueprintDebugger if needed */
	TUniquePtr<FBlueprintDebugger> BlueprintDebugger;

	/** Programmatically registered additional Blueprint categories. */
	TArray<TTuple<FAdditionalCategoryHandle, UE::BlueprintEditor::FAdditionalCategory>> RegisteredAdditionalCategories;
	uint32 NextAdditionalCategoryHandle = 0;
};

#undef UE_API
