// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"

#define UE_API KISMET_API

#define LOCTEXT_NAMESPACE "BlueprintEditor"

//////////////////////////////////////////////////////////////////////////
// FCompilerResultsSummoner

struct FCompilerResultsSummoner : public FWorkflowTabFactory
{
public:
	UE_API FCompilerResultsSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp);

	UE_API virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override
	{
		return LOCTEXT("CompilerResultsTooltip", "The compiler results tab shows any errors or warnings generated when compiling this Blueprint.");
	}
};

#undef LOCTEXT_NAMESPACE

#undef UE_API
