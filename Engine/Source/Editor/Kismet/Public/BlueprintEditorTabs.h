// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API KISMET_API

class FName;

struct FBlueprintEditorTabs
{
	// Tab identifiers
	static UE_API const FName DetailsID;
	static UE_API const FName DefaultEditorID;
	static UE_API const FName DebugID;
	static UE_API const FName PaletteID;
	static UE_API const FName BookmarksID;
	static UE_API const FName CompilerResultsID;
	static UE_API const FName FindResultsID;
	static UE_API const FName ConstructionScriptEditorID;
	static UE_API const FName SCSViewportID;
	static UE_API const FName MyBlueprintID;
	static UE_API const FName ReplaceNodeReferencesID;
	static UE_API const FName UserDefinedStructureID;
	static UE_API const FName BlueprintDebuggerID;

	// Document tab identifiers
	static UE_API const FName GraphEditorID;
	static UE_API const FName TimelineEditorID;

private:
	FBlueprintEditorTabs() {}
};

#undef UE_API
