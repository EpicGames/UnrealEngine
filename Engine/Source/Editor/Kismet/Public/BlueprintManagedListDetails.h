// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "IDetailCustomNodeBuilder.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API KISMET_API

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class SWidget;

/** Blueprint managed list details */
class FBlueprintManagedListDetails : public IDetailCustomNodeBuilder
{
protected:
	/** List item node type */
	struct FManagedListItem
	{
		FString ItemName;
		FText DisplayName;
		TWeakObjectPtr<> AssetPtr;
		uint8 bIsRemovable : 1;
		TArray<TSharedPtr<IPropertyHandle>> PropertyHandles;

		FManagedListItem()
			:bIsRemovable(false)
		{
		}
	};

	/** Customizable display options */
	struct FDisplayOptions
	{
		FText TitleText;
		FText TitleTooltipText;
		FText NoItemsLabelText;
		FText BrowseButtonToolTipText;
		FText RemoveButtonToolTipText;
		TAttribute<bool> EditCondition;
	};

	/** Mutable display options */
	FDisplayOptions DisplayOptions;

	/** IDetailCustomNodeBuilder interface */
	UE_API virtual void GenerateHeaderRowContent(FDetailWidgetRow& HeaderRow) override;
	UE_API virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;
	virtual FName GetName() const override { return *DisplayOptions.TitleText.ToString(); }
	virtual bool InitiallyCollapsed() const override { return false; }
	virtual void Tick(float DeltaTime) override {}
	virtual bool RequiresTick() const override { return false; }
	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRebuildChildren) override { RegenerateChildrenDelegate = InOnRebuildChildren; }
	/** END IDetailCustomNodeBuilder interface */

	/** Overridable interface methods */
	virtual TSharedPtr<SWidget> MakeAddItemWidget() { return nullptr; }
	virtual void GetManagedListItems(TArray<FManagedListItem>& OutListItems) const {}
	virtual void OnRemoveItem(const FManagedListItem& Item) {}

	/** Helper function to regenerate the details customization */
	UE_API void RegenerateChildContent();

private:
	/** A delegate to regenerate this list of children */
	FSimpleDelegate RegenerateChildrenDelegate;
};

#undef UE_API
