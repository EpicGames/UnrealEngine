// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"

#define UE_API KISMET_API

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyHandle;
class SMyBlueprint;

// Property type customization for FMemberReference
class FBlueprintMemberReferenceDetails : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FBlueprintMemberReferenceDetails(TWeakPtr<SMyBlueprint>()));
	}
	
	static TSharedRef<IPropertyTypeCustomization> MakeInstance(TWeakPtr<SMyBlueprint> InMyBlueprint)
	{
		return MakeShareable(new FBlueprintMemberReferenceDetails(InMyBlueprint));
	}

private:
	FBlueprintMemberReferenceDetails(TWeakPtr<SMyBlueprint> InMyBlueprint)
		: MyBlueprint(InMyBlueprint)
	{
	}
	
	// IPropertyTypeCustomization interface
	UE_API virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InStructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InStructCustomizationUtils) override {};

private:
	TWeakPtr<SMyBlueprint> MyBlueprint;
};

#undef UE_API
