// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/SoftObjectPath.h"

#define UE_API KISMET_API

class IClassViewerFilter;
class IPinTypeSelectorFilter;
class UBlueprint;
class UObject;
struct FAssetData;
struct FBlueprintNamespacePathTree;

/**
 * A helper class for managing namespace info for a single Blueprint context.
 */
class FBlueprintNamespaceHelper
{
public:
	/** Default constructor. */
	UE_API FBlueprintNamespaceHelper();

	/** Note: We explicitly declare/implement the dtor so that forward-declared types can be destroyed. */
	UE_API virtual ~FBlueprintNamespaceHelper();

	/**
	 * Add a new Blueprint into the current helper context.
	 * 
	 * @param InBlueprint	A Blueprint object being edited.
	 */
	UE_API void AddBlueprint(const UBlueprint* InBlueprint);

	/**
	 * Add a namespace identifier into the Blueprint editor context that's managed by this instance.
	 *
	 * @param Namespace		The namespace identifier string to add. Should resemble "X.Y.Z" as the format.
	 */
	UE_API void AddNamespace(const FString& Namespace);

	/**
	 * Add multiple namespace identifiers into the Blueprint editor context that's managed by this instance.
	 *
	 * @param NamespaceList	A container type containing one or more namespace identifier strings. Each entry should resemble "X.Y.Z" as the format.
	 */
	template<typename ContainerType>
	void AddNamespaces(const ContainerType& NamespaceList)
	{
		for (const FString& Namespace : NamespaceList)
		{
			AddNamespace(Namespace);
		}
	}

	/**
	 * Remove a namespace identifier from the Blueprint editor context that's managed by this instance.
	 * 
	 * @param Namespace		The namespace identifier string to remove. Should resemble "X.Y.Z" as the format.
	 */
	UE_API void RemoveNamespace(const FString& Namespace);

	/**
	 * Tests a namespace identifier to see if it's been imported. 
	 *
	 * @param TestNamespace	A namespace identifier string. Should resemble "X.Y.Z" as the format.
	 * @return TRUE if the namespace identifier exists within the Blueprint editor context that's managed by this instance.
	 */
	UE_API bool IsIncludedInNamespaceList(const FString& TestNamespace) const;

	/**
	 * Tests an asset's namespace to see if it's been imported.
	 *
	 * @param InAssetData	Asset data context.
	 * @return TRUE if the asset's underlying namespace was imported into the Blueprint editor context that's managed by this instance.
	 */
	UE_API bool IsImportedAsset(const FAssetData& InAssetData) const;

	/**
	 * Tests an object's namespace to see if it's been imported.
	 *
	 * @param InObject	Hard reference to a (loaded) object.
	 * @return TRUE if the object's underlying namespace was imported into the Blueprint editor context that's managed by this instance.
	 */
	UE_API bool IsImportedObject(const UObject* InObject) const;

	/**
	 * Tests an object's namespace to see if it's been imported.
	 *
	 * @param InObjectPath	Soft reference to an object. May be loaded or unloaded.
	 * @return TRUE if the object's underlying namespace was imported into the Blueprint editor context that's managed by this instance.
	 */
	UE_API bool IsImportedObject(const FSoftObjectPath& InObjectPath) const;

	/**
	 * @return A class viewer filter that can be used with class picker widgets to filter out class types whose namespaces were not imported into the Blueprint editor context that's managed by this instance.
	 */
	TSharedPtr<IClassViewerFilter> GetClassViewerFilter() const
	{
		return ClassViewerFilter;
	}

	/**
	 * @return A pin type selection filter that can be used with pin type selector widgets to filter out pin types whose namespaces were not imported into the Blueprint editor context that's managed by this instance.
	 */
	TSharedPtr<IPinTypeSelectorFilter> GetPinTypeSelectorFilter() const
	{
		return PinTypeSelectorFilter;
	}

	/**
	 * Utility method used to keep console flags in sync with the current Blueprint editor settings environment.
	 */
	static UE_API void RefreshEditorFeatureConsoleFlags();

private:
	// Path tree that stores imported namespace identifiers for the associated Blueprint.
	TUniquePtr<FBlueprintNamespacePathTree> NamespacePathTree;

	// For use with the class viewer widget in order to filter class type items by namespace.
	TSharedPtr<IClassViewerFilter> ClassViewerFilter;

	// For use with the pin type selector widget in order to filter pin type items by namespace.
	TSharedPtr<IPinTypeSelectorFilter> PinTypeSelectorFilter;
};

#undef UE_API
