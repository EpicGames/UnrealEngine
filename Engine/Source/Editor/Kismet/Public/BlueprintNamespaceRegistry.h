// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/IDelegateInstance.h"
#include "Templates/UniquePtr.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectGlobals.h"

#define UE_API KISMET_API

class FName;
class UObject;
struct FAssetData;
struct FBlueprintNamespacePathTree;

/**
 * A shared utility class that keeps track of registered Blueprint namespace identifiers sourced from objects and assets in the editor.
 */
class FBlueprintNamespaceRegistry
{
public:
	UE_API ~FBlueprintNamespaceRegistry();

	/**
	 * Provides public singleton access.
	 */
	static UE_API FBlueprintNamespaceRegistry& Get();

	/**
	 * One-time initialization method; separated from the ctor so it can be called explicitly.
	 */
	UE_API void Initialize();

	/**
	 * One-time shutdown method; separated from the dtor so it can be called explicitly.
	 */
	UE_API void Shutdown();

	/**
	 * @return TRUE if the given path identifier is currently registered.
	 */
	UE_API bool IsRegisteredPath(const FString& InPath) const;
	
	/**
	 * @return TRUE if the given path identifier is inclusive of any registered paths.
	 * 
	 * Example: If "MyProject.MyNamespace" is a registered path, then both "MyProject" and "MyProject.MyNamespace" are inclusive paths.
	 * 
	 * Also note if a registered path is removed, inclusive paths may still be valid. For instance, if both "MyProject.MyNamespace" and
	 * "MyProject.MyNamespace_2" are registered paths, and "MyProject.MyNamespace_2" is removed, "MyProject" is still an inclusive path.
	 */
	UE_API bool IsInclusivePath(const FString& InPath) const;

	/**
	 * @param InPath	Path identifier string (e.g. "X.Y" or "X.Y.").
	 * @param OutNames	On output, an array containing the set of names rooted to the given path (e.g. "Z" in "X.Y.Z").
	 */
	UE_API void GetNamesUnderPath(const FString& InPath, TArray<FName>& OutNames) const;

	/**
	 * @param OutPaths	On output, contains the full set of all currently-registered namespace identifier paths.
	 */
	UE_API void GetAllRegisteredPaths(TArray<FString>& OutPaths) const;

	/**
	 * Adds an explicit namespace identifier to the registry if not already included.
	 * 
	 * @param InPath	Path identifier string (e.g. "X.Y").
	 */
	UE_API void RegisterNamespace(const FString& InPath);

	/**
	 * Recreates the namespace registry.
	 */
	UE_API void Rebuild();

protected:
	UE_API FBlueprintNamespaceRegistry();

	/** Asset registry event handler methods. */
	UE_API void OnAssetAdded(const FAssetData& AssetData);
	UE_API void OnAssetRemoved(const FAssetData& AssetData);
	UE_API void OnAssetRenamed(const FAssetData& AssetData, const FString& InOldName);
	UE_API void OnAssetRegistryFilesLoaded();

	/** Namespace identifier registration methods. */
	UE_API void FindAndRegisterAllNamespaces();
	UE_API void RegisterNamespace(const UObject* InObject);
	UE_API void RegisterNamespace(const FAssetData& AssetData);

	/** Console command implementations (debugging/testing). */
	UE_API void ToggleDefaultNamespace();
	UE_API void DumpAllRegisteredPaths();
	UE_API void OnDefaultNamespaceTypeChanged();

	/** Handler for hot reload / live coding completion events. */
	UE_API void OnReloadComplete(EReloadCompleteReason InReason);

private:
	/** Indicates whether the registry has been initialized. */
	bool bIsInitialized;

	/** Delegate handles to allow for deregistration on shutdown. */
	FDelegateHandle OnAssetAddedDelegateHandle;
	FDelegateHandle OnAssetRemovedDelegateHandle;
	FDelegateHandle OnAssetRenamedDelegateHandle;
	FDelegateHandle OnFilesLoadedDelegateHandle;
	FDelegateHandle OnReloadCompleteDelegateHandle;
	FDelegateHandle OnDefaultNamespaceTypeChangedDelegateHandle;

	/** Handles storage and retrieval for namespace path identifiers. */
	TUniquePtr<FBlueprintNamespacePathTree> PathTree;

	/** Internal set of objects to exclude during namespace registration. */
	TSet<FSoftObjectPath> ExcludedObjectPaths;
};

#undef UE_API
