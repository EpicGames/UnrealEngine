// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Misc/CoreMiscDefines.h"
#include "UObject/SoftObjectPath.h"

#define UE_API KISMET_API

class FProperty;
class UBlueprint;
class UObject;
class UStruct;
struct FAssetData;

/** Default namespace type for objects/assets if one is not explicitly assigned. */
enum class EDefaultBlueprintNamespaceType
{
	/** All objects/assets belong to the global namespace by default. */
	DefaultToGlobalNamespace,

	/** All objects/assets base their default namespace on the underlying package path. */
	UsePackagePathAsDefaultNamespace,
};

/** A wrapper struct around various Blueprint namespace utility and support methods. */
struct FBlueprintNamespaceUtilities
{
public:
	/**
	 * Analyzes the given asset to determine its explicitly-assigned namespace identifier, or otherwise returns its default namespace.
	 * 
	 * @param InAssetData	Input asset data.
	 * @return The unique Blueprint namespace identifier associated with the given asset, or an empty string if the asset belongs to the global namespace (default).
	 */
	static UE_API FString GetAssetNamespace(const FAssetData& InAssetData);

	/**
	 * Analyzes the given object to determine its explicitly-assigned namespace identifier, or otherwise returns its default namespace.
	 * 
	 * @param InObject		A reference to the input object.
	 * @return The unique Blueprint namespace identifier associated with the given object, or an empty string if the object belongs to the global namespace (default).
	 */
	static UE_API FString GetObjectNamespace(const UObject* InObject);

	/**
	 * Analyzes the given object path to determine its explicitly-assigned namespace identifier, or otherwise returns its default namespace.
	 *
	 * @param InObjectPath	Path to the given object (may not be loaded yet).
	 * @return The unique Blueprint namespace identifier associated with the given object (even if unloaded), or an empty string if the object belongs to the global namespace (default).
	 */
	static UE_API FString GetObjectNamespace(const FSoftObjectPath& InObjectPath);

	/**
	 * Analyzes a property value to determine explicitly-assigned namespace identifiers from any object references, or otherwise returns the default namespace for each occurrence (default).
	 * 
	 * @param InProperty	The property for which we will analyze the value.
	 * @param InContainer	The source address of the struct/object containing the property's value.
	 * @param OutNamespaces	Zero or more unique namespace identifier(s) referenced by the property value. An entry with an empty string equates to the default global namespace.
	 */
	static UE_API void GetPropertyValueNamespaces(const FProperty* InProperty, const void* InContainer, TSet<FString>& OutNamespaces);

	/**
	 * Gathers the set of global namespaces that are implicitly imported by all Blueprint assets.
	 * 
	 * @param OutNamespaces	Zero or more unique namespace identifier(s) representing the shared global namespace set.
	 */
	static UE_API void GetSharedGlobalImports(TSet<FString>& OutNamespaces);

	/**
	 * Gathers the set of default namespaces that are implicitly imported by a given object's type.
	 *
	 * @param InObject		A reference to a loaded object or type.
	 * @param OutNamespaces	Zero or more unique namespace identifier(s) representing the type's default namespace set.
	 */
	static UE_API void GetDefaultImportsForObject(const UObject* InObject, TSet<FString>& OutNamespaces);

	/**
	 * Sets the default Blueprint namespace type that objects/assets should use when not explicitly assigned.
	 *
	 * @param InType		Default namespace type to use.
	 */
	static UE_API void SetDefaultBlueprintNamespaceType(EDefaultBlueprintNamespaceType InType);

	/** @return The default Blueprint namespace type objects/assets should use. Currently used for debugging/testing. */
	static UE_API EDefaultBlueprintNamespaceType GetDefaultBlueprintNamespaceType();

	/** Delegate invoked whenever the default Blueprint namespace type changes. */
	DECLARE_MULTICAST_DELEGATE(FOnDefaultBlueprintNamespaceTypeChanged);
	static UE_API FOnDefaultBlueprintNamespaceTypeChanged& OnDefaultBlueprintNamespaceTypeChanged();

	/**
	 * Refresh the Blueprint editor environment to align with current namespace editor feature settings.
	 */
	static UE_API void RefreshBlueprintEditorFeatures();

	/**
	 * Helper method to convert a package path to a Blueprint namespace identifier string.
	 *
	 * @param InPackagePath		Package path input string.
	 * @param OutNamespacePath	The input string converted to namespace identifier format.
	 */
	static UE_API void ConvertPackagePathToNamespacePath(const FString& InPackagePath, FString& OutNamespacePath);
};

#undef UE_API
