// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "UObject/NameTypes.h"
#include "Widgets/SCompoundWidget.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"

class FTabManager;
struct FFrame;

namespace CallStackViewer
{
	void KISMET_API UpdateDisplayedCallstack(TArrayView<const FFrame* const> ScriptStack);
	FName GetTabName();
	void RegisterTabSpawner(FTabManager& TabManager);
}
