// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Framework/Views/ITypedTableView.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateConstants.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

#define UE_API KISMET_API

class FBreakpointParentItem;
class FDebugLineItem;
class FName;
class FTraceStackParentItem;
class ITableRow;
class SCheckBox;
class SComboButton;
class SHeaderRow;
class SSearchBox;
class SWidget;
class UBlueprint;
class UBlueprintGeneratedClass;
class UEdGraphPin;
class UObject;
struct FGeometry;

//////////////////////////////////////////////////////////////////////////
// FDebugLineItem

// Shared pointer to a debugging tree line entry
typedef TSharedPtr<class FDebugLineItem> FDebugTreeItemPtr;

// The base class for a line entry in the debugging tree view
class FDebugLineItem : public TSharedFromThis<FDebugLineItem>
{
public:
	friend class FLineItemWithChildren; // used by FLineItemWithChildren::EnsureChildIsAdded
	enum EDebugLineType
	{
		DLT_Message,
		DLT_TraceStackParent,
		DLT_TraceStackChild,
		DLT_Parent,
		DLT_SelfWatch,
		DLT_Watch,
		DLT_WatchChild,
		DLT_LatentAction,
		DLT_Breakpoint,
		DLT_BreakpointParent
	};

	enum ESearchFlags
	{
		SF_None = 0,
		SF_RootNode = 1 << 0,
		SF_ContainerElement = 1 << 1
	};

	virtual ~FDebugLineItem() = default;

	// Create the widget for the name column
	UE_API virtual TSharedRef<SWidget> GenerateNameWidget(TSharedPtr<FString> InSearchString);

	// Create the widget for the value column
	UE_API virtual TSharedRef<SWidget> GenerateValueWidget(TSharedPtr<FString> InSearchString);

	// Add standard context menu items that can act on any node of the tree
	UE_API void MakeMenu(class FMenuBuilder& MenuBuilder, bool bInDebuggerTab);

	// Add context menu items that can act on this node of the tree
	UE_API virtual void ExtendContextMenu(class FMenuBuilder& MenuBuilder, bool bInDebuggerTab);

	// Gather all of the children
	virtual void GatherChildrenBase(TArray<FDebugTreeItemPtr>& OutChildren, const FString& InSearchString, bool bRespectSearch = true) {}

	// returns whether this tree node has children (used by drop down arrows)
	UE_API virtual bool HasChildren() const;

	// only line items inheriting from FLineItemWithChildren can have children
	virtual bool CanHaveChildren() { return false; }

	// @return The object that will act as a parent to more items in the tree, or nullptr if this is a leaf node
	virtual UObject* GetParentObject() const { return nullptr; }

	virtual EDebugLineType GetType() const
	{
		return Type;
	}

	// returns a widget that will go to the left of the Name Widget.
	UE_API virtual TSharedRef<SWidget> GetNameIcon();

	// returns a widget that will go to the left of the Value Widget.
	UE_API virtual TSharedRef<SWidget> GetValueIcon();

	// returns current value of Search String
	UE_API FText GetHighlightText(const TSharedPtr<FString> InSearchString) const;

	// Helper function to try to get the blueprint for a given object;
	//   Returns the blueprint that was used to create the instance if there was one
	//   Returns the object itself if it is already a blueprint
	//   Otherwise returns NULL
	static UE_API UBlueprint* GetBlueprintForObject(UObject* ParentObject);

	static UE_API UBlueprintGeneratedClass* GetClassForObject(UObject* ParentObject);

	static UE_API bool IsDebugLineTypeActive(EDebugLineType Type);
	static UE_API void OnDebugLineTypeActiveChanged(ECheckBoxState CheckState, EDebugLineType Type);

	static UE_API void SetBreakpointParentItemBlueprint(FDebugTreeItemPtr InBreakpointParentItem, TWeakObjectPtr<UBlueprint> InBlueprint);

	// updates bVisible and bParentsMatchSearch based on this node alone
	UE_API void UpdateSearch(const FString& InSearchString, ESearchFlags SearchFlags);

	UE_API bool IsVisible();
	UE_API bool DoParentsMatchSearch();

	// @return The text to display in the value column, unless GenerateValueWidget is overridden
	UE_API virtual FText GetDescription() const;
protected:

	// Cannot create an instance of this class, it's just for use as a base class
	FDebugLineItem(EDebugLineType InType)
		: Type(InType)
	{
	}

	// Duplicate this item
	virtual FDebugLineItem* Duplicate() const = 0;

	// Compare this item to another of the same type
	virtual bool Compare(const FDebugLineItem* Other) const = 0;

	// used for sets
	virtual uint32 GetHash() const = 0;

	// Used to update the state of a line item rather than replace it.
	// called after Compare returns true
	[[maybe_unused]] virtual void UpdateData(const FDebugLineItem& NewerData) {}

	// @return The actual underlying name of the item, for search compatibility
	UE_API virtual FText GetName() const;

	// @return The text to display in the name column, unless GenerateNameWidget is overridden
	UE_API virtual FText GetDisplayName() const;

	UE_API bool HasName() const;
	UE_API bool HasValue() const;
	UE_API void CopyNameToClipboard() const;
	UE_API void CopyValueToClipboard() const;

protected:
	// Type of action (a kind of RTTI for the tree, really only used to accelerate Compare checks)
	EDebugLineType Type;

	static UE_API uint16 ActiveTypeBitset;

	// true if self or any recursive children match the search
	bool bVisible = false;
	// true if self or any recursive parents match the search
	bool bParentsMatchSearch = false;
};

//////////////////////////////////////////////////////////////////////////
// SKismetDebugTreeView

class SKismetDebugTreeView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SKismetDebugTreeView)
		: _SelectionMode(ESelectionMode::Single)
		, _OnExpansionChanged()
		, _InDebuggerTab(false)
	{}

	SLATE_ATTRIBUTE(ESelectionMode::Type, SelectionMode)
	SLATE_EVENT(STreeView<FDebugTreeItemPtr>::FOnExpansionChanged, OnExpansionChanged)
	SLATE_ARGUMENT(bool, InDebuggerTab)
	SLATE_ARGUMENT(TSharedPtr<SHeaderRow>, HeaderRow)

		SLATE_END_ARGS()
public:
	UE_API void Construct(const FArguments& InArgs);

	// SWidget interface
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	// End of SWidget interface

	/** Adds a root level TreeItem */
	UE_API void AddTreeItemUnique(const FDebugTreeItemPtr& Item);

	/** Removes a root level TreeItem */
	UE_API bool RemoveTreeItem(const FDebugTreeItemPtr& Item);

	/** Clears all root level Tree Items */
	UE_API void ClearTreeItems();

	/** Updates search text and requests the Tree be updated */
	UE_API void SetSearchText(const FText& InSearchText);

	/** Requests for an update of the tree at the next tick */
	UE_API void RequestUpdateFilteredItems();

	/** Returns the Array of Root Tree Items */
	UE_API const TArray<FDebugTreeItemPtr>& GetRootTreeItems() const;

	// Passthrough functions to the TreeView
	UE_API int32 GetSelectedItems(TArray<FDebugTreeItemPtr>& OutItems);
	UE_API void ClearExpandedItems();
	UE_API bool IsScrolling() const;
	UE_API void SetItemExpansion(FDebugTreeItemPtr InItem, bool bInShouldExpandItem);
private:
	/** Updates the Filtered Items based on current Root Tree Items */
	UE_API void UpdateFilteredItems();

	UE_API TSharedRef<ITableRow> OnGenerateRow(FDebugTreeItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable);
	UE_API void OnGetChildren(FDebugTreeItemPtr InParent, TArray<FDebugTreeItemPtr>& OutChildren);
	UE_API TSharedPtr<SWidget> OnMakeContextMenu() const;

	TSharedPtr< STreeView< FDebugTreeItemPtr > > TreeView;
	TArray<FDebugTreeItemPtr> FilteredTreeRoots;
	TArray<FDebugTreeItemPtr> RootTreeItems;

	TSharedPtr<FString> SearchString;

	bool bFilteredItemsDirty;

	/** message to display when no entries in the tree match the search text */
	FDebugTreeItemPtr SearchMessageItem;

	/** whether this tree is held within the blueprint debugger tab, used to hide context menu options */
	bool bInDebuggerTab;
public:
	/** Accessible functions for creating tree items */
	static UE_API FDebugTreeItemPtr MakeTraceStackParentItem();
	static UE_API FDebugTreeItemPtr MakeBreakpointParentItem(TWeakObjectPtr<UBlueprint> InBlueprint);
	static UE_API FDebugTreeItemPtr MakeMessageItem(const FString& InMessage);
	static UE_API FDebugTreeItemPtr MakeParentItem(UObject* InObject);
	static UE_API FDebugTreeItemPtr MakeWatchLineItem(const UEdGraphPin* InPinRef, UObject* InDebugObject);
	
	/** Column Id's for custom header rows */
	static UE_API const FName ColumnId_Name;
	static UE_API const FName ColumnId_Value;
};

#undef UE_API
