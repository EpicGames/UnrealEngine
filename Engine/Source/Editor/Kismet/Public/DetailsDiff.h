// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "DiffUtils.h"
#include "PropertyPath.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SWidget.h"
#include "Widgets/Layout/LinkableScrollBar.h"

#define UE_API KISMET_API

class FPropertyPath;
class IDetailsView;
class UObject;

/** Struct to handle showing details for an object and provide an interface for listing all differences */
class FDetailsDiff
{
public:
	UE_API FDetailsDiff( const UObject* InObject, bool bScrollbarOnLeft = false );	

	UE_DEPRECATED(5.6, "FOnDisplayedPropertiesChanged removed, interact with IDetailsView directly")
	DECLARE_DELEGATE( FOnDisplayedPropertiesChanged );
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.6, "InOnDisplayedPropertiesChanged, interact with IDetailsView directly")
	UE_API FDetailsDiff( const UObject* InObject, FOnDisplayedPropertiesChanged InOnDisplayedPropertiesChanged, bool bScrollbarOnLeft = false );
	UE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Attempt to highlight the property with the given path, may not always succeed */
	void HighlightProperty( const FPropertySoftPath& PropertyName );

	/** Returns actual widget that is used to display details */
	UE_API TSharedRef< IDetailsView > DetailsWidget() const;

	/** Returns object being displayed */
	const UObject* GetDisplayedObject() const { return DisplayedObject; }

	/** Returns a list of all properties that would be diffed */
	UE_API TArray<FPropertySoftPath> GetDisplayedProperties() const;

	/** Perform a diff against another view, ordering either by display order or by remove/add/change */
	UE_API void DiffAgainst(const FDetailsDiff& Newer, TArray<FSingleObjectDiffEntry>& OutDifferences, bool bSortByDisplayOrder = false) const;

	/** Link the two details panels so they scroll in sync with one another */
	static UE_API void LinkScrolling(FDetailsDiff& LeftPanel, FDetailsDiff& RightPanel, const TAttribute<TArray<FVector2f>>& ScrollRate);

	static UE_API TSharedRef<IDetailsView> CreateDetailsView(const UObject* InObject, TSharedPtr<SScrollBar> ExternalScrollbar = nullptr, bool bScrollbarOnLeft = false);

private:
	TArray< FPropertyPath > DisplayedProperties;
	const UObject* DisplayedObject;

	TSharedPtr< class IDetailsView > DetailsView;

	TSharedPtr<SLinkableScrollBar> ScrollBar;
};

#undef UE_API
