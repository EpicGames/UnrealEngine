// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Queue.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Dom/JsonObject.h"
#include "Engine/Blueprint.h"
#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "HAL/ThreadSafeCounter.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Logging/LogMacros.h"
#include "Misc/EnumClassFlags.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "SlateFwd.h"
#include "Stats/Stats.h"
#include "Templates/Atomic.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "TickableEditorObject.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Serialization/CustomVersion.h"

#include "FindInBlueprintManager.generated.h"

#define UE_API KISMET_API

class FArchive;
class SDockTab;
class SWidget;
class UBlueprint;
class UClass;
class UObject;
struct FTopLevelAssetPath;

DECLARE_LOG_CATEGORY_EXTERN(LogFindInBlueprint, Warning, All);

/** CSV stats profiling category */
CSV_DECLARE_CATEGORY_EXTERN(FindInBlueprint);

class FFindInBlueprintsResult;
class FImaginaryBlueprint;
class FImaginaryFiBData;
class FSpawnTabArgs;
class SFindInBlueprints;
struct FAssetData;

// Shared pointers to cached imaginary data (must be declared as thread-safe).
typedef TWeakPtr<FImaginaryFiBData, ESPMode::ThreadSafe> FImaginaryFiBDataWeakPtr;
typedef TSharedPtr<FImaginaryFiBData, ESPMode::ThreadSafe> FImaginaryFiBDataSharedPtr;

#define MAX_GLOBAL_FIND_RESULTS 4

/**
 *Const values for Find-in-Blueprints to tag searchable data
 */
struct FFindInBlueprintSearchTags
{
	/** Properties tag, for Blueprint variables */
	static UE_API const FText FiB_Properties;

	/** Components tags */
	static UE_API const FText FiB_Components;
	static UE_API const FText FiB_IsSCSComponent;
	/** End Components tags */

	/** Nodes tag */
	static UE_API const FText FiB_Nodes;

	/** Schema Name tag, to identify the schema that a graph uses */
	static UE_API const FText FiB_SchemaName;
	/** Uber graphs tag */
	static UE_API const FText FiB_UberGraphs;
	/** Function graph tag */
	static UE_API const FText FiB_Functions;
	/** Macro graph tag */
	static UE_API const FText FiB_Macros;
	/** Sub graph tag, for any sub-graphs in a Blueprint */
	static UE_API const FText FiB_SubGraphs;
	/** Extension graph tag, for any graphs in a Blueprint Extension */
	static UE_API const FText FiB_ExtensionGraphs;
	/** Blueprint extension tag. */
	static UE_API const FText FiB_Extensions;

	/** Name tag */
	static UE_API const FText FiB_Name;
	/** Native Name tag */
	static UE_API const FText FiB_NativeName;
	/** Class Name tag */
	static UE_API const FText FiB_ClassName;
	/** NodeGuid tag */
	static UE_API const FText FiB_NodeGuid;
	/** Default value */
	static UE_API const FText FiB_DefaultValue;
	/** Tooltip tag */
	static UE_API const FText FiB_Tooltip;
	/** Description tag */
	static UE_API const FText FiB_Description;
	/** Comment tag */
	static UE_API const FText FiB_Comment;
	/** Path tag */
	static UE_API const FText FiB_Path;
	/** Parent Class tag */
	static UE_API const FText FiB_ParentClass;
	/** Interfaces tag */
	static UE_API const FText FiB_Interfaces;
	/** Class that originally defined the function tag */
	static UE_API const FText FiB_FuncOriginClass;

	/** Pin type tags */

	/** Pins tag */
	static UE_API const FText FiB_Pins;

	/** Pin Category tag */
	static UE_API const FText FiB_PinCategory;
	/** Pin Sub-Category tag */
	static UE_API const FText FiB_PinSubCategory;
	/** Pin object class tag */
	static UE_API const FText FiB_ObjectClass;
	/** Pin IsArray tag */
	static UE_API const FText FiB_IsArray;
	/** Pin IsReference tag */
	static UE_API const FText FiB_IsReference;
	/** Glyph icon tag */
	static UE_API const FText FiB_Glyph;
	/** Style set the glyph belongs to */
	static UE_API const FText FiB_GlyphStyleSet;
	/** Glyph icon color tag */
	static UE_API const FText FiB_GlyphColor;

	// Identifier for metadata storage, completely unsearchable tag
	static UE_API const FText FiBMetaDataTag;
	/** End const values for Find-in-Blueprint */
};

/** FiB data versioning */
UENUM()
enum EFiBVersion : int
{
	FIB_VER_NONE = -1, // Unknown version (not set)

	FIB_VER_BASE = 0, // All Blueprints prior to versioning will automatically be assumed to be at 0 if they have FiB data collected
	FIB_VER_VARIABLE_REFERENCE, // Variable references (FMemberReference) is collected in FiB
	FIB_VER_INTERFACE_GRAPHS, // Implemented Interface Graphs is collected in FiB
	FIB_VER_FUNC_CALL_SITES, // Hidden target pins and function origin class are collected in FiB for improved function call site searchability

	// -----<new versions can be added before this line>-------------------------------------------------
	FIB_VER_PLUS_ONE,
	FIB_VER_LATEST = FIB_VER_PLUS_ONE - 1 // Always the last version, we want Blueprints to be at latest
};

/** Consolidated version info for a Blueprint search data entry */
struct FSearchDataVersionInfo
{
	/** FiB asset registry tag data version */
	int32 FiBDataVersion = EFiBVersion::FIB_VER_NONE;

	/** Custom versions used to serialize values in the JSON string lookup table */
	FCustomVersionContainer CustomVersions;

	/** Current version info */
	static FSearchDataVersionInfo Current();
};

/** State flags for search database entries */
enum class ESearchDataStateFlags : uint8
{
	None = 0,
	/** Set when this search database entry has been fully indexed, which is completed asynchronously */
	IsIndexed = 1 << 0,
	/** Cached to determine if the Blueprint is seen as no longer valid, allows it to be cleared out next save to disk */
	WasRemoved = 1 << 1,
};

ENUM_CLASS_FLAGS(ESearchDataStateFlags);

/** Tracks data relevant to a Blueprint for searches */
struct FSearchData
{
	/** The Blueprint this search data points to, if available */
	TWeakObjectPtr<UBlueprint> Blueprint;

	/** The full asset path this search data is associated with of the form /Game/Path/To/Package.Package */
	FSoftObjectPath AssetPath;

	/** Encoded search data block for the Blueprint, this will not always be set if it's already been parsed */
	FString Value;

	/** Key to use to look up the encoded search data from an FAssetData, if this is set Value will probably be empty */
	FName AssetKeyForValue;

	/** Parent Class */
	FString ParentClass;

	/** Interfaces implemented by the Blueprint */
	TArray<FString> Interfaces;

	/** Cached ImaginaryBlueprint data for the searchable content, prevents having to re-parse every search */
	FImaginaryFiBDataSharedPtr ImaginaryBlueprint;

	/** Data versioning */
	FSearchDataVersionInfo VersionInfo;

	/** State flags (see enum) */
	ESearchDataStateFlags StateFlags;

	FSearchData()
		: Blueprint(nullptr)
		, StateFlags(ESearchDataStateFlags::None)
	{
	}

	/** True if this represents a valid asset */
	bool IsValid() const
	{
		return !AssetPath.IsNull();
	}

	/** True if this has an encoded value that has yet to be parsed */
	bool HasEncodedValue() const
	{
		return !Value.IsEmpty() || !AssetKeyForValue.IsNone();
	}

	/** Clear the encoded value after parsing or getting new data */
	void ClearEncodedValue()
	{
		Value.Reset();
		AssetKeyForValue = NAME_None;
	}

	bool IsIndexingCompleted() const
	{
		return EnumHasAllFlags(StateFlags, ESearchDataStateFlags::IsIndexed);
	}

	bool IsMarkedForDeletion() const
	{
		return EnumHasAllFlags(StateFlags, ESearchDataStateFlags::WasRemoved);
	}
};

/** Filters are used by functions for searching to decide whether items can call certain functions or match the requirements of a function */
enum ESearchQueryFilter
{
	BlueprintFilter = 0,
	GraphsFilter,
	UberGraphsFilter,
	FunctionsFilter,
	MacrosFilter,
	NodesFilter,
	PinsFilter,
	PropertiesFilter,
	VariablesFilter,
	ComponentsFilter,
	AllFilter, // Will search all items, when used inside of another filter it will search all sub-items of that filter
};

/** Used for external gather functions to add Key/Value pairs to be placed into Json */
struct FSearchTagDataPair
{
	FSearchTagDataPair(FText InKey, FText InValue)
		: Key(InKey)
		, Value(InValue)
	{}

	FText Key;
	FText Value;
};

struct FFiBMD
{
	static UE_API const FString FiBSearchableMD;
	static UE_API const FString FiBSearchableShallowMD;
	static UE_API const FString FiBSearchableExplicitMD;
	static UE_API const FString FiBSearchableHiddenExplicitMD;
	static UE_API const FString FiBSearchableFormatVersionMD;
};

/** Which assets to index for caching */
enum class EFiBCacheOpType
{
	CachePendingAssets,
	CacheUnindexedAssets
};

/** Flags to control the UX while caching */
enum class EFiBCacheOpFlags
{
	None = 0,
	/** Whether to show progress */
	ShowProgress = 1 << 0,
	/** Whether to hide toast popups */
	HideNotifications = 1 << 1,
	/** Whether to allow users to cancel */
	AllowUserCancel = 1 << 2,
	/** Set if the user wants to check out and save (applies to unindexed caching only) */
	CheckOutAndSave = 1 << 3,
	/** Whether to hide progress bar widgets */
	HideProgressBars = 1 << 4,
	/** Whether to allow users to hide/close progress */
	AllowUserCloseProgress = 1 << 5,
	/** Set if we are caching assets from the discovery stage */
	IsCachingDiscoveredAssets = 1 << 6,
	/** Whether to keep progress visible on completion */
	KeepProgressVisibleOnCompletion = 1 << 7,
	/** Index deferred assets on the main thread only (used for debugging) */
	ExecuteOnMainThread = 1 << 8,
	/** Don't index multiple assets in parallel (used to assist with profiling) */
	ExecuteOnSingleThread = 1 << 9,
	/** Only execute the gather phase (used to help minimize memory usage on editor/tab open) */
	ExecuteGatherPhaseOnly = 1 << 10,
};

ENUM_CLASS_FLAGS(EFiBCacheOpFlags);

/** Options to configure the bulk caching task */
struct FFindInBlueprintCachingOptions
{
	/** Type of caching operation */
	EFiBCacheOpType OpType = EFiBCacheOpType::CachePendingAssets;

	/** Initial set of control flags */
	EFiBCacheOpFlags OpFlags = EFiBCacheOpFlags::None;

	/** Callback for when caching is finished */
	FSimpleDelegate OnFinished;

	/** Minimum version requirement for caching, any Blueprints below this version will be re-indexed */
	EFiBVersion MinimiumVersionRequirement = EFiBVersion::FIB_VER_LATEST;
};

/** Options for FFindInBlueprintSearchManager::AddOrUpdateBlueprintSearchMetadata() */
enum class EAddOrUpdateBlueprintSearchMetadataFlags
{
	None = 0,
	/** Forces the Blueprint to be recache'd, regardless of what data it believes exists */
	ForceRecache = 1 << 0,
	/** Clear any cached data value for this Blueprint */
	ClearCachedValue = 1 << 1,
};

ENUM_CLASS_FLAGS(EAddOrUpdateBlueprintSearchMetadataFlags);

////////////////////////////////////
// FFindInBlueprintsResult

/* Item that matched the search results */
class FFindInBlueprintsResult : public TSharedFromThis< FFindInBlueprintsResult >
{
public:
	FFindInBlueprintsResult() = default;
	virtual ~FFindInBlueprintsResult() = default;

	/* Create a root */
	UE_API explicit FFindInBlueprintsResult(const FText& InDisplayText);

	/* Called when user clicks on the search item */
	UE_API virtual FReply OnClick();

	/* Get Category for this search result */
	UE_API virtual FText GetCategory() const;

	/* Create an icon to represent the result */
	UE_API virtual TSharedRef<SWidget>	CreateIcon() const;

	/** Finalizes any content for the search data that was unsafe to do on a separate thread */
	virtual void FinalizeSearchData() {};

	/* Gets the comment on this node if any */
	UE_API FString GetCommentText() const;

	/** gets the blueprint housing all these search results */
	UE_API UBlueprint* GetParentBlueprint() const;

	/**
	* Parses search info for specific data important for displaying the search result in an easy to understand format
	*
	* @param	InTokens		The search tokens to check results against
	* @param	InKey			This is the tag for the data, describing what it is so special handling can occur if needed
	* @param	InValue			Compared against search query to see if it passes the filter, sometimes data is rejected because it is deemed unsearchable
	* @param	InParent		The parent search result
	*/
	virtual void ParseSearchInfo(FText InKey, FText InValue) {};

	/** Returns the Object represented by this search information give the Blueprint it can be found in */
	UE_API virtual UObject* GetObject(UBlueprint* InBlueprint) const;

	/** Returns the display string for the row */
	UE_API FText GetDisplayString() const;

public:
	/*Any children listed under this category */
	TArray< TSharedPtr<FFindInBlueprintsResult> > Children;

	/*If it exists it is the blueprint*/
	TWeakPtr<FFindInBlueprintsResult> Parent;

	/*The display text for this item */
	FText DisplayText;

	/** Display text for comment information */
	FString CommentText;
};

typedef TSharedPtr<FFindInBlueprintsResult> FSearchResult;

////////////////////////////////////
// FStreamSearch

struct FStreamSearchOptions
{
	/** Filter to limit the FilteredImaginaryResults to */
	enum ESearchQueryFilter ImaginaryDataFilter;

	/** When searching, any Blueprint below this version will be considered out-of-date */
	EFiBVersion MinimiumVersionRequirement;

	/** Default constructor. */
	FStreamSearchOptions()
		:ImaginaryDataFilter(ESearchQueryFilter::AllFilter)
		,MinimiumVersionRequirement(EFiBVersion::FIB_VER_LATEST)
	{
	}
};

/**
 * Async task for searching Blueprints
 */
class FStreamSearch : public FRunnable
{
public:
	/** Constructor */
	UE_API FStreamSearch(const FString& InSearchValue, const FStreamSearchOptions& InSearchOptions = FStreamSearchOptions());

	/** Begin FRunnable Interface */
	UE_API virtual bool Init() override;

	UE_API virtual uint32 Run() override;

	UE_API virtual void Stop() override;

	UE_API virtual void Exit() override;
	/** End FRunnable Interface */

	/** Brings the thread to a safe stop before continuing. */
	UE_API void EnsureCompletion();

	/** Returns TRUE if the thread is done with it's work. */
	UE_API bool IsComplete() const;

	/** Returns TRUE if Stop() was called while work is still pending. */
	UE_API bool WasStopped() const;

	/**
	 * Appends the items filtered through the search filter to the passed array
	 *
	 * @param OutItemsFound		All the items found since last queried
	 */
	UE_API void GetFilteredItems(TArray<TSharedPtr<class FFindInBlueprintsResult>>& OutItemsFound);

	/** Helper function to query the percent complete this search is */
	UE_API float GetPercentComplete() const;

	/** Returns the Out-of-Date Blueprint count */
	int32 GetOutOfDateCount() const
	{
		return BlueprintCountBelowVersion;
	}

	/** Returns the FilteredImaginaryResults from the search query, these results have been filtered by the ImaginaryDataFilter. */
	UE_API void GetFilteredImaginaryResults(TArray<FImaginaryFiBDataSharedPtr>& OutFilteredImaginaryResults);

public:
	/** Thread to run the cleanup FRunnable on */
	TUniquePtr<FRunnableThread> Thread;

	/** A list of items found, cleared whenever the main thread pulls them to display to screen */
	TArray<TSharedPtr<class FFindInBlueprintsResult>> ItemsFound;

	/** The search value to filter results by */
	FString SearchValue;

	/** Options for setting up the search */
	FStreamSearchOptions SearchOptions;

	/** Prevents searching while other threads are pulling search results */
	FCriticalSection SearchCriticalSection;

	/** Filtered (ImaginaryDataFilter) list of imaginary data results that met the search requirements. Must be declared as thread-safe since imaginary data is a shared resource. */
	TArray<FImaginaryFiBDataSharedPtr> FilteredImaginaryResults;

	/** A going count of all Blueprints below the MinimiumVersionRequirement */
	int32 BlueprintCountBelowVersion;

	// Whether the thread has finished running
	bool bThreadCompleted;

private:
	/** Unique identifier for this search (used with benchmarking) */
	int32 SearchId;

	/** > 0 if we've been asked to abort work in progress at the next opportunity */
	FThreadSafeCounter StopTaskCounter;
};

////////////////////////////////////
// FFindInBlueprintSearchManager

/** Singleton manager for handling all Blueprint searches, helps to manage the going progress of Blueprints, and is thread-safe. */
class FFindInBlueprintSearchManager : public FTickableEditorObject
{
public:
	static UE_API FFindInBlueprintSearchManager* Instance;
	static UE_API FFindInBlueprintSearchManager& Get();

	UE_API FFindInBlueprintSearchManager();
	UE_API ~FFindInBlueprintSearchManager();

	//~ Begin FTickableObject Interface
	UE_API virtual void Tick(float DeltaTime) override;
	UE_API virtual bool IsTickable() const override;
	UE_API virtual TStatId GetStatId() const override;
	//~ End FTickableObject Interface

	/**
	 * Applies the given search data to a matching entry in the database. Optionally adds a new entry if a match is not found.
	 *
	 * @param InSearchData		Search data to be applied
	 * @param bAllowNewEntry	Whether to allow a new entry to be added if a match is not found
	 */
	UE_API void ApplySearchDataToDatabase(FSearchData InSearchData, bool bAllowNewEntry = false);

	/**
	 * Given an asset path, locate and return a copy of its matching search data in the index cache.
	 *
	 * @param InAssetPath				Asset path (search index key).
	 * @return							Matching search data from the index cache. Will return invalid (empty) search data if a matching entry was not found.
	 */
	UE_API FSearchData GetSearchDataForAssetPath(const FSoftObjectPath& InAssetPath);

	/**
	 * Gathers the Blueprint's search metadata and adds or updates it in the cache
	 *
	 * @param InBlueprint		Blueprint to cache the searchable data for
	 * @param InFlags			Flags to assist with controlling this function's behavior
	 * @param InVersion			Allows the caller to override the format version to use for caching
	 */
	UE_API void AddOrUpdateBlueprintSearchMetadata(UBlueprint* InBlueprint, EAddOrUpdateBlueprintSearchMetadataFlags InFlags = EAddOrUpdateBlueprintSearchMetadataFlags::None, EFiBVersion InVersion = EFiBVersion::FIB_VER_NONE);

	/**
	 * Starts a search query, the FiB manager handles where the thread is at in the search query at all times so that post-save of the cache to disk it can correct the index
	 *
	 * @param InSearchOriginator		Pointer to the thread object that the query originates from
	 */
	UE_API void BeginSearchQuery(const class FStreamSearch* InSearchOriginator);

	/**
	 * Continues a search query, returning a single piece of search data
	 *
	 * @param InSearchOriginator		Pointer to the thread object that the query originates from
	 * @param OutSearchData				Result of the search, if there is any Blueprint search data still available to query
	 * @return							TRUE if the search was successful, FALSE if the search is complete
	 */
	UE_API bool ContinueSearchQuery(const class FStreamSearch* InSearchOriginator, FSearchData& OutSearchData);

	/**
	 * This function ensures that the passed in search query ends in a safe manner. The search will no longer be valid to this manager, though it does not destroy any objects.
	 * Use this whenever the search is finished or canceled.
	 *
	 * @param InSearchOriginator	Identifying search stream to be stopped
	 */
	UE_API void EnsureSearchQueryEnds(const class FStreamSearch* InSearchOriginator);

	/**
	 * Query how far along a search thread is
	 *
	 * @param OutSearchData				Result of the search, if there is any Blueprint search data still available to query
	 * @return							Percent along the search thread is
	 */
	UE_API float GetPercentComplete(const class FStreamSearch* InSearchOriginator) const;

	/**
	 * Query for a single, specific Blueprint's search data.
	 *
	 * @param InBlueprint				The Blueprint to search for
	 * @param bInRebuildSearchData		When TRUE the search data will be freshly collected
	 * @return							The search data, or invalid (empty) search data if not found
	 */
	UE_API FSearchData QuerySingleBlueprint(UBlueprint* InBlueprint, bool bInRebuildSearchData);

	/** Processes the encoded string value in the SearchData into the intermediate format, return true if string and version were valid */
	UE_API bool ProcessEncodedValueForUnloadedBlueprint(FSearchData& SearchData);

	/** Returns the number of unindexed Blueprints, either due to not having been indexed before, or AR data being out-of-date */
	UE_API int32 GetNumberUnindexedAssets() const;

	/** Returns the number of uncached assets during an active indexing operation */
	UE_API int32 GetNumberUncachedAssets() const;

	/**
	 * Starts a task to cache Blueprints at a rate of 1 per tick
	 *
	 * @param InSourceWidget				The source FindInBlueprints widget, this widget will be informed when caching is complete
	 * @param InCachingOptions				Options to configure the caching task
	 */
	UE_API void CacheAllAssets(TWeakPtr< class SFindInBlueprints > InSourceWidget, const FFindInBlueprintCachingOptions& InCachingOptions);

	/**
	 * Exports a list of all unindexed assets to Saved/FindInBlueprints_OutdatedAssetList.txt
	 */
	UE_API void ExportOutdatedAssetList();

	/**
	 * Starts the actual caching process
	 *
	 * @param bInSourceControlActive		TRUE if source control is active
	 * @param bInCheckoutAndSave			TRUE if the system should checkout and save all assets that need to be reindexed
	 */
	UE_API void OnCacheAllUnindexedAssets(bool bInSourceControlActive, bool bInCheckoutAndSave);

	/** Stops the caching process where it currently is at, the rest can be continued later */
	UE_API void CancelCacheAll(SFindInBlueprints* InFindInBlueprintWidget);

	/** Returns the current index in the caching */
	UE_API int32 GetCurrentCacheIndex() const;

	/** Returns the path of the current Blueprint being cached */
	UE_API FSoftObjectPath GetCurrentCacheBlueprintPath() const;

	/** Returns the progress complete on the caching */
	UE_API float GetCacheProgress() const;

	/** Returns the list of Blueprint paths that failed to cache */
	TSet<FSoftObjectPath> GetFailedToCachePathList() const { return FailedToCachePaths; }

	/** Returns the number of Blueprints that failed to cache */
	int32 GetFailedToCacheCount() const { return FailedToCachePaths.Num(); }

	/** Returns TRUE if caching failed */
	bool HasCachingFailed() const { return FailedToCachePaths.Num() > 0; };

	/** Callback to note that Blueprint caching is started */
	UE_API void StartedCachingBlueprints(EFiBCacheOpType InCacheOpType, EFiBCacheOpFlags InCacheOpFlags);

	/**
	 * Callback to note that Blueprint caching is complete
	 *
	 * @param InNumberCached		The number of Blueprints cached, to be chopped off the existing array so the rest (if any) can be finished later
	 */
	UE_API void FinishedCachingBlueprints(EFiBCacheOpType InCacheOpType, EFiBCacheOpFlags InCacheOpFlags, int32 InNumberCached, TSet<FSoftObjectPath>& InFailedToCacheList);

	/** Returns TRUE if Blueprints are being cached. */
	UE_API bool IsCacheInProgress() const;

	/** Returns TRUE if unindexed Blueprints are being cached (since this can block the UI) */
	UE_API bool IsUnindexedCacheInProgress() const;

	/** Returns TRUE if we're still inside the initial asset discovery and registration stage */
	UE_API bool IsAssetDiscoveryInProgress() const;

	/** Returns TRUE if there are one or more active asynchronous search queries */
	UE_API bool IsAsyncSearchQueryInProgress() const;

	/** Returns a weak reference to the widget that initiated the current caching operation */
	TWeakPtr<SFindInBlueprints> GetSourceCachingWidget() const { return SourceCachingWidget; }

	void EnableGatheringData(bool bInEnableGatheringData) { bEnableGatheringData = bInEnableGatheringData; }

	bool IsGatheringDataEnabled() const { return bEnableGatheringData; }

	/** If TRUE, the developer menu tool commands will be shown in the 'Developer' section of the Blueprint Editor's menu bar */
	bool ShouldEnableDeveloperMenuTools() const { return bEnableDeveloperMenuTools; }

	/** If TRUE, search result meta will be gathered once and stored in a template. Avoids doing this work redundantly at search time. */
	bool ShouldEnableSearchResultTemplates() const { return !bDisableSearchResultTemplates; }

	/** Find or create the global find results widget */
	UE_API TSharedPtr<SFindInBlueprints> GetGlobalFindResults();

	/** Enable or disable the global find results tab feature */
	UE_API void EnableGlobalFindResults(bool bEnable);

	/** Close any orphaned global find results tabs for a particular tab manager */
	UE_API void CloseOrphanedGlobalFindResultsTabs(TSharedPtr<class FTabManager> TabManager);

	/** Returns true if a global find results tab is currently open */
	bool IsGlobalFindResultsOpen() const { return GlobalFindResults.Num() > 0; }

	UE_API void GlobalFindResultsClosed(const TSharedRef<SFindInBlueprints>& FindResults);

	/** Dumps the full index cache to the given stream (for debugging purposes) */
	UE_API void DumpCache(FArchive& Ar);

	/** Randomly initiates a partial rebuild of the index cache (for debugging purposes) */
	UE_API void DoRandomIndexCacheRebuild();

public:
	/** Converts a string of hex characters, previously converted by ConvertFTextToHexString, to an FText. */
	static UE_API FText ConvertHexStringToFText(FString InHexString);

	/** Serializes an FText to memory and converts the memory into a string of hex characters */
	static UE_API FString ConvertFTextToHexString(FText InValue);

	/** Given a fully constructed Find-in-Blueprint FString of searchable data, will parse and construct a JsonObject */
	static UE_API TSharedPtr< class FJsonObject > ConvertJsonStringToObject(FSearchDataVersionInfo InVersionInfo, const FString& InJsonString, TMap<int32, FText>& OutFTextLookupTable);

	/** Generates a human-readable search index for the given Blueprint (for debugging purposes) */
	static UE_API FString GenerateSearchIndexForDebugging(UBlueprint* InBlueprint);

private:
	/** Initializes the FiB manager */
	UE_API void Initialize();

	/** Callback hook during pre-garbage collection, pauses all processing searches so that the GC can do it's job */ 
	UE_API void PauseFindInBlueprintSearch();

	/** Callback hook during post-garbage collection, saves the cache to disk and unpauses all processing searches */
	UE_API void UnpauseFindInBlueprintSearch();

	/** Callback hook from the Asset Registry when an asset is added */
	UE_API void OnAssetAdded(const struct FAssetData& InAssetData);

	/** Callback hook from the Asset Registry, marks the asset for deletion from the cache */
	UE_API void OnAssetRemoved(const struct FAssetData& InAssetData);

	/** Callback hook from the Asset Registry, marks the asset for deletion from the cache */
	UE_API void OnAssetRenamed(const struct FAssetData& InAssetData, const FString& InOldName);

	/** Callback hook from the Asset Registry, signals that the initial asset discovery stage has been completed */
	UE_API void OnAssetRegistryFilesLoaded();

	/** Callback hook from the Asset Registry when an asset is loaded */
	UE_API void OnAssetLoaded(class UObject* InAsset);

	/** Callback from Kismet when a Blueprint is unloaded */
	UE_API void OnBlueprintUnloaded(class UBlueprint* InBlueprint);

	/** Callback hook from the Reload manager that indicates that a module has been reloaded */
	UE_API void OnReloadComplete(EReloadCompleteReason Reason);

	/** Returns a copy of the search data that's cached at the given index. Will return invalid (empty) search data if the index is out of range */
	UE_API FSearchData GetSearchDataForIndex(int32 CacheIndex);

	/** Cleans the cache of any excess data from Blueprints that have been moved, renamed, or deleted. Occurs during post-garbage collection */
	UE_API void CleanCache();

	/** Builds the cache from all available Blueprint assets that the asset registry has discovered at the time of this function. Occurs on startup */
	UE_API void BuildCache();

	/**
	 * Helper to properly add a Blueprint's SearchData to the database
	 *
	 * @param InSearchData		Data to add to the database
	 * @return					Index into the SearchArray for looking up the added item
	 */
	UE_API int32 AddSearchDataToDatabase(FSearchData InSearchData);

	/** Removes a Blueprint from being managed by the FiB system by passing in the UBlueprint's path */
	UE_API void RemoveBlueprintByPath(const FSoftObjectPath& InPath);

	/** Adds a new search database entry for unloaded asset data */
	UE_API void AddUnloadedBlueprintSearchMetadata(const FAssetData& InAssetData);

	/** Begins the process of extracting FiB data from an unloaded asset */
	UE_API void ExtractUnloadedFiBData(const FAssetData& InAssetData, FString* InFiBData, FName InKeyForFiBData, EFiBVersion InFiBDataVersion);

	/** Determines the global find results tab label */
	UE_API FText GetGlobalFindResultsTabLabel(int32 TabIdx);

	/** Handler for a request to spawn a new global find results tab */
	UE_API TSharedRef<SDockTab> SpawnGlobalFindResultsTab(const FSpawnTabArgs& SpawnTabArgs, int32 TabIdx);

	/** Creates and opens a new global find results tab */
	UE_API TSharedPtr<SFindInBlueprints> OpenGlobalFindResultsTab();

protected:
	/** Contains info about an active search query */
	struct FActiveSearchQuery
	{
		/** Current search array index */
		TAtomic<int32> NextIndex;
		/** Current count of assets searched */
		TAtomic<int32> SearchCount;
		/** Asset paths for which searching was deferred due to being indexed */
		TQueue<FSoftObjectPath> DeferredAssetPaths;

		FActiveSearchQuery()
			:NextIndex(0)
			,SearchCount(0)
		{
		}
	};

	typedef TSharedPtr<FActiveSearchQuery, ESPMode::ThreadSafe> FActiveSearchQueryPtr;

	/** Thread-safe access to the active search query that's mapped to the given stream search */
	UE_API FActiveSearchQueryPtr FindSearchQuery(const class FStreamSearch* InSearchOriginator) const;

	/** Returns the next pending search data for the given query and advances the index to the next entry */
	UE_API FSearchData GetNextSearchDataForQuery(const FStreamSearch* InSearchOriginator, FActiveSearchQueryPtr InSearchQueryPtr, bool bCheckDeferredList);

	/** If searches are paused, blocks the calling thread until searching is resumed */
	UE_API void BlockSearchQueryIfPaused();

private:
	/** Maps the Blueprint paths to their index in the SearchArray */
	TMap<FSoftObjectPath, int32> SearchMap;

	/** Stores the Blueprint search data and is used to iterate over in small chunks */
	TArray<FSearchData> SearchArray;

	/** Counter of active searches */
	FThreadSafeCounter ActiveSearchCounter;

	/** A mapping of active search queries and where they are currently at in the search data */
	TMap<const class FStreamSearch*, FActiveSearchQueryPtr> ActiveSearchQueries;

	/** Critical section to safely add, remove, and find data in ActiveSearchQueries */
	mutable FCriticalSection SafeQueryModifyCriticalSection;

	/** Critical section to lock threads during the pausing procedure */
	FCriticalSection PauseThreadsCriticalSection;

	/** Critical section to safely modify cached data */
	FCriticalSection SafeModifyCacheCriticalSection;

	/** Because we are unable to query for the module on another thread, cache it for use later */
	class FAssetRegistryModule* AssetRegistryModule;

	/** FindInBlueprints widget that started the cache process */
	TWeakPtr<SFindInBlueprints> SourceCachingWidget;

	/** Asset paths that were discovered, loaded or modified and now require indexing (or re-indexing) */
	TSet<FSoftObjectPath> PendingAssets;

	/** Asset paths that have not been cached for searching due to lack of FiB data, this means that they are either older Blueprints, or the DDC cannot find the data */
	TSet<FSoftObjectPath> UnindexedAssets;

	/** List of paths for Blueprints that failed to cache */
	TSet<FSoftObjectPath> FailedToCachePaths;

	/** List of paths that require a full index pass during the first global search */
	TSet<FSoftObjectPath> AssetsToIndexOnFirstSearch;

	/** Tickable object that does the caching of uncached Blueprints at a rate of once per tick */
	TUniquePtr<class FCacheAllBlueprintsTickableObject> CachingObject;

	/** Stores the type of caching operation that's currently in progress */
	EFiBCacheOpType CurrentCacheOpType;

	/** Mapping between a class name and its UClass instance - used for faster look up in FFindInBlueprintSearchManager::OnAssetAdded */
	TMap<FTopLevelAssetPath, TWeakObjectPtr<const UClass>> CachedAssetClasses;

	/** The tab identifier/instance name for global find results */
	FName GlobalFindResultsTabIDs[MAX_GLOBAL_FIND_RESULTS];

	/** Array of open global find results widgets */
	TArray<TWeakPtr<SFindInBlueprints>> GlobalFindResults;

	/** Global Find Results workspace menu item */
	TSharedPtr<class FWorkspaceItem> GlobalFindResultsMenuItem;

	/** Size of a single async indexing task batch; zero or negative means that a caching operation will consist of a single batch */
	int32 AsyncTaskBatchSize;

	/** TRUE when the the FiB manager wants to pause all searches, helps manage the pausing procedure */
	TAtomic<bool> bIsPausing;

	/** Currently used to delay the full indexing pass until the first search in order to control memory usage */
	TAtomic<bool> bHasFirstSearchOccurred;

	/** Tells if gathering data is currently allowed */
	bool bEnableGatheringData;

	/** If true, search metadata will be regenerated for loaded assets on the main thread immediately at discovery or load time. By default, this work is deferred to avoid hitching. */
	bool bDisableDeferredIndexing;

	/** If true, and if deferred indexing is not disabled, search metadata will be regenerated for loaded assets on the main thread at a rate of one asset per tick. Additional indexing
		work will be deferred to the search thread. By default, all indexing work is deferred to a background thread and both loaded and unloaded assets are fully indexed asynchronously. */
	bool bDisableThreadedIndexing;

	/** Whether CSV profiling has been enabled (default=false) */
	bool bEnableCSVStatsProfiling;

	/** Whether to enable Blueprint editor developer menu tools */
	bool bEnableDeveloperMenuTools;

	/** Disable the use of search result templates. Setting this to TRUE will slightly decrease overall memory usage, but will also increase global search times */
	bool bDisableSearchResultTemplates;

	/** Defers the cost to extract metadata for each discovered asset during the initial asset registry scan into a single pass over the full asset registry once the scan is complete. */
	bool bDisableImmediateAssetDiscovery;
};

struct FDisableGatheringDataOnScope
{
	bool bOriginallyEnabled;
	FDisableGatheringDataOnScope() : bOriginallyEnabled(FFindInBlueprintSearchManager::Get().IsGatheringDataEnabled())
	{
		FFindInBlueprintSearchManager::Get().EnableGatheringData(false);
	}
	~FDisableGatheringDataOnScope()
	{
		FFindInBlueprintSearchManager::Get().EnableGatheringData(bOriginallyEnabled);
	}
};

#undef UE_API
