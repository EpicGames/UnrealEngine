// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeature.h"

class UBlueprint;
struct FTypedElementHandle;
class IAssetEditorInstance;

/**
 * Modular feature interface for Blueprint Editor selection notifications
 * Implemented by plugins that want to be notified when Blueprint Editor selection changes
 */
class UE_INTERNAL IBlueprintSelectionNotifier : public IModularFeature
{
public:
    static FName GetModularFeatureName()
    {
        static FName ModularFeatureName = FName(TEXT("BlueprintSelectionNotifier"));
        return ModularFeatureName;
    }

    /**
     * Called when Blueprint Editor selection changes
     * @param AssetEditorInstance The asset editor instance that triggered the selection change
     * @param Blueprint The blueprint being edited
     * @param SelectedElements The currently selected elements in the blueprint
     */
    virtual void OnBlueprintSelectionUpdated(IAssetEditorInstance* AssetEditorInstance, UBlueprint* Blueprint, const TArray<FTypedElementHandle>& SelectedElements) = 0;
};