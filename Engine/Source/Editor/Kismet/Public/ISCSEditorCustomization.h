// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "EditorViewportClient.h"

class USceneComponent;

/**
 * Customization class that allows per-component type handling of various SCS editor functionality.
 */
class ISCSEditorCustomization
{
public:
	virtual ~ISCSEditorCustomization() = default;

	/** 
	 * Process click in the viewport.
	 * @param	InViewportClient	The viewport client receiving the click
	 * @param	InView				The scene view receiving the click
	 * @param	InHitProxy			The hit proxy that was clicked
	 * @param	InKey				The key that was pressed
	 * @param	InEvent				The input event that occurred
	 * @param	InHitX				The X location of the click
	 * @param	InHitY				The Y location of the click
	 * @return true if the operation was handled.
	 */
	virtual bool HandleViewportClick(const TSharedRef<FEditorViewportClient>& InViewportClient, class FSceneView& InView, class HHitProxy* InHitProxy, FKey InKey, EInputEvent InEvent, uint32 InHitX, uint32 InHitY) = 0;

	/**
	 * Handle drag in the viewport
	 * @param	InSceneComponent		The scene component being dragged
	 * @param	InComponentTemplate		The template corresponding to the scene component
	 * @param	InDeltaTranslation		The delta translation being applied
	 * @param	InDeltaRotation			The delta rotation being applied
	 * @param	InDeltaScale			The delta scale being applied
	 * @param	InPivot					The povot point for any transformations
	 * @return true if the operation was handled.
	 */
	virtual bool HandleViewportDrag(const USceneComponent* InSceneComponent, class USceneComponent* InComponentTemplate, const FVector& InDeltaTranslation, const FRotator& InDeltaRotation, const FVector& InDeltaScale, const FVector& InPivot) = 0;

	/**
	 * Get the widget location for this scene component
	 * @param	InSceneComponent		The scene component to get location for
	 * @param	OutWidgetLocation		The output widget location
	 * @return true if the operation was handled.
	 */
	virtual bool HandleGetWidgetLocation(const USceneComponent* InSceneComponent, FVector& OutWidgetLocation) = 0;

	/**
	 * Get the widget transform for this scene component
	 * @param	InSceneComponent		The scene component to get transform for
	 * @param	OutWidgetLocation		The output widget transform
	 * @return true if the operation was handled.
	 */
	virtual bool HandleGetWidgetTransform(const USceneComponent* InSceneComponent, FMatrix& OutWidgetTransform) = 0;
};
