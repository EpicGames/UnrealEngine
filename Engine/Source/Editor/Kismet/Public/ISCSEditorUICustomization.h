// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ChildActorComponent.h"
#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"
#include "Containers/ArrayView.h"
#include "Layout/Visibility.h"
#include "Widgets/SNullWidget.h"

class UActorComponent;
struct FSubobjectData;
struct FSubobjectDataHandle;
class SSubobjectEditor;
namespace UE::Editor
{
	struct FSubobjectEditorContext;
}

/** SCSEditor UI customization */
class ISCSEditorUICustomization
{
public:
	virtual ~ISCSEditorUICustomization() = default;

	/** @return Whether to hide the components tree */
	virtual bool HideComponentsTree(TArrayView<UObject*> Context) const { return false; }

	/** @return Whether to hide the components filter box */
	virtual bool HideComponentsFilterBox(TArrayView<UObject*> Context) const { return false; }

	/** @return Whether to hide the "Add Component" combo button */
	virtual bool HideAddComponentButton(TArrayView<UObject*> Context) const { return false; }

	/** @return Whether to hide the "Edit Blueprint" and "Blueprint/Add Script" buttons */
	virtual bool HideBlueprintButtons(TArrayView<UObject*> Context) const { return false; }

	/** @return The icon that should be used for the given SubobjectData
	 * 
	 * Implementations that return nullptr will lead to existing built-in functionality of the
	 * subobject editor being used to determine the Icon for the given FSubobjectData 
	 */
	virtual const FSlateBrush* GetIconBrush(const FSubobjectData&) const { return nullptr; }

	/** Allows customizer to inject a right-side justified widget on the subobject editor
	 * Typically these widgets would convey actions that can be taken and may optionally provide
	 * status information through their visualization.
	 * As an example, a subobject that is a non-native ActorComponent may provide a button via these widgets
	 * to open a code editor to the C++ version of that type.
	 * Returning nullptr will cause built-in behaviour to be used, indicating that the UICustomization does not customize this
	 * UI element. To override existing behaviour and actually show nothing, return a single null widget
	 */
	UE_DEPRECATED(5.8, "Use GetControlsWidget(TSharedRef<SSubobjectEditor>&, const FSubobjectData&")
	virtual TSharedPtr<SWidget> GetControlsWidget(const FSubobjectData&) const { return {}; }
	
	/** Allows customizer to inject a right-side justified widget on the subobject editor
	 * Typically these widgets would convey actions that can be taken and may optionally provide
	 * status information through their visualization.
	 * As an example, a subobject that is a non-native ActorComponent may provide a button via these widgets
	 * to open a code editor to the C++ version of that type.
	 * Returning nullptr will cause built-in behaviour to be used, indicating that the UICustomization does not customize this
	 * UI element. To override existing behaviour and actually show nothing, return a single null widget
	 */
	virtual TSharedPtr<SWidget> GetControlsWidget(TSharedRef<SSubobjectEditor>& SubobjectEditor, const FSubobjectData& Data) const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GetControlsWidget(Data);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * When it returns true, overrides the value returned by SSubobjectEditor::GetDesiredVisibility().
	 *  Otherwise, it is ignored.
	 * @param VisibilityOut The visibility to return
	 * @return true if the customization wants VisibilityOut to be used. 
	 */
	UE_EXPERIMENTAL(5.8, "Signature may change in the future")
	virtual bool OverrideDesiredVisibility(const UE::Editor::FSubobjectEditorContext& Context, EVisibility& VisibilityOut) const { return false; }

	/** 
	 * @return An override for the default ChildActorComponentTreeViewVisualizationMode from the project settings, if different from UseDefault.
	 * @note Setting an override also forces child actor tree view expansion to be enabled.
	 */
	virtual EChildActorComponentTreeViewVisualizationMode GetChildActorVisualizationMode() const { return EChildActorComponentTreeViewVisualizationMode::UseDefault; }

	/** @return A component type that limits visible nodes when filtering the tree view */
	virtual TSubclassOf<UActorComponent> GetComponentTypeFilter(TArrayView<UObject*> Context) const { return nullptr; }

	/** Optionally filter the given SubobjectDataHandles for the tree view as appropriate for this editor.
	 * @return Whether any edits were made */
	UE_EXPERIMENTAL(5.7, "The interface for filtering the sub object editor may change.")
	virtual bool FilterSubobjectData(TArray<FSubobjectDataHandle>& SubobjectData) const { return false; }

	/** Optionally sorts the given SubobjectDataHandles for the tree view as appropriate for this editor.
	 * @return Whether any sorting was attempted */
	virtual bool SortSubobjectData(TArray<FSubobjectDataHandle>& SubobjectData) { return false; }

	/**
	 * Called before events that might change the targets of the subobject editor, particularly
	 *  useful for gathering data to use in a subsequent SelectDefaultSubobject call.
	 */
	UE_EXPERIMENTAL(5.8, "This function is experimental for now.")
	virtual void PrePotentialTargetChange(SSubobjectEditor& SubobjectEditor) {};

	/**
	 * Gives a customization the opportunity to control what the "default" subobject selection is.
	 * @return true if custom selection was performed, false if subobject editor should perform its
	 *  own default selection (typically selecting root).
	 */
	UE_EXPERIMENTAL(5.8, "This function is experimental for now.")
	virtual bool SelectDefaultSubobject(SSubobjectEditor& SubobjectEditor) { return false; }

	/** Decides if this UI Customization provides an overlay for the tree view */
	UE_EXPERIMENTAL(5.8, "This function is experimental for now.")
	virtual bool SupportsTreeViewOverlay(const SSubobjectEditor& Editor) const { return false; }

	/** Create an overlay for the tree view widget */
	UE_EXPERIMENTAL(5.8, "This function is experimental for now.")
	virtual TSharedRef<class SWidget> ConstructTreeViewOverlay(TWeakPtr<class SSubobjectEditor> Editor) const { return SNullWidget::NullWidget; }
};