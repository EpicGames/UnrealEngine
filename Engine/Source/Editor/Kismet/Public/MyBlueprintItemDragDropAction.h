// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintEditor.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Engine/Blueprint.h"
#include "GraphEditorDragDropAction.h"
#include "Input/DragAndDrop.h"
#include "Input/Reply.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"
#include "UObject/UnrealType.h"

#define UE_API KISMET_API

class FText;
class UBlueprint;
class UEdGraph;
struct FEdGraphSchemaAction;

/** DragDropAction class for drag and dropping an item from the My Blueprints tree (e.g., variable or function) */
class FMyBlueprintItemDragDropAction : public FGraphSchemaActionDragDropAction
{
public:
	DRAG_DROP_OPERATOR_TYPE(FMyBlueprintItemDragDropAction, FGraphSchemaActionDragDropAction)

	// FGraphEditorDragDropAction interface
	UE_API virtual FReply DroppedOnAction(TSharedRef<FEdGraphSchemaAction> Action) override;
	UE_API virtual FReply DroppedOnCategory(FText Category) override;
	UE_API virtual void HoverTargetChanged() override;
	// End of FGraphEditorDragDropAction

	/** Set if operation is modified by alt */
	void SetAltDrag(bool InIsAltDrag) {	bAltDrag = InIsAltDrag; }

	/** Set if operation is modified by the ctrl key */
	void SetCtrlDrag(bool InIsCtrlDrag) { bControlDrag = InIsCtrlDrag; }

protected:
	 /** Constructor */
	UE_API FMyBlueprintItemDragDropAction();

	virtual UBlueprint* GetSourceBlueprint() const
	{
		return nullptr;
	}

	/** Helper method to see if we're dragging in the same blueprint */
	bool IsFromBlueprint(UBlueprint* InBlueprint) const
	{
		return GetSourceBlueprint() == InBlueprint;
	}

	UE_API void SetFeedbackMessageError(const FText& Message);
	UE_API void SetFeedbackMessageOK(const FText& Message);

protected:
	/** Was ctrl held down at start of drag */
	bool bControlDrag;
	/** Was alt held down at the start of drag */
	bool bAltDrag;
	/** Analytic delegate to track node creation */
	FNodeCreationAnalytic AnalyticCallback;
};

#undef UE_API
