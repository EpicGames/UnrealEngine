// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Engine/MemberReference.h"
#include "FindInBlueprintManager.h"
#include "Misc/ScopedSlowTask.h"
#include "ScopedTransaction.h"
#include "Stats/Stats.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "TickableEditorObject.h"

class FScopedTransaction;
class UBlueprint;
struct FScopedSlowTask;

/** A helper class for Replacing Variable references in blueprints */
struct FReplaceNodeReferencesHelper : FTickableEditorObject
{
	/** Constructs a ReplaceNodeReferencesHelper with the specified Source and Replacement variables */
	FReplaceNodeReferencesHelper(const FMemberReference& Source, const FMemberReference& Replacement, UBlueprint* InBlueprint);

	/** Constructs a ReplaceNodeReference Helper with the specified Source and Replacement variables */
	FReplaceNodeReferencesHelper(FMemberReference&& Source, FMemberReference&& Replacement, UBlueprint* InBlueprint);

	virtual ~FReplaceNodeReferencesHelper();

	/** Triggers a FindInBlueprints cache of all blueprints, and submits a search query when it is done, this could take a while */
	void BeginFindAndReplace(const FSimpleDelegate& InOnCompleted = FSimpleDelegate());

	/** Callback to replace references when Search is completed */
	void ReplaceReferences(TArray<FImaginaryFiBDataSharedPtr>& InRawDataList);

	/** Returns true when the Find/Replace operation is finished */
	bool IsCompleted() const { return bCompleted; }

	/** Returns the MemberReference for the Source */
	const FMemberReference& GetSource() const { return SourceReference; }

	/** Returns the MemberReference for the Replacement */
	const FMemberReference& GetReplacement() const { return ReplacementReference; }

	/** Keeps a scoped transaction alive while it does it's job (call with nullptr to reset if needed) */
	const void SetTransaction(TSharedPtr<FScopedTransaction> InTransaction);

	//~ Begin FTickableEditorObject Interface
	virtual bool IsTickable() const override;

	virtual void Tick(float DeltaSeconds) override;

	virtual TStatId GetStatId() const override;
	//~ End FTickableEditorObject Interface

	/**
	 * Helper function to replace references
	 *
	 * @param InSource		Variable reference to replace
	 * @param InReplacement Variable reference to replace with
	 * @param InBlueprint   Blueprint that InReplacement belongs to
	 * @param InRawDataList Raw find in blueprints search results
	 */
	static void ReplaceReferences(const FMemberReference& InSource, const FMemberReference& InReplacement, UBlueprint* InBlueprint, TArray<FImaginaryFiBDataSharedPtr>& InRawDataList);

private:

	/** Submits a search query and calls ReplaceReferences when complete */
	void OnSubmitSearchQuery();

	/** Updates the active search and ends it when complete */
	void UpdateSearchQuery();

	/** The source variable to be replaced */
	FMemberReference SourceReference;

	/** The variable to replace references to the source with */
	FMemberReference ReplacementReference;

	/** The Class that owns the variables */
	UBlueprint* Blueprint;

	/** Callback for when the FindAndReplace is completed */
	FSimpleDelegate OnCompleted;

	/** Used when starting a full find and replace task */
	TUniquePtr<FScopedSlowTask> SlowTask;

	/** In Progress Search Object */
	TSharedPtr<FStreamSearch> StreamSearch;

	/** Transaction if the user wants us to keep one alive */
	TSharedPtr<FScopedTransaction> Transaction;

	/** Whether a search has been started and finished */
	bool bCompleted;
};