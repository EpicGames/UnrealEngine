// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "ReviewComments.generated.h"

#define UE_API KISMET_API
class FJsonObject;

UENUM()
enum class EReviewCommentTaskState : uint8
{
	// regular comment (not a task)
	Comment,
	// unaddressed task
	Open,
	// completed task
	Addressed,
	// verified completed task (documentation is limited so I'm not positive on this one)
	Verified,
};

// swarm stores shelved file reviews separately from comments on submitted CLs
UENUM()
enum class EReviewTopicType : uint8
{
	// shelved for review
	Review,
	// change is already submitted
	Change,

	// this should never happen
	Unrecognised
};

struct FReviewCommentContext
{
public:
	// file being commented on
	TOptional<FString> File;
	// what version of the file this comment is associated with
	TOptional<int32> Version;
	// (text only) preview of text being commented on
	TOptional<TArray<FString>> Content;
	// (text only) line number on left panel of this comment
	TOptional<int32> LeftLine;
	// (text only) line number on right panel of this comment
	TOptional<int32> RightLine;

	// id of the comment that this one is replying to
	TOptional<int32> ReplyTo;

	// category this comment will appear under in the diff results
	TOptional<FString> Category;
	
	UE_API FString ToJson() const;
	UE_API TSharedPtr<FJsonObject> ToJsonObject() const;
	static UE_API FReviewCommentContext FromJson(const FString& Json);
	static UE_API FReviewCommentContext FromJson(const TSharedPtr<FJsonObject>& JsonObject);
};

struct FReviewTopic
{
public:
	FString ChangelistNum;
	EReviewTopicType TopicType;

	// returns topic in format "TopicType/ChangelistNum"
	UE_API FString ToString() const;
	static UE_API FReviewTopic FromString(const FString& ReviewTopic);
	bool operator==(const FReviewTopic& Other) const
	{
		return ChangelistNum == Other.ChangelistNum;
	}
};

inline uint32 GetTypeHash(const FReviewTopic& Topic)
{
	return GetTypeHash(Topic.ChangelistNum);
}

struct FReviewComment
{
public:	
	// info about CL being commented on
	TOptional<FReviewTopic> Topic;
	// unique id associated with this comment
	TOptional<int32> CommentID;
	// the comment text
	TOptional<FString> Body;
	
	// names of all users who have liked this comment
	TOptional<TSet<FString>> Likes;
	// names of all users who have marked the comment as read
	TOptional<TSet<FString>> ReadBy;
	// signifies whether this is a task and if it's been addressed
	TOptional<EReviewCommentTaskState> TaskState;
	// comment author
	TOptional<FString> User;
	// when the comment was created
	TOptional<FDateTime> CreatedTime;
	// when the comment was last edited
	TOptional<FDateTime> EditedTime;
	// when the comment was created
	TOptional<FDateTime> UpdatedTime;
	
	// metadata about the comment
	FReviewCommentContext Context;
	
	// flags
	bool bIsClosed : 1 = false;

	UE_API FString ToJson() const;
	UE_API TSharedPtr<FJsonObject> ToJsonObject() const;
	static UE_API FReviewComment FromJson(const FString& Json);
	static UE_API FReviewComment FromJson(const TSharedPtr<FJsonObject>& JsonObject);
};

class IReviewCommentAPI : public TSharedFromThis<IReviewCommentAPI>
{
public:
	DECLARE_DELEGATE_TwoParams(OnGetCommentsComplete, const TArray<FReviewComment>&, const FString& /*ErrorMessage*/)
	DECLARE_DELEGATE_TwoParams(OnPostCommentComplete, const FReviewComment&, const FString& /*ErrorMessage*/)
	DECLARE_DELEGATE_TwoParams(OnEditCommentComplete, const FReviewComment&, const FString& /*ErrorMessage*/)
	DECLARE_DELEGATE_TwoParams(OnGetReviewTopicForCLComplete, const FReviewTopic&, const FString& /*ErrorMessage*/)
	
	virtual ~IReviewCommentAPI() = default;

	// returns the name of the user logged in
	virtual FString GetUsername() const = 0;
	
	virtual void GetComments(const FReviewTopic& Topic, const OnGetCommentsComplete& OnComplete) const = 0;
	// Body must be set; username will get set by PostComment
	virtual void PostComment(FReviewComment& Comment, const OnPostCommentComplete& OnComplete, bool bSilenceNotification = false) const = 0;
	// all unset members will be left unchanged.
	virtual void EditComment(const FReviewComment& Comment, const OnEditCommentComplete& OnComplete, bool bSilenceNotification = false) const = 0;

	// retrieves the review associated with a CL.
	virtual void GetReviewTopicForCL(const FString &ChangelistNum, const OnGetReviewTopicForCLComplete& OnComplete) const = 0;
	// retrieves the review associated with a CL. If there is none, it creates a new review and returns that.
	virtual void GetOrCreateReviewTopicForCL(const FString &ChangelistNum, const OnGetReviewTopicForCLComplete& OnComplete) const = 0;
};

#undef UE_API
