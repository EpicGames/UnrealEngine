// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "DiffResults.h"
#include "DiffUtils.h"
#include "GraphEditor.h"
#include "HAL/Platform.h"
#include "IAssetTypeActions.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "SKismetInspector.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

#define UE_API KISMET_API

class FAsyncDetailViewDiff;
class SLinkableScrollBar;
class FBlueprintDifferenceTreeEntry;
class FSpawnTabArgs;
class FTabManager;
class FText;
class FUICommandList;
class IDiffControl;
class SBox;
class SMyBlueprint;
class SOverlay;
class SSplitter;
class SWidget;
class SWindow;
class UBlueprint;
class UEdGraph;
class UEdGraphNode;
class UEdGraphPin;
class UObject;
struct FGraphToDiff;
template <typename ItemType> class SListView;

enum class EAssetEditorCloseReason : uint8;

/** Individual Diff item shown in the list of diffs */
struct FDiffResultItem : public TSharedFromThis<FDiffResultItem>
{
	FDiffResultItem(FDiffSingleResult InResult) : Result(InResult){}

	FDiffSingleResult Result;

	TSharedRef<SWidget> KISMET_API GenerateWidget() const;
};

DECLARE_DELEGATE_OneParam(FOnMyBlueprintActionSelected, UObject*);

namespace DiffWidgetUtils
{
	KISMET_API void SelectNextRow(SListView< TSharedPtr< struct FDiffSingleResult> >& ListView, const TArray< TSharedPtr< struct FDiffSingleResult > >& ListViewSource );
	KISMET_API void SelectPrevRow(SListView< TSharedPtr< struct FDiffSingleResult> >& ListView, const TArray< TSharedPtr< struct FDiffSingleResult > >& ListViewSource );
	KISMET_API bool HasNextDifference(SListView< TSharedPtr< struct FDiffSingleResult> >& ListView, const TArray< TSharedPtr< struct FDiffSingleResult > >& ListViewSource);
	KISMET_API bool HasPrevDifference(SListView< TSharedPtr< struct FDiffSingleResult> >& ListView, const TArray< TSharedPtr< struct FDiffSingleResult > >& ListViewSource);
}

/** Panel used to display the blueprint */
struct FDiffPanel
{
	UE_API FDiffPanel();

	/** Initializes the panel, can be moved into constructor if diff and merge clients are made more uniform: */
	UE_API void InitializeDiffPanel();

	/** Generate a panel for NewGraph diffed against OldGraph */
	UE_API void GeneratePanel(UEdGraph* NewGraph, UEdGraph* OldGraph);
	
	/** Generate a panel that displays the Graph and reflects the items in the DiffResults */
	UE_API void GeneratePanel(UEdGraph* Graph, TSharedPtr<TArray<FDiffSingleResult>> DiffResults, TAttribute<int32> FocusedDiffResult);

	/** Generate the 'MyBlueprint' widget, which is private to this module */
	UE_API TSharedRef<class SWidget> GenerateMyBlueprintWidget();

	/** Called when user hits keyboard shortcut to copy nodes */
	UE_API void CopySelectedNodes();

	/** Gets whatever nodes are selected in the Graph Editor */
	UE_API FGraphPanelSelectionSet GetSelectedNodes() const;

	/** Can user copy any of the selected nodes? */
	UE_API bool CanCopyNodes() const;

	/** Functions used to focus/find a particular change in a diff result */
	UE_API void FocusDiff(UEdGraphPin& Pin);
	UE_API void FocusDiff(UEdGraphNode& Node);

	UE_API TSharedRef<SWidget> GetMyBlueprintWidget() const;
	UE_API TSharedRef<SWidget> GetDetailsWidget() const;

	/** The blueprint that owns the graph we are showing */
	const UBlueprint*				Blueprint;

	/** The box around the graph editor, used to change the content when new graphs are set */
	TSharedPtr<SBox>				GraphEditorBox;

	/** The actual my blueprint panel, used to regenerate the panel when the new graphs are set */
	TSharedPtr<class SMyBlueprint>	MyBlueprint;

	/** The details view associated with the graph editor */
	TSharedPtr<class SKismetInspector>	DetailsView;

	/** The graph editor which does the work of displaying the graph */
	TWeakPtr<class SGraphEditor>	GraphEditor;

	/** Revision information for this blueprint */
	FRevisionInfo					RevisionInfo;

	/** True if we should show a name identifying which asset this panel is displaying */
	bool							bShowAssetName;

	/** The widget that contains the revision info in graph mode */
	TSharedPtr<SWidget>				OverlayGraphRevisionInfo;

	TSharedPtr<SLinkableScrollBar> DetailScrollbar;
private:
	/** Command list for this diff panel */
	TSharedPtr<FUICommandList> GraphEditorCommands;
};

/* Visual Diff between two Blueprints*/
class  SBlueprintDiff: public SCompoundWidget
{
public:
	DECLARE_DELEGATE_TwoParams( FOpenInDefaults, const class UBlueprint* , const class UBlueprint* );

	SLATE_BEGIN_ARGS( SBlueprintDiff ){}
			SLATE_ARGUMENT( const class UBlueprint*, BlueprintOld )
			SLATE_ARGUMENT( const class UBlueprint*, BlueprintNew )
			SLATE_ARGUMENT( struct FRevisionInfo, OldRevision )
			SLATE_ARGUMENT( struct FRevisionInfo, NewRevision )
			SLATE_ARGUMENT( bool, ShowAssetNames )
			SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);
	UE_API virtual ~SBlueprintDiff();

	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/** Called when a new Graph is clicked on by user */
	UE_API void OnGraphChanged(FGraphToDiff* Diff);

	/** Called when blueprint is modified */
	UE_API void OnBlueprintChanged(UBlueprint* InBlueprint);

	/** Called when user clicks on a new graph list item */
	UE_API void OnGraphSelectionChanged(TSharedPtr<FGraphToDiff> Item, ESelectInfo::Type SelectionType);

	/** Called when user clicks on an entry in the listview of differences */
	UE_API void OnDiffListSelectionChanged(TSharedPtr<struct FDiffResultItem> TheDiff);

	/** Helper function for generating an empty widget */
	static UE_API TSharedRef<SWidget> DefaultEmptyPanel();

	/** Helper function to create a window that holds a diff widget */
	static UE_API TSharedPtr<SWindow> CreateDiffWindow(FText WindowTitle, const UBlueprint* OldBlueprint, const UBlueprint* NewBlueprint, const struct FRevisionInfo& OldRevision, const struct FRevisionInfo& NewRevision);

	/** Helper function to create a window that holds a diff widget that defaults a window title*/
	static UE_API TSharedPtr<SWindow> CreateDiffWindow(const UBlueprint* OldBlueprint, const UBlueprint* NewBlueprint, const FRevisionInfo& OldRevision, const FRevisionInfo& NewRevision, const UClass* BlueprintClass);

protected:
	/** Called when user clicks button to go to next difference */
	UE_API void NextDiff();

	/** Called when user clicks button to go to prev difference */
	UE_API void PrevDiff();

	/** Called to determine whether we have a list of differences to cycle through */
	UE_API bool HasNextDiff() const;
	UE_API bool HasPrevDiff() const;

	/** Find the FGraphToDiff that displays the graph with GraphPath relative path */
	UE_API FGraphToDiff* FindGraphToDiffEntry(const FString& GraphPath);

	/** Bring these revisions of graph into focus on main display*/
	UE_API void FocusOnGraphRevisions(FGraphToDiff* Diff);

	/** Create a list item entry graph that exists in at least one of the blueprints */
	UE_API void CreateGraphEntry(class UEdGraph* GraphOld, class UEdGraph* GraphNew);
		
	/** Disable the focus on a particular pin */
	UE_API void DisablePinDiffFocus();

	/** User toggles the option to lock the views between the two blueprints */
	UE_API void OnToggleLockView();
	
	/** User toggles the option to change the split view mode betwwen vertical and horizontal */
	UE_API void OnToggleSplitViewMode();

	/** Reset the graph editor, called when user switches graphs to display*/
	UE_API void ResetGraphEditors();

	/** Get the image to show for the toggle lock option*/
	UE_API FSlateIcon GetLockViewImage() const;
	
	/** Get the image to show for the toggle split view mode option*/
	UE_API FSlateIcon GetSplitViewModeImage() const;

	/** List of graphs to diff, are added to panel last */
	TArray<TSharedPtr<FGraphToDiff>> Graphs;

	/** Get Graph editor associated with this Graph */
	UE_API FDiffPanel& GetDiffPanelForNode(UEdGraphNode& Node);

	/** Event handler that updates the graph view when user selects a new graph */
	UE_API void HandleGraphChanged(FGraphToDiff* Diff);
	
	/** Function used to generate the list of differences and the widgets needed to calculate that list */
	UE_API void GenerateDifferencesList();

	/** Checks if a graph is valid for diffing */
	UE_API bool IsGraphDiffNeeded(class UEdGraph* InGraph) const;

	/** Called when editor may need to be closed */
	UE_API void OnCloseAssetEditor(UObject* Asset, EAssetEditorCloseReason CloseReason);

	struct FDiffControl
	{
		FDiffControl()
		: Widget()
		, DiffControl(nullptr)
		{
		}

		TSharedPtr<SWidget> Widget;
		TSharedPtr< class IDiffControl > DiffControl;
	};

	UE_API FDiffControl GenerateBlueprintTypePanel();
	UE_API FDiffControl GenerateMyBlueprintPanel();
	UE_API FDiffControl GenerateGraphPanel();
	UE_API FDiffControl GenerateDefaultsPanel();
	UE_API FDiffControl GenerateClassSettingsPanel();
	UE_API FDiffControl GenerateComponentsPanel();
	UE_API FDiffControl GenerateGeneralFileCommentEntries();

	UE_API TSharedRef<SOverlay> GenerateGraphWidgetForPanel(FDiffPanel& OutDiffPanel) const;
	UE_API TSharedRef<SBox> GenerateRevisionInfoWidgetForPanel(TSharedPtr<SWidget>& OutGeneratedWidget,const FText& InRevisionText) const;

	/** Accessor and event handler for toggling between diff view modes (defaults, components, graph view, interface, macro): */
	UE_API void SetCurrentMode(FName NewMode);
	FName GetCurrentMode() const { return CurrentMode; }
	UE_API void OnModeChanged(const FName& InNewViewMode) const;

	UE_API void UpdateTopSectionVisibility(const FName& InNewViewMode) const;

	FName CurrentMode;

	/*The two panels used to show the old & new revision*/ 
	FDiffPanel				PanelOld, PanelNew;

	/* Diff info about the kismet panels in blueprint graph diff view */
	TSharedPtr<FAsyncDetailViewDiff> GraphDetailDiff;
	
	/** If the two views should be locked */
	bool	bLockViews;

	/** If the view on Graph Mode should be divided vertically */
	bool bVerticalSplitGraphMode = true;

	/** Contents widget that we swap when mode changes (defaults, components, etc) */
	TSharedPtr<SBox> ModeContents;

	TSharedPtr<SSplitter> TopRevisionInfoWidget;

	TSharedPtr<SSplitter> DiffGraphSplitter;
	
	TSharedPtr<SSplitter> GraphToolBarWidget;

	friend struct FListItemGraphToDiff;

	/** We can't use the global tab manager because we need to instance the diff control, so we have our own tab manager: */
	TSharedPtr<FTabManager> TabManager;

	/** Tree of differences collected across all panels: */
	TArray< TSharedPtr<class FBlueprintDifferenceTreeEntry> > PrimaryDifferencesList;

	/** List of all differences, cached so that we can iterate only the differences and not labels, etc: */
	TArray< TSharedPtr<class FBlueprintDifferenceTreeEntry> > RealDifferences;

	/** Tree view that displays the differences, cached for the buttons that iterate the differences: */
	TSharedPtr< STreeView< TSharedPtr< FBlueprintDifferenceTreeEntry > > > DifferencesTreeView;

	/** Stored references to widgets used to display various parts of a blueprint, from the mode name */
	TMap<FName, FDiffControl> ModePanels;

	/** A pointer to the window holding this */
	TWeakPtr<SWindow> WeakParentWindow;

	FDelegateHandle AssetEditorCloseDelegate;

	/** To make diffing more accurate and friendly, UBlueprint::CategorySorting gets modified. this will revert to the
	 *  old version when the window closes */
	class FScopedCategorySortChange
	{
	public:
		~FScopedCategorySortChange();
		void SetBlueprint(UBlueprint* Blueprint);
	private:
		UBlueprint* Blueprint = nullptr;
		TArray<FName> Backup = {};
	} ScopedCategorySortChange;
};


#undef UE_API
