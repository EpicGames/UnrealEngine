// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintEditor.h"
#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Misc/Optional.h"
#include "Serialization/Archive.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API KISMET_API

class STextComboBox;
class SWidget;
class UBlueprint;
class UObject;
class UWorld;
struct FGeometry;
struct FSlateBrush;

struct FBlueprintDebugObjectInstance
{
	/** Actual object to debug, can be null */
	TWeakObjectPtr<UObject> ObjectPtr;

	/** Friendly label for object to debug */
	FString ObjectLabel;

	/** Raw object path of spawned PIE object, this is not a SoftObjectPath because we don't want it to get fixed up */
	FString ObjectPath;

	/** Object path to object in the editor, will only be set for static objects */
	FString EditorObjectPath;

	bool bPinned;

	FBlueprintDebugObjectInstance(TWeakObjectPtr<UObject> InPtr, const FString& InLabel)
		: ObjectPtr(InPtr)
		, ObjectLabel(InLabel)
		, bPinned(false)
	{
	}

	/** Returns true if this is the special entry for no specific object */
	bool IsEmptyObject() const
	{
		return ObjectPath.IsEmpty();
	}

	/** If this has no editor path, it was spawned */
	bool IsSpawnedObject() const 
	{
		return !ObjectPath.IsEmpty() && EditorObjectPath.IsEmpty();
	}
	
	/** If editor and object path are the same length because there's no prefix, this is the editor object */
	bool IsEditorObject() const 
	{
		return !ObjectPath.IsEmpty() && ObjectPath.Len() == EditorObjectPath.Len();
	}
};

class SBlueprintEditorSelectedDebugObjectWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SBlueprintEditorSelectedDebugObjectWidget){}
	SLATE_END_ARGS()

	UE_API virtual ~SBlueprintEditorSelectedDebugObjectWidget();

	UE_API void Construct(const FArguments& InArgs, TSharedPtr<FBlueprintEditor> InBlueprintEditor);

	// SWidget interface
	UE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	// End of SWidget interface

	/** Adds an object to the list of debug choices */
	UE_API void AddDebugObject(UObject* TestObject, const FString& TestObjectName = FString());

private:
	UBlueprint* GetBlueprintObj() const { return BlueprintEditor.Pin()->GetBlueprintObj(); }

	/** Creates a list of all debug objects **/
	UE_API void GenerateDebugObjectInstances(bool bRestoreSelection);

	/** Generate list of active PIE worlds to debug **/
	UE_API void GenerateDebugWorldNames(bool bRestoreSelection);

	/** Refresh the widget. **/
	UE_API void OnRefresh();

	/** Returns the entry for the current debug actor */
	UE_API TSharedPtr<FBlueprintDebugObjectInstance> GetDebugObjectInstance() const;

	/** Returns the name of the current debug actor */
	UE_API TSharedPtr<FString> GetDebugWorldName() const;

	/** Handles the selection changed event for the debug actor combo box */
	UE_API void DebugObjectSelectionChanged(TSharedPtr<FBlueprintDebugObjectInstance> NewSelection, ESelectInfo::Type SelectInfo);

	UE_API void DebugWorldSelectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);

	/** Called when user clicks button to select the current object being debugged */
	UE_API void SelectedDebugObject_OnClicked();

	/** Returns true if a debug actor is currently selected */
	UE_API bool IsDebugObjectSelected() const;

	UE_API EVisibility IsDebugWorldComboVisible() const;

	/* Returns the string to indicate no debug object is selected */
	UE_API static const FString& GetNoDebugString();

	UE_API const FString& GetDebugAllWorldsString() const;

	UE_API TSharedRef<SWidget> OnGetActiveDetailSlotContent(bool bChangedToHighDetail);

	/** Helper method to construct a debug object label string */
	UE_API FString MakeDebugObjectLabel(UObject* TestObject, bool bAddContextIfSelectedInEditor, bool bAddSpawnedContext) const;

	/** Fills in data for a specific instance */
	UE_API void FillDebugObjectInstance(TSharedPtr<FBlueprintDebugObjectInstance> Instance);

	/** Called to create a widget for each debug object item */
	UE_API TSharedRef<SWidget> CreateDebugObjectItemWidget(TSharedPtr<FBlueprintDebugObjectInstance> InItem);

	/** Returns the combo button label to use for the currently-selected debug object item */
	UE_API FText GetSelectedDebugObjectTextLabel() const;

	/** Handle selection of debug object through debug object access. */
	UE_API void HandleOnDebugObjectAccess(UObject* InDebugObject);

	/** Sorts the debug objects based on pinned status. */
	UE_API void SortDebugObjects();

	/** Returns either a pinned or unpinned image based on object pin status. */
	const FSlateBrush* GetDebugObjectItemWidgetPinBrush(TWeakPtr<FBlueprintDebugObjectInstance> InInstanceWeak) const;

	/** Returns Visible if the item is pinned or hovered. */
	EVisibility GetDebugObjectItemWidgetPinVisibility(TWeakPtr<FBlueprintDebugObjectInstance> InInstanceWeak, TWeakPtr<SWidget> InRowWidgetWeak) const;

	/** Pins or unpins the debug object. */
	FReply OnDebugObjectItemWidgetPinMouseDown(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent, TWeakPtr<FBlueprintDebugObjectInstance> InInstanceWeak);

private:
	/** Pointer back to the blueprint editor tool that owns us */
	TWeakPtr<FBlueprintEditor> BlueprintEditor;

	static TSharedRef<FBlueprintDebugObjectInstance> NoDebugInstance;

	/** Lists of actors of a given blueprint type and their names */
	TArray<TSharedPtr<FBlueprintDebugObjectInstance>> DebugObjects;

	/** PIE worlds that we can debug */
	TArray< TWeakObjectPtr<UWorld> > DebugWorlds;
	TArray< TSharedPtr<FString> > DebugWorldNames;

	/** Widget containing the names of all possible debug actors. This is a "generic" SComboBox rather than an STextComboBox so that we can customize the label on the combo button widget. */
	TSharedPtr<SComboBox<TSharedPtr<FBlueprintDebugObjectInstance>>> DebugObjectsComboBox;

	TSharedPtr<STextComboBox> DebugWorldsComboBox;

	TWeakObjectPtr<UObject> LastObjectObserved;
};

#undef UE_API
