// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "SourceControlOperations.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "Widgets/SWidget.h"
#include "WorkflowOrientedApp/SModeWidget.h"

#define UE_API KISMET_API

class FBlueprintEditor;
class FExtender;
class FMenuBuilder;
class FToolBarBuilder;
class FUICommandInfo;
class SWidget;
class UBlueprintEditorToolMenuContext;
class UToolMenu;
struct FToolMenuContext;

/**
 * Kismet menu
 */
class FKismet2Menu
{
public:
	static UE_API void SetupBlueprintEditorMenu(const FName MainMenuName);
	
protected:
	static UE_API void FillFileMenuBlueprintSection(UToolMenu* Menu);

	static UE_API void FillEditMenu(UToolMenu* Menu);

	static UE_API void FillViewMenu(UToolMenu* Menu);

	static UE_API void FillDebugMenu(UToolMenu* Menu);

	static UE_API void FillDeveloperMenu(UToolMenu* Menu);
};


class FFullBlueprintEditorCommands : public TCommands<FFullBlueprintEditorCommands>
{
public:
	/** Constructor */
	FFullBlueprintEditorCommands() 
		: TCommands<FFullBlueprintEditorCommands>("FullBlueprintEditor", NSLOCTEXT("Contexts", "FullBlueprintEditor", "Full Blueprint Editor"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}

	/** Compile the blueprint */
	TSharedPtr<FUICommandInfo> Compile;
	TSharedPtr<FUICommandInfo> SaveOnCompile_Never;
	TSharedPtr<FUICommandInfo> SaveOnCompile_SuccessOnly;
	TSharedPtr<FUICommandInfo> SaveOnCompile_Always;
	TSharedPtr<FUICommandInfo> JumpToErrorNode;

	/** Switch between modes in the blueprint editor */
	TSharedPtr<FUICommandInfo> SwitchToScriptingMode;
	TSharedPtr<FUICommandInfo> SwitchToBlueprintDefaultsMode;
	TSharedPtr<FUICommandInfo> SwitchToComponentsMode;
	
	/** Edit Blueprint global options */
	TSharedPtr<FUICommandInfo> EditGlobalOptions;
	TSharedPtr<FUICommandInfo> EditClassDefaults;

	/** Initialize commands */
	virtual void RegisterCommands() override;
};



class FBlueprintEditorToolbar : public TSharedFromThis<FBlueprintEditorToolbar>
{
public:
	FBlueprintEditorToolbar(TSharedPtr<FBlueprintEditor> InBlueprintEditor)
		: BlueprintEditor(InBlueprintEditor) {}

	UE_API void AddBlueprintGlobalOptionsToolbar(UToolMenu* InMenu, bool bRegisterViewport = false);
	UE_API void AddCompileToolbar(UToolMenu* InMenu);
	UE_API void AddNewToolbar(UToolMenu* InMenu);
	UE_API void AddScriptingToolbar(UToolMenu* InMenu);
	UE_API void AddDebuggingToolbar(UToolMenu* InMenu);

	/** Returns the current status icon for the blueprint being edited */
	UE_API FSlateIcon GetStatusImage() const;

	/** Returns the current status as text for the blueprint being edited */
	UE_API FText GetStatusTooltip() const;

	/** Diff current blueprint against the specified revision */
	static UE_API void DiffAgainstRevision(class UBlueprint* Current, int32 OldRevision);

	static UE_API TSharedRef<SWidget> MakeDiffMenu(const UBlueprintEditorToolMenuContext* InContext);

protected:
	/** Pointer back to the blueprint editor tool that owns us */
	TWeakPtr<FBlueprintEditor> BlueprintEditor;
};

#undef UE_API
