// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "SlateFwd.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateConstants.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

#define UE_API KISMET_API

class ITableRow;
class SComboButton;
class SSearchBox;
class SSuggestionTextBox;
class SWidget;

/** A custom widget class that provides support for Blueprint namespace entry and/or selection. */
class SBlueprintNamespaceEntry : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnNamespaceSelected, const FString&);
	DECLARE_DELEGATE_OneParam(FOnGetNamespacesToExclude, TSet<FString>&);

	SLATE_BEGIN_ARGS(SBlueprintNamespaceEntry)
	: _Font(FCoreStyle::Get().GetFontStyle(TEXT("NormalFont")))
	, _AllowTextEntry(true)
	{}
		/** Current namespace value. */
		SLATE_ARGUMENT(FString, CurrentNamespace)

		/** Font color and opacity. */
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)

		/** Allow text input to manually set arbitrary values. */
		SLATE_ARGUMENT(bool, AllowTextEntry)

		/** Allow external code to set custom combo button content. */
		SLATE_NAMED_SLOT(FArguments, ButtonContent)

		/** Called when a valid namespace is either entered or selected. */
		SLATE_EVENT(FOnNamespaceSelected, OnNamespaceSelected)

		/** Called to allow external code to exclude one or more namespaces from the list. */
		SLATE_EVENT(FOnGetNamespacesToExclude, OnGetNamespacesToExclude)

		/** Tooltip used for excluded namespaces that are visible in the selection drop-down. */
		SLATE_ATTRIBUTE(FText, ExcludedNamespaceTooltipText)

	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

	/**
	 * Set the current namespace to the given identifier.
	 * 
	 * @param InNamespace	New namespace identifier. May be an empty string.
	 */
	UE_API void SetCurrentNamespace(const FString& InNamespace);

protected:
	struct FPathTreeNodeItem
	{
		FString NodePath;
		TArray<TSharedPtr<FPathTreeNodeItem>> ChildNodes;
		bool bIsSelectable = true;
	};
	
	UE_API void OnTextChanged(const FText& InText);
	UE_API void OnTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit);
	UE_API void OnShowingSuggestions(const FString& InputText, TArray<FString>& OutSuggestions);
	UE_API TSharedRef<SWidget> OnGetNamespaceTreeMenuContent();
	UE_API TSharedRef<ITableRow> OnGenerateRowForNamespaceTreeItem(TSharedPtr<FPathTreeNodeItem> Item, const TSharedRef<STableViewBase>& OwnerTable);
	UE_API void OnGetChildrenForNamespaceTreeItem(TSharedPtr<FPathTreeNodeItem> Item, TArray<TSharedPtr<FPathTreeNodeItem>>& OutChildren);
	UE_API void OnNamespaceTreeFilterTextChanged(const FText& InText);
	UE_API void OnNamespaceTreeSelectionChanged(TSharedPtr<FPathTreeNodeItem> Item, ESelectInfo::Type SelectInfo);
	UE_API bool OnIsNamespaceTreeItemSelectable(TSharedPtr<FPathTreeNodeItem> Item) const;
	UE_API FText GetCurrentNamespaceText() const;

	UE_API void PopulateNamespaceTree();
	UE_API void SelectNamespace(const FString& InNamespace);
	UE_API void ExpandAllTreeViewItems(const TArray<TSharedPtr<FPathTreeNodeItem>>* NodeListPtr = nullptr);
	UE_API const TSharedPtr<FPathTreeNodeItem>* FindTreeViewNode(const FString& NodePath, const TArray<TSharedPtr<FPathTreeNodeItem>>* NodeListPtr = nullptr) const;

private:
	FString CurrentNamespace;
	TArray<FString> AllRegisteredPaths;
	TSet<FString> ExcludedTreeViewPaths;
	TArray<TSharedPtr<FPathTreeNodeItem>> RootNodes;

	TSharedPtr<SComboButton> ComboButton;
	TSharedPtr<SSuggestionTextBox> TextBox;
	TSharedPtr<SSearchBox> SearchBox;
	TSharedPtr<STreeView<TSharedPtr<FPathTreeNodeItem>>> TreeView;

	FOnNamespaceSelected OnNamespaceSelected;
	FOnGetNamespacesToExclude OnGetNamespacesToExclude;
	TAttribute<FText> ExcludedNamespaceTooltipText;

	static UE_API float NamespaceListBorderPadding;
	static UE_API float NamespaceListMinDesiredWidth;
};

#undef UE_API
