// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "ISourceControlOperation.h"
#include "ISourceControlProvider.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "SourceControlOperations.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API KISMET_API

class FUpdateStatus;
class SVerticalBox;
class UBlueprint;
struct FRevisionInfo;

class SBlueprintRevisionMenu : public SCompoundWidget
{
	DECLARE_DELEGATE_OneParam(FOnRevisionSelected, FRevisionInfo const&)

public:
	SLATE_BEGIN_ARGS(SBlueprintRevisionMenu)
		: _bIncludeLocalRevision(false)
	{}
		SLATE_ARGUMENT(bool, bIncludeLocalRevision)
		SLATE_EVENT(FOnRevisionSelected, OnRevisionSelected)
	SLATE_END_ARGS()

	UE_API ~SBlueprintRevisionMenu();

	UE_API void Construct(const FArguments& InArgs, UBlueprint const* Blueprint);

private: 
	/** Delegate used to determine the visibility 'in progress' widgets */
	UE_API EVisibility GetInProgressVisibility() const;
	/** Delegate used to determine the visibility of the cancel button */
	UE_API EVisibility GetCancelButtonVisibility() const;

	/** Delegate used to cancel a source control operation in progress */
	UE_API FReply OnCancelButtonClicked() const;
	/** Callback for when the source control operation is complete */
	UE_API void OnSourceControlQueryComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);

	/**  */
	bool bIncludeLocalRevision;
	/**  */
	FOnRevisionSelected OnRevisionSelected;
	/** The name of the file we want revision info for */
	FString Filename;
	/** The box we are using to display our menu */
	TSharedPtr<SVerticalBox> MenuBox;
	/** The source control operation in progress */
	TSharedPtr<FUpdateStatus, ESPMode::ThreadSafe> SourceControlQueryOp;
	/** The state of the SCC query */
	uint32 SourceControlQueryState;
};

#undef UE_API
