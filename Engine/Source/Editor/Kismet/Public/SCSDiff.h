// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "DiffUtils.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "Widgets/SWidget.h"

#define UE_API KISMET_API

class FSCSEditorTreeNode;
class FSubobjectEditorTreeNode;
class SKismetInspector;
class SSubobjectBlueprintEditor;
class SWidget;
class UBlueprint;

/** Struct to support diffing the component tree for a blueprint */
class FSCSDiff
{
public:
	UE_API FSCSDiff(const class UBlueprint* InBlueprint);

	UE_API void HighlightProperty(FName VarName, FPropertySoftPath Property);
	UE_API TSharedRef< SWidget > TreeWidget();

	UE_API TArray< FSCSResolvedIdentifier > GetDisplayedHierarchy() const;

	const UBlueprint* GetBlueprint() const { return Blueprint; }

protected:
	UE_API void OnSCSEditorUpdateSelectionFromNodes(const TArray<TSharedPtr<FSubobjectEditorTreeNode>>& SelectedNodes);
	UE_API void OnSCSEditorHighlightPropertyInDetailsView(const class FPropertyPath& InPropertyPath);

private:
	TSharedPtr< class SWidget > ContainerWidget;
	TSharedPtr< class SSubobjectBlueprintEditor > SubobjectEditor;
	TSharedPtr< class SKismetInspector > Inspector;

	/** Blueprint we are inspecting */
	UBlueprint* Blueprint = nullptr;
};

#undef UE_API
