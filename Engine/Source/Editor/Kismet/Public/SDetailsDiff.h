// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AsyncDetailViewDiff.h"
#include "DiffUtils.h"
#include "IAssetTypeActions.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWindow.h"
#include "Widgets/SCompoundWidget.h"

enum class ETreeDiffResult;
class FSpawnTabArgs;
class FTabManager;
class IDiffControl;
class FUICommandList;
class SDetailsSplitter;
class FDetailsDiffControl;
enum class EAssetEditorCloseReason : uint8;

/** Panel used to display the details */
struct FDetailsDiffPanel
{
	/** The asset that owns the details view we are showing */
	const UObject*				Object = nullptr;

	/** Revision information for this asset */
	FRevisionInfo					RevisionInfo;

	/** True if we should show a name identifying which asset this panel is displaying */
	bool							bShowAssetName = true;

	/** The widget that contains the revision info in graph mode */
	TSharedPtr<SWidget>				OverlayRevisionInfo;
private:
	/** Command list for this diff panel */
	TSharedPtr<FUICommandList> GraphEditorCommands;
};

/* Visual Diff between two Assets */
class  SDetailsDiff: public SCompoundWidget
{
public:

	DECLARE_DELEGATE_OneParam(FOnCustomizeDetailsWidget, const TSharedRef<IDetailsView>&)
	DECLARE_DELEGATE_RetVal_OneParam(FLinearColor, FRowHighlightColor, const TUniquePtr<FAsyncDetailViewDiff::DiffNodeType>&)
	DECLARE_DELEGATE_RetVal_OneParam(bool, FShouldHighlightRow, const TUniquePtr<FAsyncDetailViewDiff::DiffNodeType>&)

	SLATE_BEGIN_ARGS(SDetailsDiff) {}
		SLATE_ARGUMENT_DEPRECATED(const class UObject*, AssetOld, 5.5, "Use OldAsset instead")
		SLATE_ARGUMENT_DEPRECATED(const class UObject*, AssetNew, 5.5, "Use NewAsset instead")
		SLATE_ARGUMENT(const class UObject*, OldAsset)
		SLATE_ARGUMENT(const class UObject*, NewAsset)
		SLATE_ARGUMENT(struct FRevisionInfo, OldRevision)
		SLATE_ARGUMENT(struct FRevisionInfo, NewRevision)
		SLATE_ARGUMENT(bool, ShowAssetNames)
		SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)
		SLATE_ARGUMENT(TSharedPtr<IDetailPropertyExtensionHandler>, ExtensionHandler)

		SLATE_EVENT(FOnCustomizeDetailsWidget, OnCustomizeDetailsWidget)
		SLATE_EVENT(DiffUtils::FOnGenerateCustomDiffEntries, OnGenerateCustomDiffEntries)
		SLATE_EVENT(DiffUtils::FOnGenerateCustomDiffEntryWidget, OnGenerateCustomDiffEntryWidget)
		SLATE_EVENT(DiffUtils::FOnOrganizeDiffEntries, OnOrganizeDiffEntries)
		SLATE_EVENT(FShouldHighlightRow, ShouldHighlightRow)
		SLATE_EVENT(FRowHighlightColor, RowHighlightColor)
	SLATE_END_ARGS()

	KISMET_API void Construct(const FArguments& InArgs);
	KISMET_API virtual ~SDetailsDiff();

	KISMET_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/** Called when user clicks on an entry in the listview of differences */
	KISMET_API void OnDiffListSelectionChanged(TSharedPtr<struct FDiffResultItem> TheDiff);

	/** Helper function for generating an empty widget */
	static KISMET_API TSharedRef<SWidget> DefaultEmptyPanel();

	/** Helper function to create a window that holds a diff widget */
	static KISMET_API TSharedRef<SDetailsDiff> CreateDiffWindow(FText WindowTitle, const UObject* OldObject, const UObject* NewObject, const struct FRevisionInfo& OldRevision, const struct FRevisionInfo& NewRevision);

	/** Helper function to create a window that holds a diff widget (default window title) */
	static KISMET_API TSharedRef<SDetailsDiff> CreateDiffWindow(const UObject* OldObject, const UObject* NewObject, const FRevisionInfo& OldRevision, const FRevisionInfo& NewRevision, const UClass* ObjectClass);

	/** Enables actions like "Choose Left" and "Choose Right" which will modify the OutputObject */
	KISMET_API void SetOutputObject(UObject* OutputObject);
	
	/** Return a serialized buffer of change requests made by the user */
	KISMET_API UObject* GetOutputObject() const;

	/** Returns whether SetOutputObject was called with a valid object */
	KISMET_API bool IsOutputEnabled() const;

	KISMET_API void ReportMergeConflicts(const TMap<FString, TMap<FPropertySoftPath, ETreeDiffResult>>& Conflicts);

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnWindowClosedEvent, TSharedRef<SDetailsDiff>)
	FOnWindowClosedEvent OnWindowClosedEvent;
protected:
	/** Called when user clicks button to go to next difference */
	KISMET_API void NextDiff();

	/** Called when user clicks button to go to prev difference */
	KISMET_API void PrevDiff();

	/** Called to determine whether we have a list of differences to cycle through */
	KISMET_API bool HasNextDiff() const;
	KISMET_API bool HasPrevDiff() const;
	
	/** Function used to generate the list of differences and the widgets needed to calculate that list */
	KISMET_API void GenerateDifferencesList();

	/** Called when editor may need to be closed */
	KISMET_API void OnCloseAssetEditor(UObject* Asset, EAssetEditorCloseReason CloseReason);

	KISMET_API void OnObjectReplaced(const FCoreUObjectDelegates::FReplacementObjectMap& Replacements);

	struct FDiffControl
	{
		FDiffControl()
		: Widget()
		, DiffControl(nullptr)
		{
		}

		TSharedPtr<SWidget> Widget;
		TSharedPtr< class IDiffControl > DiffControl;
	};

	KISMET_API FDiffControl GenerateDetailsPanel(const TFunction<const UObject*(const UObject*)>& Redirector = nullptr);

	KISMET_API TSharedRef<SBox> GenerateRevisionInfoWidgetForPanel(TSharedPtr<SWidget>& OutGeneratedWidget,const FText& InRevisionText) const;

	/** Accessor and event handler for toggling between diff view modes (defaults, components, graph view, interface, macro): */
	KISMET_API void SetCurrentMode(FName NewMode);
	KISMET_API void RefreshCurrentModePanel();
	FName GetCurrentMode() const { return CurrentMode; }
	KISMET_API void OnModeChanged(const FName& InNewViewMode) const;

	KISMET_API void UpdateTopSectionVisibility(const FName& InNewViewMode) const;

	FName CurrentMode;

	/** The two panels used to show the old & new revision */
	FDetailsDiffPanel PanelOld;
	FDetailsDiffPanel PanelNew;
	
	/** If set, actions like "Choose Left" and "Choose Right" will be enabled and modify the OutputObject */
	UObject* OutputObject = nullptr; // clone that can be mutated by user actions

	DECLARE_MULTICAST_DELEGATE(FOnSetOutputObjectEvent)
	FOnSetOutputObjectEvent OnOutputObjectSetEvent;

	FOnCustomizeDetailsWidget OnCustomizeDetailsWidget;
	DiffUtils::FOnGenerateCustomDiffEntries OnGenerateCustomDiffEntries;
	DiffUtils::FOnGenerateCustomDiffEntryWidget OnGenerateCustomDiffEntryWidget;
	DiffUtils::FOnOrganizeDiffEntries OnOrganizeDiffEntries;
	FShouldHighlightRow ShouldHighlightRow;
	FRowHighlightColor RowHighlightColor;

	/** Contents widget that we swap when mode changes (defaults, components, etc) */
	TSharedPtr<SBox> ModeContents;

	TSharedPtr<SSplitter> TopRevisionInfoWidget;

	/** We can't use the global tab manager because we need to instance the diff control, so we have our own tab manager: */
	TSharedPtr<FTabManager> TabManager;

	/** Tree of differences collected across all panels: */
	TArray< TSharedPtr<class FBlueprintDifferenceTreeEntry> > PrimaryDifferencesList;

	/** List of all differences, cached so that we can iterate only the differences and not labels, etc: */
	TArray< TSharedPtr<class FBlueprintDifferenceTreeEntry> > RealDifferences;

	/** Tree view that displays the differences, cached for the buttons that iterate the differences: */
	TSharedPtr< STreeView< TSharedPtr< FBlueprintDifferenceTreeEntry > > > DifferencesTreeView;

	/** Stored references to widgets used to display various parts of an object, from the mode name */
	TMap<FName, FDiffControl> ModePanels;

	/** A pointer to the window holding this */
	TWeakPtr<SWindow> WeakParentWindow;

	FDelegateHandle AssetEditorCloseDelegate;

	TMap<FString, TMap<FPropertySoftPath, ETreeDiffResult>> MergeConflicts;
};


