// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Framework/SlateDelegates.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Misc/Optional.h"
#include "SlateFwd.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "UObject/Object.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

#define UE_API KISMET_API

class ITableRow;
class SSearchBox;
class STableViewBase;
class SWidget;

//////////////////////////////////////////////////////////////////////////
// SFilterableObjectList

class SFilterableObjectList : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SFilterableObjectList ){}
	SLATE_END_ARGS()
public:
	UE_API void Construct(const FArguments& InArgs);

	UE_API void InternalConstruct();

protected:
	// Interface for derived classes to implement
	UE_API virtual void RebuildObjectList();
	UE_API virtual FString GetSearchableText(UObject* Object);
	
	/** Return value for GenerateRowForObject */
	struct FListRow
	{
		FListRow( const TSharedRef<SWidget>& InWidget, const FOnDragDetected& OnDragDetected )
		: Widget(InWidget)
		, OnDragDetected_Handler( OnDragDetected )
		{
		}

		/** The widget to place into the table row. */
		TSharedRef<SWidget> Widget;
		/** The Delegate to invoke when the user started dragging a row. */
		FOnDragDetected OnDragDetected_Handler;
	};

	/** Make a table row for the list of filterable widgets */
	UE_API virtual FListRow GenerateRowForObject(UObject* Object);
	

	// End of interface for derived classes to implement
protected:
	UE_API void RefilterObjectList();

	UE_API TSharedRef<ITableRow> OnGenerateTableRow(UObject* InData, const TSharedRef<STableViewBase>& OwnerTable);

	UE_API FReply OnRefreshButtonClicked();

	UE_API void OnFilterTextChanged(const FText& InFilterText);

	UE_API EVisibility GetFilterStatusVisibility() const;
	UE_API FText GetFilterStatusText() const;

	UE_API bool IsFilterActive() const;
	UE_API void ReapplyFilter();
protected:
	/** Widget containing the object list */
	TSharedPtr< SListView<UObject*> > ObjectListWidget;

	/* Widget containing the filtering text box */
	TSharedPtr< SSearchBox > FilterTextBoxWidget;

	/** List of objects that can be shown */
	TArray<UObject*> LoadedObjectList;

	/** List of objects to show that have passed the keyword filtering */
	TArray<UObject*> FilteredObjectList;
};

#undef UE_API
