// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "K2Node_CallFunction.h"
#include "Layout/Visibility.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

class FBlueprintEditor;
class FString;
class ITableRow;
class STableViewBase;
class STextComboBox;
class SWidget;
class SWindow;
class UBlueprint;
class UK2Node_CallFunction;

struct FFixupSelfContextItem
{
public:
	/** Enum to describe how the node will be fixed up (or not) */
	enum class EFixupStrategy 
	{ 
		DoNothing, 
		CreateNewFunction, 
		RemoveNode
	};
	
	/** Constructs a FixupSelfContextItem for a given function name */
	FFixupSelfContextItem(FName Function) : FuncName(Function) {}

	/** Callback to create a widget for the Item */
	TSharedRef<SWidget> CreateWidget(TArray<TSharedPtr<FString>>& InFixupOptions);

	/** Function this item relates to */
	FName FuncName;
	
	/** Nodes that reference this function */
	TArray<UK2Node_CallFunction*> Nodes;
	
	/** Combo box for user to decide fixup strategy */
	TSharedPtr<STextComboBox> ComboBox;
};

/** Widget to prompt the user to fixup self context on bad K2Node_CallFunction Pastes */
class SFixupSelfContextDialog : public SCompoundWidget
{
private:
	typedef TSharedPtr<FFixupSelfContextItem> FListViewItem;
public:
	SLATE_BEGIN_ARGS(SFixupSelfContextDialog)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TArray<UK2Node_CallFunction*>& InNodesToFixup, UBlueprint* InFromBP, const FBlueprintEditor* InBlueprintEditorPtr, bool bInOtherPastedNodes);

	/**
	 * Creates a Confirmation Modal, this function will not return until the Dialog is closed
	 *
	 * @param NodesToFixup Array of unresolved function nodes
	 * @param InFromBP	The blueprint that the function was copied from
	 * @param BlueprintEditorPtr the BlueprintEditor this modal was created by
	 * @param bOtherPastedNodes Whether other nodes were involved in this paste operation
	 * @returns True if the user confirmed the operation, False if it was canceled
	 */
	static bool CreateModal(const TArray<UK2Node_CallFunction*>& NodesToFixup, UBlueprint* InFromBP, const FBlueprintEditor* BlueprintEditorPtr, bool bOtherPastedNodes);

private:
	/** Generates a row for a List Item */
	TSharedRef<ITableRow> OnGenerateRow(FListViewItem Item, const TSharedRef<STableViewBase>& OwnerTable);

	/** Visibility callback for "Nothing pasted" warning */
	EVisibility GetNoneWarningVisibility() const;

	/** Closes the window */
	FReply CloseWindow(bool bConfirmed);

	/** 
	* Creates the missing function or event in the blueprint we are copying to. If the FuncToFix came from
	* a custom event, then create a matching one here. 
	* 
	* @param FuncToFix		The function that needs to be created that was on the old BP, but not this one
	*/
	void CreateMissingFunctions(FListViewItem FuncToFix);

	/** The list of unique functions that need to be fixed */
	TArray<FListViewItem> FunctionsToFixup;

	/** The find results to modify */
	TArray<UK2Node_CallFunction*> NodesToFixup;

	/** The blueprint for the new context */
	UBlueprint* Blueprint;

	/** The blueprint editor this modal was created by */
	const FBlueprintEditor* BlueprintEditor;

	/** The blueprint that the incoming function is coming from */
	UBlueprint* FromBP;

	/** Window to close when dialog completed */
	TSharedPtr<SWindow> MyWindow;

	/** Options for fixup */
	TArray<TSharedPtr<FString>> Options;

	/** Whether other nodes were involved in this paste operation */
	bool bOtherNodes;

	/** Whether the user confirmed or canceled the action */
	bool bOutConfirmed;
};