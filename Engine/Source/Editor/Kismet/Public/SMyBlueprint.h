// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintEditor.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2_Actions.h"
#include "Framework/Commands/Commands.h"
#include "Input/Reply.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "SGraphActionMenu.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

#define UE_API KISMET_API

class FBlueprintEditor;
class FMenuBuilder;
class FUICommandInfo;
class FUICommandList;
class SComboButton;
class SKismetInspector;
class SSearchBox;
class SWidget;
class UBlueprint;
class UEdGraph;
class UFunction;
class UObject;
class UUserDefinedEnum;
class UUserDefinedStruct;
struct FComponentEventConstructionData;
struct FEdGraphSchemaAction;
struct FEdGraphSchemaAction_K2Struct;
struct FGeometry;
struct FGraphActionListBuilderBase;
struct FGraphActionNode;
struct FGraphActionSort;
struct FKeyEvent;
struct FPointerEvent;
struct FPropertyChangedEvent;
struct FReplaceNodeReferencesHelper;

class FMyBlueprintCommands : public TCommands<FMyBlueprintCommands>
{
public:
	/** Constructor */
	FMyBlueprintCommands() 
		: TCommands<FMyBlueprintCommands>(TEXT("MyBlueprint"), NSLOCTEXT("Contexts", "My Blueprint", "My Blueprint"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}

	// Basic operations
	TSharedPtr<FUICommandInfo> OpenGraph;
	TSharedPtr<FUICommandInfo> OpenGraphInNewTab;
	TSharedPtr<FUICommandInfo> OpenExternalGraph;
	TSharedPtr<FUICommandInfo> FocusNode;
	TSharedPtr<FUICommandInfo> FocusNodeInNewTab;
	TSharedPtr<FUICommandInfo> ImplementFunction;
	TSharedPtr<FUICommandInfo> DeleteEntry;
	TSharedPtr<FUICommandInfo> PasteVariable;
	TSharedPtr<FUICommandInfo> PasteLocalVariable;
	TSharedPtr<FUICommandInfo> PasteFunction;
	TSharedPtr<FUICommandInfo> PasteMacro;
	TSharedPtr<FUICommandInfo> GotoNativeVarDefinition;
	TSharedPtr<FUICommandInfo> MoveVariableToParent;
	TSharedPtr<FUICommandInfo> MoveFunctionToParent;
	// Add New Item
	/** Initialize commands */
	virtual void RegisterCommands() override;
};

class SMyBlueprint : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SMyBlueprint ) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, TWeakPtr<FBlueprintEditor> InBlueprintEditor, const UBlueprint* InBlueprint = nullptr);
	UE_API ~SMyBlueprint();

	void SetInspector( TSharedPtr<SKismetInspector> InInspector ) { Inspector = InInspector ; }

	/* SWidget interface */
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime);
	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);

	/* Reset the last pin type settings to default. */
	UE_API void ResetLastPinType();

	/** Refreshes the graph action menu */
	UE_API void Refresh();
	void SetFocusedGraph(UEdGraph* InEdGraph) { EdGraph = InEdGraph; }
	
	/** Accessor for getting the current selection as a K2 graph */
	UE_API FEdGraphSchemaAction_K2Graph* SelectionAsGraph() const;

	/** Accessor for getting the current selection as a K2 enum */
	UE_API FEdGraphSchemaAction_K2Enum* SelectionAsEnum() const;

	/** Accessor for getting the current selection as a K2 enum */
	UE_API FEdGraphSchemaAction_K2Struct* SelectionAsStruct() const;

	/** Accessor for getting the current selection as a K2 var */
	UE_API FEdGraphSchemaAction_K2Var* SelectionAsVar() const;
	
	/** Accessor for getting the current selection as a K2 delegate */
	UE_API FEdGraphSchemaAction_K2Delegate* SelectionAsDelegate() const;

	/** Accessor for getting the current selection as a K2 event */
	UE_API FEdGraphSchemaAction_K2Event* SelectionAsEvent() const;
	
	/** Accessor for getting the current selection as a K2 Input Action */
	UE_API FEdGraphSchemaAction_K2InputAction* SelectionAsInputAction() const;

	/** Accessor for getting the current selection as a K2 local var */
	UE_API FEdGraphSchemaAction_K2LocalVar* SelectionAsLocalVar() const;

	/** Accessor for getting the current selection as a K2 blueprint base variable */
	UE_API FEdGraphSchemaAction_BlueprintVariableBase* SelectionAsBlueprintVariable() const;

	/** Accessor for determining if the current selection is a category*/
	UE_API bool SelectionIsCategory() const;
	
	UE_API void EnsureLastPinTypeValid();

	/** Gets the last pin type selected by this widget, or by the function editor */
	FEdGraphPinType& GetLastPinTypeUsed() {EnsureLastPinTypeValid(); return LastPinType;}
	FEdGraphPinType& GetLastFunctionPinTypeUsed() {EnsureLastPinTypeValid(); return LastFunctionPinType;}

	/** Accessor the blueprint object from the main editor */
	UBlueprint* GetBlueprintObj() const {return Blueprint;}

	/** Gets whether we are showing user variables only or not */
	bool ShowUserVarsOnly() const { return !IsShowingInheritedVariables(); }

	/** Gets our parent blueprint editor */
	TWeakPtr<FBlueprintEditor> GetBlueprintEditor() {return BlueprintEditorPtr;}

	/**
	 * Fills the supplied array with the currently selected objects
	 * @param OutSelectedItems The array to fill.
	 */
	UE_API void GetSelectedItemsForContextMenu(TArray<FComponentEventConstructionData>& OutSelectedItems) const;

	/** Called to reset the search filter */
	UE_API void OnResetItemFilter();

	/** Selects an item by name in either the main graph action menu or the local one */
	UE_API void SelectItemByName(const FName& ItemName, ESelectInfo::Type SelectInfo = ESelectInfo::Direct, int32 SectionId = INDEX_NONE, bool bIsCategory = false);

	/** Clears the selection in the graph action menus */
	UE_API void ClearGraphActionMenuSelection();

	/** Initiates a rename on the selected action node, if possible */
	UE_API void OnRequestRenameOnActionNode();

	/** Expands any category with the associated name */
	UE_API void ExpandCategory(const FText& CategoryName);

	/** Move the category before the target category */
	UE_API bool MoveCategoryBeforeCategory( const FText& CategoryToMove, const FText& TargetCategory );
	
	/** Callbacks for Paste Commands */
	UE_API void OnPasteGeneric();
	UE_API bool CanPasteGeneric();
private:
	/** Creates widgets for the graph schema actions */
	UE_API TSharedRef<SWidget> OnCreateWidgetForAction(struct FCreateWidgetForActionData* const InCreateData);

	/** Callback used to populate all actions list in SGraphActionMenu */
	UE_API void CollectAllActions(FGraphActionListBuilderBase& OutAllActions);
	UE_API void CollectStaticSections(TArray<int32>& StaticSectionIDs);
	UE_API void AddEventForFunctionGraph(UEdGraph* EdGraph, int32 const SectionId, FGraphActionSort& SortList, const FText& ParentCategory = FText::GetEmpty(), bool bAddChildGraphs = true) const;
	UE_API void GetChildGraphs(UEdGraph* EdGraph, int32 const SectionId, FGraphActionSort& SortList, const FText& ParentCategory = FText::GetEmpty()) const;
	UE_API void GetChildEvents(UEdGraph const* EdGraph, int32 const SectionId, FGraphActionSort& SortList, const FText& ParentCategory = FText::GetEmpty(), bool bInAddChildGraphs = true) const;
	UE_API void GetLocalVariables(FGraphActionSort& SortList) const;
	
	/** Handles the visibility of the local action list */
	UE_API EVisibility GetLocalActionsListVisibility() const;

	/** Callbacks for the graph action menu */
	UE_API FReply OnActionDragged(const TArray< TSharedPtr<FEdGraphSchemaAction> >& InActions, const FPointerEvent& MouseEvent);
	UE_API FReply OnCategoryDragged(const FText& InCategory, const FPointerEvent& MouseEvent);
	UE_API void OnActionSelected(const TArray< TSharedPtr<FEdGraphSchemaAction> >& InActions);
	UE_API void OnActionSelectedHelper(TSharedPtr<FEdGraphSchemaAction> InAction, TWeakPtr< FBlueprintEditor > InBlueprintEditor, UBlueprint* Blueprint, TSharedRef<SKismetInspector> Inspector);
	UE_API void OnGlobalActionSelected(const TArray< TSharedPtr<FEdGraphSchemaAction> >& InActions, ESelectInfo::Type InSelectionType);
	UE_API void OnActionDoubleClicked(const TArray< TSharedPtr<FEdGraphSchemaAction> >& InActions);
	UE_API void ExecuteAction(TSharedPtr<FEdGraphSchemaAction> InAction);
	UE_API TSharedPtr<SWidget> OnContextMenuOpening();

	UE_API TSharedRef<SWidget> CreateAddNewMenuWidget();
	UE_API void BuildAddNewMenu(FMenuBuilder& MenuBuilder);
	UE_API TSharedRef<SWidget> CreateAddToSectionButton(int32 InSectionID, TWeakPtr<SWidget> WeakRowWidget, FText AddNewText, FName MetaDataTag);

	UE_API void OnCategoryNameCommitted(const FText& InNewText, ETextCommit::Type InTextCommit, TWeakPtr< struct FGraphActionNode > InAction );
	UE_API bool CanRequestRenameOnActionNode(TWeakPtr<struct FGraphActionNode> InSelectedNode) const;
	UE_API FText OnGetSectionTitle( int32 InSectionID );
	UE_API TSharedRef<SWidget> OnGetSectionWidget( TSharedRef<SWidget> RowWidget, int32 InSectionID );
	UE_API EVisibility OnGetSectionTextVisibility(TWeakPtr<SWidget> RowWidget, int32 InSectionID) const;
	UE_API TSharedRef<SWidget> OnGetFunctionListMenu();
	UE_API void BuildOverridableFunctionsMenu(FMenuBuilder& MenuBuilder);
	UE_API FReply OnAddButtonClickedOnSection(int32 InSectionID);
	UE_API bool CanAddNewElementToSection(int32 InSectionID) const;

	UE_API bool HandleActionMatchesName(FEdGraphSchemaAction* InAction, const FName& InName) const;

	/** Support functions for checkbox to manage displaying user variables only */
	UE_API bool IsShowingInheritedVariables() const;
	UE_API void OnToggleShowInheritedVariables();

	/** Support functions for view options for Show Empty Sections */
	UE_API void OnToggleShowEmptySections();
	UE_API bool IsShowingEmptySections() const;
	
	/** Support functions for view options for Show Replicated Variables only */
	UE_API void OnToggleShowReplicatedVariablesOnly();
	UE_API bool IsShowingReplicatedVariablesOnly() const;

	/** Support functions for view options for bAlwaysShowInterfacesInOverrides blueprint editor setting */
	UE_API void OnToggleAlwaysShowInterfacesInOverrides();
	UE_API bool GetAlwaysShowInterfacesInOverrides() const;

	/** Support functions for view options for bShowParentClassInOverrides blueprint editor setting */
	UE_API void OnToggleShowParentClassInOverrides();
	UE_API bool GetShowParentClassInOverrides() const;

	/** Support functions for view options for bShowAccessSpecifier blueprint editor setting */
	UE_API void OnToggleShowAccessSpecifier();
	UE_API bool GetShowAccessSpecifier() const;

	/** Helper function to open the selected graph */
	UE_API void OpenGraph(FDocumentTracker::EOpenDocumentCause InCause, bool bOpenExternalGraphInNewEditor = false);

	/**
	* Check if the override of a given function is most likely desired as a blueprint function 
	* or as an event. 
	* 
	* @param OverrideFunc	Desired function to override
	* 
	* @return	True if the function is desired as a function, false if desired as an event
	*/
	UE_API bool IsImplementationDesiredAsFunction(const UFunction* OverrideFunc) const;

	/** Callbacks for commands */
	UE_API void OnOpenGraph();
	UE_API void OnOpenGraphInNewTab();
	UE_API void OnOpenExternalGraph();
	UE_API bool CanOpenGraph() const;
	UE_API bool CanOpenExternalGraph() const;
	UE_API bool CanFocusOnNode() const;
	UE_API void OnFocusNode();
	UE_API void OnFocusNodeInNewTab();
	UE_API void OnImplementFunction();
	UE_API void ImplementFunction(TSharedPtr<FEdGraphSchemaAction_K2Graph> GraphAction);
	UE_API void ImplementFunction(FEdGraphSchemaAction_K2Graph* GraphAction);
	UE_API bool CanImplementFunction() const;
	UE_API void OnFindReference(bool bSearchAllBlueprints, const EGetFindReferenceSearchStringFlags Flags);
	UE_API bool CanFindReference() const;
	UE_API void OnFindAndReplaceReference();
	UE_API bool CanFindAndReplaceReference() const;
	UE_API void OnDeleteEntry();
	UE_API bool CanDeleteEntry() const;
	UE_API FReply OnAddNewLocalVariable();
	UE_API bool CanRequestRenameOnActionNode() const;
	UE_API bool IsDuplicateActionVisible() const;
	UE_API bool CanDuplicateAction() const;
	UE_API void OnDuplicateAction();
	UE_API void GotoNativeCodeVarDefinition();
	UE_API bool IsNativeVariable() const;
	UE_API void OnMoveToParent();
	UE_API bool CanMoveVariableToParent() const;
	UE_API bool CanMoveFunctionToParent() const;
	UE_API void OnCopy();
	UE_API bool CanCopy() const;
	UE_API void OnCut();
	UE_API bool CanCut() const;
	UE_API void OnPasteVariable();
	UE_API void OnPasteLocalVariable();
	UE_API bool CanPasteVariable() const;
	UE_API bool CanPasteLocalVariable() const;
	UE_API void OnPasteFunction();
	UE_API bool CanPasteFunction() const;
	UE_API void OnPasteMacro();
	UE_API bool CanPasteMacro() const;

	/** Gets the currently selected Category or returns default category name */
	UE_API FText GetPasteCategory() const;

	/** Callback when the filter is changed, forces the action tree(s) to filter */
	UE_API void OnFilterTextChanged( const FText& InFilterText );

	/** Callback for the action trees to get the filter text */
	UE_API FText GetFilterText() const;

	/** Checks if the selected action has context menu */
	UE_API bool SelectionHasContextMenu() const;

	/** Update Node Create Analytic */
	UE_API void UpdateNodeCreation();

	/** Returns the displayed category, if any, of a graph */
	UE_API FText GetGraphCategory(UEdGraph* InGraph) const;

	/** Helper function to delete a graph in the MyBlueprint window */
	UE_API void OnDeleteGraph(UEdGraph* InGraph, EEdGraphSchemaAction_K2Graph::Type);

	/** Helper function to delete a delegate in the MyBlueprint window */
	UE_API void OnDeleteDelegate(FEdGraphSchemaAction_K2Delegate* InDelegateAction);

	UE_API UEdGraph* GetFocusedGraph() const;

	/** Delegate to hook us into non-structural Blueprint object post-change events */
	UE_API void OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent);

	/** Helper function indicating whehter we're in editing mode, and can modify the target blueprint */
	UE_API bool IsEditingMode() const;

	/** Determine whether an FEdGraphSchemaAction is associated with an event */
	static UE_API bool IsAnInterfaceEvent(FEdGraphSchemaAction_K2Graph* InAction);

private:
	/** List of UI Commands for this scope */
	TSharedPtr<FUICommandList> CommandList;

	/** Pointer back to the blueprint editor that owns us */
	TWeakPtr<FBlueprintEditor> BlueprintEditorPtr;
	
	/** Graph Action Menu for displaying all our variables and functions */
	TSharedPtr<class SGraphActionMenu> GraphActionMenu;

	/** The +Function button in the function section */
	TSharedPtr<SComboButton> FunctionSectionButton;

	/** When we rebuild the view of members, we cache (but don't display) any overridable functions for user in popup menus. */
	TArray< TSharedPtr<FEdGraphSchemaAction_K2Graph> > OverridableFunctionActions;

	/** When we refresh the list of functions we cache off the implemented ones to ask questions for overridable functions. */
	TSet<FName> ImplementedFunctionCache;

	/** The last pin type used (including the function editor last pin type) */
	FEdGraphPinType LastPinType;
	FEdGraphPinType LastFunctionPinType;

	/** Enums created from 'blueprint' level */
	TArray<TWeakObjectPtr<UUserDefinedEnum>> EnumsAddedToBlueprint;

	/** The filter box that handles filtering for both graph action menus. */
	TSharedPtr< SSearchBox > FilterBox;

	/** Enums created from 'blueprint' level */
	TArray<TWeakObjectPtr<UUserDefinedStruct>> StructsAddedToBlueprint;

	/** The blueprint being displayed: */
	UBlueprint* Blueprint;

	/** The Ed Graph being displayed: */
	UEdGraph* EdGraph;

	/** The Kismet Inspector used to display properties: */
	TWeakPtr<SKismetInspector> Inspector;

	/** Flag to indicate whether or not we need to refresh the panel */
	bool bNeedsRefresh;

	/** If set we'll show only replicated variables (local to a particular blueprint view). */
	bool bShowReplicatedVariablesOnly;
};

#undef UE_API
