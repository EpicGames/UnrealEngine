// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Debugging/SKismetDebugTreeView.h"
#include "EdGraph/EdGraphPin.h"
#include "Layout/Visibility.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API KISMET_API

class FText;
class SSearchBox;
class UBlueprint;
class UObject;

typedef TSharedPtr<struct FPinValueInspectorTreeViewNode> FPinValueInspectorTreeViewNodePtr;

/**
 * Inspects the referenced pin object's underlying property value and presents it within a tree view.
 * Compound properties (e.g. structs/containers) will be broken down into a hierarchy of child nodes.
 */
class SPinValueInspector : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPinValueInspector)
	{}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

	/** Whether the search filter UI should be visible. */
	UE_API bool ShouldShowSearchFilter() const;

	/** Gets the current watched pin reference */
	UE_API const FEdGraphPinReference& GetPinRef() const;

	friend class FPinValueInspectorTooltip;

protected:
	/** @return Visibility of the search box filter widget. */
	UE_API virtual EVisibility GetSearchFilterVisibility() const;

	/** Passes SearchText through to tree view */
	UE_API virtual void OnSearchTextChanged(const FText& InSearchText);

	/** requests the constrained box be resized */
	UE_API virtual void OnExpansionChanged(FDebugTreeItemPtr InItem, bool bItemIsExpanded);

	/** Adds a unique tree item to the TreeView. */
	UE_API virtual void AddTreeItemUnique(const FDebugTreeItemPtr& Item);

	/** Adds relevant pins to the tree view */
	UE_API virtual void PopulateTreeView();

	/** Adds a single pin to the tree view */
	UE_API void AddPinToTreeView(const UEdGraphPin* Pin, UBlueprint* Blueprint);

	/** Marks tree view for updating currently filtered items. */
	UE_API void RequestUpdateFilteredItems();

	/** Sets the current watched pin */
	UE_API void SetPinRef(const FEdGraphPinReference& InPinRef);

private:
	/** Holds a weak reference to the target pin. */
	FEdGraphPinReference PinRef;

	/** The instance that's currently selected as the debugging target. */
	TWeakObjectPtr<UObject> TargetObject;

	/** Presents a hierarchical display of the inspected value along with any sub-values as children. */
	TSharedPtr<SKismetDebugTreeView> TreeViewWidget;

	/** The box that handles resizing of the Tree View */
	TSharedPtr<class SPinValueInspector_ConstrainedBox> ConstrainedBox;
};

/** class holding functions to spawn a pin value inspector tooltip */
class FPinValueInspectorTooltip
{
public:
	/** Moves the tooltip to the new location */
	UE_API void MoveTooltip(const UE::Slate::FDeprecateVector2DParameter& InNewLocation);

	/** 
	 * Dismisses the current tooltip, if it is not currently hovered
	 * NOTE: Pin a shared ptr before calling this function 
	 * 
	 * @param bForceDismiss	true to dismiss regardless of hover & tooltip ownership 
	 */
	UE_API void TryDismissTooltip(bool bForceDismiss = false);

	/** @returns whether or not the tooltip can close */
	UE_API bool TooltipCanClose() const;

	/** @returns whether this tooltip is the host for a context menu */
	UE_API bool TooltipHostsMenu() const;

private:
	/** Dismisses the current tooltip (internal implementation) */
	UE_API void DismissTooltip();

public:
	/** Summons a new tooltip in the shared window. If provided, uses a custom implementation of a pin value inspector widget. */
	static UE_API TWeakPtr<FPinValueInspectorTooltip> SummonTooltip(FEdGraphPinReference InPinRef, TSharedPtr<SPinValueInspector> InContentWidgetOverride = { });

	/** Release the references to the static widget shared pointers */
	static UE_API void ShutdownTooltip();

	/** Default inspector widget in the tooltip */
	static UE_API TSharedPtr<SPinValueInspector> ValueInspectorWidget;

private:
	/** Handles Creating a custom tooltip window for all PinValueInspector tooltips */
	static UE_API void CreatePinValueTooltipWindow();

	/** A reusable tooltip window for PinValueInspector */
	static UE_API TSharedPtr<class SWindow> TooltipWindow;

	/** Tooltip widget housed in the window */
	static UE_API TSharedPtr<class SToolTip> TooltipWidget;

	/** The current "live" tooltip */
	static UE_API TSharedPtr<FPinValueInspectorTooltip> Instance;
};

#undef UE_API
