// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintEditor.h"
#include "Components/ActorComponent.h"
#include "Components/SceneComponent.h"
#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Engine/SCS_Node.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/SlateDelegates.h"
#include "Framework/Views/ITypedTableView.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Clipping.h"
#include "Layout/Visibility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "SComponentClassCombo.h"
#include "ScopedTransaction.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "Templates/TypeHash.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SToolTip.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

#define UE_API KISMET_API

class FDragDropEvent;
class FExtender;
class FMenuBuilder;
class FSCSEditorTreeNode;
class FUICommandList;
class ISCSEditorUICustomization;
class ITableRow;
class SHeaderRow;
class SHorizontalBox;
class SInlineEditableTextBlock;
class SSCSEditor;
class SScrollBar;
class SSearchBox;
class SToolTip;
class SVerticalBox;
class SWidget;
class UBlueprint;
class UChildActorComponent;
class UClass;
class UObject;
class UPrimitiveComponent;
class USCS_Node;
class USimpleConstructionScript;
class UToolMenu;
struct EventData;
struct FGeometry;
struct FKeyEvent;
struct FPointerEvent;
struct FSlateBrush;
template <typename FuncType> class TFunctionRef;

// SCS editor tree node pointer types
using FSCSEditorTreeNodePtrType = TSharedPtr<class FSCSEditorTreeNode>;
using FSCSEditorActorNodePtrType = TSharedPtr<class FSCSEditorTreeNodeActorBase>;
using FSCSEditorChildActorNodePtrType = TSharedPtr<class FSCSEditorTreeNodeChildActor>;

/**
 * FSCSEditorTreeNode
 *
 * Wrapper class for nodes displayed in the SCS (Simple Construction Script) editor tree widget.
 * 
 * NOTE: Mostly deprecated, switch to FSubobjectEditorTreeNode
 */
class FSCSEditorTreeNode : public TSharedFromThis<FSCSEditorTreeNode>
{
public:
	/** Delegate for when the context menu requests a rename */
	DECLARE_DELEGATE(FOnRenameRequested);

	enum ENodeType
	{
		ComponentNode,
		RootActorNode,
		SeparatorNode,
		ChildActorNode,
	};

	/**
	 * Constructs an empty tree node.
	 */
	UE_API FSCSEditorTreeNode(FSCSEditorTreeNode::ENodeType InNodeType);

	/**
	* @return The name to identify this node.
	*/
	UE_API virtual FName GetNodeID() const;

	/**
	 * @return The name of the variable represented by this node.
	 */
	UE_API virtual FName GetVariableName() const;
	/**
	 * @return The string to be used in the tree display.
	 */
	UE_API virtual FString GetDisplayString() const;
	/**
	* @return The name of this node in text.
	*/
	UE_API virtual FText GetDisplayName() const;
	/**
	 * @return The SCS node that is represented by this object, or NULL if there is no associated SCS node.
	 */
	UE_API virtual class USCS_Node* GetSCSNode() const;
	/**
	 * @param ActualEditedBlueprint currently edited blueprint
	 * @note Derived classes should override GetOrCreateEditableObjectForBlueprint().
	 * @return The component template that can be editable for actual class.
	 */
	inline UActorComponent* GetOrCreateEditableComponentTemplate(UBlueprint* ActualEditedBlueprint) const
	{
		// @TODO - Deprecate this public API in favor of GetEditableObjectForBlueprint().
		return Cast<UActorComponent>(GetOrCreateEditableObjectForBlueprint(ActualEditedBlueprint));
	}
	/**
	 * Finds the component instance represented by this node contained within a given Actor instance.
	 *
	 * @param InActor The Actor instance to use as the container object for finding the component instance.
	 * @return The component instance represented by this node and contained within the given Actor instance, or NULL if not found.
	 */
	UE_API virtual UActorComponent* FindComponentInstanceInActor(const AActor* InActor) const;
	/**
	 * @return This object's parent node (or an invalid reference if no parent is assigned).
	 */
	FSCSEditorTreeNodePtrType GetParent() const { return ParentNodePtr; }
	/**
	 * @return The set of nodes which are parented to this node (read-only).
	 */
	const TArray<FSCSEditorTreeNodePtrType>& GetChildren() const { return Children; }
	/**
	 * @return The root of the actor subtree to which this node belongs.
	 */
	FSCSEditorActorNodePtrType GetActorRootNode() const { return ActorRootNodePtr; }
	/**
	 * Sets the actor root to the given node for this node along with any children.
	 */
	UE_API void SetActorRootNode(FSCSEditorActorNodePtrType InActorNode);
	/**
	 * @return Type of node
	 */
	UE_API ENodeType GetNodeType() const;
	/**
	 * @return True if this represents an actor node type
	 */
	bool IsActorNode() const
	{
		return NodeType == ENodeType::RootActorNode || NodeType == ENodeType::ChildActorNode;
	}
	/**
	 * @return True if this represents a component node type
	 */
	bool IsComponentNode() const
	{
		return NodeType == ENodeType::ComponentNode;
	}
	/**
	 * @param	bEvenIfPendingKill	If false, nullptr will be returned if the cached component template is pending kill.
	 *								If true, it will be returned regardless (this is used for recaching the component template if the objects
	 *								have been reinstanced following construction script execution).
	 *
	 * @note	Deliberately non-virtual, for performance reasons.
	 * @warning This will not return the right component for components overridden by the inherited component handler, you need to call GetOrCreateEditableComponentTemplate instead
	 * @return	The component template or instance represented by this node, if it's a component node.
	 */
	inline UActorComponent* GetComponentTemplate(bool bEvenIfPendingKill = false) const
	{
		// @todo - Deprecate this API? For backwards-compatibility, this continues to provide non-const access to the internal object instance specifically as a component reference.
		return Cast<UActorComponent>(WeakObjectPtr.Get(bEvenIfPendingKill));
	}
	/**
	 * @param	bEvenIfPendingKill	If false, nullptr will be returned if the cached object instance is pending kill.
	 *								If true, it will be returned regardless (this is used for recaching the object if the objects
	 *								have been reinstanced following construction script execution).
	 *
	 * @note	Deliberately non-virtual, for performance reasons.
	 * @return	A read-only reference to the object represented by this node.
	 */
	template<class T>
	inline const T* GetObject(bool bEvenIfPendingKill = false) const
	{
		return Cast<T>(WeakObjectPtr.Get(bEvenIfPendingKill));
	}
	/**
	 * @param	InBlueprint			The Blueprint in which the object will be edited.
	 *
	 * @note	May not be the same as the value returned by GetObject().
	 * @return	A reference to the object represented by this node that can be modified within the given Blueprint.
	 */
	template<class T>
	inline T* GetEditableObjectForBlueprint(UBlueprint* InBlueprint) const
	{
		return Cast<T>(GetOrCreateEditableObjectForBlueprint(InBlueprint));
	}
	/**
	 * Sets the internal object instance represented by this node.
	 */
	inline void SetObject(UObject* InObject)
	{
		WeakObjectPtr = InObject;
	}
	/**
	 * @return Whether or not this node is a direct child of the given node.
	 */
	bool IsDirectlyAttachedTo(FSCSEditorTreeNodePtrType InNodePtr) const { return ParentNodePtr == InNodePtr; }
	/**
	 * @return Whether or not this node is a child (direct or indirect) of the given node.
	 */
	UE_API bool IsAttachedTo(FSCSEditorTreeNodePtrType InNodePtr) const;	

	/**
	 * Finds the closest ancestor node in the given node set.
	 *
	 * @param InNodes The given node set.
	 * @return One of the nodes from the set, or an invalid node reference if the set does not contain any ancestor nodes.
	 */
	UE_API FSCSEditorTreeNodePtrType FindClosestParent(TArray<FSCSEditorTreeNodePtrType> InNodes);

	/**
	 * Adds the given node as a child node.
	 *
	 * @param InChildNodePtr The node to add as a child node.
	 */
	UE_API virtual void AddChild(FSCSEditorTreeNodePtrType InChildNodePtr);

	/**
	 * Adds a child node for the given SCS node.
	 *
	 * @param InSCSNode The SCS node to for which to create a child node.
	 * @param bIsInheritedSCS Whether or not the given SCS node is inherited from a parent.
	 * @return A reference to the new child node or the existing child node if a match is found.
	 */
	UE_API FSCSEditorTreeNodePtrType AddChild(USCS_Node* InSCSNode, bool bIsInheritedSCS);

	/**
	 * Adds a child node for the given component template.
	 *
	 * @param InComponentTemplate The component template for which to create a child node.
	 * @return A reference to the new child node or the existing child node if a match is found.
	 */
	UE_API FSCSEditorTreeNodePtrType AddChildFromComponent(UActorComponent* InComponentTemplate);

	/**
	 * Attempts to find a reference to the child node that matches the given SCS node.
	 *
	 * @param InSCSNode The SCS node to match.
	 * @param bRecursiveSearch Whether or not to recursively search child nodes (default == false).
	 * @param OutDepth If non-NULL, the depth of the child node will be returned in this parameter on success (default == NULL).
	 * @return The child node that matches the given SCS node, or an invalid node reference if no match was found.
	 */
	UE_API FSCSEditorTreeNodePtrType FindChild(const USCS_Node* InSCSNode, bool bRecursiveSearch = false, uint32* OutDepth = NULL) const;

	/**
	 * Attempts to find a reference to the child node that matches the given component template.
	 *
	 * @param InComponentTemplate The component template instance to match.
	 * @param bRecursiveSearch Whether or not to recursively search child nodes (default == false).
	 * @param OutDepth If non-NULL, the depth of the child node will be returned in this parameter on success (default == NULL).
	 * @return The child node with a component template that matches the given component template instance, or an invalid node reference if no match was found.
	 */
	UE_API FSCSEditorTreeNodePtrType FindChild(const UActorComponent* InComponentTemplate, bool bRecursiveSearch = false, uint32* OutDepth = NULL) const;

	/**
	 * Attempts to find a reference to the child node that matches the given component variable or instance name.
	 *
	 * @param InVariableOrInstanceName The component variable or instance name to match.
	 * @param bRecursiveSearch Whether or not to recursively search child nodes (default == false).
	 * @param OutDepth If non-NULL, the depth of the child node will be returned in this parameter on success (default == NULL).
	 * @return The child node with a component variable or instance name that matches the given name, or an invalid node reference if no match was found.
	 */
	UE_API FSCSEditorTreeNodePtrType FindChild(const FName& InVariableOrInstanceName, bool bRecursiveSearch = false, uint32* OutDepth = NULL) const;

	/**
	 * Removes the given node from the list of child nodes.
	 *
	 * @param InChildNodePtr The child node to remove.
	 */
	UE_API virtual void RemoveChild(FSCSEditorTreeNodePtrType InChildNodePtr);

	bool IsSceneComponent() const
	{
		return Cast<USceneComponent>(GetComponentTemplate()) != nullptr;
	}

	/** Returns the associated child actor node if applicable to this node type. */
	virtual FSCSEditorChildActorNodePtrType GetChildActorNode() { return nullptr; }

	// Tries to find a SCS node that was likely responsible for creating the specified instance component.  Note: This is not always possible to do!
	static UE_API USCS_Node* FindSCSNodeForInstance(const UActorComponent* InstanceComponent, UClass* ClassToSearch);

	/**
	 * Creates the correct type of node based on the component (instanced or not, etc...)
	 */
	static UE_API FSCSEditorTreeNodePtrType FactoryNodeFromComponent(UActorComponent* InComponent);

	/** Returns a color to use for this type of node */
	static UE_API FSlateColor GetColorTintForIcon(FSCSEditorTreeNodePtrType InNode);

	// Destructor
	virtual ~FSCSEditorTreeNode() = default;

	/**
	 * Ends the 'Create + enter initial name' transaction of this node. The creation of a node is 'ongoing' as long as the initial name of
	 * the node is in edition mode. When the text is not in edit mode anymore, the ongoing create transaction ends and the node
	 * is considered fully created.
	 */
	UE_API void CloseOngoingCreateTransaction();

protected:
	// Called when this node is being removed via a RemoveChild call
	virtual void RemoveMeAsChild() {}

	// Provides derived classes with non-const access to the object represented by this node (e.g. for rename operations, etc.). This should not be made public.
	template<class T>
	inline T* GetMutableObject() const
	{
		return Cast<T>(WeakObjectPtr.Get());
	}

	// Derived classes can override to create and/or return a reference to an alternate editable object.
	UE_API virtual UObject* GetOrCreateEditableObjectForBlueprint(UBlueprint* InBlueprint) const;

public:
	/**
	* @return The Blueprint to which the object represented by this node belongs (requires implementation in subclass).
	*/
	virtual UBlueprint* GetBlueprint() const = 0;

	/**
	 * @return Whether or not this object represents a "native" component template (i.e. one that is not found in the SCS tree).
	 */
	virtual bool IsNativeComponent() const { return false; }

	/**
	 * @return Whether or not this object represents a root component.
	 */
	virtual bool IsRootComponent() const { return false; }

	/**
	 * @return Whether or not this object represents an inherited SCS node (one from a SCS node in a parent Blueprint).
	 */
	virtual bool IsInheritedSCSNode() const { return false; }

	/**
	 * @return Whether or not this object was declared in the current class (or instance).  Anything inherited cannot be reorganized (renamed, reparented, etc...).
	 */
	virtual bool IsInheritedComponent() const
	{
		return IsNativeComponent() || IsInheritedSCSNode();
	}

	/**
	 * @return Whether or not this node represents an instanced object (i.e. not a template).
	 */
	virtual bool IsInstanced() const { return false; }

	/**
	 * @return Whether or not this node represents an instanced actor object.
	 */
	bool IsInstancedActor() const
	{
		return IsInstanced() && IsActorNode();
	}

	/**
	 * @return Whether or not this node represents an instanced component object.
	 */
	bool IsInstancedComponent() const
	{
		return IsInstanced() && IsComponentNode();
	}

	/**
	 * @return Whether or not this node represents a Blueprint (i.e. non-native) component object.
	 */
	bool IsBlueprintComponent() const
	{
		return IsComponentNode() && !IsNativeComponent();
	}

	/**
	 * @return Whether or not this object represents a component instance that was created by the user and not by a native or Blueprint-generated class.
	 */
	virtual bool IsUserInstancedComponent() const { return false; }

	/**
	* @return Whether or not this object represents the default SCS scene root component.
	*/
	virtual bool IsDefaultSceneRoot() const { return false; }

	/**
	 * @return Whether or not this object represents a node that can be deleted from the SCS tree.
	 */
	virtual bool CanDelete() const { return false; }

	/**
	 * @return Whether or not this object represents a node that can be reparented to other nodes based on its context.
	 */
	virtual bool CanReparent() const { return false; }

	/**
	 * @return Whether or not we can edit properties for the object represented by this node.
	 */
	virtual bool CanEdit() const { return false; }

	/**
	 * @return Whether or not we can rename the object or variable represented by this node.
	 */
	virtual bool CanRename() const { return false; }

	/**
	 * Requests a rename on the node.
	 * @param OngoingCreateTransaction The transaction scoping the node creation which will end once the node is named by the user or null if the rename is not part of a the creation process.
	 */
	UE_API void OnRequestRename(TUniquePtr<FScopedTransaction> OngoingCreateTransaction);

	/** Renames the object or variable represented by this node */
	UE_API virtual void OnCompleteRename(const FText& InNewName);

	/** Sets up the delegate for a rename operation */
	void SetRenameRequestedDelegate(FOnRenameRequested InRenameRequested) { RenameRequestedDelegate = InRenameRequested; }

	/** Query that determines if this item should be filtered out or not */
	virtual bool IsFlaggedForFiltration() const 
	{
		return ensureMsgf(FilterFlags != EFilteredState::Unknown, TEXT("Querying a bad filtration state.")) ? 
			(FilterFlags & EFilteredState::FilteredInMask) == 0 : false; 
	}

	/** Returns whether the node will match the given type (for filtering) */
	UE_API virtual bool MatchesFilterType(const UClass* InFilterType) const;

	/** Refreshes this item's filtration state. Set bRecursive to 'true' to refresh any child nodes as well */
	UE_API virtual bool RefreshFilteredState(const UClass* InFilterType, const TArray<FString>& InFilterTerms, bool bRecursive);

protected:
	/** Sets this item's filtration state. Use bUpdateParent to make sure the parent's EFilteredState::ChildMatches flag is properly updated based off the new state */
	UE_API void SetCachedFilterState(bool bMatchesFilter, bool bUpdateParent);
	/** Updates the EFilteredState::ChildMatches flag, based off of children's current state */
	UE_API void RefreshCachedChildFilterState(bool bUpdateParent);
	/** Used to update the EFilteredState::ChildMatches flag for parent nodes, when this item's filtration state has changed */
	UE_API void ApplyFilteredStateToParent();
	
	// Scope the creation of a node which ends when the initial 'name' is given/accepted by the user, which can be several frames after the node was actually created.
	TUniquePtr<FScopedTransaction> OngoingCreateTransaction;

private:
	// The type of tree node
	ENodeType NodeType;

	// Weak ptr to the object instance represented by this node (e.g. component template)
	TWeakObjectPtr<UObject> WeakObjectPtr;

	// Actual tree structure
	FSCSEditorTreeNodePtrType ParentNodePtr;
	FSCSEditorActorNodePtrType ActorRootNodePtr;
	TArray<FSCSEditorTreeNodePtrType> Children;

	/** Handles rename requests */
	FOnRenameRequested RenameRequestedDelegate;

	enum EFilteredState
	{
		FilteredOut    = 0x00,
		MatchesFilter  = (1 << 0),
		ChildMatches   = (1 << 1),

		FilteredInMask = (MatchesFilter | ChildMatches),
		Unknown = 0xFC // ~FilteredInMask
	};
	uint8 FilterFlags;
};

//////////////////////////////////////////////////////////////////////////
//

class FSCSEditorTreeNodeComponentBase : public FSCSEditorTreeNode
{
protected:
	FSCSEditorTreeNodeComponentBase()
		: FSCSEditorTreeNode(FSCSEditorTreeNode::ComponentNode)
	{
	}

public:
	// FSCSEditorTreeNode interface
	UE_API virtual FName GetVariableName() const override;
	UE_API virtual FString GetDisplayString() const override;
	virtual bool CanRename() const override { return !IsInheritedComponent() && !IsDefaultSceneRoot(); }
	virtual bool CanDelete() const override { return !IsInheritedComponent() && !IsDefaultSceneRoot(); }
	UE_API virtual bool CanReparent() const override;
	UE_API virtual UBlueprint* GetBlueprint() const override;
	UE_API virtual FSCSEditorChildActorNodePtrType GetChildActorNode();
	UE_API virtual bool MatchesFilterType(const UClass* InFilterType) const override;
	// End of FSCSEditorTreeNode interface

private:
	/** Child actor node associated with this component node, if applicable. */
	FSCSEditorChildActorNodePtrType ChildActorNodePtr;
};


//////////////////////////////////////////////////////////////////////////
// FSCSEditorTreeNodeInstancedInheritedComponent - A inherited component in the instanced case (either an inherited SCS node or an inherited native component)

class FSCSEditorTreeNodeInstancedInheritedComponent : public FSCSEditorTreeNodeComponentBase
{
public:
	/**
	 * Constructs a wrapper around a component template not contained within an SCS tree node (e.g. "native" components).
	 *
	 * @param InComponentTemplate The component template represented by this object.
	 */
	UE_API FSCSEditorTreeNodeInstancedInheritedComponent(AActor* Owner, UActorComponent* InComponentTemplate);

	// FSCSEditorTreeNode public interface
	UE_API virtual bool IsNativeComponent() const override;
	UE_API virtual bool IsRootComponent() const override;
	UE_API virtual bool IsInheritedSCSNode() const override;
	virtual bool IsInstanced() const override { return true; }
	virtual bool IsInheritedComponent() const override { return true; }
	UE_API virtual bool IsDefaultSceneRoot() const override;
	UE_API virtual bool CanEdit() const override;
	UE_API virtual FText GetDisplayName() const override;
	// End of FSCSEditorTreeNode public interface

private:
	TWeakObjectPtr<AActor> InstancedComponentOwnerPtr;
};

//////////////////////////////////////////////////////////////////////////
// FSCSEditorTreeNodeInstanceAddedComponent - A unique-to-this instance component

class FSCSEditorTreeNodeInstanceAddedComponent : public FSCSEditorTreeNodeComponentBase
{
public:
	/**
	* Constructs a wrapper around a component template not contained within an SCS tree node (e.g. "native" components).
	*
	* @param InComponentTemplate The component template represented by this object.
	*/
	UE_API FSCSEditorTreeNodeInstanceAddedComponent(AActor* Owner, UActorComponent* InComponentTemplate);

	// FSCSEditorTreeNode public interface
	UE_API virtual bool IsRootComponent() const override;
	virtual bool IsInstanced() const override { return true; }
	virtual bool IsUserInstancedComponent() const override { return true; }
	UE_API virtual bool IsDefaultSceneRoot() const override;
	virtual bool CanEdit() const override { return true; }
	virtual FName GetVariableName() const override { return NAME_None; }
	UE_API virtual FString GetDisplayString() const override;
	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual void OnCompleteRename(const FText& InNewName) override;
	// End of FSCSEditorTreeNode public interface

protected:
	// FSCSEditorTreeNode protected interface
	UE_API virtual void RemoveMeAsChild() override;
	// End of FSCSEditorTreeNode protected interface

private:
	FName InstancedComponentName;
	TWeakObjectPtr<AActor> InstancedComponentOwnerPtr;
};

//////////////////////////////////////////////////////////////////////////
// FSCSEditorTreeNodeComponent - A generic component in the non-instanced case (either a SCS node or an inherited native component)

class FSCSEditorTreeNodeComponent : public FSCSEditorTreeNodeComponentBase
{
public:
	/**
	 * Constructs a wrapper around a component template contained within an SCS tree node.
	 *
	 * @param InSCSNode The SCS tree node represented by this object.
	 * @param bInIsInherited Whether or not the SCS tree node is inherited from a parent Blueprint class.
	 */
	UE_API FSCSEditorTreeNodeComponent(class USCS_Node* InSCSNode, bool bInIsInherited = false);

	/**
	 * Constructs a wrapper around a component template not contained within an SCS tree node (e.g. "native" components).
	 *
	 * @param InComponentTemplate The component template represented by this object.
	 */
	UE_API FSCSEditorTreeNodeComponent(UActorComponent* InComponentTemplate);


	// FSCSEditorTreeNode public interface
	UE_API virtual bool IsNativeComponent() const override;
	UE_API virtual bool IsRootComponent() const override;
	UE_API virtual bool IsInheritedSCSNode() const override;
	UE_API virtual bool IsDefaultSceneRoot() const override;
	UE_API virtual bool CanEdit() const override;
	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual class USCS_Node* GetSCSNode() const override;
	UE_API virtual void OnCompleteRename(const FText& InNewName) override;
	// End of FSCSEditorTreeNode public interface

protected:
	// FSCSEditorTreeNode protected interface
	UE_API virtual void RemoveMeAsChild() override;
	UE_API virtual UObject* GetOrCreateEditableObjectForBlueprint(UBlueprint* InBlueprint) const override;
	// End of FSCSEditorTreeNode protected interface

	/** Get overridden template component, specialized in given blueprint */
	UE_API UActorComponent* INTERNAL_GetOverridenComponentTemplate(UBlueprint* Blueprint) const;

private:
	// Was this component inherited from a parent class or introduced in this class?
	bool bIsInheritedSCS;

	// Is this the template coming from an SCS node?
	TWeakObjectPtr<class USCS_Node> SCSNodePtr;
};

class FSCSEditorTreeNodeActorBase : public FSCSEditorTreeNode
{
public:
	FSCSEditorTreeNodeActorBase(FSCSEditorTreeNode::ENodeType InNodeType, AActor* InActor)
		: FSCSEditorTreeNode(InNodeType)
	{
		SetObject(InActor);
	}

	UE_API FSCSEditorTreeNodePtrType GetOwnerNode() const;
	UE_API void SetOwnerNode(FSCSEditorTreeNodePtrType NewOwnerNode);

	UE_API FSCSEditorTreeNodePtrType GetSceneRootNode() const;
	UE_API void SetSceneRootNode(FSCSEditorTreeNodePtrType NewSceneRootNode);

	/** Returns the set of root nodes */
	UE_API const TArray<FSCSEditorTreeNodePtrType>& GetComponentNodes() const;

	// FSCSEditorTreeNode public interface
	UE_API virtual FName GetNodeID() const override;
	UE_API virtual bool IsInstanced() const override;
	virtual bool CanEdit() const override { return true; }
	UE_API virtual void AddChild(FSCSEditorTreeNodePtrType InChildNodePtr) override;
	UE_API virtual void RemoveChild(FSCSEditorTreeNodePtrType InChildNodePtr) override;
	UE_API virtual UBlueprint* GetBlueprint() const override;
	// End of FSCSEditorTreeNode public interface

protected:
	using Super = FSCSEditorTreeNode;

private:
	/** The actor's subtree owner (if valid) */
	FSCSEditorTreeNodePtrType OwnerNodePtr;
	/** The actor's scene root node (if valid) */
	FSCSEditorTreeNodePtrType SceneRootNodePtr;
	/** Root set of components (contains the root scene component and any non-scene component nodes) */
	TArray<FSCSEditorTreeNodePtrType> ComponentNodes;
};

class FSCSEditorTreeNodeRootActor : public FSCSEditorTreeNodeActorBase
{
public:
	FSCSEditorTreeNodeRootActor(AActor* InActor, bool bInAllowRename)
		: FSCSEditorTreeNodeActorBase(FSCSEditorTreeNode::RootActorNode, InActor)
		, bAllowRename(bInAllowRename)
		, CachedFilterType(nullptr)
	{
	}

	// FSCSEditorTreeNode public interface
	virtual bool CanRename() const override { return bAllowRename; }
	UE_API virtual void OnCompleteRename(const FText& InNewName) override;
	UE_API virtual void AddChild(FSCSEditorTreeNodePtrType InChildNodePtr) override;
	UE_API virtual void RemoveChild(FSCSEditorTreeNodePtrType InChildNodePtr) override;
	UE_API virtual bool RefreshFilteredState(const UClass* InFilterType, const TArray<FString>& InFilterTerms, bool bRecursive) override;
	// End of FSCSEditorTreeNode public interface

private:
	bool bAllowRename;
	const UClass* CachedFilterType;
	TArray<FString> CachedFilterTerms;
	TWeakPtr<class FSCSEditorTreeNodeSeparator> SceneComponentSeparatorNodePtr;
	TWeakPtr<class FSCSEditorTreeNodeSeparator> NonSceneComponentSeparatorNodePtr;
};

class FSCSEditorTreeNodeChildActor : public FSCSEditorTreeNodeActorBase
{
public:
	FSCSEditorTreeNodeChildActor(AActor* InActor)
		: FSCSEditorTreeNodeActorBase(FSCSEditorTreeNode::ChildActorNode, InActor)
	{
	}

	// FSCSEditorTreeNode public interface
	UE_API virtual bool IsFlaggedForFiltration() const override;
	// End of FSCSEditorTreeNode public interface

	UE_API UChildActorComponent* GetChildActorComponent() const;
};

class FSCSEditorTreeNodeSeparator : public FSCSEditorTreeNode
{
public:
	FSCSEditorTreeNodeSeparator()
		: FSCSEditorTreeNode(FSCSEditorTreeNode::SeparatorNode)
	{
	}

	// FSCSEditorTreeNode public interface
	virtual UBlueprint* GetBlueprint() const override { return nullptr; }
	UE_API virtual bool MatchesFilterType(const UClass* InFilterType) const override;
	// End of FSCSEditorTreeNode public interface

	/** If the given type matches the tree view filter, the separator will also be flagged for filtration. */
	UE_API void AddFilteredComponentType(const TSubclassOf<UActorComponent>& InFilteredType);

private:
	TArray<const UClass*> FilteredTypes;
};

class UE_DEPRECATED(5.8, "SSCSEditor was removed, use either SSubObjectEditor or FSCSEditorTreeNode functions") SSCS_RowWidget;

class SSCS_RowWidget
{
	UE_DEPRECATED(5.8, "Use FSCSEditorTreeNode::GetColorTintForIcon")
	static FSlateColor GetColorTintForIcon(FSCSEditorTreeNodePtrType InNode)
	{
		return FSCSEditorTreeNode::GetColorTintForIcon(InNode);
	}
};

#undef UE_API
