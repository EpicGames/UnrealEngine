// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "SubobjectEditorExtensionContext.generated.h"

class SSubobjectBlueprintEditor;
class SSubobjectEditor;
class SSubobjectInstanceEditor;

UCLASS(MinimalAPI)
class USubobjectEditorExtensionContext : public UObject
{
	GENERATED_BODY()

public:
	
	const TWeakPtr<SSubobjectEditor>& GetSubobjectEditor() const { return SubobjectEditor; }

private:
	friend SSubobjectEditor;
	friend SSubobjectInstanceEditor;
	friend SSubobjectBlueprintEditor;

	TWeakPtr<SSubobjectEditor> SubobjectEditor;
};
